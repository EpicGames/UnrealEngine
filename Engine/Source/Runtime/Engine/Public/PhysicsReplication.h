// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsReplication.h
	Manage replication of physics bodies
=============================================================================*/

#pragma once

#include "Engine/EngineTypes.h"
#include "Engine/ReplicatedState.h"
#include "PhysicsReplicationInterface.h"
#include "Physics/PhysicsInterfaceDeclares.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "Chaos/PhysicsObject.h"
#include "Chaos/ParticleDirtyFlags.h"
#include "Chaos/SimCallbackObject.h"
#include "Physics/PhysicsInterfaceUtils.h"
#include "Physics/NetworkPhysicsSettingsComponent.h"

namespace Chaos
{
	class FRewindData;
	class FPBDRigidsSolver;
	namespace Private
	{
		class FPBDIsland;
	}
}

namespace CharacterMovementCVars
{
	extern ENGINE_API int32 SkipPhysicsReplication;
	extern ENGINE_API float NetPingExtrapolation;
	extern ENGINE_API float NetPingLimit;
	extern ENGINE_API float ErrorPerLinearDifference;
	extern ENGINE_API float ErrorPerAngularDifference;
	extern ENGINE_API float ErrorAccumulationSeconds;
	extern ENGINE_API float ErrorAccumulationDistanceSq;
	extern ENGINE_API float ErrorAccumulationSimilarity;
	extern ENGINE_API float MaxLinearHardSnapDistance;
	extern ENGINE_API float MaxRestoredStateError;
	extern ENGINE_API float PositionLerp;
	extern ENGINE_API float LinearVelocityCoefficient;
	extern ENGINE_API float AngleLerp;
	extern ENGINE_API float AngularVelocityCoefficient;
	extern ENGINE_API int32 AlwaysHardSnap;
	extern ENGINE_API int32 AlwaysResetPhysics;
	extern ENGINE_API int32 ApplyAsyncSleepState;
}

#if !UE_BUILD_SHIPPING
namespace PhysicsReplicationCVars
{
	extern ENGINE_API int32 LogPhysicsReplicationHardSnaps;
}
#endif

#pragma region FPhysicsReplicationAsync

struct FPhysicsRepErrorCorrectionData
{
	float LinearVelocityCoefficient;
	float AngularVelocityCoefficient;
	float PositionLerp;
	float AngleLerp;
};

/** Final computed desired state passed into the physics sim */
struct FPhysicsRepAsyncInputData
{
	FRigidBodyState TargetState;
	Chaos::FSingleParticlePhysicsProxy* Proxy; // Used for legacy (BodyInstance) flow
	Chaos::FConstPhysicsObjectHandle PhysicsObject;
	TOptional<FPhysicsRepErrorCorrectionData> ErrorCorrection;
	EPhysicsReplicationMode RepMode;
	int32 ServerFrame;
	float LatencyOneWay;

	FPhysicsRepAsyncInputData(Chaos::FConstPhysicsObjectHandle POHandle)
		: Proxy(nullptr)
		, PhysicsObject(POHandle)
		, RepMode(EPhysicsReplicationMode::Default)
	{};
};

struct FPhysicsReplicationAsyncInput : public Chaos::FSimCallbackInput
{
	FPhysicsRepErrorCorrectionData ErrorCorrection;
	TArray<FPhysicsRepAsyncInputData> InputData;
	bool NetworkPhysicsTickOffsetAssigned;
	int32 NetworkPhysicsTickOffset;

	void Reset()
	{
		InputData.Reset();
		NetworkPhysicsTickOffsetAssigned = false;
		NetworkPhysicsTickOffset = 0;
	}
};

struct FReplicatedPhysicsTargetAsync
{
	FReplicatedPhysicsTargetAsync()
		: TargetState()
		, AccumulatedErrorSeconds(0.0f)
		, TickCount(0)
		, ServerFrame(INDEX_NONE)
		, FrameOffset(0)
		, ReceiveFrame(INDEX_NONE)
		, ReceiveInterval(5)
		, AverageReceiveInterval(5.0f)
		, RepMode(EPhysicsReplicationMode::Default)
		, RepModeOverride(EPhysicsReplicationMode::Default)
		, PrevPosTarget()
		, PrevRotTarget()
		, PrevPos()
		, PrevLinVel()
		, AccumulatedSleepSeconds(0.0f)
		, bAllowTargetAltering(false)
		, WaitForServerFrame(INDEX_NONE)
		, SimDecayTimeScale(1.0f)
		, LastLODFrame(INDEX_NONE)
		, PrevTickCountCached(0)
		, bPrevAllowTargetAlteringCached(false)
		, bFrameOffsetCorrectedCached(false)
	{ }

	/** The target state replicated by server */
	FRigidBodyState TargetState;

	/** Physics sync error accumulation */
	float AccumulatedErrorSeconds;

	/** The amount of simulation ticks this target has been used for */
	int32 TickCount;

	/** ServerFrame this target was replicated on. (LocalFrame = ServerFrame - FrameOffset) */
	int32 ServerFrame;

	/** The frame offset between local client and server. (LocalFrame = ServerFrame - FrameOffset) */
	int32 FrameOffset;

	/** The local client frame when receiving this target from the server */
	int32 ReceiveFrame;

	/** Local physics frames between received targets */
	int32 ReceiveInterval;
	float AverageReceiveInterval;

	/** The replication mode this PhysicsObject should use */
	EPhysicsReplicationMode RepMode;
	EPhysicsReplicationMode RepModeOverride;

	/** Correction values from previous update */
	FVector PrevPosTarget;
	FQuat PrevRotTarget;
	FVector PrevPos;
	FVector PrevLinVel;

	/** Accumulated seconds asleep */
	float AccumulatedSleepSeconds;

	/** If this target is allowed to be altered, via extrapolation or target alignment via TickCount */
	bool bAllowTargetAltering;

	/** ServerFrame for the target to wait on, no replication will be performed while waiting for up to date data */
	int32 WaitForServerFrame;

	/** Per-frame time-scale used by SimulationDecay during resim integration steps. 1.0 is full integration, 0.0 is integration fully reverted */
	float SimDecayTimeScale;

	/** Solver frame at which ApplyPhysicsReplicationLOD last ran for this target */
	int32 LastLODFrame;

	/** Per-frame transient helper properties */
	int32 PrevTickCountCached;
	bool bPrevAllowTargetAlteringCached;
	bool bFrameOffsetCorrectedCached;

public:
	/** Is this target waiting for up to date data? */
	const bool IsWaiting() const { return WaitForServerFrame > INDEX_NONE; }

	/** Set target to wait for data newer than @param InWaitForServerFrame and while waiting replicate via @param InRepModeOverride */
	void SetWaiting(int32 InWaitForServerFrame, EPhysicsReplicationMode InRepModeOverride)
	{
		SetWaiting(InWaitForServerFrame);
		RepModeOverride = InRepModeOverride;
	}

	/** Set target to wait for data newer than @param InWaitForServerFrame */
	void SetWaiting(int32 InWaitForServerFrame)
	{
		RepModeOverride = RepMode;
		WaitForServerFrame = InWaitForServerFrame;
	}

	/** Update waiting status and clear waiting if @param InServerFrame is newer than the frame we are waiting for */
	void UpdateWaiting(int32 InServerFrame)
	{
		if (InServerFrame > WaitForServerFrame)
		{
			SetWaiting(INDEX_NONE);
		}
	}
};

class FPhysicsReplicationAsync : public IPhysicsReplicationAsync,
	public Chaos::TSimCallbackObject<
	FPhysicsReplicationAsyncInput,
	Chaos::FSimCallbackNoOutput,
	Chaos::ESimCallbackOptions::Presimulate | Chaos::ESimCallbackOptions::PostIntegrate | Chaos::ESimCallbackOptions::PostSolve | Chaos::ESimCallbackOptions::PhysicsObjectUnregister>
{
	virtual FName GetFNameForStatId() const override;
	virtual void OnPostInitialize_Internal() override;
	virtual void OnPreSimulate_Internal() override;
	virtual void OnPostIntegrate_Internal() override;
	virtual void OnPostSolve_Internal() override;
	virtual void OnPhysicsObjectUnregistered_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;

	virtual void ApplyTargetStatesAsync(const float DeltaSeconds);
	UE_DEPRECATED(5.6, "Deprecated, call the function with just @param DeltaSeconds instead.")
	virtual void ApplyTargetStatesAsync(const float DeltaSeconds, const FPhysicsRepErrorCorrectionData& ErrorCorrection, const TArray<FPhysicsRepAsyncInputData>& TargetStates) { ApplyTargetStatesAsync(DeltaSeconds); };

	// Replication functions
	virtual void DefaultReplication_DEPRECATED(Chaos::FRigidBodyHandle_Internal* Handle, const FPhysicsRepAsyncInputData& State, const float DeltaSeconds, const FPhysicsRepErrorCorrectionData& ErrorCorrection);
	virtual bool DefaultReplication(Chaos::FPBDRigidParticleHandle* Handle, FReplicatedPhysicsTargetAsync& Target, const float DeltaSeconds);
	virtual bool PredictiveInterpolation(Chaos::FPBDRigidParticleHandle* Handle, FReplicatedPhysicsTargetAsync& Target, const float DeltaSeconds);
	virtual bool ResimulationReplication(Chaos::FPBDRigidParticleHandle* Handle, FReplicatedPhysicsTargetAsync& Target, const float DeltaSeconds);

public:
	//~ Begin IPhysicsReplicationAsync interface
	virtual void RegisterSettings(Chaos::FConstPhysicsObjectHandle PhysicsObject, TWeakPtr<const FNetworkPhysicsSettingsData> InSettings) override;
	virtual void AddResimulationRequest_Internal(const float DeltaSeconds) override;

	virtual int32 GetNetworkPhysicsTickOffset_Internal() const override { return NetworkPhysicsTickOffset; }

	virtual TWeakPtr<FParticleSimDecaySettings> FindOrAddParticleSimDecaySettings(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;
	virtual void RemoveParticleSimDecaySettings(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;
	//~ End IPhysicsReplicationAsync interface

private:
	float LatencyOneWay;
	FRigidBodyErrorCorrection ErrorCorrectionDefault;
	FNetworkPhysicsSettingsData SettingsCurrent;
	FNetworkPhysicsSettingsData SettingsDefault;
	TMap<Chaos::FConstPhysicsObjectHandle, FReplicatedPhysicsTargetAsync> ObjectToTarget;
	TArray<Chaos::FConstPhysicsObjectHandle> PendingDeleteFromObjectToTarget;
	TMap<Chaos::FConstPhysicsObjectHandle, TWeakPtr<const FNetworkPhysicsSettingsData>> ObjectToSettings;
	TMap<Chaos::FConstPhysicsObjectHandle, TSharedPtr<FParticleSimDecaySettings>> ParticleSimDecaySettings;

	TArray<const Chaos::Private::FPBDIsland*> ResimIslands;
	TArray<const Chaos::FGeometryParticleHandle*> ResimIslandsParticles;
	TArray<int32> ParticlesInResimIslands;
	TArray<Chaos::FParticleID> ReplicatedParticleIDs;

	int32 ResimOutOfBoundsCounter = 0;
	float ResimErrorLogTimer = 0.0f;

	bool NetworkPhysicsTickOffsetAssigned = false;
	int32 NetworkPhysicsTickOffset = 0;

private:
	FReplicatedPhysicsTargetAsync* AddObjectToReplication(Chaos::FConstPhysicsObjectHandle PhysicsObject);
	void RemoveObjectFromReplication(Chaos::FConstPhysicsObjectHandle PhysicsObject);
	void UpdateAsyncTarget(const FPhysicsRepAsyncInputData& Input, Chaos::FPBDRigidsSolver* RigidsSolver);
	void UpdateRewindDataTarget(const FPhysicsRepAsyncInputData& Input);
	void CacheResimInteractions();
	// Sets SettingsCurrent to either the objects custom settings or to the default settings
	void FetchObjectSettings(Chaos::FConstPhysicsObjectHandle PhysicsObject); 
	bool UsePhysicsReplicationLOD();
	void CheckTargetResimValidity(FReplicatedPhysicsTargetAsync& Target);
	void ApplyPhysicsReplicationLOD(Chaos::FConstPhysicsObjectHandle PhysicsObjectHandle, FReplicatedPhysicsTargetAsync& Target, const uint32 LODFlags);
	void DebugDrawReplicationMode(Chaos::FConstPhysicsObjectHandle PhysicsObjectHandle, const FReplicatedPhysicsTargetAsync& Target);

	/** Check if we have to resimulate the particle or not */
	bool ShouldResimulateParticle(const Chaos::FPBDRigidParticleHandle* Handle, const FReplicatedPhysicsTargetAsync& Target, const float DeltaSeconds) const;

	/** Check if a target is valid for resimulation or not */
	bool IsTargetValidForResim(const FReplicatedPhysicsTargetAsync& Target) const;

	/** Returns RewindData if it already exists, otherwise enables RewindCapture (when network physics prediction and a fixed timestep are in use) and returns the freshly enabled instance. Returns nullptr if neither conditions allow enabling. */
	Chaos::FRewindData* GetOrEnableRewindData(Chaos::FPBDRigidsSolver* RigidsSolver);

	/** Static function to extrapolate a target for N ticks using X DeltaSeconds */
	static void ExtrapolateTarget(FReplicatedPhysicsTargetAsync& Target, const int32 ExtrapolateFrames, const float DeltaSeconds);
	/** Static function to extrapolate a target for N Seconds */
	static void ExtrapolateTarget(FReplicatedPhysicsTargetAsync& Target, const float ExtrapolationTime);

public:
	void Setup(FRigidBodyErrorCorrection ErrorCorrection)
	{
		ErrorCorrectionDefault = ErrorCorrection;
	}
};

#pragma endregion // FPhysicsReplicationAsync



struct FReplicatedPhysicsTarget
{
	FReplicatedPhysicsTarget(Chaos::FConstPhysicsObjectHandle POHandle = nullptr)
		: ArrivedTimeSeconds(0.0f)
		, AccumulatedErrorSeconds(0.0f)
		, ServerFrame(0)
		, PhysicsObject(POHandle)
	{ }

	/** The target state replicated by server */
	FRigidBodyState TargetState;

	/** The bone name used to find the body */
	FName BoneName;

	/** Client time when target state arrived */
	float ArrivedTimeSeconds;

	/** Physics sync error accumulation */
	float AccumulatedErrorSeconds;

	/** Correction values from previous update */
	FVector PrevPosTarget;
	FVector PrevPos;

	/** ServerFrame this target was replicated on (must be converted to local frame prior to client-side use) */
	int32 ServerFrame;

	/** Index of physics object on component */
	Chaos::FConstPhysicsObjectHandle PhysicsObject;

	/** The replication mode the target should be used with */
	EPhysicsReplicationMode ReplicationMode;

#if !UE_BUILD_SHIPPING
	FDebugFloatHistory ErrorHistory;
#endif
};

class FPhysicsReplication : public IPhysicsReplication
{
public:
	ENGINE_API FPhysicsReplication(FPhysScene* PhysScene);
	ENGINE_API virtual ~FPhysicsReplication();

	/** Helper method so the Skip Replication CVar can be check elsewhere (including game extensions to this class) */
	static ENGINE_API bool ShouldSkipPhysicsReplication();

	/** Tick and update all body states according to replicated targets */
	ENGINE_API virtual void Tick(float DeltaSeconds) override;

	/** Sets the latest replicated target for a body instance */
	ENGINE_API virtual void SetReplicatedTarget(UPrimitiveComponent* Component, FName BoneName, const FRigidBodyState& ReplicatedTarget, int32 ServerFrame) override;

	ENGINE_API void SetReplicatedTarget(Chaos::FConstPhysicsObjectHandle PhysicsObject, const FRigidBodyState& ReplicatedTarget, int32 ServerFrame, EPhysicsReplicationMode ReplicationMode = EPhysicsReplicationMode::Default);

	/** Remove the replicated target*/
	ENGINE_API virtual void RemoveReplicatedTarget(UPrimitiveComponent* Component) override;

	/** Remove the replicated target*/
	ENGINE_API void RemoveReplicatedTarget(Chaos::FConstPhysicsObjectHandle PhysicsObject);

	virtual int32 GetNetworkPhysicsTickOffset() const override { return NetworkPhysicsTickOffset; }

protected:

	/** Update the physics body state given a set of replicated targets */
	ENGINE_API virtual void OnTick(float DeltaSeconds, TMap<TWeakObjectPtr<UPrimitiveComponent>, FReplicatedPhysicsTarget>& ComponentsToTargets);
	virtual void OnTargetRestored(TWeakObjectPtr<UPrimitiveComponent> Component, const FReplicatedPhysicsTarget& Target) {}
	virtual void OnSetReplicatedTarget(UPrimitiveComponent* Component, FName BoneName, const FRigidBodyState& ReplicatedTarget, int32 ServerFrame, FReplicatedPhysicsTarget& Target) {}

	/** Called when a dynamic rigid body receives a physics update */
	ENGINE_API virtual bool ApplyRigidBodyState(float DeltaSeconds, FBodyInstance* BI, FReplicatedPhysicsTarget& PhysicsTarget, const FRigidBodyErrorCorrection& ErrorCorrection, const float PingSecondsOneWay, int32 LocalFrame, int32 NumPredictedFrames);
	ENGINE_API virtual bool ApplyRigidBodyState(float DeltaSeconds, FBodyInstance* BI, FReplicatedPhysicsTarget& PhysicsTarget, const FRigidBodyErrorCorrection& ErrorCorrection, const float PingSecondsOneWay, bool* bDidHardSnap = nullptr); // deprecated path with no localframe/numpredicted

	ENGINE_API UWorld* GetOwningWorld();
	ENGINE_API const UWorld* GetOwningWorld() const;

private:

	/** Get the ping from this machine to the server */
	ENGINE_API float GetLocalPing() const;

	/** Get the ping from  */
	ENGINE_API float GetOwnerPing(const AActor* const Owner, const FReplicatedPhysicsTarget& Target) const;

private:
	TMap<TWeakObjectPtr<UPrimitiveComponent>, FReplicatedPhysicsTarget> ComponentToTargets_DEPRECATED; // This collection is keeping the legacy flow working until fully deprecated in a future release
	TArray<TWeakObjectPtr<UPrimitiveComponent>> PendingDeleteFromComponentsToTargets;
	TArray<FReplicatedPhysicsTarget> ReplicatedTargetsQueue;
	FPhysScene* PhysScene;
	TWeakObjectPtr<UNetworkPhysicsSettingsComponent> SettingsCurrent;


	FPhysicsReplicationAsync* PhysicsReplicationAsync;
	FPhysicsReplicationAsyncInput* AsyncInput;	//async data being written into before we push into callback

	bool NetworkPhysicsTickOffsetAssigned = false;
	int32 NetworkPhysicsTickOffset = 0;

	ENGINE_API void PrepareAsyncData_External(const FRigidBodyErrorCorrection& ErrorCorrection);	//prepare async data for writing. Call on external thread (i.e. game thread)
};
