// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/PhysicsObject.h"
#include "Physics/NetworkPhysicsSettingsComponent.h" // Temporary until RegisterSettings with FNetworkPhysicsSettingsAsync param is deprecated

class UPrimitiveComponent;
struct FRigidBodyState;
struct FNetworkPhysicsSettingsData;

/** Per-particle settings for SimulationDecay */
struct FParticleSimDecaySettings
{
	/** Running-average of how many frames ahead of the latest received input this sim-proxy is predicting. */
	float InputPredictionFramesAverage = 0.0f;
	/** Static clamp value used when bUseDynamicTimeScale is false. Also reused as the runtime-decay TimeScale when bApplyDecayAtRuntime is true. Expected range 0.0 -> 1.0. */
	float StaticTimeScale = 0.9f;
	/** Base value added to the dynamic clamp formula, expected range 0.0 -> 1.0. Recommended to set via SetDynamicSettings(). */
	float DynamicBase = 0.1f;
	/** Minimum clamp value allowed by the dynamic clamp formula, expected range 0.0 -> 1.0. Recommended to set via SetDynamicSettings(). */
	float DynamicMin = 0.25f;
	/** Maximum clamp value allowed by the dynamic clamp formula, expected range 0.0 -> 1.0. Recommended to set via SetDynamicSettings(). */
	float DynamicMax = 1.0f;
	/** When true, compute the clamp from InputPredictionFramesAverage. */
	bool bUseDynamicTimeScale = false;
	/** When true, apply SimulationDecay during regular (non-resim) frames as well, in addition to resim frames. Only effective when LOD leaves the particle in EPhysicsReplicationMode::Resimulation. */
	bool bApplyDecayAtRuntime = false;

	void SetDynamicSettings(const bool bInUseDynamicTimeScale, const float InDynamicBase, const float InDynamicMin, const float InDynamicMax)
	{
		bUseDynamicTimeScale = bInUseDynamicTimeScale;
		DynamicBase = FMath::Clamp(InDynamicBase, 0.0f, 1.0f);
		DynamicMin = FMath::Clamp(InDynamicMin, 0.0f, 1.0f);
		DynamicMax = FMath::Clamp(InDynamicMax, 0.0f, 1.0f);
	}
};

class IPhysicsReplication // Game Thread API
{
public:
	virtual ~IPhysicsReplication() = default;

	virtual void Tick(float DeltaSeconds) { }

	virtual void SetReplicatedTarget(UPrimitiveComponent* Component, FName BoneName, const FRigidBodyState& ReplicatedTarget, int32 ServerFrame) = 0;

	virtual void RemoveReplicatedTarget(UPrimitiveComponent* Component) = 0;

	/** Get the network physics tick offset on the game thread */
	virtual int32 GetNetworkPhysicsTickOffset() const { return 0; }
};

class IPhysicsReplicationAsync // Physics Thread API
{
public:
	virtual ~IPhysicsReplicationAsync() = default;

	UE_DEPRECATED(5.7, "Deprecated, use the RegisterSettings that passes through a TWeakPtr<FNetworkPhysicsSettingsData> parameter instead ")
	virtual void RegisterSettings(Chaos::FConstPhysicsObjectHandle PhysicsObject, FNetworkPhysicsSettingsAsync InSettings) { }

	virtual void RegisterSettings(Chaos::FConstPhysicsObjectHandle PhysicsObject, TWeakPtr<const FNetworkPhysicsSettingsData> InSettings) = 0;

	/** Add potential resimulation request from the physics replication */
	virtual void AddResimulationRequest_Internal( const float DeltaSeconds) = 0;

	/** Get the network physics tick offset on the physics thread */
	virtual int32 GetNetworkPhysicsTickOffset_Internal() const { return 0; }

	/** Return a writable entry in the per-particle SimulationDecay settings map. */
	virtual TWeakPtr<FParticleSimDecaySettings> FindOrAddParticleSimDecaySettings(Chaos::FConstPhysicsObjectHandle PhysicsObject) { return nullptr; }
	
	/** Remove entry in the per-particle SimulationDecay settings map. */
	virtual void RemoveParticleSimDecaySettings(Chaos::FConstPhysicsObjectHandle PhysicsObject) { }
};
