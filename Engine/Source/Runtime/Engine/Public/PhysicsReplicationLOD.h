// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "PhysicsReplicationLODInterface.h"
#include "Chaos/SimCallbackObject.h"
#include "Physics/PhysicsInterfaceDeclares.h"
#include "Physics/PhysicsInterfaceUtils.h"
#include "PhysicsEngine/PhysicsSettings.h"

namespace Chaos
{
	namespace Private
	{
		class FPBDIsland;
	}
}

enum EPhysicsReplicationLODFlags : uint32
{
	LODFlag_None = 0,
	LODFlag_IslandCheck = (1 << 0),
	LODFlag_DistanceCheck = (1 << 1),
	LODFlag_All = UINT_MAX,
};

class UPrimitiveComponent;
class FPhysicsReplicationLODAsync;

class FPhysicsReplicationLOD : public IPhysicsReplicationLOD
{
public:
	FPhysicsReplicationLOD(FPhysScene* PhysScene);

	virtual ~FPhysicsReplicationLOD();

public: // IPhysicsReplicationLOD Functions

	virtual void SetEnabled(const bool bInEnabled) override;
	virtual bool IsEnabled() const override;

	/** Register a components physics object as a Focal Particle in LOD */
	virtual void RegisterFocalPoint_External(const UPrimitiveComponent* Component, FName BoneName = NAME_None) override;
	/** Unregister a components physics object as a Focal Particle in LOD */
	virtual void UnregisterFocalPoint_External(const UPrimitiveComponent* Component, FName BoneName = NAME_None) override;

	/** Register PhysicsObject as a Focal Particle in LOD */
	virtual void RegisterFocalPoint_External(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;
	/** Unregister PhysicsObject as a Focal Particle in LOD */
	virtual void UnregisterFocalPoint_External(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;
	
	/** Get the PhysicsThread instance of the PhysicsReplicationLOD */
	virtual IPhysicsReplicationLODAsync* GetPhysicsReplicationLOD_Internal() override;

private:
	bool bEnabled;
	FPhysScene* PhysScene;
	FPhysicsReplicationLODAsync* PhysicsReplicationLODAsync;
};




// ------------------------------- Async -------------------------------


struct FPhysicsReplicationLODAsyncInput : public Chaos::FSimCallbackInput
{
	TOptional<bool> bEnabled;
	TArray<Chaos::FConstPhysicsObjectHandle> PhysicsObjectsToRegister;
	TArray<Chaos::FConstPhysicsObjectHandle> PhysicsObjectsToUnregister;

	void Reset()
	{
		bEnabled.Reset();
		PhysicsObjectsToRegister.Reset();
		PhysicsObjectsToUnregister.Reset();
	}
};


class FPhysicsReplicationLODAsync : public IPhysicsReplicationLODAsync
	, public Chaos::TSimCallbackObject<
	FPhysicsReplicationLODAsyncInput,
	Chaos::FSimCallbackNoOutput,
	Chaos::ESimCallbackOptions::Presimulate | Chaos::ESimCallbackOptions::Rewind | Chaos::ESimCallbackOptions::PhysicsObjectUnregister | Chaos::ESimCallbackOptions::PostSolve>
{
friend FPhysicsReplicationLOD;

public:

	FPhysicsReplicationLODAsync() 
	: bEnabled(false)
	, DefaultSettings()
	, LodData()
	, FocalParticles()
	, ParticlesInFocalIslands()
	, CachedIslandLodData()
	, ParticleIslands()
	, IslandParticles()
	, IslandAABB()
	, ParticleAABB()
	{ 
	}

	virtual ~FPhysicsReplicationLODAsync() = default;

private: // TSimCallbackObject Functions
	virtual void OnPostInitialize_Internal() override;
	virtual void ProcessInputs_Internal(int32 PhysicsStep) override; // Called before OnPreSimulate_Internal()
	virtual void OnPreSimulate_Internal() override;
	virtual void OnPostSolve_Internal() override;
	virtual void OnPhysicsObjectUnregistered_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;


public: // IPhysicsReplicationLODAsync Functions
	virtual bool IsEnabled() const override;

	/** Register PhysicsObject as a Focal Particle in LOD */
	virtual void RegisterFocalPoint_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;
	/** Unregister PhysicsObject as a Focal Particle in LOD */
	virtual void UnregisterFocalPoint_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;

	/** Receive the LOD data for @param PhysicsObject, based on its relation to registered Focal Particles in LOD.
	* NOTE: @param PhysicsObject will no be manipulated by the LOD, the returning FPhysicsRepLodData has data and recommendation that can be used by the caller.
	* @param LODFlags is of type EPhysicsReplicationLODFlags */
	virtual FPhysicsRepLodData* GetLODData_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject, const uint32 LODFlags = EPhysicsReplicationLODFlags::LODFlag_All) override;

private:
	void ConsumeAsyncInput();
	void CacheParticlesInFocalIslands();

	bool PerformIslandLOD(const Chaos::FGeometryParticleHandle& ParticleHandle, const uint32 LODFlags);
	bool PerformDistanceLOD(const Chaos::FGeometryParticleHandle& ParticleHandle, const uint32 LODFlags);

private:
	bool bEnabled;
	FPhysicsReplicationLODSettings DefaultSettings;
	FPhysicsRepLodData LodData;
	TArray<Chaos::FConstPhysicsObjectHandle> FocalParticles;
	
	// Particles in the same islands as focal particles
	TArray<int32> ParticlesInFocalIslands;

	// LOD Data cached per island, valid for one physics frame
	TMap<int32, FPhysicsRepLodData> CachedIslandLodData;

	// Solver frame on which ConsumeAsyncInput was last performed, ensures we only process it once per frame.
	int32 LastInputProcessStep = INDEX_NONE;

	// Solver frame on which CacheParticlesInFocalIslands was last performed, ensures we only process it once per frame.
	int32 LastFocalCacheStep = INDEX_NONE;

	// Transient array to hold islands affected by LOD
	TArray<const Chaos::Private::FPBDIsland*> ParticleIslands;
	
	// Transient array to hold particles that are part of islands
	TArray<const Chaos::FGeometryParticleHandle*> IslandParticles;
	
	// Transient BoundingBox for island
	Chaos::FAABB3 IslandAABB;
	
	// Transient BoundingBox for particle, used to cache LocalBounds and apply offset from world location and then add to IslandAABB
	Chaos::FAABB3 ParticleAABB;
};
