// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/PhysicsObject.h"
#include "Engine/EngineTypes.h"

class UPrimitiveComponent;
class IPhysicsReplicationLODAsync;

class IPhysicsReplicationLOD // Game Thread API
{
public:
	virtual ~IPhysicsReplicationLOD() = default;

	virtual void SetEnabled(const bool bInEnabled) = 0;
	virtual bool IsEnabled() const = 0;

	virtual void RegisterFocalPoint_External(const UPrimitiveComponent* Component, FName BoneName = NAME_None) = 0;
	virtual void UnregisterFocalPoint_External(const UPrimitiveComponent* Component, FName BoneName = NAME_None) = 0;
	virtual void RegisterFocalPoint_External(Chaos::FConstPhysicsObjectHandle PhysicsObject) = 0;
	virtual void UnregisterFocalPoint_External(Chaos::FConstPhysicsObjectHandle PhysicsObject) = 0;

	virtual IPhysicsReplicationLODAsync* GetPhysicsReplicationLOD_Internal() = 0;
};

struct FPhysicsRepLodData
{
	FPhysicsRepLodData()
	: DataAssigned(false)
	, AlignedTime(0.0f)
	, AlignedFrame(0)
	, bKinematic(false)
	{};

	bool DataAssigned;
	float AlignedTime;
	int32 AlignedFrame;
	bool bKinematic;

	void Reset()
	{
		DataAssigned = false;
		AlignedTime = 0.0f;
		AlignedFrame = 0;
		bKinematic = false;
	}
};

class IPhysicsReplicationLODAsync // Physics Thread API
{
public:
	virtual ~IPhysicsReplicationLODAsync() = default;

	virtual bool IsEnabled() const = 0;

	virtual void RegisterFocalPoint_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject) = 0;
	virtual void UnregisterFocalPoint_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject) = 0;

	virtual FPhysicsRepLodData* GetLODData_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject, const uint32 LODFlags = UINT_MAX) = 0;
};
