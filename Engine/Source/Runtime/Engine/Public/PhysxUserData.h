// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Physics/PhysicsInterfaceDeclares.h"

class UPhysicalMaterial;
class UPrimitiveComponent;
struct FBodyInstance;
struct FConstraintInstance;
struct FKShapeElem;
class FPhysScene_PhysX;

/** Forward declarations */
struct FKShapeElem;
struct FCustomPhysXPayload;

/** PhysX user data type*/
namespace EPhysxUserDataType
{
	enum Type
	{
		Invalid,
		BodyInstance,
		PhysicalMaterial,
		PhysScene,
		ConstraintInstance,
		PrimitiveComponent,
		AggShape,
		CustomPayload,	//This is intended for plugins
	};
};

/** PhysX user data */
struct FPhysxUserData
{
protected:
	EPhysxUserDataType::Type	Type;
	void*						Payload;

public:
	FPhysxUserData()									:Type(EPhysxUserDataType::Invalid), Payload(nullptr) {}
	FPhysxUserData(FBodyInstance* InPayload)			:Type(EPhysxUserDataType::BodyInstance), Payload(InPayload) {}
	FPhysxUserData(UPhysicalMaterial* InPayload)		:Type(EPhysxUserDataType::PhysicalMaterial), Payload(InPayload) {}
	FPhysxUserData(FPhysScene* InPayload)			    :Type(EPhysxUserDataType::PhysScene), Payload(InPayload) {}
	FPhysxUserData(FConstraintInstance* InPayload)		:Type(EPhysxUserDataType::ConstraintInstance), Payload(InPayload) {}
	FPhysxUserData(UPrimitiveComponent* InPayload)		:Type(EPhysxUserDataType::PrimitiveComponent), Payload(InPayload) {}
	FPhysxUserData(FKShapeElem* InPayload)				:Type(EPhysxUserDataType::AggShape), Payload(InPayload) {}
	FPhysxUserData(FCustomPhysXPayload* InPayload)		:Type(EPhysxUserDataType::CustomPayload), Payload(InPayload) {}
	
	template <class T> static T* Get(void* UserData);
	template <class T> static void Set(void* UserData, T* Payload);

	//helper function to determine if userData is garbage (maybe dangling pointer)
	static bool IsGarbage(void* UserData){ return ((FPhysxUserData*)UserData)->Type < EPhysxUserDataType::Invalid || ((FPhysxUserData*)UserData)->Type > EPhysxUserDataType::CustomPayload; }
};

template <> inline FBodyInstance* FPhysxUserData::Get(void* UserData)			{ if (!UserData || ((FPhysxUserData*)UserData)->Type != EPhysxUserDataType::BodyInstance) { return nullptr; } return (FBodyInstance*)((FPhysxUserData*)UserData)->Payload; }
template <> inline UPhysicalMaterial* FPhysxUserData::Get(void* UserData)		{ if (!UserData || ((FPhysxUserData*)UserData)->Type != EPhysxUserDataType::PhysicalMaterial) { return nullptr; } return (UPhysicalMaterial*)((FPhysxUserData*)UserData)->Payload; }
template <> inline FPhysScene* FPhysxUserData::Get(void* UserData)				{ if (!UserData || ((FPhysxUserData*)UserData)->Type != EPhysxUserDataType::PhysScene) { return nullptr; }return (FPhysScene*)((FPhysxUserData*)UserData)->Payload; }
template <> inline FConstraintInstance* FPhysxUserData::Get(void* UserData)	{ if (!UserData || ((FPhysxUserData*)UserData)->Type != EPhysxUserDataType::ConstraintInstance) { return nullptr; } return (FConstraintInstance*)((FPhysxUserData*)UserData)->Payload; }
template <> inline UPrimitiveComponent* FPhysxUserData::Get(void* UserData)	{ if (!UserData || ((FPhysxUserData*)UserData)->Type != EPhysxUserDataType::PrimitiveComponent) { return nullptr; } return (UPrimitiveComponent*)((FPhysxUserData*)UserData)->Payload; }
template <> inline FKShapeElem* FPhysxUserData::Get(void* UserData)	{ if (!UserData || ((FPhysxUserData*)UserData)->Type != EPhysxUserDataType::AggShape) { return nullptr; } return (FKShapeElem*)((FPhysxUserData*)UserData)->Payload; }
template <> inline FCustomPhysXPayload* FPhysxUserData::Get(void* UserData) { if (!UserData || ((FPhysxUserData*)UserData)->Type != EPhysxUserDataType::CustomPayload) { return nullptr; } return (FCustomPhysXPayload*)((FPhysxUserData*)UserData)->Payload; }

template <> inline void FPhysxUserData::Set(void* UserData, FBodyInstance* Payload)			{ check(UserData); ((FPhysxUserData*)UserData)->Type = EPhysxUserDataType::BodyInstance; ((FPhysxUserData*)UserData)->Payload = Payload; }
template <> inline void FPhysxUserData::Set(void* UserData, UPhysicalMaterial* Payload)		{ check(UserData); ((FPhysxUserData*)UserData)->Type = EPhysxUserDataType::PhysicalMaterial; ((FPhysxUserData*)UserData)->Payload = Payload; }
template <> inline void FPhysxUserData::Set(void* UserData, FPhysScene* Payload)				{ check(UserData); ((FPhysxUserData*)UserData)->Type = EPhysxUserDataType::PhysScene; ((FPhysxUserData*)UserData)->Payload = Payload; }
template <> inline void FPhysxUserData::Set(void* UserData, FConstraintInstance* Payload)		{ check(UserData); ((FPhysxUserData*)UserData)->Type = EPhysxUserDataType::ConstraintInstance; ((FPhysxUserData*)UserData)->Payload = Payload; }
template <> inline void FPhysxUserData::Set(void* UserData, UPrimitiveComponent* Payload)		{ check(UserData); ((FPhysxUserData*)UserData)->Type = EPhysxUserDataType::PrimitiveComponent; ((FPhysxUserData*)UserData)->Payload = Payload; }
template <> inline void FPhysxUserData::Set(void* UserData, FKShapeElem* Payload)	{ check(UserData); ((FPhysxUserData*)UserData)->Type = EPhysxUserDataType::AggShape; ((FPhysxUserData*)UserData)->Payload = Payload; }
template <> inline void FPhysxUserData::Set(void* UserData, FCustomPhysXPayload* Payload) { check(UserData); ((FPhysxUserData*)UserData)->Type = EPhysxUserDataType::CustomPayload; ((FPhysxUserData*)UserData)->Payload = Payload; }