// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PlanarReflectionSceneProxy.h: 
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RenderResource.h"
#include "UnrealClient.h"
#include "Matrix3x4.h"

// Currently we support at most 2 views for each planar reflection, one view per stereo pass
// Must match FPlanarReflectionUniformParameters.
inline const int32 GMaxPlanarReflectionViews = 2;

class UPlanarReflectionComponent;

class FPlanarReflectionRenderTarget : public FTexture, public FRenderTarget
{
public:

	FPlanarReflectionRenderTarget(FIntPoint InSize) :
		Size(InSize)
	{}

	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer
		(
			SF_Bilinear,
			AM_Clamp,
			AM_Clamp,
			AM_Clamp
		);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FPlanarReflectionRenderTarget"))
			.SetExtent(GetSizeXY())
			.SetFormat(PF_FloatRGBA)
			.SetClearValue(FClearValueBinding::Black)
			.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask);

		RenderTargetTextureRHI = TextureRHI = RHICmdList.CreateTexture(Desc);
	}

	virtual FIntPoint GetSizeXY() const { return Size; }

	/** Returns the width of the texture in pixels. */
	virtual uint32 GetSizeX() const
	{
		return Size.X;
	}
	/** Returns the height of the texture in pixels. */
	virtual uint32 GetSizeY() const
	{
		return Size.Y;
	}

	virtual float GetDisplayGamma() const { return 1.0f; }

	virtual FString GetFriendlyName() const override { return TEXT("FPlanarReflectionRenderTarget"); }

	virtual FRHIGPUMask GetGPUMask(FRHICommandListImmediate& RHICmdList) const override
	{
		return ActiveGPUMask;
	}

	// Changes the GPUMask used when updating the reflection capture with multi-GPU.
	void SetActiveGPUMask(FRHIGPUMask InGPUMask)
	{
		check(IsInRenderingThread());
		ActiveGPUMask = InGPUMask;
	}

private:

	FRHIGPUMask ActiveGPUMask; // GPU mask copied from parent render target for multi-GPU
	FIntPoint Size;
};

class FPlanarReflectionSceneProxy
{
public:

	FPlanarReflectionSceneProxy(UPlanarReflectionComponent* Component);

	void UpdateTransform(const FMatrix& NewTransform)
	{

		PlanarReflectionOrigin = NewTransform.TransformPosition(FVector::ZeroVector);
		ReflectionPlane = FPlane(PlanarReflectionOrigin, NewTransform.TransformVector(FVector(0, 0, 1)));

		// Extents of the mesh used to visualize the reflection plane
		const float MeshExtent = 2000.0f;
		FVector LocalExtent(MeshExtent, MeshExtent, DistanceFromPlaneFadeEnd);
		FBox LocalBounds(-LocalExtent, LocalExtent);
		WorldBounds = LocalBounds.TransformBy(NewTransform);

		const FVector XAxis = NewTransform.TransformVector(FVector(1, 0, 0));
		const FVector::FReal XAxisLength = XAxis.Size();
		PlanarReflectionXAxis = FVector4(XAxis / FMath::Max(XAxisLength, UE_DELTA), XAxisLength * MeshExtent);

		const FVector YAxis = NewTransform.TransformVector(FVector(0, 1, 0));
		const FVector::FReal YAxisLength = YAxis.Size();
		PlanarReflectionYAxis = FVector4(YAxis / FMath::Max(YAxisLength, UE_DELTA), YAxisLength * MeshExtent);

		const FMirrorMatrix MirrorMatrix(ReflectionPlane);
		// Using TransposeAdjoint instead of full inverse because we only care about transforming normals
		const FMatrix InverseTransposeMirrorMatrix4x4 = MirrorMatrix.TransposeAdjoint();
		InverseTransposeMirrorMatrix.SetMatrix(InverseTransposeMirrorMatrix4x4);
	}

	void ApplyWorldOffset(const FVector& InOffset)
	{
		WorldBounds = WorldBounds.ShiftBy(InOffset);
		PlanarReflectionOrigin+= InOffset;
		ReflectionPlane = FPlane(PlanarReflectionOrigin, ReflectionPlane /*Normal*/);
	}
	

	FBox WorldBounds;
	bool bIsStereo;
	FPlane ReflectionPlane;
	FVector PlanarReflectionOrigin;
	float DistanceFromPlaneFadeEnd;
	FVector4 PlanarReflectionXAxis;
	FVector4 PlanarReflectionYAxis;
	FVector PlanarReflectionParameters;
	FVector2D PlanarReflectionParameters2;
	int32 PlanarReflectionId;
	float PrefilterRoughness;
	float PrefilterRoughnessDistance;
	FMatrix ProjectionWithExtraFOV[GMaxPlanarReflectionViews];
	FIntRect ViewRect[GMaxPlanarReflectionViews];
	FMatrix3x4 InverseTransposeMirrorMatrix;
	FName OwnerName;

	/** This is specific to a certain view and should actually be stored in FSceneViewState. */
	FPlanarReflectionRenderTarget* RenderTarget;
};
