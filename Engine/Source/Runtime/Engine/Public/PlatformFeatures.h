// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IVideoRecordingSystem;

/**
 * Interface for platform feature modules
 */

/** Defines the interface of a module implementing platform feature collector class. */
class IPlatformFeaturesModule : public IModuleInterface
{
public:

	static IPlatformFeaturesModule& Get()
	{
		static IPlatformFeaturesModule* StaticModule = NULL;
		// first time initialization
		if (!StaticModule)
		{
			const TCHAR* PlatformModuleName = FPlatformMisc::GetPlatformFeaturesModuleName();
			if (PlatformModuleName)
			{
				StaticModule = &FModuleManager::LoadModuleChecked<IPlatformFeaturesModule>(PlatformModuleName);
			}
			else
			{
				// if the platform doesn't care about a platform features module, then use this generic almost empty implementation
				StaticModule = new IPlatformFeaturesModule;
			}
		}

		return *StaticModule;
	}

	ENGINE_API virtual class ISaveGameSystem* GetSaveGameSystem();

	ENGINE_API virtual class IDVRStreamingSystem* GetStreamingSystem();

	ENGINE_API virtual FString GetUniqueAppId();

	ENGINE_API virtual IVideoRecordingSystem* GetVideoRecordingSystem();

	ENGINE_API virtual void SetScreenshotEnableState(bool bEnabled);
};
