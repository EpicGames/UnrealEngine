// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"

#include "PluginBlueprintLibrary.generated.h"


/**
 * A function library of utilities for querying information about plugins.
 */
UCLASS(MinimalAPI)
class UPluginBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Get the list of extra directories that are recursively searched for
	 * plugins (aside from the engine and project plugin directories).
	 *
	 * @return The additional filesystem plugin search paths.
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static TArray<FString> GetAdditionalPluginSearchPaths();

	/**
	 * Get the list of extra directories added by the project that are
	 * recursively searched for plugins.
	 *
	 * @return The additional project filesystem plugin search paths.
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static const TArray<FString>& GetAdditionalProjectPluginSearchPaths();

	/**
	 * Get the names of all enabled plugins.
	 *
	 * @return The names of all enabled plugins.
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static TArray<FString> GetEnabledPluginNames();

	/**
	 * Get the name of the plugin containing an object.
	 *
	 * @param ObjectPath - Path to the object
	 * @param OutPluginName - Name of the plugin containing the object, if found
	 *
	 * @return true if the object is contained within a plugin and the plugin
	 *         name was stored in OutPluginName, or false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static ENGINE_API bool GetPluginNameForObjectPath(
			const FSoftObjectPath& ObjectPath,
			FString& OutPluginName);

	/**
	 * Get the filesystem path to a plugin's descriptor.
	 *
	 * @param PluginName - Name of the plugin
	 * @param OutFilePath - Filesystem path to the plugin's descriptor, if found
	 *
	 * @return true if the named plugin was found and the plugin descriptor
	 *         filesystem path was stored in OutFilePath, or false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static bool GetPluginDescriptorFilePath(
			const FString& PluginName,
			FString& OutFilePath);

	/**
	 * Get the filesystem path to a plugin's base directory.
	 *
	 * @param PluginName - Name of the plugin
	 * @param OutBaseDir - Filesystem path to the plugin's base directory, if found
	 *
	 * @return true if the named plugin was found and the plugin base directory
	 *         filesystem path was stored in OutBaseDir, or false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static bool GetPluginBaseDir(
			const FString& PluginName,
			FString& OutBaseDir);

	/**
	 * Get the filesystem path to a plugin's content directory.
	 *
	 * @param PluginName - Name of the plugin
	 * @param OutContentDir - Filesystem path to the plugin's content directory, if found
	 *
	 * @return true if the named plugin was found and the plugin content
	 *         directory filesystem path was stored in OutContentDir, or false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static ENGINE_API bool GetPluginContentDir(
			const FString& PluginName,
			FString& OutContentDir);

	/**
	 * Get the virtual root path for assets in a plugin.
	 *
	 * @param PluginName - Name of the plugin
	 * @param OutAssetPath - Virtual root path for the plugin's assets, if found
	 *
	 * @return true if the named plugin was found and the plugin's virtual
	 *         root path was stored in OutAssetPath, or false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static bool GetPluginMountedAssetPath(
			const FString& PluginName,
			FString& OutAssetPath);

	/**
	 * Get the version number of a plugin.
	 *
	 * @param PluginName - Name of the plugin
	 * @param OutVersion - Version number of the plugin, if found
	 *
	 * @return true if the named plugin was found and the plugin's version
	 *         number was stored in OutVersion, or false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static bool GetPluginVersion(const FString& PluginName, int32& OutVersion);

	/**
	 * Get the version name of a plugin.
	 *
	 * @param PluginName - Name of the plugin
	 * @param OutVersionName - Version name of the plugin, if found
	 *
	 * @return true if the named plugin was found and the plugin's version
	 *         name was stored in OutVersionName, or false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static bool GetPluginVersionName(
			const FString& PluginName,
			FString& OutVersionName);

	/**
	 * Get the description of a plugin.
	 *
	 * @param PluginName - Name of the plugin
	 * @param OutDescription - Description of the plugin, if found
	 *
	 * @return true if the named plugin was found and the plugin's description
	 *         was stored in OutDescription, or false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static bool GetPluginDescription(
			const FString& PluginName,
			FString& OutDescription);

	/**
	 * Get the editor custom virtual path of a plugin.
	 *
	 * @param PluginName - Name of the plugin
	 * @param OutVirtualPath - Editor custom virtual path of the plugin, if found
	 *
	 * @return true if the named plugin was found and the plugin's editor
	 *         custom virtual path was stored in OutVirtualPath, or false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static bool GetPluginEditorCustomVirtualPath(
			const FString& PluginName,
			FString& OutVirtualPath);

	/**
	 * Determine whether a plugin is mounted.
	 *
	 * @param PluginName - Name of the plugin
	 *
	 * @return true if the named plugin is mounted, or false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Engine Scripting | Plugin Utilities")
	static bool IsPluginMounted(const FString& PluginName);
};
