// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PointLightSceneProxy.h: Point light scene info definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "LocalLightSceneProxy.h"

class UPointLightComponent;
struct FPointLightSceneProxyDesc;

class FPointLightSceneProxy : public FLocalLightSceneProxy
{
public:
	/** The light falloff exponent. */
	float FalloffExponent;

	/** Radius of light source shape */
	float SourceRadius;

	/** Soft radius of light source shape */
	float SoftSourceRadius;

	/** Length of light source shape */
	float SourceLength;

	/** Whether light uses inverse squared falloff. */
	const uint32 bInverseSquared : 1;

	/** Initialization constructor. */
	FPointLightSceneProxy(const UPointLightComponent* Component);
	ENGINE_API FPointLightSceneProxy(const FPointLightSceneProxyDesc& LightDesc);

	//~Begin FLightSceneProxy Interface
	virtual float GetSourceRadius() const override;
	virtual bool IsInverseSquared() const override;
	virtual void GetLightShaderParameters(FLightRenderParameters& LightParameters, uint32 Flags=0) const override;
	virtual bool GetWholeSceneProjectedShadowInitializer(const FSceneViewFamily& ViewFamily, TArray<FWholeSceneProjectedShadowInitializer, TInlineAllocator<6> >& OutInitializers) const override;
	//~End FLightSceneProxy Interface

	//~Begin FLocalLightSceneProxy Interface
	virtual FVector GetPerObjectProjectedShadowProjectionPoint(const FBoxSphereBounds& SubjectBounds) const override;
	//~End FLocalLightSceneProxy Interface
};