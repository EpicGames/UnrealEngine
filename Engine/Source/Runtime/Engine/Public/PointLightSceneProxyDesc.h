// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LocalLightSceneProxyDesc.h"

struct FPointLightSceneProxyDesc : public FLocalLightSceneProxyDesc
{
	typedef FLocalLightSceneProxyDesc Super;

	FPointLightSceneProxyDesc() = default;
	virtual ~FPointLightSceneProxyDesc() = default;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** The light falloff exponent. */
	float LightFalloffExponent = 0.0f;

	/** Radius of light source shape */
	float SourceRadius = 0.0f;

	/** Soft radius of light source shape */
	float SoftSourceRadius = 0.0f;

	/** Length of light source shape */
	float SourceLength = 0.0f;

	/** Whether light uses inverse squared falloff. */
	bool bUseInverseSquaredFalloff = false;
};