// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PrecomputedLightVolume.h: Declarations for precomputed light volumes.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Math/SHMath.h"
#include "Math/GenericOctreePublic.h"
#include "Math/GenericOctree.h"

class FSceneInterface;

/** Incident radiance stored for a point. */
template <int32 SHOrder>
class TVolumeLightingSample
{
public:
	/** World space position of the sample. */
	FVector3f Position;	// LWC_TODO: Update to FVector once FVector4 supports doubles
	/** World space radius that determines how far the sample can be interpolated. */
	float Radius;

	/** 
	 * Incident lighting at the sample position. 
	 * For high quality lightmaps, only diffuse GI and static lights is represented here.
	 * For low quality lightmaps, diffuse GI and static and stationary light direct lighting is contained in the sample, with the exception of the stationary directional light. 
	 * @todo - quantize to reduce memory cost
	 */
	TSHVectorRGB<SHOrder> Lighting;

	/** BentNormal occlusion of the sky, packed into an FColor.  This is only valid in the high quality lightmap data. */
	FColor PackedSkyBentNormal;

	/** Shadow factor for the stationary directional light. */
	float DirectionalLightShadowing;

	TVolumeLightingSample() :
		Position(FVector(0, 0, 0)),
		Radius(0),
		PackedSkyBentNormal(FColor(127, 127, 255)),
		DirectionalLightShadowing(1)
	{
	}

	TVolumeLightingSample(const TVolumeLightingSample<2>& Other);
	TVolumeLightingSample(const TVolumeLightingSample<3>& Other);

	void SetPackedSkyBentNormal(FVector InSkyBentNormal)
	{
		PackedSkyBentNormal = FColor(FMath::TruncToInt((InSkyBentNormal.X * .5f + .5f) * 255.0f), FMath::TruncToInt((InSkyBentNormal.Y * .5f + .5f) * 255.0f), FMath::TruncToInt((InSkyBentNormal.Z * .5f + .5f) * 255.0f));
	}

	inline FVector GetSkyBentNormalUnpacked() const
	{
		return FVector(PackedSkyBentNormal.R / 255.0f * 2.0f - 1.0f, PackedSkyBentNormal.G / 255.0f * 2.0f - 1.0f, PackedSkyBentNormal.B / 255.0f * 2.0f - 1.0f);
	}

	friend FArchive& operator<<(FArchive& Ar, TVolumeLightingSample<SHOrder>& Sample);
};

typedef TVolumeLightingSample<3> FVolumeLightingSample;
typedef TVolumeLightingSample<2> FVolumeLightingSample2Band;
struct FLightVolumeOctreeSemantics
{
	enum { MaxElementsPerLeaf = 4 };
	enum { MaxNodeDepth = 12 };

	/** Using the heap allocator instead of an inline allocator to trade off add/remove performance for memory. */
	typedef FDefaultAllocator ElementAllocator;

	// LWC_TODO: Update to return FVector::FReal once FVector4 supports doubles
	inline static const float* GetBoundingBox(const FVolumeLightingSample& Sample)
	{
		FPlatformMisc::Prefetch( &Sample, PLATFORM_CACHE_LINE_SIZE );
		// here we require that the position and radius are contiguous in memory
		static_assert(STRUCT_OFFSET(FVolumeLightingSample, Position) + 3 * sizeof(float) == STRUCT_OFFSET(FVolumeLightingSample, Radius), "FVolumeLightingSample radius must follow position.");
		return &Sample.Position.X;
	}

	static void SetElementId(const FVolumeLightingSample& Element, FOctreeElementId2 Id)
	{
	}

	inline static void ApplyOffset(FVolumeLightingSample& Element, FVector Offset)
	{
		Element.Position+= FVector3f(Offset);	//LWC_TODO: Precision loss
	}
};

typedef TOctree2<FVolumeLightingSample, FLightVolumeOctreeSemantics> FLightVolumeOctree;

/** Set of volume lighting samples belonging to one streaming level, which can be queried about the lighting at a given position. */
class FPrecomputedLightVolumeData
{
public:

	ENGINE_API FPrecomputedLightVolumeData();
	~FPrecomputedLightVolumeData();

	friend FArchive& operator<<(FArchive& Ar, FPrecomputedLightVolumeData& Volume);
	friend FArchive& operator<<(FArchive& Ar, FPrecomputedLightVolumeData*& Volume);

	/** Frees any previous samples, prepares the volume to have new samples added. */
	ENGINE_API void Initialize(const FBox& NewBounds);

	/** Adds a high quality lighting sample. */
	ENGINE_API void AddHighQualityLightingSample(const FVolumeLightingSample& NewHighQualitySample);

	/** Adds a low quality lighting sample. */
	ENGINE_API void AddLowQualityLightingSample(const FVolumeLightingSample& NewLowQualitySample);

	/** Shrinks the octree and updates memory stats. */
	ENGINE_API void FinalizeSamples();

	/** Invalidates anything produced by the last lighting build. */
	ENGINE_API void InvalidateLightingCache();

	SIZE_T GetAllocatedBytes() const;

	bool IsInitialized() const
	{
		return bInitialized;
	}

	FBox& GetBounds()
	{
		return Bounds;
	}

private:

	bool bInitialized;
	FBox Bounds;

	/** Octree containing lighting samples to be used with high quality lightmaps. */
	FLightVolumeOctree HighQualityLightmapOctree;

	/** Octree containing lighting samples to be used with low quality lightmaps. */
	FLightVolumeOctree LowQualityLightmapOctree;

	friend class FPrecomputedLightVolume;
};


/** Set of volume lighting samples belonging to one streaming level, which can be queried about the lighting at a given position. */
class FPrecomputedLightVolume
{
public:

	ENGINE_API FPrecomputedLightVolume();
	~FPrecomputedLightVolume();

	ENGINE_API void AddToScene(class FSceneInterface* Scene, class UMapBuildDataRegistry* Registry, FGuid LevelBuildDataId);

	ENGINE_API void RemoveFromScene(FSceneInterface* Scene);
	
	ENGINE_API void SetData(const FPrecomputedLightVolumeData* NewData, FSceneInterface* Scene);

	/** Interpolates incident radiance to Position. */
	ENGINE_API void InterpolateIncidentRadiancePoint(
		const FVector& Position, 
		float& AccumulatedWeight,
		float& AccumulatedDirectionalLightShadowing,
		FSHVectorRGB3& AccumulatedIncidentRadiance,
		FVector& SkyBentNormal) const;
	
	/** Interpolates incident radiance to Position. */
	ENGINE_API void InterpolateIncidentRadianceBlock(
		const FBoxCenterAndExtent& BoundingBox, 
		const FIntVector& QueryCellDimensions,
		const FIntVector& DestCellDimensions,
		const FIntVector& DestCellPosition,
		TArray<float>& AccumulatedWeights,
		TArray<FSHVectorRGB2>& AccumulatedIncidentRadiance) const;

	ENGINE_API void DebugDrawSamples(class FPrimitiveDrawInterface* PDI, bool bDrawDirectionalShadowing) const;

	ENGINE_API bool IntersectBounds(const FBoxSphereBounds& InBounds) const;

	SIZE_T GetAllocatedBytes() const;

	bool IsAddedToScene() const
	{
		return bAddedToScene;
	}

	double GetNodeLevelExtent(int32 Level) const
	{
		return OctreeForRendering->GetNodeLevelExtent(Level);
	}

	ENGINE_API void ApplyWorldOffset(const FVector& InOffset);

	// temporary, for ES 3.1 preview verification.
	inline bool IsUsingHighQualityLightMap() const
	{
		return OctreeForRendering == &Data->HighQualityLightmapOctree;
	}

	const FPrecomputedLightVolumeData* Data;

private:

	bool bAddedToScene;

	/** Octree used to accelerate interpolation searches. */
	const FLightVolumeOctree* OctreeForRendering;

	/** Offset from world origin. Non-zero only when world origin was rebased */
	FVector WorldOriginOffset;
};
