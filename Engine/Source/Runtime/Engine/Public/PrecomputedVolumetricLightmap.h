// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PrecomputedVolumetricLightmap.h: Declarations for precomputed volumetric lightmap.
=============================================================================*/

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Math/SHMath.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "PixelFormat.h"
#include "Math/PackedVector.h"
#include "RHI.h"
#include "RenderResource.h"
#include "ShaderParameterStruct.h"
#include "GlobalShader.h"
#include "UObject/WeakObjectPtr.h"

class FSceneInterface;

class FVolumetricLightmapDataLayer : public FResourceBulkDataInterface
{
public:

	FVolumetricLightmapDataLayer() :
		DataSize(0),
		Format(PF_Unknown),
		bNeedsCPUAccess(false)
	{}

	friend FArchive& operator<<(FArchive& Ar, FVolumetricLightmapDataLayer& Volume);

	virtual const void* GetResourceBulkData() const override
	{
		return Data.GetData();
	}

	virtual uint32 GetResourceBulkDataSize() const override
	{
		return Data.Num() * Data.GetTypeSize();
	}

	virtual void Discard() override
	{
		if (!bNeedsCPUAccess)
		{
			Data.Empty();
		}
	}

	void Resize(int32 NewSize)
	{
		Data.Empty(NewSize);
		Data.AddZeroed(NewSize);
		DataSize = NewSize;
	}

	ENGINE_API void CreateTexture(FRHICommandListBase& RHICmdList, FIntVector Dimensions);
	ENGINE_API void CreateTargetTexture(FRHICommandListBase& RHICmdList, FIntVector Dimensions);
	ENGINE_API void CreateUAV(FRHICommandListBase& RHICmdList);
	
	UE_DEPRECATED(5.7, "CreateTexture now requires a command list.")
	ENGINE_API void CreateTexture(FIntVector Dimensions);
	UE_DEPRECATED(5.7, "CreateTargetTexture now requires a command list.")
	ENGINE_API void CreateTargetTexture(FIntVector Dimensions);

	TArray<uint8> Data;
	// Stored redundantly for stats after Data has been discarded
	int32 DataSize;
	EPixelFormat Format;
	FTextureRHIRef Texture;
	FUnorderedAccessViewRHIRef UAV;

	bool bNeedsCPUAccess;
};

struct FVolumetricLightmapBasicBrickDataLayers
{
	FVolumetricLightmapDataLayer AmbientVector;
	FVolumetricLightmapDataLayer SHCoefficients[6];
	FVolumetricLightmapDataLayer SkyBentNormal;
	FVolumetricLightmapDataLayer DirectionalLightShadowing;
};

class FVolumetricLightmapBrickData : public FVolumetricLightmapBasicBrickDataLayers
{
public:

	ENGINE_API int32 GetMinimumVoxelSize() const;

	void ReleaseRHI()
	{
		AmbientVector.Texture.SafeRelease();

		for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
		{
			SHCoefficients[i].Texture.SafeRelease();
		}

		SkyBentNormal.Texture.SafeRelease();
		DirectionalLightShadowing.Texture.SafeRelease();

		AmbientVector.UAV.SafeRelease();

		for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
		{
			SHCoefficients[i].UAV.SafeRelease();
		}

		SkyBentNormal.UAV.SafeRelease();
		DirectionalLightShadowing.UAV.SafeRelease();
	}

	SIZE_T GetAllocatedBytes() const
	{
		SIZE_T NumBytes = AmbientVector.DataSize + SkyBentNormal.DataSize + DirectionalLightShadowing.DataSize;

		for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
		{
			NumBytes += SHCoefficients[i].DataSize;
		}

		return NumBytes;
	}

	void SetNeedsCPUAccess(bool InAccess)
	{
		AmbientVector.bNeedsCPUAccess = InAccess;

		for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
		{
			SHCoefficients[i].bNeedsCPUAccess = InAccess;
		}

		SkyBentNormal.bNeedsCPUAccess = InAccess;
		DirectionalLightShadowing.bNeedsCPUAccess = InAccess;
	}
};

/** 
 * Data for a Volumetric Lightmap, built during import from Lightmass.
 * Its lifetime is managed by UMapBuildDataRegistry. 
 */
class FPrecomputedVolumetricLightmapData : public FRenderResource
{
public:

	ENGINE_API FPrecomputedVolumetricLightmapData();
	ENGINE_API virtual ~FPrecomputedVolumetricLightmapData();

	friend FArchive& operator<<(FArchive& Ar, FPrecomputedVolumetricLightmapData& Volume);
	friend FArchive& operator<<(FArchive& Ar, FPrecomputedVolumetricLightmapData*& Volume);

	ENGINE_API void InitializeOnImport(const FBox& NewBounds, int32 InBrickSize);
	ENGINE_API void FinalizeImport();

	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;

	ENGINE_API void InitRHIForSubLevelResources(FRHICommandListBase& RHICmdList);
	UE_DEPRECATED(5.7, "InitRHIForSubLevelResources now requires a command list.")
	ENGINE_API void InitRHIForSubLevelResources();
	ENGINE_API void ReleaseRHIForSubLevelResources();

	UE_DEPRECATED(5.8, "HandleDataMovementInAtlas now requires a command list.")
	void HandleDataMovementInAtlas(int32 OldOffset, int32 NewOffset)
	{
		HandleDataMovementInAtlas(FRHICommandListImmediate::Get(), OldOffset, NewOffset);
	}

	ENGINE_API void HandleDataMovementInAtlas(FRHIComputeCommandList& RHICmdList, int32 OldOffset, int32 NewOffset);
	
	ENGINE_API void AddToSceneData(FRHICommandListImmediate& RHICmdList, FPrecomputedVolumetricLightmapData* SceneData);
	
	UE_DEPRECATED(5.8, "AddToSceneData now requires a command list.")
	void AddToSceneData(FPrecomputedVolumetricLightmapData* SceneData)
	{
		AddToSceneData(FRHICommandListImmediate::Get(), SceneData);
	}

	ENGINE_API void RemoveFromSceneData(FRHICommandListImmediate& RHICmdList, FPrecomputedVolumetricLightmapData* SceneData, int32 PersistentLevelBrickDataBaseOffset);

	UE_DEPRECATED(5.8, "RemoveFromSceneData now requires a command list.")
	void RemoveFromSceneData(FPrecomputedVolumetricLightmapData* SceneData, int32 PersistentLevelBrickDataBaseOffset)
	{
		RemoveFromSceneData(FRHICommandListImmediate::Get(), SceneData, PersistentLevelBrickDataBaseOffset);
	}

	SIZE_T GetAllocatedBytes() const;

	const FBox& GetBounds() const
	{
		return Bounds;
	}

	FBox Bounds;

	bool bTransient;

	FIntVector IndirectionTextureDimensions;
	FVolumetricLightmapDataLayer IndirectionTexture;

	int32 BrickSize;
	FIntVector BrickDataDimensions;
	FVolumetricLightmapBrickData BrickData;

	/**
	 * Position data for sub level streaming
	 */
	// Brick positions in the persistent level's indirection texture
	TResourceArray<FIntVector> SubLevelBrickPositions;
	TResourceArray<FColor> IndirectionTextureOriginalValues;

	/**
	 * Runtime data for sub level streaming
	 */
	FBufferRHIRef SubLevelBrickPositionsBuffer;
	FShaderResourceViewRHIRef SubLevelBrickPositionsSRV;

	FBufferRHIRef IndirectionTextureOriginalValuesBuffer;
	FShaderResourceViewRHIRef IndirectionTextureOriginalValuesSRV;

	int32 BrickDataBaseOffsetInAtlas;
	TArray<FPrecomputedVolumetricLightmapData*> SceneDataAdded;

	// CPU indirection table for mobile path
	TArray<uint8> CPUSubLevelIndirectionTable;
	TArray<FPrecomputedVolumetricLightmapData*> CPUSubLevelBrickDataList;

private:

	friend class FPrecomputedVolumetricLightmap;
};


/** 
 * Represents the Volumetric Lightmap for a specific ULevel.  
 */
class FPrecomputedVolumetricLightmap
{
public:

	ENGINE_API FPrecomputedVolumetricLightmap();
	ENGINE_API ~FPrecomputedVolumetricLightmap();

	ENGINE_API void AddToScene(class FSceneInterface* Scene, class UMapBuildDataRegistry* Registry, FGuid LevelBuildDataId, bool bIsPersistentLevel);
	ENGINE_API void AddToScene(class FSceneInterface* Scene, class UMapBuildDataRegistry* Registry, FPrecomputedVolumetricLightmapData* NewData, bool bIsPersistentLevel);

	ENGINE_API void RemoveFromScene(FSceneInterface* Scene);
	
	ENGINE_API void SetData(FRHICommandListBase& RHICmdList, FPrecomputedVolumetricLightmapData* NewData, FSceneInterface* Scene);

	bool IsAddedToScene() const
	{
		return bAddedToScene;
	}

	ENGINE_API void ApplyWorldOffset(const FVector& InOffset);

	// Owned by rendering thread
	// ULevel's MapBuildData GC-visible property guarantees that the FPrecomputedVolumetricLightmapData will not be deleted during the lifetime of FPrecomputedVolumetricLightmap.
	FPrecomputedVolumetricLightmapData* Data;

private:

	bool bAddedToScene;

	TWeakObjectPtr<UMapBuildDataRegistry> SourceRegistry;

	/** Offset from world origin. Non-zero only when world origin was rebased */
	FVector WorldOriginOffset;
};

template<typename T>
inline FLinearColor ConvertToLinearColor(T InColor)
{
	return FLinearColor(InColor);
};

template<typename T>
inline T ConvertFromLinearColor(const FLinearColor& InColor)
{
	return T(InColor);
};

template<>
inline FLinearColor ConvertToLinearColor<FColor>(FColor InColor)
{
	return InColor.ReinterpretAsLinear();
};

template<>
inline FColor ConvertFromLinearColor<FColor>(const FLinearColor& InColor)
{
	return InColor.QuantizeRound();
};

template<>
inline FLinearColor ConvertToLinearColor<FFloat3Packed>(FFloat3Packed InColor)
{
	return InColor.ToLinearColor();
};

template<>
inline FFloat3Packed ConvertFromLinearColor<FFloat3Packed>(const FLinearColor& InColor)
{
	return FFloat3Packed(InColor);
};

template<>
inline FLinearColor ConvertToLinearColor<FFixedRGBASigned8>(FFixedRGBASigned8 InColor)
{
	return InColor.ToLinearColor();
};

template<>
inline FFixedRGBASigned8 ConvertFromLinearColor<FFixedRGBASigned8>(const FLinearColor& InColor)
{
	return FFixedRGBASigned8(InColor);
};

template<>
inline uint8 ConvertFromLinearColor<uint8>(const FLinearColor& InColor)
{
	return (uint8)FMath::Clamp<int32>(FMath::RoundToInt(InColor.R * MAX_uint8), 0, MAX_uint8);
};

template<>
inline FLinearColor ConvertToLinearColor<uint8>(uint8 InColor)
{
	const float Scale = 1.0f / MAX_uint8;
	return FLinearColor(InColor * Scale, 0, 0, 0);
};

inline static const float GPointFilteringThreshold = .001f;

template<typename VoxelDataType>
FLinearColor FilteredVolumeLookup(FVector Coordinate, FIntVector DataDimensions, const VoxelDataType* Data)
{
	FVector CoordinateFraction(FMath::Frac(Coordinate.X), FMath::Frac(Coordinate.Y), FMath::Frac(Coordinate.Z));
	FIntVector FilterNeighborSize(CoordinateFraction.X > GPointFilteringThreshold ? 2 : 1, CoordinateFraction.Y > GPointFilteringThreshold ? 2 : 1, CoordinateFraction.Z > GPointFilteringThreshold ? 2 : 1);
	FIntVector CoordinateInt000(Coordinate);

	FLinearColor FilteredValue(0, 0, 0, 0);
	FVector FilterWeight(1.0f, 1.0f, 1.0f);

	for (int32 Z = 0; Z < FilterNeighborSize.Z; Z++)
	{
		if (FilterNeighborSize.Z > 1)
		{
			FilterWeight.Z = (Z == 0 ? 1.0f - CoordinateFraction.Z : CoordinateFraction.Z);
		}

		for (int32 Y = 0; Y < FilterNeighborSize.Y; Y++)
		{
			if (FilterNeighborSize.Y > 1)
			{
				FilterWeight.Y = (Y == 0 ? 1.0f - CoordinateFraction.Y : CoordinateFraction.Y);
			}

			for (int32 X = 0; X < FilterNeighborSize.X; X++)
			{
				if (FilterNeighborSize.X > 1)
				{
					FilterWeight.X = (X == 0 ? 1.0f - CoordinateFraction.X : CoordinateFraction.X);
				}

				FIntVector CoordinateInt = CoordinateInt000 + FIntVector(X, Y, Z);
				CoordinateInt.X = FMath::Clamp(CoordinateInt.X, 0, DataDimensions.X - 1);
				CoordinateInt.Y = FMath::Clamp(CoordinateInt.Y, 0, DataDimensions.Y - 1);
				CoordinateInt.Z = FMath::Clamp(CoordinateInt.Z, 0, DataDimensions.Z - 1);
				const int32 LinearIndex = ((CoordinateInt.Z * DataDimensions.Y) + CoordinateInt.Y) * DataDimensions.X + CoordinateInt.X;

				FilteredValue += ConvertToLinearColor<VoxelDataType>(Data[LinearIndex]) * FilterWeight.X * FilterWeight.Y * FilterWeight.Z;
			}
		}
	}

	return FilteredValue;
}

template<typename VoxelDataType>
VoxelDataType FilteredVolumeLookupReconverted(FVector Coordinate, FIntVector DataDimensions, const VoxelDataType* Data)
{
	FVector CoordinateFraction(FMath::Frac(Coordinate.X), FMath::Frac(Coordinate.Y), FMath::Frac(Coordinate.Z));
	FIntVector FilterNeighborSize(CoordinateFraction.X > GPointFilteringThreshold ? 2 : 1, CoordinateFraction.Y > GPointFilteringThreshold ? 2 : 1, CoordinateFraction.Z > GPointFilteringThreshold ? 2 : 1);

	if (FilterNeighborSize.X == 1 && FilterNeighborSize.Y == 1 && FilterNeighborSize.Z == 1)
	{
		FIntVector CoordinateInt000(Coordinate);
		const int32 LinearIndex = ((CoordinateInt000.Z * DataDimensions.Y) + CoordinateInt000.Y) * DataDimensions.X + CoordinateInt000.X;
		return Data[LinearIndex];
	}
	else
	{
		FLinearColor FilteredValue = FilteredVolumeLookup<VoxelDataType>(Coordinate, DataDimensions, Data);
		return ConvertFromLinearColor<VoxelDataType>(FilteredValue);
	}
}

template<typename VoxelDataType>
VoxelDataType NearestVolumeLookup(FVector Coordinate, FIntVector DataDimensions, const VoxelDataType* Data)
{
	FIntVector NearestCoordinateInt(FMath::RoundToInt(Coordinate.X), FMath::RoundToInt(Coordinate.Y), FMath::RoundToInt(Coordinate.Z));
	const int32 LinearIndex = ((NearestCoordinateInt.Z * DataDimensions.Y) + NearestCoordinateInt.Y) * DataDimensions.X + NearestCoordinateInt.X;
	return Data[LinearIndex];
}

extern ENGINE_API FVector ComputeIndirectionCoordinate(FVector LookupPosition, const FBox& VolumeBounds, FIntVector IndirectionTextureDimensions);

extern ENGINE_API void SampleIndirectionTexture(
	FVector IndirectionDataSourceCoordinate,
	FIntVector IndirectionTextureDimensions,
	const uint8* IndirectionTextureData,
	FIntVector& OutIndirectionBrickOffset,
	int32& OutIndirectionBrickSize);

extern ENGINE_API void SampleIndirectionTextureWithSubLevel(
	FVector IndirectionDataSourceCoordinate,
	FIntVector IndirectionTextureDimensions,
	const uint8* IndirectionTextureData,
	const TArray<uint8>& CPUSubLevelIndirectionTable,
	FIntVector& OutIndirectionBrickOffset,
	int32& OutIndirectionBrickSize,
	int32& OutSubLevelIndex);

extern ENGINE_API FVector ComputeBrickTextureCoordinate(
	FVector IndirectionDataSourceCoordinate,
	FIntVector IndirectionBrickOffset,
	int32 IndirectionBrickSize,
	int32 BrickSize);

class FRemoveSubLevelBricksCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRemoveSubLevelBricksCS);
	SHADER_USE_PARAMETER_STRUCT(FRemoveSubLevelBricksCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumBricks)
		SHADER_PARAMETER(uint32, PersistentLevelBrickDataBaseOffset)
		SHADER_PARAMETER_SRV(Buffer<uint>, SubLevelBrickPositions)
		SHADER_PARAMETER_SRV(Buffer<uint4>, IndirectionTextureOriginalValues)
		SHADER_PARAMETER_UAV(RWTexture3D<uint4>, IndirectionTexture)
	END_SHADER_PARAMETER_STRUCT()
};

class FCopyResidentBricksCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyResidentBricksCS);
	SHADER_USE_PARAMETER_STRUCT(FCopyResidentBricksCS, FGlobalShader);

	class FHasSkyBentNormal : SHADER_PERMUTATION_BOOL("HAS_SKY_BENT_NORMAL");
	using FPermutationDomain = TShaderPermutationDomain<FHasSkyBentNormal>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, StartPosInOldVolume)
		SHADER_PARAMETER(uint32, StartPosInNewVolume)
		SHADER_PARAMETER_TEXTURE(Texture3D, AmbientVector)
		SHADER_PARAMETER_TEXTURE(Texture3D, SkyBentNormal)
		SHADER_PARAMETER_TEXTURE(Texture3D, DirectionalLightShadowing)

		SHADER_PARAMETER_UAV(RWTexture3D<float3>, OutAmbientVector)
		SHADER_PARAMETER_UAV(RWTexture3D<float4>, OutSkyBentNormal)
		SHADER_PARAMETER_UAV(RWTexture3D<float>, OutDirectionalLightShadowing)
	END_SHADER_PARAMETER_STRUCT()
};

class FCopyResidentBrickSHCoefficientsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyResidentBrickSHCoefficientsCS);
	SHADER_USE_PARAMETER_STRUCT(FCopyResidentBrickSHCoefficientsCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, StartPosInOldVolume)
		SHADER_PARAMETER(uint32, StartPosInNewVolume)
		SHADER_PARAMETER_TEXTURE(Texture3D, SHCoefficients)
		SHADER_PARAMETER_UAV(RWTexture3D<float4>, OutSHCoefficients)
	END_SHADER_PARAMETER_STRUCT()
};

class FPatchIndirectionTextureCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPatchIndirectionTextureCS);
	SHADER_USE_PARAMETER_STRUCT(FPatchIndirectionTextureCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumBricks)
		SHADER_PARAMETER(uint32, StartPosInNewVolume)
		SHADER_PARAMETER_SRV(Buffer<uint>, SubLevelBrickPositions)
		SHADER_PARAMETER_UAV(RWTexture3D<uint4>, IndirectionTexture)
	END_SHADER_PARAMETER_STRUCT()
};

class FMoveWholeIndirectionTextureCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMoveWholeIndirectionTextureCS);
	SHADER_USE_PARAMETER_STRUCT(FMoveWholeIndirectionTextureCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumBricks)
		SHADER_PARAMETER(uint32, StartPosInOldVolume)
		SHADER_PARAMETER(uint32, StartPosInNewVolume)
		SHADER_PARAMETER_TEXTURE(Texture3D<uint4>, OldIndirectionTexture)
		SHADER_PARAMETER_UAV(RWTexture3D<uint4>, IndirectionTexture)
	END_SHADER_PARAMETER_STRUCT()
};

struct FVolumetricLightmapBrickTextureSet : public FVolumetricLightmapBasicBrickDataLayers
{
	FIntVector BrickDataDimensions;

	template<class VolumetricLightmapBrickDataType> // Can be either FVolumetricLightmapBrickData or FVolumetricLightmapBrickTextureSet
	void Initialize(FIntVector InBrickDataDimensions, VolumetricLightmapBrickDataType& BrickData)
	{
		FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

		BrickDataDimensions = InBrickDataDimensions;

		AmbientVector.Format = BrickData.AmbientVector.Format;
		SkyBentNormal.Format = BrickData.SkyBentNormal.Format;
		DirectionalLightShadowing.Format = BrickData.DirectionalLightShadowing.Format;

		for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
		{
			SHCoefficients[i].Format = BrickData.SHCoefficients[i].Format;
		}

		AmbientVector.CreateTargetTexture(RHICmdList, BrickDataDimensions);
		AmbientVector.CreateUAV(RHICmdList);

		for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
		{
			SHCoefficients[i].CreateTargetTexture(RHICmdList, BrickDataDimensions);
			SHCoefficients[i].CreateUAV(RHICmdList);
		}

		if (BrickData.SkyBentNormal.Texture.IsValid())
		{
			SkyBentNormal.CreateTargetTexture(RHICmdList, BrickDataDimensions);
			SkyBentNormal.CreateUAV(RHICmdList);
		}

		DirectionalLightShadowing.CreateTargetTexture(RHICmdList, BrickDataDimensions);
		DirectionalLightShadowing.CreateUAV(RHICmdList);
	}

	void Release()
	{
		AmbientVector.Texture.SafeRelease();
		for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
		{
			SHCoefficients[i].Texture.SafeRelease();
		}
		SkyBentNormal.Texture.SafeRelease();
		DirectionalLightShadowing.Texture.SafeRelease();

		AmbientVector.UAV.SafeRelease();
		for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
		{
			SHCoefficients[i].UAV.SafeRelease();
		}
		SkyBentNormal.UAV.SafeRelease();
		DirectionalLightShadowing.UAV.SafeRelease();
	}
};

class FVolumetricLightmapBrickAtlas : public FRenderResource
{
public:
	ENGINE_API FVolumetricLightmapBrickAtlas();

	FVolumetricLightmapBrickTextureSet TextureSet;

	ENGINE_API virtual void ReleaseRHI() override;

	struct Allocation
	{
		// The data being allocated, as an identifier for the entry
		class FPrecomputedVolumetricLightmapData* Data = nullptr;

		int32 Size = 0;
		int32 StartOffset = 0;
	};

	TArray<Allocation> Allocations;

	ENGINE_API void Insert(FRHICommandListImmediate& RHICmdList, int32 Index, FPrecomputedVolumetricLightmapData* Data);
	ENGINE_API void Remove(FRHICommandListImmediate& RHICmdList, FPrecomputedVolumetricLightmapData* Data);

	UE_DEPRECATED(5.8, "Insert now requires a command list.")
	void Insert(int32 Index, FPrecomputedVolumetricLightmapData* Data)
	{
		Insert(FRHICommandListImmediate::Get(), Index, Data);
	}

	UE_DEPRECATED(5.8, "Remove now requires a command list.")
	void Remove(FPrecomputedVolumetricLightmapData* Data)
	{
		Remove(FRHICommandListImmediate::Get(), Data);
	}

private:
	bool bInitialized;
	int32 PaddedBrickSize;
};

extern ENGINE_API TGlobalResource<FVolumetricLightmapBrickAtlas> GVolumetricLightmapBrickAtlas;