// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

class FSceneInterface;
class UWorld;
class IBulkDataIORequest;
class FPrecomputedVolumetricLightmap;

struct FVolumetricLightMapGridDesc;
struct FVolumetricLightMapGridCell;
class FVolumetricLightmapGridStreamingManager;
class UMapBuildDataRegistry;
class FPrecomputedVolumetricLightmapData;

class FVolumetricLightmapGridManager
{	
public:

	FVolumetricLightmapGridManager(UWorld* InWorld, FVolumetricLightMapGridDesc* Grid);
	~FVolumetricLightmapGridManager();

	void UpdateBounds(const FBox& Bounds);
	int32 ProcessRequests();
	void RemoveFromScene(class FSceneInterface* InScene);
	int32 WaitForPendingRequest(float Timelimit);
	int32 GetNumPendingRequests();

private:
	
	friend class FVolumetricLightmapGridStreamingManager;

	UWorld* World = nullptr;
	UMapBuildDataRegistry* Registry = nullptr;
	FVolumetricLightMapGridDesc* Grid = nullptr;
	
	struct FCellRequest
	{
		enum EStatus
		{
			Created,
			Requested,
			Ready,
			Cancelled,
			Done
		};

		EStatus Status = Created;
		IBulkDataIORequest* IORequest = nullptr;
		TNotNull<FVolumetricLightMapGridCell*> Cell;
		FPrecomputedVolumetricLightmapData* Data = nullptr;
	};	

	struct FLoadedCellData
	{
		FPrecomputedVolumetricLightmapData* Data = nullptr;
		TNotNull<FVolumetricLightMapGridCell*> Cell;
		FPrecomputedVolumetricLightmap* Lightmap = nullptr;
	};

	void RequestVolumetricLightMapCell(FCellRequest& CellRequest);
	void ReleaseCellData(FLoadedCellData* LoadedCell, FSceneInterface* InScene);
	
	TArray<FCellRequest>	PendingCellRequests;
	TMap<FVolumetricLightMapGridCell*, FLoadedCellData> LoadedCells;
	
	FBox Bounds;

	FCriticalSection PendingRequestsCriticalSection;

	TUniquePtr<FVolumetricLightmapGridStreamingManager> StreamingManager;
};

