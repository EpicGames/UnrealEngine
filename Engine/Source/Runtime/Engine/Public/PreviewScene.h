// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PreviewScene.h: Preview scene definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Engine/World.h"

class FSceneInterface;

/**
 * Encapsulates a simple scene setup for preview or thumbnail rendering.
 */
class FPreviewScene : public FGCObject
{
public:
	struct ConstructionValues
	{
		ConstructionValues()
			: LightRotation(-40.f,-67.5f,0.f)
			, SkyBrightness(1.0f)
			, LightBrightness(UE_PI)
			, bDefaultLighting(true)
			, bAllowAudioPlayback(false)
			, bForceMipsResident(true)
			, bCreatePhysicsScene(true)
			, bShouldSimulatePhysics(false)
			, bTransactional(true)
			, bEditor(true)
			, bForceUseMovementComponentInNonGameWorld(false)
			, bAllowLumenPrimitiveTrackingInPreviewWorld(false)
		{
		}

		FRotator LightRotation;
		float SkyBrightness;
		float LightBrightness;
		uint32 bDefaultLighting : 1;
		uint32 bAllowAudioPlayback:1;
		uint32 bForceMipsResident:1;
		uint32 bCreatePhysicsScene:1;
		uint32 bShouldSimulatePhysics:1;
		uint32 bTransactional:1;
		uint32 bEditor:1;
		uint32 bForceUseMovementComponentInNonGameWorld:1;
		uint32 bAllowLumenPrimitiveTrackingInPreviewWorld:1;

		TSubclassOf<class AGameModeBase> DefaultGameMode;
		class UGameInstance* OwningGameInstance = nullptr;

		ConstructionValues& SetCreateDefaultLighting(const bool bDefault) { bDefaultLighting = bDefault; return *this; }
		ConstructionValues& SetLightRotation(const FRotator& Rotation) { LightRotation = Rotation; return *this; }
		ConstructionValues& SetSkyBrightness(const float Brightness) { SkyBrightness = Brightness; return *this; }
		ConstructionValues& SetLightBrightness(const float Brightness) { LightBrightness = Brightness; return *this; }

		ConstructionValues& AllowAudioPlayback(const bool bAllow) { bAllowAudioPlayback = bAllow; return *this; }
		ConstructionValues& SetForceMipsResident(const bool bForce) { bForceMipsResident = bForce; return *this; }
		ConstructionValues& SetCreatePhysicsScene(const bool bCreate) { bCreatePhysicsScene = bCreate; return *this; }
		ConstructionValues& ShouldSimulatePhysics(const bool bInShouldSimulatePhysics) { bShouldSimulatePhysics = bInShouldSimulatePhysics; return *this; }
		ConstructionValues& SetTransactional(const bool bInTransactional) { bTransactional = bInTransactional; return *this; }
		ConstructionValues& SetEditor(const bool bInEditor) { bEditor = bInEditor; return *this; }
		ConstructionValues& ForceUseMovementComponentInNonGameWorld(const bool bInForceUseMovementComponentInNonGameWorld) { bForceUseMovementComponentInNonGameWorld = bInForceUseMovementComponentInNonGameWorld; return *this; }
		ConstructionValues& AllowLumenPrimitiveTrackingInPreviewWorld(const bool bInAllowLumenPrimitiveTrackingInPreviewWorld) { bAllowLumenPrimitiveTrackingInPreviewWorld = bInAllowLumenPrimitiveTrackingInPreviewWorld; return *this; }

		ConstructionValues& SetDefaultGameMode(TSubclassOf<class AGameModeBase> GameMode) { DefaultGameMode = GameMode; return *this; }
		ConstructionValues& SetOwningGameInstance(class UGameInstance* InGameInstance) { OwningGameInstance = InGameInstance; return *this; }
	};

	// for physical correct light computations we multiply diffuse and specular lights by PI (see LABEL_RealEnergy)
	ENGINE_API FPreviewScene(ConstructionValues CVS = ConstructionValues());
	ENGINE_API virtual ~FPreviewScene();

	/**
	 * Adds a component to the preview scene.  This attaches the component to the scene, and takes ownership of it.
	 */
	ENGINE_API virtual void AddComponent(class UActorComponent* Component,const FTransform& LocalToWorld, bool bAttachToRoot=false);

	/**
	 * Removes a component from the preview scene.  This detaches the component from the scene, and returns ownership of it.
	 */
	ENGINE_API virtual void RemoveComponent(class UActorComponent* Component);

	// Serializer.
	ENGINE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	ENGINE_API virtual FString GetReferencerName() const override;

	// Accessors.
	UWorld* GetWorld() const { return PreviewWorld; }
	FSceneInterface* GetScene() const { return PreviewWorld->Scene; }
	bool IsInitialized() const { return PreviewWorld != nullptr; }

	/** Access to line drawing */
	class ULineBatchComponent* GetLineBatcher() const { return LineBatcher; }
	/** Clean out the line batcher each frame */
	ENGINE_API void ClearLineBatcher();

	/** Update sky and reflection captures */
	ENGINE_API void UpdateCaptureContents();

	ENGINE_API FRotator GetLightDirection();
	ENGINE_API void SetLightDirection(const FRotator& InLightDir);
	ENGINE_API void SetLightBrightness(float LightBrightness);
	ENGINE_API void SetLightColor(const FColor& LightColor);

	ENGINE_API void SetSkyBrightness(float SkyBrightness);
	ENGINE_API void SetSkyCubemap(class UTextureCube* Cubemap);

	/** Get the background color we use by default */
	ENGINE_API virtual FLinearColor GetBackgroundColor() const;

	/** Load/Save settings to the config, specifying the key */
	ENGINE_API void LoadSettings(const TCHAR* Section);
	ENGINE_API void SaveSettings(const TCHAR* Section);

	class UDirectionalLightComponent* DirectionalLight = nullptr;;
	class USkyLightComponent* SkyLight = nullptr;;

private:
	void Uninitialize();

	TArray<TObjectPtr<class UActorComponent>> Components;

protected:
	TObjectPtr<class UWorld> PreviewWorld = nullptr;
	class ULineBatchComponent* LineBatcher = nullptr;

	/** This controls whether or not all mip levels of textures used by UMeshComponents added to this preview window should be loaded and remain loaded. */
	bool bForceAllUsedMipsResident;
};
