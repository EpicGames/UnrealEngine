// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialRelevance.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "MaterialCachedData.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "RHIFeatureLevel.h"
#include "RenderUtils.h"
#include "SceneTypes.h"
#include "Engine/EngineTypes.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "AI/NavigationModifier.h"

/** Helper class used to share implementation for different PrimitiveComponent types */
class FPrimitiveComponentHelper
{
public:
	template<class T>
	static FPrimitiveMaterialPropertyDescriptor GetUsedMaterialPropertyDesc(const T& Component, EShaderPlatform InShaderPlatform);

	template<class T>
	static void GetNavigationData(const T& Component, FNavigationRelevantData& OutData);

	template<class T>
	static void AddNavigationModifier(const T& Component, FNavigationRelevantData& OutData);
};

template<class T>
FPrimitiveMaterialPropertyDescriptor FPrimitiveComponentHelper::GetUsedMaterialPropertyDesc(const T& Component, EShaderPlatform InShaderPlatform)
{
	FPrimitiveMaterialPropertyDescriptor Result;
	TArray<UMaterialInterface*> UsedMaterials;
	Component.GetUsedMaterials(UsedMaterials);

	const bool bUseTessellation = UseNaniteTessellation();

	for (const UMaterialInterface* MaterialInterface : UsedMaterials)
	{
		if (MaterialInterface)
		{
			FMaterialRelevance MaterialRelevance = MaterialInterface->GetRelevance_Concurrent(InShaderPlatform);

			Result.bAnyMaterialHasWorldPositionOffset = Result.bAnyMaterialHasWorldPositionOffset || MaterialRelevance.bUsesWorldPositionOffset;

			if (MaterialInterface->HasPixelAnimation() && IsOpaqueOrMaskedBlendMode(MaterialInterface->GetBlendMode()))
			{
				Result.bAnyMaterialHasPixelAnimation = true;
			}

			if (bUseTessellation && MaterialRelevance.bUsesDisplacement)
			{
				FDisplacementScaling DisplacementScaling = MaterialInterface->GetDisplacementScaling();

				const float MinDisplacement = (0.0f - DisplacementScaling.Center) * DisplacementScaling.Magnitude;
				const float MaxDisplacement = (1.0f - DisplacementScaling.Center) * DisplacementScaling.Magnitude;

				Result.MinMaxMaterialDisplacement.X = FMath::Min(Result.MinMaxMaterialDisplacement.X, MinDisplacement);
				Result.MinMaxMaterialDisplacement.Y = FMath::Max(Result.MinMaxMaterialDisplacement.Y, MaxDisplacement);
			}

			Result.MaxWorldPositionOffsetDisplacement = FMath::Max(Result.MaxWorldPositionOffsetDisplacement, MaterialInterface->GetMaxWorldPositionOffsetDisplacement());

			const FMaterialCachedExpressionData& CachedMaterialData = MaterialInterface->GetCachedExpressionData();

			Result.bAnyMaterialHasPerInstanceRandom = Result.bAnyMaterialHasPerInstanceRandom || CachedMaterialData.bHasPerInstanceRandom;
			Result.bAnyMaterialHasPerInstanceCustomData = Result.bAnyMaterialHasPerInstanceCustomData || CachedMaterialData.bHasPerInstanceCustomData;
		}
	}

	return Result;
}

template<class T>
void FPrimitiveComponentHelper::GetNavigationData(const T& Component, FNavigationRelevantData& OutData)
{
	FPrimitiveComponentHelper::AddNavigationModifier(Component, OutData);
}

template<class T>
void FPrimitiveComponentHelper::AddNavigationModifier(const T& Component, FNavigationRelevantData& OutData)
{
	if (Component.bFillCollisionUnderneathForNavmesh || Component.bRasterizeAsFilledConvexVolume)
	{
		FCompositeNavModifier CompositeNavModifier;
		CompositeNavModifier.SetFillCollisionUnderneathForNavmesh(Component.bFillCollisionUnderneathForNavmesh);
		CompositeNavModifier.SetRasterizeAsFilledConvexVolume(Component.bRasterizeAsFilledConvexVolume);
		OutData.Modifiers.Add(CompositeNavModifier);
	}
}