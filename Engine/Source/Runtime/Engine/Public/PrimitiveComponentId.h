// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/TypeHash.h"

/**
 * Class used to identify UPrimitiveComponents on the rendering thread without having to pass the pointer around,
 * Which would make it easy for people to access game thread variables from the rendering thread.
 */
class FPrimitiveComponentId
{
public:

	FPrimitiveComponentId() : PrimIDValue(0)
	{}

	inline bool IsValid() const
	{
		return PrimIDValue > 0;
	}

	inline bool operator==(FPrimitiveComponentId OtherId) const
	{
		return PrimIDValue == OtherId.PrimIDValue;
	}

	inline bool operator!=(FPrimitiveComponentId OtherId) const
	{
		return PrimIDValue != OtherId.PrimIDValue;
	}

	friend uint32 GetTypeHash(FPrimitiveComponentId Id)
	{
		return GetTypeHash(Id.PrimIDValue);
	}

	uint32 PrimIDValue;
};
