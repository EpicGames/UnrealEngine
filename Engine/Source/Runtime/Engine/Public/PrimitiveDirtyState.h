// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

enum class EPrimitiveDirtyState : uint8
{
	None = 0U,
	ChangedTransform = (1U << 1U),
	ChangedStaticLighting = (1U << 2U),
	ChangedOther = (1U << 3U),
	/** The Added flag is a bit special, as it is used to skip invalidations in the VSM, and thus must only be set if the primitive is in fact added
	 * (a previous remove must have been processed by GPU scene, or it is new). If in doubt, don't set this. */
	 Added = (1U << 4U),
	 Removed = (1U << 5U), // Only used to make sure we don't process something that has been marked as Removed (more a debug feature, can be trimmed if need be)
	 ChangedAll = ChangedTransform | ChangedStaticLighting | ChangedOther,
	 /** Mark all data as changed and set Added flag. Must ONLY be used when a primitive is added, c.f. Added, above. */
	 AddedMask = ChangedAll | Added,
};
ENUM_CLASS_FLAGS(EPrimitiveDirtyState);
