// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

class FDynamicPrimitiveResource;
class FSceneView;
class FTexture;
class HHitProxy;
struct FMeshBatch;

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * The base interface used to query a primitive for its dynamic elements.
 */
class FPrimitiveDrawInterface
{
public:

	const FSceneView* View;

	/** Initialization constructor. */
	FPrimitiveDrawInterface(const FSceneView* InView):
		View(InView)
	{}

	virtual ~FPrimitiveDrawInterface() = default;

	virtual bool IsHitTesting() = 0;
	virtual void SetHitProxy(HHitProxy* HitProxy) = 0;

	virtual void RegisterDynamicResource(FDynamicPrimitiveResource* DynamicResource) = 0;

	virtual void AddReserveLines(uint8 DepthPriorityGroup, int32 NumLines, bool bDepthBiased = false, bool bThickLines = false) = 0;

	virtual void DrawSprite(
		const FVector& Position,
		float SizeX,
		float SizeY,
		const FTexture* Sprite,
		const FLinearColor& Color,
		uint8 DepthPriorityGroup,
		float U,
		float UL,
		float V,
		float VL,
		uint8 BlendMode = 1, /*SE_BLEND_Masked*/
		float OpacityMaskRefVal = .5f
		) = 0;

	// Draw an opaque line. The alpha component of Color is ignored.
	virtual void DrawLine(
		const FVector& Start,
		const FVector& End,
		const FLinearColor& Color,
		uint8 DepthPriorityGroup,
		float Thickness = 0.0f,
		float DepthBias = 0.0f,
		bool bScreenSpace = false
		) = 0;

	// Draw a translucent line. The alpha component of Color determines the transparency.
	virtual void DrawTranslucentLine(
		const FVector& Start,
		const FVector& End,
		const FLinearColor& Color,
		uint8 DepthPriorityGroup,
		float Thickness = 0.0f,
		float DepthBias = 0.0f,
		bool bScreenSpace = false
	) = 0;

	virtual void DrawPoint(
		const FVector& Position,
		const FLinearColor& Color,
		float PointSize,
		uint8 DepthPriorityGroup
		) = 0;

	/**
	 * Draw a mesh element.
	 * This should only be called through the DrawMesh function.
	 *
	 * @return Number of passes rendered for the mesh
	 */
	virtual int32 DrawMesh(const FMeshBatch& Mesh) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
