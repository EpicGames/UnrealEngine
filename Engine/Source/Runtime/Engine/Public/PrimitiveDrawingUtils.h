// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "PrimitiveDrawInterface.h" // IWYU pragma: keep

class FMaterialRenderProxy;
class FMeshElementCollector;

////////////////////////////////////////////////////////////////////////////////////////////////////

//
// Primitive drawing utility functions.
//

// Solid shape drawing utility functions. Not really designed for speed - more for debugging.

// 10x10 tessellated plane at x=-1..1 y=-1...1 z=0
extern ENGINE_API void DrawPlane10x10(class FPrimitiveDrawInterface* PDI,const FMatrix& ObjectToWorld,float Radii,FVector2D UVMin, FVector2D UVMax,const FMaterialRenderProxy* MaterialRenderProxy,uint8 DepthPriority);

// draw simple triangle with material
extern ENGINE_API void DrawTriangle(class FPrimitiveDrawInterface* PDI, const FVector& A, const FVector& B, const FVector& C, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup);
extern ENGINE_API void DrawBox(class FPrimitiveDrawInterface* PDI,const FMatrix& BoxToWorld,const FVector& Radii,const FMaterialRenderProxy* MaterialRenderProxy,uint8 DepthPriority);
extern ENGINE_API void DrawSphere(class FPrimitiveDrawInterface* PDI,const FVector& Center,const FRotator& Orientation,const FVector& Radii,int32 NumSides,int32 NumRings,const FMaterialRenderProxy* MaterialRenderProxy,uint8 DepthPriority,bool bDisableBackfaceCulling=false);
extern ENGINE_API void DrawCone(class FPrimitiveDrawInterface* PDI,const FMatrix& ConeToWorld, float Angle1, float Angle2, uint32 NumSides, bool bDrawSideLines, const FLinearColor& SideLineColor, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority);

extern ENGINE_API void DrawCylinder(class FPrimitiveDrawInterface* PDI,const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis,
	double Radius, double HalfHeight, uint32 Sides, const FMaterialRenderProxy* MaterialInstance, uint8 DepthPriority);

extern ENGINE_API void DrawCylinder(class FPrimitiveDrawInterface* PDI, const FMatrix& CylToWorld, const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis,
	double Radius, double HalfHeight, uint32 Sides, const FMaterialRenderProxy* MaterialInstance, uint8 DepthPriority);

//Draws a cylinder along the axis from Start to End
extern ENGINE_API void DrawCylinder(class FPrimitiveDrawInterface* PDI, const FVector& Start, const FVector& End, double Radius, int32 Sides, const FMaterialRenderProxy* MaterialInstance, uint8 DepthPriority);


extern ENGINE_API void GetBoxMesh(const FMatrix& BoxToWorld, const FVector& Radii, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy = nullptr);
extern ENGINE_API void GetOrientedHalfSphereMesh(const FVector& Center, const FRotator& Orientation, const FVector& Radii, int32 NumSides, int32 NumRings, float StartAngle, float EndAngle, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, bool bDisableBackfaceCulling,
	int32 ViewIndex, FMeshElementCollector& Collector, bool bUseSelectionOutline = false, HHitProxy* HitProxy = nullptr);
extern ENGINE_API void GetHalfSphereMesh(const FVector& Center, const FVector& Radii, int32 NumSides, int32 NumRings, float StartAngle, float EndAngle, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, bool bDisableBackfaceCulling,
	int32 ViewIndex, FMeshElementCollector& Collector, bool bUseSelectionOutline = false, HHitProxy* HitProxy = nullptr);
extern ENGINE_API void GetSphereMesh(const FVector& Center, const FVector& Radii, int32 NumSides, int32 NumRings, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority,
	bool bDisableBackfaceCulling, int32 ViewIndex, FMeshElementCollector& Collector);
extern ENGINE_API void GetSphereMesh(const FVector& Center, const FVector& Radii, int32 NumSides, int32 NumRings, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority,
	bool bDisableBackfaceCulling, int32 ViewIndex, FMeshElementCollector& Collector, bool bUseSelectionOutline, HHitProxy* HitProxy);

extern ENGINE_API void GetDiscMesh(const FVector& Center, const FVector& XAxis, const FVector& YAxis, double Radius, int32 Sides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, 
	int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy = nullptr);
extern ENGINE_API void GetDiscMesh(const FMatrix& ToWorld, const FVector& Center, const FVector& XAxis, const FVector& YAxis, double Radius, int32 Sides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority,
	int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy = nullptr);

extern ENGINE_API void GetCylinderMesh(const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis,
	double Radius, double HalfHeight, int32 Sides, const FMaterialRenderProxy* MaterialInstance, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy = nullptr);
extern ENGINE_API void GetCylinderMesh(const FMatrix& CylToWorld, const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis,
	double Radius, double HalfHeight, uint32 Sides, const FMaterialRenderProxy* MaterialInstance, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy = nullptr);
//Draws a cylinder along the axis from Start to End
extern ENGINE_API void GetCylinderMesh(const FVector& Start, const FVector& End, double Radius, int32 Sides, const FMaterialRenderProxy* MaterialInstance, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy = nullptr);


extern ENGINE_API void GetConeMesh(const FMatrix& LocalToWorld, float AngleWidth, float AngleHeight, uint32 NumSides,
	const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy = nullptr);
extern ENGINE_API void GetCapsuleMesh(const FVector& Origin, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis, const FLinearColor& Color, double Radius, double HalfHeight, int32 NumSides,
	const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, bool bDisableBackfaceCulling, int32 ViewIndex, FMeshElementCollector& Collector, HHitProxy* HitProxy = nullptr);


/**
 * Draws a torus using triangles.
 *
 * @param	PDI						Draw interface.
 * @param	Transform				Generic transform to apply (ex. a local-to-world transform).
 * @param	XAxis					Normalized X alignment axis.
 * @param	YAxis					Normalized Y alignment axis.
 * @param	OuterRadius				Radius of the torus center-line. Viewed from above, the outside of the torus has a radius 
 *                                  of OuterRadius + InnerRadius and the hole of the torus has a radius of OuterRadius - InnerRadius.
 * @param	InnerRadius				Radius of the torus's cylinder.
 * @param	OuterSegments			Numbers of segment divisions for outer circle.
 * @param	InnerSegments			Numbers of segment divisions for inner circle.
 * @param	MaterialRenderProxy		Material to use for render
 * @param	DepthPriority			Depth priority for the circle.
 * @param	bPartial				Whether full or partial torus should be rendered.
 * @param	Angle					If partial, angle in radians of the arc clockwise beginning at the XAxis.
 * @param	bEndCaps				If partial, whether the ends should be capped with triangles.
 */
extern ENGINE_API void DrawTorus(FPrimitiveDrawInterface* PDI, const FMatrix& Transform, const FVector& XAxis, const FVector& YAxis,
								 double OuterRadius, double InnerRadius, int32 OuterSegments, int32 InnerSegments, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, bool bPartial, float Angle, bool bEndCaps);

/**
 * Draws a torus using triangles.
 *
 * @param	PDI						Draw interface.
 * @param	Transform				Generic transform to apply (ex. a local-to-world transform).
 * @param	XAxis					Normalized X alignment axis.
 * @param	YAxis					Normalized Y alignment axis.
 * @param	Color					Color of the circle.
 * @param	OuterRadius				Radius of the torus center-line. Viewed from above, the outside of the torus has a radius 
 *                                  of OuterRadius + InnerRadius and the hole of the torus has a radius of OuterRadius - InnerRadius.
 * @param	InnerRadius				Radius of the torus's cylinder.
 * @param	OuterSegments			Numbers of segment divisions for outer circle.
 * @param	InnerSegments			Numbers of segment divisions for inner circle.
 * @param	MaterialRenderProxy		Material to use for render
 * @param	DepthPriority			Depth priority for the circle.
 * @param	bPartial				Whether full or partial torus should be rendered.
 * @param	Angle					If partial, angle in radians of the arc clockwise beginning at the XAxis.
 * @param	bEndCaps				If partial, whether the ends should be capped with triangles.
 */
extern ENGINE_API void DrawTorus(FPrimitiveDrawInterface* PDI, const FMatrix& Transform, const FVector& XAxis, const FVector& YAxis, FColor Color,
								 double OuterRadius, double InnerRadius, int32 OuterSegments, int32 InnerSegments, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, bool bPartial, float Angle, bool bEndCaps);

/**
 * Draws a circle using triangles.
 *
 * @param	PDI						Draw interface.
 * @param	Base					Center of the circle.
 * @param	XAxis					X alignment axis to draw along.
 * @param	YAxis					Y alignment axis to draw along.
 * @param	Color					Color of the circle.
 * @param	Radius					Radius of the circle.
 * @param	NumSides				Numbers of sides that the circle has.
 * @param	MaterialRenderProxy		Material to use for render 
 * @param	DepthPriority			Depth priority for the circle.
 */
extern ENGINE_API void DrawDisc(class FPrimitiveDrawInterface* PDI,const FVector& Base,const FVector& XAxis,const FVector& YAxis,FColor Color,double Radius,int32 NumSides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority);

/**
 * Draws a rectangle using triangles.
 *
 * @param	PDI						Draw interface.
 * @param	Center					Center of the rectangle.
 * @param	XAxis					Normalized X alignment axis.
 * @param	YAxis					Normalized Y alignment axis.
 * @param	Color					Color of the circle.
 * @param	Width					Width of rectangle along the X dimension.
 * @param	Height					Height of rectangle along the Y dimension.
 * @param	MaterialRenderProxy		Material to use for render
 * @param	DepthPriority			Depth priority for the rectangle.
 */
extern ENGINE_API void DrawRectangleMesh(FPrimitiveDrawInterface* PDI, const FVector& Center, const FVector& XAxis, const FVector& YAxis, 
										 FColor Color, float Width, float Height, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority);

/**
 * Draws a flat arrow with an outline.
 *
 * @param	PDI						Draw interface.
 * @param	Base					Base of the arrow.
 * @param	XAxis					X alignment axis to draw along.
 * @param	YAxis					Y alignment axis to draw along.
 * @param	Color					Color of the circle.
 * @param	Length					Length of the arrow, from base to tip.
 * @param	Width					Width of the base of the arrow, head of the arrow will be 2x.
 * @param	MaterialRenderProxy		Material to use for render 
 * @param	DepthPriority			Depth priority for the circle.
 * @param	Thickness				Thickness of the lines comprising the arrow
 */

/*
x-axis is from point 0 to point 2
y-axis is from point 0 to point 1
		6
		/\
	   /  \
	  /    \
	 4_2  3_5
	   |  |
	   0__1
*/
extern ENGINE_API void DrawFlatArrow(class FPrimitiveDrawInterface* PDI,const FVector& Base,const FVector& XAxis,const FVector& YAxis,FColor Color,float Length,int32 Width, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, float Thickness = 0.0f);

// Line drawing utility functions.

/**
 * Draws a wireframe box.
 *
 * @param	PDI				Draw interface.
 * @param	Box				The FBox to use for drawing.
 * @param	Color			Color of the box.
 * @param	DepthPriority	Depth priority for the circle.
 * @param	Thickness		Thickness of the lines comprising the box
 */
extern ENGINE_API void DrawWireBox(class FPrimitiveDrawInterface* PDI, const FBox& Box, const FLinearColor& Color, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);
extern ENGINE_API void DrawWireBox(class FPrimitiveDrawInterface* PDI, const FMatrix& Matrix, const FBox& Box, const FLinearColor& Color, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

/**
 * Draws an arc using lines.
 *
 * @param	PDI				Draw interface.
 * @param	Base			Center of the circle.
 * @param	X				Normalized axis from one point to the center
 * @param	Y				Normalized axis from other point to the center
 * @param   MinAngle        The minimum angle
 * @param   MaxAngle        The maximum angle
 * @param   Radius          Radius of the arc
 * @param	Sections		Numbers of sides that the circle has.
 * @param	Color			Color of the circle.
 * @param	DepthPriority	Depth priority for the circle.
 */
extern ENGINE_API void DrawArc(FPrimitiveDrawInterface* PDI, const FVector Base, const FVector X, const FVector Y, const float MinAngle, const float MaxAngle, const double Radius, const int32 Sections, const FLinearColor& Color, uint8 DepthPriority);

/**
 * Draws a rectangle using lines.
 *
 * @param	PDI						Draw interface.
 * @param	Center					Center of the rectangle.
 * @param	XAxis					Normalized X alignment axis.
 * @param	YAxis					Normalized Y alignment axis.
 * @param	Color					Color of the circle.
 * @param	Width					Width of rectangle along the X dimension.
 * @param	Height					Height of rectangle along the Y dimension.
 * @param	MaterialRenderProxy		Material to use for render
 * @param	DepthPriority			Depth priority for the rectangle.
 * @param	Thickness				Thickness of the lines comprising the rectangle.
 */
extern ENGINE_API void DrawRectangle(FPrimitiveDrawInterface* PDI, const FVector& Center, const FVector& XAxis, const FVector& YAxis, 
									 FColor Color, float Width, float Height, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

/**
 * Draws a sphere using circles.
 *
 * @param	PDI				Draw interface.
 * @param	Base			Center of the sphere.
 * @param	Color			Color of the sphere.
 * @param	Radius			Radius of the sphere.
 * @param	NumSides		Numbers of sides that the circle has.
 * @param	DepthPriority	Depth priority for the circle.
 * @param	Thickness		Thickness of the lines comprising the sphere
 */
extern ENGINE_API void DrawWireSphere(class FPrimitiveDrawInterface* PDI, const FVector& Base, const FLinearColor& Color, double Radius, int32 NumSides, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);
extern ENGINE_API void DrawWireSphere(class FPrimitiveDrawInterface* PDI, const FTransform& Transform, const FLinearColor& Color, double Radius, int32 NumSides, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

/**
 * Draws a sphere using circles, automatically calculating a reasonable number of sides
 *
 * @param	PDI				Draw interface.
 * @param	Base			Center of the sphere.
 * @param	Color			Color of the sphere.
 * @param	Radius			Radius of the sphere.
 * @param	DepthPriority	Depth priority for the circle.
 * @param	Thickness		Thickness of the lines comprising the sphere
 */
extern ENGINE_API void DrawWireSphereAutoSides(class FPrimitiveDrawInterface* PDI, const FVector& Base, const FLinearColor& Color, double Radius, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);
extern ENGINE_API void DrawWireSphereAutoSides(class FPrimitiveDrawInterface* PDI, const FTransform& Transform, const FLinearColor& Color, double Radius, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

/**
 * Draws a wireframe circle.
 *
 * @param	PDI				Draw interface.
 * @param	Center			Center of the circle.
 * @param	X				X alignment axis to draw along.
 * @param	Y				Y alignment axis to draw along.
 * @param	Color			Color of the cylinder.
 * @param	Radius			Radius of the cylinder.
 * @param	NumSides		Numbers of sides that the cylinder has.
 * @param	DepthPriority	Depth priority for the cylinder.
 * @param	Thickness		Thickness of the lines comprising the circle
 */
extern ENGINE_API void DrawCircle(class FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, const FLinearColor& Color, double Radius, int32 NumSides, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

/**
 * Draws a wireframe cylinder.
 *
 * @param	PDI				Draw interface.
 * @param	Base			Center pointer of the base of the cylinder.
 * @param	X				X alignment axis to draw along.
 * @param	Y				Y alignment axis to draw along.
 * @param	Z				Z alignment axis to draw along.
 * @param	Color			Color of the cylinder.
 * @param	Radius			Radius of the cylinder.
 * @param	HalfHeight		Half of the height of the cylinder.
 * @param	NumSides		Numbers of sides that the cylinder has.
 * @param	DepthPriority	Depth priority for the cylinder.
 * @param	Thickness		Thickness of the lines comprising the cylinder
 */
extern ENGINE_API void DrawWireCylinder(class FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, const FVector& Z, const FLinearColor& Color, double Radius, double HalfHeight, int32 NumSides, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

/**
 * Draws a wireframe capsule.
 *
 * @param	PDI				Draw interface.
 * @param	Base			Center pointer of the base of the cylinder.
 * @param	X				X alignment axis to draw along.
 * @param	Y				Y alignment axis to draw along.
 * @param	Z				Z alignment axis to draw along.
 * @param	Color			Color of the cylinder.
 * @param	Radius			Radius of the cylinder.
 * @param	HalfHeight		Half of the height of the cylinder.
 * @param	NumSides		Numbers of sides that the cylinder has.
 * @param	DepthPriority	Depth priority for the cylinder.
 * @param	Thickness		Thickness of the lines comprising the cylinder
 */
extern ENGINE_API void DrawWireCapsule(class FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, const FVector& Z, const FLinearColor& Color, double Radius, double HalfHeight, int32 NumSides, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

/**
 * Draws a wireframe chopped cone (cylinder with independent top and bottom radius).
 *
 * @param	PDI				Draw interface.
 * @param	Base			Center pointer of the base of the cone.
 * @param	X				X alignment axis to draw along.
 * @param	Y				Y alignment axis to draw along.
 * @param	Z				Z alignment axis to draw along.
 * @param	Color			Color of the cone.
 * @param	Radius			Radius of the cone at the bottom.
 * @param	TopRadius		Radius of the cone at the top.
 * @param	HalfHeight		Half of the height of the cone.
 * @param	NumSides		Numbers of sides that the cone has.
 * @param	DepthPriority	Depth priority for the cone.
 */
extern ENGINE_API void DrawWireChoppedCone(class FPrimitiveDrawInterface* PDI,const FVector& Base,const FVector& X,const FVector& Y,const FVector& Z,const FLinearColor& Color,double Radius,double TopRadius,double HalfHeight,int32 NumSides,uint8 DepthPriority);

/**
 * Draws a wireframe cone
 *
 * @param	PDI				Draw interface.
 * @param	Transform		Generic transform to apply (ex. a local-to-world transform).
 * @param	ConeLength		Pre-transform distance from apex to the perimeter of the cone base.  The Radius of the base is ConeLength * sin(ConeAngle).
 * @param	ConeAngle		Angle of the cone in degrees. This is 1/2 the cone aperture.
 * @param	ConeSides		Numbers of sides that the cone has.
 * @param	Color			Color of the cone.
 * @param	DepthPriority	Depth priority for the cone.
 * @param	Verts			Out param, the positions of the verts at the cone base.
 * @param	Thickness		Thickness of the lines comprising the cone
 */
extern ENGINE_API void DrawWireCone(class FPrimitiveDrawInterface* PDI, TArray<FVector>& Verts, const FMatrix& Transform, double ConeLength, double ConeAngle, int32 ConeSides, const FLinearColor& Color, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);
extern ENGINE_API void DrawWireCone(class FPrimitiveDrawInterface* PDI, TArray<FVector>& Verts, const FTransform& Transform, double ConeLength, double ConeAngle, int32 ConeSides, const FLinearColor& Color, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

/**
 * Draws a wireframe cone with a arcs on the cap
 *
 * @param	PDI				Draw interface.
 * @param	Transform		Generic transform to apply (ex. a local-to-world transform).
 * @param	ConeLength		Pre-transform distance from apex to the perimeter of the cone base.  The Radius of the base is ConeLength * sin(ConeAngle).
 * @param	ConeAngle		Angle of the cone in degrees. This is 1/2 the cone aperture.
 * @param	ConeSides		Numbers of sides that the cone has.
 * @param   ArcFrequency    How frequently to draw an arc (1 means every vertex, 2 every 2nd etc.)
 * @param	CapSegments		How many lines to use to make the arc
 * @param	Color			Color of the cone.
 * @param	DepthPriority	Depth priority for the cone.
 */
extern ENGINE_API void DrawWireSphereCappedCone(FPrimitiveDrawInterface* PDI, const FTransform& Transform, double ConeLength, double ConeAngle, int32 ConeSides, int32 ArcFrequency, int32 CapSegments, const FLinearColor& Color, uint8 DepthPriority);

/**
 * Draws an oriented box.
 *
 * @param	PDI				Draw interface.
 * @param	Base			Center point of the box.
 * @param	X				X alignment axis to draw along.
 * @param	Y				Y alignment axis to draw along.
 * @param	Z				Z alignment axis to draw along.
 * @param	Color			Color of the box.
 * @param	Extent			Vector with the half-sizes of the box.
 * @param	DepthPriority	Depth priority for the cone.
 * @param	Thickness		Thickness of the lines comprising the box
 */
extern ENGINE_API void DrawOrientedWireBox(class FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, const FVector& Z, FVector Extent, const FLinearColor& Color, uint8 DepthPriority, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

/**
 * Draws a directional arrow (starting at ArrowToWorld.Origin and continuing for Length units in the X direction of ArrowToWorld).
 *
 * @param	PDI				Draw interface.
 * @param	ArrowToWorld	Transform matrix for the arrow.
 * @param	InColor			Color of the arrow.
 * @param	Length			Length of the arrow
 * @param	ArrowSize		Size of the arrow head.
 * @param	DepthPriority	Depth priority for the arrow.
 * @param	Thickness		Thickness of the lines comprising the arrow
 */
extern ENGINE_API void DrawDirectionalArrow(class FPrimitiveDrawInterface* PDI, const FMatrix& ArrowToWorld, const FLinearColor& InColor, float Length, float ArrowSize, uint8 DepthPriority, float Thickness = 0.0f);

/**
 * Draws a directional arrow with connected spokes.
 *
 * @param	PDI				Draw interface.
 * @param	ArrowToWorld	Transform matrix for the arrow.
 * @param	Color			Color of the arrow.
 * @param	ArrowHeight		Height of the the arrow head.
 * @param	ArrowWidth		Width of the arrow head.
 * @param	DepthPriority	Depth priority for the arrow.
 * @param	Thickness		Thickness of the lines used to draw the arrow.
 * @param	NumSpokes		Number of spokes used to make the arrow head.
 */
extern ENGINE_API void DrawConnectedArrow(class FPrimitiveDrawInterface* PDI, const FMatrix& ArrowToWorld, const FLinearColor& Color, float ArrowHeight, float ArrowWidth, uint8 DepthPriority, float Thickness = 0.5f, int32 NumSpokes = 6);

/**
 * Draws a axis-aligned 3 line star.
 *
 * @param	PDI				Draw interface.
 * @param	Position		Position of the star.
 * @param	Size			Size of the star
 * @param	InColor			Color of the arrow.
 * @param	DepthPriority	Depth priority for the star.
 */
extern ENGINE_API void DrawWireStar(class FPrimitiveDrawInterface* PDI, const FVector& Position, float Size, const FLinearColor& Color, uint8 DepthPriority);

/**
 * Draws a dashed line.
 *
 * @param	PDI				Draw interface.
 * @param	Start			Start position of the line.
 * @param	End				End position of the line.
 * @param	Color			Color of the arrow.
 * @param	DashSize		Size of each of the dashes that makes up the line.
 * @param	DepthPriority	Depth priority for the line.
 */
extern ENGINE_API void DrawDashedLine(class FPrimitiveDrawInterface* PDI, const FVector& Start, const FVector& End, const FLinearColor& Color, double DashSize, uint8 DepthPriority, float DepthBias = 0.0f);

/**
 * Draws a wireframe diamond.
 *
 * @param	PDI				Draw interface.
 * @param	DiamondMatrix	Transform Matrix for the diamond.
 * @param	Size			Size of the diamond.
 * @param	InColor			Color of the diamond.
 * @param	DepthPriority	Depth priority for the diamond.
 * @param	Thickness		How thick to draw diamond lines
 */
extern ENGINE_API void DrawWireDiamond(class FPrimitiveDrawInterface* PDI, const FMatrix& DiamondMatrix, float Size, const FLinearColor& InColor, uint8 DepthPriority, float Thickness = 0.0f);

/**
 * Draws a coordinate system (Red for X axis, Green for Y axis, Blue for Z axis).
 *
 * @param	PDI				Draw interface.
 * @param	AxisLoc			Location of the coordinate system.
 * @param	AxisRot			Location of the coordinate system.
 * @param	Scale			Scale for the axis lines.
 * @param	DepthPriority	Depth priority coordinate system.
 * @param	Thickness		How thick to draw the axis lines
 */
extern ENGINE_API void DrawCoordinateSystem(FPrimitiveDrawInterface* PDI, FVector const& AxisLoc, FRotator const& AxisRot, float Scale, uint8 DepthPriority, float Thickness = 0.0f);

/**
 * Draws a coordinate system with a fixed color.
 *
 * @param	PDI				Draw interface.
 * @param	AxisLoc			Location of the coordinate system.
 * @param	AxisRot			Location of the coordinate system.
 * @param	Scale			Scale for the axis lines.
 * @param	InColor			Color of the axis lines.
 * @param	DepthPriority	Depth priority coordinate system.
 * @param	Thickness		How thick to draw the axis lines
 */
extern ENGINE_API void DrawCoordinateSystem(FPrimitiveDrawInterface* PDI, FVector const& AxisLoc, FRotator const& AxisRot, float Scale, const FLinearColor& InColor, uint8 DepthPriority, float Thickness = 0.0f);

/**
 * Draws a wireframe of the bounds of a frustum as defined by a transform from clip-space into world-space.
 * @param PDI - The interface to draw the wireframe.
 * @param FrustumToWorld - A transform from clip-space to world-space that defines the frustum.
 * @param Color - The color to draw the wireframe with.
 * @param DepthPriority - The depth priority group to draw the wireframe with.
 */
extern ENGINE_API void DrawFrustumWireframe(
	FPrimitiveDrawInterface* PDI,
	const FMatrix& WorldToFrustum,
	FColor Color,
	uint8 DepthPriority
	);

////////////////////////////////////////////////////////////////////////////////////////////////////
