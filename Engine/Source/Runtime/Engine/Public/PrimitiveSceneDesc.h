// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/BoxSphereBounds.h"
#include "PrimitiveComponentId.h"
#include "PrimitiveSceneInfoData.h"
#include "Engine/EngineTypes.h"
#include "Math/BoxSphereBounds.h"

class FPrimitiveSceneProxy;
struct FPrimitiveSceneProxyDesc;
class HHitProxy;
class IPrimitiveComponent;

/**
* FPrimitiveSceneDesc is a structure that can be used to Add/Remove/Update primitives in an FScene. 
* 
* It encapsulates all the necessary information to create/update the primitive. Usage of an PrimitiveComponentInterface
* is optional, but if one is not provided the ProxyDesc must already be created and passed in the ProxyDesc member.
* 
*/

struct FPrimitiveSceneDesc 
{
	FPrimitiveSceneProxyDesc* ProxyDesc = nullptr;
	IPrimitiveComponent* PrimitiveComponentInterface = nullptr;
	FPrimitiveSceneInfoData* PrimitiveSceneData = nullptr;
	FPrimitiveSceneProxy* SceneProxy = nullptr;

	FPrimitiveComponentId LightingAttachmentComponentId;
	FPrimitiveComponentId LodParentComponentId;

	bool bShouldAddtoScene = true; // for UpdatePrimitiveAttachment
	bool bRecreateProxyOnUpdateTransform = false; 
	bool bIsUnreachable = false;
	bool bBulkReregister = false;

	EComponentMobility::Type Mobility = EComponentMobility::Movable;
	FBoxSphereBounds Bounds;
	FBoxSphereBounds LocalBounds;

	FMatrix RenderMatrix;
	FVector AttachmentRootPosition;

	UObject* PrimitiveUObject = nullptr;
	
	// @todo: Possibly add uninitialized FStrings in this object to allow overriding the name without having a corresponding UObject
	FString GetFullName() { return PrimitiveUObject->GetFullName(); }
	FString GetName() { return PrimitiveUObject->GetName(); }

	bool IsUnreachable() { return bIsUnreachable; }
	bool ShouldRecreateProxyOnUpdateTransform() { return bRecreateProxyOnUpdateTransform; }

	FAttachmentCounter* GetAttachmentCounter() const { return PrimitiveSceneData ? &PrimitiveSceneData->AttachmentCounter : nullptr; }
	FPrimitiveComponentId GetPrimitiveSceneId() const { return PrimitiveSceneData->PrimitiveSceneId; }
	FPrimitiveComponentId GetLODParentId() const { return LodParentComponentId; }
	FPrimitiveComponentId GetLightingAttachmentId() const { return LightingAttachmentComponentId; }

	void SetLODParentId(FPrimitiveComponentId  Id) { LodParentComponentId = Id; }
	void SetLightingAttachmentId(FPrimitiveComponentId  Id) { LightingAttachmentComponentId = Id; }

	UE_DEPRECATED(5.5, "GetLastSubmitTime is no longer used")
	double GetLastSubmitTime() { return 0.0; }

	UE_DEPRECATED(5.5, "SetLastSubmitTime is no longer used.")
	void SetLastSubmitTime(double InSubmitTime) {}
	
	EComponentMobility::Type GetMobility() { return Mobility; }
	FMatrix GetRenderMatrix() { return RenderMatrix; }
	const FMatrix& GetRenderMatrix() const { return RenderMatrix; }
	FVector GetActorPositionForRenderer() { return AttachmentRootPosition; }
	
	#if !WITH_STATE_STREAM
	UE_DEPRECATED(5.6, "World should not be used by rendering")
	UWorld* GetWorld() { return World; }
	UWorld*  World = nullptr;
	#endif

	FBoxSphereBounds GetBounds() { return Bounds; }
	FBoxSphereBounds GetLocalBounds() { return LocalBounds; }
	
	FPrimitiveSceneProxy* GetSceneProxy() const
	{
		if(PrimitiveSceneData)
		{
			return PrimitiveSceneData->SceneProxy;
		}
		return SceneProxy;
	}
	
	FPrimitiveSceneProxyDesc* GetSceneProxyDesc() {  return ProxyDesc; }
	FPrimitiveSceneInfoData& GetSceneData() { return *PrimitiveSceneData; }

	void ReleaseSceneProxy()
	{ 
		SceneProxy = nullptr;
		if(PrimitiveSceneData)
		{
			PrimitiveSceneData->SceneProxy = nullptr;
		}
	}

	IPrimitiveComponent* GetPrimitiveComponentInterface() { return PrimitiveComponentInterface; }
	const IPrimitiveComponent* GetPrimitiveComponentInterface() const { return PrimitiveComponentInterface; }

	UPackage* GetOutermost() const { return PrimitiveUObject->GetOutermost(); }
};

struct FInstancedStaticMeshSceneDesc
{
	FInstancedStaticMeshSceneDesc(FPrimitiveSceneDesc& InPrimitiveSceneDesc):
		PrimitiveSceneDesc(InPrimitiveSceneDesc)
	{
	}

	operator FPrimitiveSceneDesc*() { return &PrimitiveSceneDesc; }
	FPrimitiveSceneProxy* GetSceneProxy() { return PrimitiveSceneDesc.GetSceneProxy(); }	
	FBoxSphereBounds GetBounds() { return PrimitiveSceneDesc.GetBounds(); }
	FBoxSphereBounds GetLocalBounds() { return PrimitiveSceneDesc.GetLocalBounds(); }
	

	// Using composition to refer to the PrimitiveSceneDesc instead of inheritance for easier 
	// usage of a class member instead of an heap allocated struct in implementers
	FPrimitiveSceneDesc& PrimitiveSceneDesc;
	UStaticMesh* StaticMesh = nullptr;

	UStaticMesh* GetStaticMesh() { check(StaticMesh); return StaticMesh; }
};