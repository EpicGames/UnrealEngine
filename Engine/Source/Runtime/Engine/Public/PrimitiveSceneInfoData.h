// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "AutoRTFM.h"
#include "CoreGlobals.h"
#include "HAL/ThreadSafeCounter.h"
#include "PrimitiveComponentId.h"
#include "Templates/RefCounting.h"

class FPrimitiveSceneProxy;
struct FActorLastRenderTime;

// An attachment counter is a thread-safe counter that is only ever
// incremented on the game thread, and decremented on the rendering
// thread.
//
// It also supports transactions, and will undo any increments on the
// game thread if a transaction aborts.
struct FAttachmentCounter final
{
	UE_NONCOPYABLE(FAttachmentCounter);

	AUTORTFM_OPEN FAttachmentCounter() = default;

#if UE_AUTORTFM
	void Increment()
	{
		// TODO: should we `check(IsInGameThread());`?

		UE_AUTORTFM_OPEN
		{
			Payload->Payload++;
		};

		UE_AUTORTFM_ONABORT(this)
		{
			Payload->Payload--;
		};
	}

	void Decrement()
	{
		// TODO: should we `check(IsInRenderingThread());`?
		AutoRTFM::UnreachableIfClosed("Should only be decremented on the render thread in the open");
		Payload->Payload--;
	}

	int32 GetValue() const
	{
		int32 Result = 0;

		UE_AUTORTFM_OPEN
		{
			Result = Payload->Payload;
		};

		return Result;
	}
private:

	// Under AutoRTFM because we are creating and *sometimes* destroying
	// `FAttachmentCounter`'s in the same transaction, we need it to be
	// heap allocated, so that the memory doesn't just instantly go away
	// and our on-abort handlers would then use-after-free.
	struct FPayload final : public UE::Private::FQueryableRefCountedObject
	{
		std::atomic<int32> Payload = 0;
	};

	TRefCountPtr<FPayload> Payload = new FPayload();

#else

	void Increment()
	{
		// TODO: should we `check(IsInGameThread());`?
		Payload++;
	}

	void Decrement()
	{
		// TODO: should we `check(IsInRenderingThread());`?
		Payload--;
	}

	int32 GetValue() const
	{
		return Payload;
	}
private:
	std::atomic<int32> Payload = 0;
#endif
};

/*
 * All the necessary information for scene primitive to component feedback 
 */
struct FPrimitiveSceneInfoData
{
	/** The primitive's scene info. */
	FPrimitiveSceneProxy* SceneProxy = nullptr;

	/**
	 * The value of WorldSettings->TimeSeconds for the frame when this component was last rendered.  This is written
	 * from the render thread, which is up to a frame behind the game thread, so you should allow this time to
	 * be at least a frame behind the game thread's world time before you consider the actor non-visible.
	 */
	UE_DEPRECATED(5.8, "Use GetLastRenderTime/SetLastRenderTime instead.")
	mutable std::atomic<float> LastRenderTime = -1000.0f;

	/** Same as LastRenderTime but only updated if the component is on screen. Used by the texture streamer. */
	UE_DEPRECATED(5.8, "Use GetLastRenderTimeOnScreen/SetLastRenderTimeOnScreen instead.")
	mutable std::atomic<float> LastRenderTimeOnScreen = -1000.0f;

	/**
	* Incremented by the main thread before being attached to the scene, decremented
	* by the rendering thread after removal. This counter exists to assert that 
	* operations are safe in order to help avoid race conditions.
	*
	*           *** Runtime logic should NEVER rely on this value. ***
	*
	* The only safe assertions to make are:
	*
	*     AttachmentCounter == 0: The primitive is not exposed to the rendering
	*                             thread, it is safe to modify shared members.
	*                             This assertion is valid ONLY from the main thread.
	*
	*     AttachmentCounter >= 1: The primitive IS exposed to the rendering
	*                             thread and therefore shared members must not
	*                             be modified. This assertion may be made from
	*                             any thread. Note that it is valid and expected
	*                             for AttachmentCounter to be larger than 1, e.g.
	*                             during reattachment.
	*/
	FAttachmentCounter AttachmentCounter;

	/** Used by the renderer, to identify a primitive across re-registers. */
	FPrimitiveComponentId PrimitiveSceneId;

	/** Whether the primitive is always visible. If true the last render time will be unset. */
	int32 bAlwaysVisible : 1;

	/**
	 * Pointer to the last render time variable on the primitive's owning actor or other UObject (if owned), which is written to by the RT and read by the GT.
	 * The value of LastRenderTime will therefore not be deterministic due to race conditions, but the GT uses it in a way that allows this.
	 * Storing a pointer to the UObject member variable only works in the AActor/UPrimitiveComponent case because:
	 *	UPrimitiveComponent's outer is its owning AActor, so it prevents the owner from being garbage collected while the component lives.
	 *  If the UPrimitiveComponent is GC'd during the Actor's lifetime, OwnerLastRenderTime is still valid so there is no issue.
	 *	If the UPrimitiveComponent and the Actor are GC'd together, neither will be deleted until FinishDestroy has been executed on both.
	 *	UPrimitiveComponent's FinishDestroy will not execute until the primitive has been detached from the Scene through it's DetachFence.
	 * In general feedback from the renderer to the game thread like this should be avoided.
	 *
	 * Any other user of this struct that intends to add it's own primitives in the Scene must provide the same guarantees. 
	 *
	 */
	FActorLastRenderTime* OwnerLastRenderTimePtr = nullptr;

	ENGINE_API void SetLastRenderTime(float InLastRenderTime, bool bUpdateLastRenderTimeOnScreen) const;

	float GetLastRenderTime() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return LastRenderTime.load(std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetLastRenderTime(float InTime) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LastRenderTime.store(InTime, std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	float GetLastRenderTimeOnScreen() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return LastRenderTimeOnScreen.load(std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetLastRenderTimeOnScreen(float InTime) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LastRenderTimeOnScreen.store(InTime, std::memory_order_relaxed);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

protected:

	/** Next id to be used by a component. */
	static ENGINE_API FThreadSafeCounter NextPrimitiveId;

public:

	FPrimitiveSceneInfoData()
		: bAlwaysVisible(false)
	{
		PrimitiveSceneId.PrimIDValue = AutoRTFM::Open([] { return NextPrimitiveId.Increment(); });
	}
};
