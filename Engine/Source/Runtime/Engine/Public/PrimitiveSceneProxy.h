// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PrimitiveSceneProxy.h: Primitive scene proxy definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "SceneViewOwner.h"
#include "Stats/Stats.h"
#include "Misc/MemStack.h"
#include "PrimitiveViewRelevance.h"
#include "SceneTypes.h"
#include "Engine/Scene.h"
#include "UniformBuffer.h"
#include "PrimitiveUniformShaderParameters.h"
#include "DrawDebugHelpers.h"
#include "Math/CapsuleShape.h"
#include "SceneDefinitions.h"
#include "MeshDrawCommandStatsDefines.h"
#include "InstanceDataTypes.h"
#include "RaytracingVFAttributeData.h"
#include "PSOPrecacheFwd.h"
#include "RayTracingGeometryManagerTypes.h"
#include "Runtime/Engine/Internal/MaterialCache/MaterialCacheDescriptor.h"
#include "Streaming/SimpleStreamableAssetManagerHandle.h"

class FLightSceneInfo;
class FLightSceneProxy;
class FPrimitiveDrawInterface;
class FPrimitiveSceneInfo;
class FStaticPrimitiveDrawInterface;
class FSkinningSceneExtensionProxy;
class HHitProxy;
class UPrimitiveComponent;
class URuntimeVirtualTexture;
class UTexture2D;
enum class ERuntimeVirtualTextureMaterialType : uint8;
struct FMeshBatch;
class FColorVertexBuffer;
class FRayTracingGeometry;
class FVertexFactory;
class IHeterogeneousVolumeInterface;
struct FPrimitiveUniformShaderParametersBuilder;
struct FPrimitiveSceneProxyDesc;
class IPrimitiveComponent;
struct FEngineShowFlags;
struct FRenderTransform;
struct FRenderBounds;
struct FRenderCurveResourceData;

namespace Nanite
{
	using CoarseMeshStreamingHandle = int16;
}

struct FDesiredLODLevel
{
	enum class EType : uint8
	{
		// LOD is the first in a set of LODs that can be selected from.
		First,

		// LOD is fixed to a pre-selected level.
		Fixed
	};

	static FDesiredLODLevel CreateFixed(uint8 LOD)
	{
		return FDesiredLODLevel(LOD, EType::Fixed);
	}
	
	static FDesiredLODLevel CreateFirst(uint8 LOD)
	{
		return FDesiredLODLevel(LOD, EType::First);
	}

	FDesiredLODLevel(uint8 InLOD, EType InType)
		: LOD(InLOD)
		, Type(InType)
	{}

	bool IsFixed() const { return Type == EType::Fixed; }
	bool IsFirst() const { return Type == EType::First; }

	uint8 LOD = 0;
	EType Type = EType::First;
};

union FSimpleLightId
{
	struct 
	{
		int32 LocalId;
		int32 SourcePrimitiveId;
	};
	uint64 Packed;

	FSimpleLightId()
		: LocalId(INDEX_NONE)
		, SourcePrimitiveId(INDEX_NONE)
	{}

	FSimpleLightId(int32 InLocalId, int32 InSourcePrimitiveId)
		: LocalId(InLocalId)
		, SourcePrimitiveId(InSourcePrimitiveId)
	{}

	bool operator==(FSimpleLightId Other) const
	{
		return Other.Packed == Packed;
	}

	bool IsValid() const
	{
		return LocalId != INDEX_NONE && SourcePrimitiveId != INDEX_NONE;
	}
};

inline uint32 GetTypeHash(FSimpleLightId In)
{
	return GetTypeHash(In.Packed);
}

/** Data for a simple dynamic light. */
class FSimpleLightEntry
{
public:
	FVector3f Color;
	float Radius;
	float Exponent;
	float InverseExposureBlend = 0.0f;
	float VolumetricScatteringIntensity;
	float SpecularScale = 1.0f;
	float DiffuseScale = 1.0f;
	bool bAffectTranslucency;
	bool bAllowMegaLights = false;
	bool bMegaLightsCastShadows = false;
	FSimpleLightId LightId;
};

/** Data for a simple dynamic light which could change per-view. */
class FSimpleLightPerViewEntry
{
public:
	FVector Position;
};

/**
* Index into the Per-view data for each instance.
* Most uses wont need to add > 1 per view data.
* This array will be the same size as InstanceData for uses that require per view data. Otherwise it will be empty.
*/
class FSimpleLightInstacePerViewIndexData
{
public:
	uint32 PerViewIndex : 31;
	uint32 bHasPerViewData;
};

/** Data pertaining to a set of simple dynamic lights */
class FSimpleLightArray
{
public:
	/** Data per simple dynamic light instance, independent of view */
	TArray<FSimpleLightEntry, SceneRenderingAllocator> InstanceData;
	/** Per-view data for each light */
	TArray<FSimpleLightPerViewEntry, SceneRenderingAllocator> PerViewData;
	/** Indices into the per-view data for each light. */
	TArray<FSimpleLightInstacePerViewIndexData, SceneRenderingAllocator> InstancePerViewDataIndices;

public:

	/**
	* Returns the per-view data for a simple light entry.
	* @param LightIndex - The index of the simple light
	* @param ViewIndex - The index of the view
	* @param NumViews - The number of views in the ViewFamily.
	*/
	inline const FSimpleLightPerViewEntry& GetViewDependentData(int32 LightIndex, int32 ViewIndex, int32 NumViews) const
	{
		// If InstanceData has an equal number of elements to PerViewData then all views share the same PerViewData.
		if (InstanceData.Num() == PerViewData.Num())
		{
			check(InstancePerViewDataIndices.Num() == 0);
			return PerViewData[LightIndex];
		}
		else 
		{
			check(InstancePerViewDataIndices.Num() == InstanceData.Num());

			// Calculate per-view index.
			FSimpleLightInstacePerViewIndexData PerViewIndex = InstancePerViewDataIndices[LightIndex];
			const int32 PerViewDataIndex = PerViewIndex.PerViewIndex + ( PerViewIndex.bHasPerViewData ? ViewIndex : 0 );
			return PerViewData[PerViewDataIndex];
		}
	}
};

/** Information about a heightfield gathered by the renderer for heightfield lighting. */
class FHeightfieldComponentDescription
{
public:
	FVector4f HeightfieldScaleBias = FVector4f(ForceInit);
	FVector4f MinMaxUV = FVector4f(ForceInit);
	FMatrix LocalToWorld = FMatrix::Identity;
	FVector2D LightingAtlasLocation = FVector2D(ForceInit);
	FIntRect HeightfieldRect; // Default initialized

	uint32 GPUSceneInstanceIndex = 0;
	int32 NumSubsections = 0;
	FVector4 SubsectionScaleAndBias = FVector4(ForceInit);
	int32 VisibilityChannel;

	FHeightfieldComponentDescription(const FMatrix& InLocalToWorld, uint32 InGPUSceneInstanceIndex) :
		LocalToWorld(InLocalToWorld),
		GPUSceneInstanceIndex(InGPUSceneInstanceIndex),
		VisibilityChannel(-1)
	{}
};

ENGINE_API extern bool ShouldOptimizedWPOAffectNonNaniteShaderSelection();
extern bool IsAllowingApproximateOcclusionQueries();
extern bool CacheShadowDepthsFromPrimitivesUsingWPO();

enum class ERayTracingPrimitiveFlags : uint8
{
	None = 0,

	// Visibility flags:

	// This type of geometry is not supported in ray tracing and all proxies will be excluded
	// If a proxy decides to return UnsupportedProxyType it must be consistent across all proxies of this type
	// This value is used for primitives that return false from FPrimitiveSceneProxy::IsRayTracingRelevant().
	UnsupportedProxyType = 1 << 0,

	// This scene proxy will be excluded, because it decides to be invisible in ray tracing (probably due to other flags)
	// Excluded proxies will skip visibility checks
	Exclude = 1 << 1,

	// Similar to Exclude however scene proxy will still go through culling and run relevant logic when it is deemed visible
	Skip = 1 << 2,

	// Caching flags:

	// Fully dynamic (the ray tracing representation of this scene proxy will be polled every frame)
	// Ray tracing mesh commands generated from this proxy's materials can be cached
	// (not compatible with CacheInstances or ComputeLOD)
	Dynamic = 1 << 3,
	// Instances from this proxy can be cached
	// (not compatible with Dynamic or ComputeLOD)
	CacheInstances = 1 << 4,

	// Misc flags:

	// Static meshes with multiple LODs will want to select a LOD index based on screen size
	// (not compatible with Dynamic or CacheInstances)
	ComputeLOD = 1 << 5, 

	// Primitive is masked as a far field object
	FarField = 1 << 6,

	// Raytracing data is streamable (TODO: support in dynamic path)
	Streaming = 1 << 7,
};
ENUM_CLASS_FLAGS(ERayTracingPrimitiveFlags);

enum class ERayTracingProxyType : uint8
{
	None								= 0,
	StaticMesh							= 1 << 0,
	HierarchicalInstancedStaticMesh		= 1 << 1,
	LandscapeGrass						= 1 << 2,
	NaniteProxy							= 1 << 3,
	SkeletalMesh						= 1 << 4,
	InstanceSkeletalMesh				= 1 << 5,

	// Don't use as ERayTracingProxyType is bitpacked in 7 bits in FPrimitiveRayTracingData
	//DO_NOT_USE						= 1 << 7,

	All									= StaticMesh | HierarchicalInstancedStaticMesh | LandscapeGrass | NaniteProxy | SkeletalMesh | InstanceSkeletalMesh,
};
ENUM_CLASS_FLAGS(ERayTracingProxyType);

/**
 * Encapsulates the data which is mirrored to render a UPrimitiveComponent parallel to the game thread.
 * This is intended to be subclassed to support different primitive types.  
 */
class FPrimitiveSceneProxy
{
public:

	/** Initialization constructor. */
	ENGINE_API FPrimitiveSceneProxy(const UPrimitiveComponent* InComponent, FName ResourceName = NAME_None);
	ENGINE_API FPrimitiveSceneProxy(const FPrimitiveSceneProxyDesc& InDesc, FName ResourceName = NAME_None);

	/** Remaining constructors, and assignment operators. */
	ENGINE_API FPrimitiveSceneProxy();
	ENGINE_API FPrimitiveSceneProxy(FPrimitiveSceneProxy const&);
	ENGINE_API FPrimitiveSceneProxy(FPrimitiveSceneProxy&&);
	ENGINE_API FPrimitiveSceneProxy& operator=(const FPrimitiveSceneProxy&);
	ENGINE_API FPrimitiveSceneProxy& operator=(FPrimitiveSceneProxy&&);
	/** Virtual destructor. */
	ENGINE_API virtual ~FPrimitiveSceneProxy();

	/** Return a type (or subtype) specific hash for sorting purposes */
	virtual SIZE_T GetTypeHash() const = 0;

	/**
	 * Updates selection for the primitive proxy. This simply sends a message to the rendering thread to call SetSelection_RenderThread.
	 * This is called in the game thread as selection is toggled.
	 * @param bInParentSelected - true if the parent actor is selected in the editor
	 * @param bInIndividuallySelected - true if the component is selected in the editor directly
	 */
	ENGINE_API void SetSelection_GameThread(const bool bInParentSelected, const bool bInIndividuallySelected=false);

	/**
	 * Updates the LevelInstance editing state for the primitive proxy. This simply sends a message to the rendering thread to call SetLevelInstanceEditingState_RenderThread.
	 * This is called in the game thread when the object enters/leaves LevelInstance levels.
	 * @param bInEditingState - true if the parent actor belongs to an editing LevelInstance sublevel
	 */
	ENGINE_API void SetLevelInstanceEditingState_GameThread(const bool bInEditingState);

	/**
	 * Updates hover state for the primitive proxy. This simply sends a message to the rendering thread to call SetHovered_RenderThread.
	 * This is called in the game thread as hover state changes
	 * @param bInHovered - true if the parent actor is hovered
	 */
	void SetHovered_GameThread(const bool bInHovered);

	/**
	 * Updates the lighting channels for the primitive proxy.
	 */
	void SetLightingChannels_GameThread(FLightingChannels LightingChannels);

	/**
	 * Updates the hidden editor view visibility map on the game thread which just enqueues a command on the render thread
	 */
	void SetHiddenEdViews_GameThread( uint64 InHiddenEditorViews );

#if WITH_EDITOR
	/**
	 * Enqueue and update for the render thread to notify it that the editor is currently moving the owning component with gizmos.
	 */
	void SetIsBeingMovedByEditor_GameThread(bool bIsBeingMoved);

	/**
	 * Force the proxy to render as though it is selected. Shows outlines and overlays.
	 */
	ENGINE_API void SetSelectionOverride_GameThread(bool bForceSelection);
	
	/**
	 * Enqueue updated selection outline color for the render thread to use.
	 */
	ENGINE_API void SetSelectionOutlineColorIndex_GameThread(uint8 ColorIndex);

	/**
	 * Enqueue updated overlay color for the render thread to use.
	 */
	void SetOverlayColor_GameThread(FColor OverlayColor);
#endif // WITH_EDITOR

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/**
	 * Enqueue and update for the render thread to notify it that the primitive color changed.
	 */
	ENGINE_API void SetPrimitiveColor_GameThread(const FLinearColor& InPrimitiveColor);
#endif	

	/** Enqueue and update for the render thread to remove the velocity data for this component from the scene. */
	ENGINE_API void ResetSceneVelocity_GameThread();

	/** Enqueue updated setting for evaluation of World Position Offset. */
	void SetEvaluateWorldPositionOffset_GameThread(bool bEvaluate);

	virtual void SetWorldPositionOffsetDisableDistance_GameThread(int32 NewValue) {}

	virtual void SetInstanceCullDistance_RenderThread(float StartCullDistance, float EndCullDistance) {}

	/** @return True if the primitive is visible in the given View. */
	ENGINE_API bool IsShown(const FSceneView* View) const;

	/** @return True if the primitive is casting a shadow. */
	ENGINE_API bool IsShadowCast(const FSceneView* View) const;

	/** Helper for components that want to render bounds. */
	ENGINE_API void RenderBounds(FPrimitiveDrawInterface* PDI, const FEngineShowFlags& EngineShowFlags, const FBoxSphereBounds& Bounds, bool bRenderInEditor) const;

	/** Verifies that a material used for rendering was present in the component's GetUsedMaterials list. */
	ENGINE_API bool VerifyUsedMaterial(const class FMaterialRenderProxy* MaterialRenderProxy) const;

	/** Returns the LOD that the primitive will render at for this view. */
	virtual int32 GetLOD(const FSceneView* View) const { return INDEX_NONE; }

	/** 
	* All FPrimitiveSceneProxy derived classes can decide to fully override the HHitProxy
	* creation, or add their own and call any of their base classes to add theirs. 
	* 
	* Classes deriving from FPrimitiveSceneProxy which are meant to be used with 
	* IPrimitiveComponent should override both CreateHitProxies(UPrimitiveComponent*, ...)
	* and CreateHitProxies(IPrimitiveComponent*, ...) and make the UPrimitiveComponent 
	* version call into the IPrimitiveComponent one. This allows their derived classes that
	* are exclusive to UPrimitiveComponent to call into them and to reroute the proxy 
	* creation to the IPrimitiveComponent path.
	*/

	/**
	 * Creates the hit proxies are used when DrawDynamicElements is called.
	 * Called in the game thread.
	 * @param OutHitProxies - Hit proxes which are created should be added to this array.
	 * @return The hit proxy to use by default for elements drawn by DrawDynamicElements.
	 */
	ENGINE_API virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component,TArray<TRefCountPtr<HHitProxy> >& OutHitProxies);

	/**
	 * Creates the hit proxies are used when DrawDynamicElements is called.
	 * Called in the game thread.
	 * @param OutHitProxies - Hit proxes which are created should be added to this array.
	 * @return The hit proxy to use by default for elements drawn by DrawDynamicElements.
	 */
	ENGINE_API virtual HHitProxy* CreateHitProxies(IPrimitiveComponent* ComponentInterface,TArray<TRefCountPtr<HHitProxy> >& OutHitProxies);

#if WITH_EDITOR
	/** 
	 * Allows a scene proxy to override hit proxy ids and generate more than one hit proxy id per draw call
	 * Useful for sub-section selection (faces, vertices, bones, etc)
	 * 
	 * @return The vertex buffer  to be use for custom hit proxy ids or null if not used.  Each color represents an id in a HHitProxy
	 */
	virtual const FColorVertexBuffer* GetCustomHitProxyIdBuffer() const { return nullptr; }
#endif
	/**
	 * Draws the primitive's static elements.  This is called from the rendering thread once when the scene proxy is created.
	 * The static elements will only be rendered if GetViewRelevance declares static relevance.
	 * @param PDI - The interface which receives the primitive elements.
	 */
	virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) {}

	/** Gathers a description of the mesh elements to be rendered for the given LOD index, without consideration for views. */
	virtual void GetMeshDescription(int32 LODIndex, TArray<FMeshBatch>& OutMeshElements) const {}

	/** Gathers shadow shapes from this proxy. */
	virtual void GetShadowShapes(FVector PreViewTranslation, TArray<FCapsuleShape3f>& OutCapsuleShapes) const {}

	virtual bool IsCullingReversedByComponent() const { return false; }

#if RHI_RAYTRACING
	// TODO: remove these individual functions in favor of ERayTracingPrimitiveFlags
	virtual bool IsRayTracingRelevant() const { return false; }
	virtual bool IsRayTracingStaticRelevant() const { return false; }

	/** Return whether proxy has a valid ray tracing representation and can be used for ray tracing. */
	virtual bool HasRayTracingRepresentation() const { return false; }

	/** Gathers dynamic ray tracing instances from this proxy. */
	virtual void GetDynamicRayTracingInstances(class FRayTracingInstanceCollector& Collector) {}

	virtual TArray<FRayTracingGeometry*> GetStaticRayTracingGeometries() const { return {}; }

	ENGINE_API virtual ERayTracingPrimitiveFlags GetRayTracingPrimitiveFlags();
	ENGINE_API virtual void GetStaticRayTracingInstances(struct FStaticRayTracingInstances& OutRayTracingInstances);

	/** 
	 * Gathers static ray tracing primitives from this proxy. 
	 * Fields of the instances can be partially filled, depending on what is desired to be cached, which is described in the returned flags
	 */
	UE_DEPRECATED(5.8, "Use GetRayTracingPrimitiveFlags() and GetStaticRayTracingInstances() instead.")
	ENGINE_API virtual ERayTracingPrimitiveFlags GetCachedRayTracingInstance(struct FRayTracingInstance& OutRayTracingInstance);

	/**
	 * If the ray tracing data is streaming then get the coarse mesh streaming handle 
	 */
	UE_DEPRECATED(5.8, "Coarse Mesh Streaming is deprecated. Use reference based residency instead.")
	virtual Nanite::CoarseMeshStreamingHandle GetCoarseMeshStreamingHandle() const { return INDEX_NONE; }

	/** @return The handle of the ray tracing geometry group used by this primitive */
	virtual RayTracing::FGeometryGroupHandle GetRayTracingGeometryGroupHandle() const { return {}; }

	/** @return The handle of the ray tracing geometry group used by this primitive */
	ENGINE_API ERayTracingProxyType GetRayTracingProxyType() const;

	virtual FRaytracingVFAttributeData GetRaytracingVFAttributeData(const FSceneView* View) { return {}; }
#endif // RHI_RAYTRACING

	virtual void GetStreamableRenderAssetInfo(const FBoxSphereBounds& PrimitiveBounds, TArray<struct FStreamingRenderAssetPrimitiveInfo>& StreamableRenderAssets) const { check(IsSupportingStreamableRenderAssetsGathering()); }
	bool IsSupportingStreamableRenderAssetsGathering() const { return bImplementsStreamableAssetGathering; }
	virtual bool CanApplyStreamableRenderAssetScaleFactor() const { return false; }
	bool IsForceMipStreaming() const { return bForceMipStreaming; }
	
	/** 
	 * Gathers the primitive's dynamic mesh elements.  This will only be called if GetViewRelevance declares dynamic relevance.
	 * This is called from the rendering thread for each set of views that might be rendered.  
	 * Game thread state like UObjects must have their properties mirrored on the proxy to avoid race conditions.  The rendering thread must not dereference UObjects.
	 * The gathered mesh elements will be used multiple times, any memory referenced must last as long as the Collector (eg no stack memory should be referenced).
	 * This function should not modify the proxy but simply collect a description of things to render.  Updates to the proxy need to be pushed from game thread or external events.
	 *
	 * @param Views - the array of views to consider.  These may not exist in the ViewFamily.
	 * @param ViewFamily - the view family, for convenience
	 * @param VisibilityMap - a bit representing this proxy's visibility in the Views array
	 * @param Collector - gathers the mesh elements to be rendered and provides mechanisms for temporary allocations
	 */
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, class FMeshElementCollector& Collector) const {}

	virtual const class FCardRepresentationData* GetMeshCardRepresentation() const { return nullptr; }

	ENGINE_API virtual FMatrix GetMeshCardsToWorld(int32 InstanceIndex) const;

	/** 
	* Gives the primitive an opportunity to override MeshBatch arguments for a specific View
	* Only called for a MeshBatch with a bViewDependentArguments property set
	* @param View - the view to override for
	* @param ViewDependentMeshBatch - view dependent mesh copy (does not affect a cached FMeshBatch)
	*/
	virtual void ApplyViewDependentMeshArguments(const FSceneView& View, FMeshBatch& ViewDependentMeshBatch) const {};

	/** 
	 * Gets the boxes for sub occlusion queries
	 * @param View - the view the occlusion results are for
	 * @return pointer to the boxes, must remain valid until the queries are built
	 */
	virtual const TArray<FBoxSphereBounds>* GetOcclusionQueries(const FSceneView* View) const 
	{
		return nullptr;
	}

	/** 
	 * Gives the primitive the results of sub-occlusion-queries
	 * @param View - the view the occlusion results are for
	 * @param Results - visibility results, allocated from the scene allocator, so valid until the end of the frame
	 * @param NumResults - number of visibility bools
	 */
	virtual void AcceptOcclusionResults(const FSceneView* View, TArray<bool>* Results, int32 ResultsStart, int32 NumResults) {}

	/**
	 * Determines the relevance of this primitive's elements to the given view.
	 * Called in the rendering thread.
	 * @param View - The view to determine relevance for.
	 * @return The relevance of the primitive's elements to the view.
	 */
	ENGINE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const;

	/** Callback from the renderer to gather simple lights that this proxy wants renderered. */
	virtual void GatherSimpleLights(const FSceneViewFamily& ViewFamily, FSimpleLightArray& OutParticleLights) const {}

	/**
	 *	Determines the relevance of this primitive's elements to the given light.
	 *	@param	LightSceneProxy			The light to determine relevance for
	 *	@param	bDynamic (output)		The light is dynamic for this primitive
	 *	@param	bRelevant (output)		The light is relevant for this primitive
	 *	@param	bLightMapped (output)	The light is light mapped for this primitive
	 */
	virtual void GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const
	{
		// Determine the lights relevance to the primitive.
		bDynamic = true;
		bRelevant = true;
		bLightMapped = false;
		bShadowMapped = false;
	}

	virtual void GetDistanceFieldAtlasData(const class FDistanceFieldVolumeData*& OutDistanceFieldData, float& SelfShadowBias) const
	{
		OutDistanceFieldData = nullptr;
		SelfShadowBias = 0;
	}

	virtual bool HeightfieldHasPendingStreaming() const { return false; }

	virtual bool StaticMeshHasPendingStreaming() const { return false; }

	virtual void GetHeightfieldRepresentation(UTexture2D*& OutHeightmapTexture, UTexture2D*& OutVisibilityTexture, FHeightfieldComponentDescription& OutDescription) const
	{
		OutHeightmapTexture = nullptr;
		OutVisibilityTexture = nullptr;
	}

	virtual TConstArrayView<FPrimitiveComponentId> GetSourceLandscapeComponentIds() const { return {}; }

	/**
	 *	Called when the rendering thread adds the proxy to the scene.
	 *	This function allows for generating renderer-side resources.
	 *	Called in the rendering thread.
	 */
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) {}

	/**
	 *	Called when the rendering thread removes the proxy from the scene.
	 *	This function allows for removing renderer-side resources.
	 *	Called in the rendering thread.
	 */
	virtual void DestroyRenderThreadResources() {}

	/**
	 */
	UE_DEPRECATED(5.8, "No longer called.")
	virtual void OnDetachLight(const FLightSceneInfo* Light) final
	{
	}

	/**
	 * Called to notify the proxy when its transform has been updated.
	 * Called in the thread that owns the proxy; game or rendering.
	 */
	virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) { }

	/**
	* @return true if the proxy can be culled when occluded by other primitives
	*/
	virtual bool CanBeOccluded() const
	{
		return true;
	}

	/**
	* @return true if per-pixel visibility tests are allowed for instances of this primitive during GPU instance culling.
	* This may yield significant performance benefit for some cases, such as relatively high-poly instanced static meshes.
	*/
	virtual bool AllowInstanceCullingOcclusionQueries() const
	{
		return false;
	}

	/**
	* @return true if the proxy can skip redundant transform updates where applicable.
	*/
	UE_DEPRECATED(5.8, "Will be removed as it is not generally thread safe.")
	bool CanSkipRedundantTransformUpdates() const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bCanSkipRedundantTransformUpdates;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	* Set whether the proxy can skip redundant transform updates where applicable.
	*/
	UE_DEPRECATED(5.8, "Will be removed as it is not generally thread safe.")
	void SetCanSkipRedundantTransformUpdates(bool bCanSkip)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bCanSkipRedundantTransformUpdates = bCanSkip;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	* @return true if the proxy uses distance cull fade.
	*/
	virtual bool IsUsingDistanceCullFade() const
	{
		return false;
	}

	/**
	* @return true if the proxy has custom occlusion queries
	*/
	virtual bool HasSubprimitiveOcclusionQueries() const
	{
		return false;
	}

	virtual bool ShowInBSPSplitViewmode() const
	{
		return false;
	}

	/**
	 * Determines the DPG to render the primitive in regardless of view.
	 * Should only be called if HasViewDependentDPG()==true.
	 */
	virtual ESceneDepthPriorityGroup GetStaticDepthPriorityGroup() const
	{
		check(!HasViewDependentDPG());
		return (ESceneDepthPriorityGroup)StaticDepthPriorityGroup;
	}

	/**
	 * Determines the DPG to render the primitive in for the given view.
	 * May be called regardless of the result of HasViewDependentDPG.
	 * @param View - The view to determine the primitive's DPG for.
	 * @return The DPG the primitive should be rendered in for the given view.
	 */
	ENGINE_API ESceneDepthPriorityGroup GetDepthPriorityGroup(const FSceneView* View) const;

	/** Every derived class should override these functions */
	virtual uint32 GetMemoryFootprint( void ) const = 0;
	SIZE_T GetAllocatedSize( void ) const { return( Owners.GetAllocatedSize() ); }

	/**
	 * Set the collision flag on the scene proxy to enable/disable collision drawing
	 *
	 * @param const bool bNewEnabled new state for collision drawing
	 */
	void SetCollisionEnabled_GameThread(const bool bNewEnabled);

	/**
	 * Set the collision flag on the scene proxy to enable/disable collision drawing (RENDER THREAD)
	 *
	 * @param const bool bNewEnabled new state for collision drawing
	 */
	void SetCollisionEnabled_RenderThread(const bool bNewEnabled);

	/**
	* Set the custom depth enabled flag
	*
	* @param the new value
	*/
	void SetCustomDepthEnabled_GameThread(const bool bInRenderCustomDepth);

	/**
	* Set the custom depth enabled flag (RENDER THREAD)
	*
	* @param the new value
	*/
	void SetCustomDepthEnabled_RenderThread(const bool bInRenderCustomDepth);

	/**
	* Set the custom depth stencil value
	*
	* @param the new value
	*/
	void SetCustomDepthStencilValue_GameThread(const int32 InCustomDepthStencilValue );

	/**
	* Set the custom depth stencil value (RENDER THREAD)
	*
	* @param the new value
	*/
	void SetCustomDepthStencilValue_RenderThread(const int32 InCustomDepthStencilValue);

	void SetDistanceFieldSelfShadowBias_RenderThread(float NewBias);

	ENGINE_API void SetDrawDistance_RenderThread(float MinDrawDistance, float MaxDrawDistance, float VirtualTextureMaxDrawDistance);

	// Accessors.
	inline FSceneInterface& GetScene() const { return *Scene; }
	inline FPrimitiveComponentId GetPrimitiveComponentId() const { return PrimitiveComponentId; }
	inline FPrimitiveSceneInfo* GetPrimitiveSceneInfo() const { return PrimitiveSceneInfo; }
	inline const FMatrix& GetLocalToWorld() const { return LocalToWorld; }
	inline bool IsLocalToWorldDeterminantNegative() const { return bIsLocalToWorldDeterminantNegative; }
	inline const FBoxSphereBounds& GetBounds() const { return Bounds; }
	inline const FBoxSphereBounds& GetLocalBounds() const { return LocalBounds; }
	ENGINE_API virtual void GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const;
	inline FName GetOwnerName() const { return OwnerName; }
	inline FName GetResourceName() const { return ResourceName; }
	inline FName GetLevelName() const { return LevelName; }
	inline TStatId GetStatId() const 
	{ 
		return StatId; 
	}	
	inline float GetMinDrawDistance() const { return MinDrawDistance; }
	inline float GetMaxDrawDistance() const { return MaxDrawDistance; }
	inline int32 GetVisibilityId() const { return VisibilityId; }
	inline int16 GetTranslucencySortPriority() const { return TranslucencySortPriority; }
	inline float GetTranslucencySortDistanceOffset() const { return TranslucencySortDistanceOffset; }

	inline int32 GetVirtualTextureLodBias() const { return VirtualTextureLodBias; }
	inline int32 GetVirtualTextureCullMips() const { return VirtualTextureCullMips; }
	inline int32 GetVirtualTextureMinCoverage() const {	return VirtualTextureMinCoverage; }
	inline const TArray<int32>& GetRuntimeVirtualTextureIds() const { return RuntimeVirtualTextures; }

	inline bool IsMovable() const 
	{ 
		// Note: primitives with EComponentMobility::Stationary can still move (as opposed to lights with EComponentMobility::Stationary)
		return Mobility == EComponentMobility::Movable || Mobility == EComponentMobility::Stationary; 
	}

	inline bool IsOftenMoving() const { return Mobility == EComponentMobility::Movable; }

	inline bool IsMeshShapeOftenMoving() const 
	{ 
		return Mobility == EComponentMobility::Movable || !bGoodCandidateForCachedShadowmap; 
	}

	inline ELightmapType GetLightmapType() const { return LightmapType; }
	inline bool IsStatic() const { return Mobility == EComponentMobility::Static; }
	inline bool IsSelectable() const { return bSelectable; }
	inline bool IsParentSelected() const { return bParentSelected; }
	inline bool IsIndividuallySelected() const { return bIndividuallySelected; }
	inline bool IsEditingLevelInstanceChild() const { return bLevelInstanceEditingState; }
	inline bool IsSelected() const { return IsParentSelected() || IsIndividuallySelected(); }
	inline bool WantsEditorEffects() const { return bWantsEditorEffects; }
	inline bool WantsSelectionOutline() const { return bWantsSelectionOutline; }
	ENGINE_API bool ShouldRenderCustomDepth() const;
	inline bool IsVisibleInSceneCaptureOnly() const { return bVisibleInSceneCaptureOnly; }
	inline bool IsHiddenInSceneCapture() const { return bHiddenInSceneCapture; }
	inline uint8 GetCustomDepthStencilValue() const { return CustomDepthStencilValue; }
	inline EStencilMask GetStencilWriteMask() const { return CustomDepthStencilWriteMask; }
	inline uint8 GetLightingChannelMask() const { return LightingChannelMask; }
	inline uint8 GetLightingChannelStencilValue() const 
	{ 
		// Flip the default channel bit so that the default value is 0, to align with the default stencil clear value and GBlackTexture value
		return (LightingChannelMask & 0x6) | (~LightingChannelMask & 0x1); 
	}
	inline bool IsVisibleInReflectionCaptures() const { return bVisibleInReflectionCaptures; }
	inline bool IsVisibleInRealTimeSkyCaptures() const { return bVisibleInRealTimeSkyCaptures; }
	inline bool IsVisibleInRayTracing() const { return bVisibleInRayTracing; }
	inline bool IsVisibleInReflections() const { return bVisibleInReflections; }
	inline bool IsVisibleInLumenScene() const { return bVisibleInLumenScene; }
	inline bool IsOpaqueOrMasked() const { return bOpaqueOrMasked; }
	inline bool SupportsMaterialCache() const { return bSupportsMaterialCache; }
	inline bool ShouldRenderInMainPass() const { return bRenderInMainPass; }
	inline bool ShouldRenderInDepthPass() const { return bRenderInMainPass || bRenderInDepthPass; }
	inline bool SupportsParallelGDME() const { return bSupportsParallelGDME; }
	inline bool SinglePassGDME() const { return bSinglePassGDME; }
	inline bool SupportsParallelGDRTI() const { return bSupportsParallelGDRTI; }
	inline bool IsCollisionEnabled() const { return bCollisionEnabled; }
	inline bool IsHovered() const { return bHovered; }
	inline bool IsOwnedBy(const FSceneViewOwner& Owner) const { return Owner.IsPartOf(Owners); }
	inline bool HasViewDependentDPG() const { return bUseViewOwnerDepthPriorityGroup; }
	inline bool HasStaticLighting() const { return bStaticLighting; }
	inline bool NeedsUnbuiltPreviewLighting() const { return bNeedsUnbuiltPreviewLighting; }
	inline bool CastsStaticShadow() const { return bCastStaticShadow; }
	inline bool CastsDynamicShadow() const { return bCastDynamicShadow; }
	inline bool IsEmissiveLightSource() const { return bEmissiveLightSource; }
	inline bool WritesVirtualTexture() const{ return bSupportsRuntimeVirtualTexture && RuntimeVirtualTextures.Num() > 0; }
	inline bool AffectsDynamicIndirectLighting() const { return bAffectDynamicIndirectLighting; }
	inline bool AffectsDistanceFieldLighting() const { return bAffectDistanceFieldLighting; }
	inline bool AffectsIndirectLightingWhileHidden() const { return bAffectIndirectLightingWhileHidden; }
	inline EIndirectLightingCacheQuality GetIndirectLightingCacheQuality() const { return IndirectLightingCacheQuality; }
	inline bool CastsVolumetricTranslucentShadow() const { return bCastVolumetricTranslucentShadow; }
	inline bool CastsContactShadow() const { return bCastContactShadow; }
	inline bool CastsDeepShadow() const { return bCastDeepShadow; }
	inline bool CastsCapsuleDirectShadow() const { return bCastCapsuleDirectShadow; }
	inline bool CastsDynamicIndirectShadow() const { return bCastsDynamicIndirectShadow; }
	inline float GetDynamicIndirectShadowMinVisibility() const { return DynamicIndirectShadowMinVisibility; }
	inline bool CastsHiddenShadow() const { return bCastHiddenShadow; }
	inline bool CastsShadowAsTwoSided() const { return bCastShadowAsTwoSided; }
	inline bool CastsSelfShadowOnly() const { return bSelfShadowOnly; }
	inline bool CastsInsetShadow() const { return bCastInsetShadow; }
	inline bool CastsCinematicShadow() const { return bCastCinematicShadow; }
	inline bool CastsFarShadow() const { return bCastFarShadow; }
	inline bool LightAttachmentsAsGroup() const { return bLightAttachmentsAsGroup; }
	ENGINE_API bool UseSingleSampleShadowFromStationaryLights() const;
	inline bool StaticElementsAlwaysUseProxyPrimitiveUniformBuffer() const { return bStaticElementsAlwaysUseProxyPrimitiveUniformBuffer; }
	inline bool DoesVFRequirePrimitiveUniformBuffer() const { return bVFRequiresPrimitiveUniformBuffer; }

	/** Whether instance data only exists in the GPU Scene (instance data not present in CPU memory). */
	ENGINE_API bool IsInstanceDataGPUOnly() const;

	/** 
	 * Returns true to inform scene update that the mesh batches produced makes use of the (GPU)Scene instance count, and thus don't require recaching if the instance count changed. 
	 * Defaults to false, the proxy should only opt in if the above condition is true (or risk GPU-crashes).
	 * Requires FMeshBatchElement::bFetchInstanceCountFromScene to be true.
	 */
	inline bool DoesMeshBatchesUseSceneInstanceCount() const { return bDoesMeshBatchesUseSceneInstanceCount; }

	inline bool ShouldUseAsOccluder() const { return bUseAsOccluder; }
	inline bool AllowApproximateOcclusion() const { return bAllowApproximateOcclusion; }
	inline bool Holdout() const { return bHoldout; }
	inline bool IsSplineMesh() const { return bSplineMesh; }
	inline bool IsSkinnedMesh() const { return bSkinnedMesh; }
	inline bool IsFirstPerson() const { return bIsFirstPerson; }
	inline bool IsFirstPersonWorldSpaceRepresentation() const { return bIsFirstPersonWorldSpaceRepresentation; }
	inline bool IsOwnerNoSee() const { return bOwnerNoSee; }
	inline bool IsOnlyOwnerSee() const { return bOnlyOwnerSee; }

	inline FRHIUniformBuffer* GetUniformBuffer() const
	{
		return UniformBuffer.GetReference(); 
	}

	inline bool HasPerInstanceHitProxies() const { return bHasPerInstanceHitProxies; }

#if WITH_EDITOR
	inline uint8 GetSelectionOutlineColorIndex() const { return SelectionOutlineColorIndex; }
	inline FColor GetOverlayColor() const { return OverlayColor; }
#endif // WITH_EDITOR
	
	ENGINE_API bool UseEditorCompositing(const FSceneView* View) const;
	inline const FVector& GetActorPosition() const { return ActorPosition; }
	inline const bool ReceivesDecals() const { return bReceivesDecals; }
	inline bool WillEverBeLit() const { return bWillEverBeLit; }
	inline bool HasValidSettingsForStaticLighting() const { return bHasValidSettingsForStaticLighting; }
	inline bool SupportsDistanceFieldRepresentation() const { return bSupportsDistanceFieldRepresentation; }
	inline bool SupportsHeightfieldRepresentation() const { return bSupportsHeightfieldRepresentation; }
	inline bool IsLumenHeightfield() const { return bLumenHeightfield; }
	inline bool IsLandscapeProxy() const { return bIsLandscapeProxy; }
	inline bool IsLandscapeNaniteProxy() const { return bIsLandscapeNaniteProxy; }
	inline bool IsCompatibleWithLumenCardSharing() const { return bCompatibleWithLumenCardSharing; }
	inline bool SupportsInstanceDataBuffer() const { return InstanceSceneDataBuffersInternal != nullptr; }
	inline bool SupportsSortedTriangles() const { return bSupportsSortedTriangles; }
	inline bool TreatAsBackgroundForOcclusion() const { return bTreatAsBackgroundForOcclusion; }
	inline bool ShouldNotifyOnWorldAddRemove() const { return bShouldNotifyOnWorldAddRemove; }
	inline bool IsForceHidden() const {return bForceHidden;}
	inline bool ShouldReceiveMobileCSMShadows() const { return bReceiveMobileCSMShadows; }
	inline bool IsRayTracingFarField() const { return bRayTracingFarField; }
	inline int32 GetRayTracingGroupId() const { return RayTracingGroupId; }
	inline uint8 GetRayTracingGroupCullingPriority() const { return RayTracingGroupCullingPriority; }

	static constexpr int32 InvalidRayTracingGroupId = -1;

	inline bool EvaluateWorldPositionOffset() const { return bEvaluateWorldPositionOffset; }
	inline bool AnyMaterialHasWorldPositionOffset() const { return bAnyMaterialHasWorldPositionOffset; }
	inline bool AnyMaterialAlwaysEvaluatesWorldPositionOffset() const { return bAnyMaterialAlwaysEvaluatesWorldPositionOffset; }
	inline bool AnyMaterialHasPixelAnimation() const { return bAnyMaterialHasPixelAnimation; }
	inline bool AnyMaterialUsesTemporalResponsiveness() const { return bAnyMaterialUsesTemporalResponsiveness; }
	inline bool AnyMaterialUsesMotionVectorWorldOffset() const { return bAnyMaterialUsesMotionVectorWorldOffset; }
	inline float GetMaxWorldPositionOffsetExtent() const
	{
		if ((EvaluateWorldPositionOffset() && AnyMaterialHasWorldPositionOffset())
			|| AnyMaterialAlwaysEvaluatesWorldPositionOffset())
		{
			return MaxWPOExtent;
		}
		return 0.0f;
	}

	inline const FVector2f& GetMinMaxMaterialDisplacement() const
	{
		return MinMaxMaterialDisplacement;
	}

	inline float GetAbsMaxDisplacement() const
	{
		// TODO: DISP - Fix me
		const float AbsMaxMaterialDisplacement = FMath::Max(-GetMinMaxMaterialDisplacement().X, GetMinMaxMaterialDisplacement().Y);
		return GetMaxWorldPositionOffsetExtent() + AbsMaxMaterialDisplacement;
	}
	
	/** Returns true if this proxy can change transform so that we should cache previous transform for calculating velocity. */
	inline bool HasDynamicTransform() const { return IsMovable() || bIsBeingMovedByEditor; }
	/** Returns true if this proxy can write velocity. This is used for setting velocity relevance. */
	inline bool DrawsVelocity() const { return HasDynamicTransform() || bAlwaysHasVelocity || AnyMaterialHasPixelAnimation() || bHasWorldPositionOffsetVelocity || AnyMaterialUsesTemporalResponsiveness() || AnyMaterialUsesMotionVectorWorldOffset(); }
	/** Returns true if this proxy should write velocity even when the transform isn't changing. Usually this is combined with a check for the transform changing. */
	inline bool AlwaysHasVelocity() const {	return bAlwaysHasVelocity || AnyMaterialHasPixelAnimation() || (bHasWorldPositionOffsetVelocity && EvaluateWorldPositionOffset()) || AnyMaterialUsesTemporalResponsiveness() || AnyMaterialUsesMotionVectorWorldOffset(); }

#if WITH_EDITOR
	inline int32 GetNumUncachedStaticLightingInteractions() { return NumUncachedStaticLightingInteractions; }

	/** This function exist only to perform an update of the UsedMaterialsForVerification on the render thread*/
	ENGINE_API void SetUsedMaterialForVerification(const TArray<UMaterialInterface*>& InUsedMaterialsForVerification);
#endif

#if !UE_BUILD_TEST
	inline FLinearColor GetWireframeColor() const { return WireframeColor; }
	inline FLinearColor GetPrimitiveColor() const { return PrimitiveColor; }
	inline void SetWireframeColor(const FLinearColor& InWireframeColor) { WireframeColor = InWireframeColor; }
	inline void SetPrimitiveColor(const FLinearColor& InPrimitiveColor) { PrimitiveColor = InPrimitiveColor; }
#else
	inline FLinearColor GetWireframeColor() const { return FLinearColor::White; }
	inline FLinearColor GetPrimitiveColor() const { return FLinearColor::White; }
	inline void SetWireframeColor(const FLinearColor& InWireframeColor) {}
	inline void SetPrimitiveColor(const FLinearColor& InPrimitiveColor) {}
#endif

	/**
	* Returns whether this proxy should be considered a "detail mesh".
	* Detail meshes are distance culled even if distance culling is normally disabled for the view. (e.g. in editor)
	*/
	virtual bool IsDetailMesh() const
	{
		return false;
	}

	virtual FSkinningSceneExtensionProxy* GetSkinningSceneExtensionProxy() const
	{
		return nullptr;
	}

	/**
	* Returns whether this proxy is a Nanite mesh.
	*/
	inline bool IsNaniteMesh() const
	{
		return bIsNaniteMesh;
	}

	/**
	* Returns whether this proxy is always visible.
	*/
	inline bool IsAlwaysVisible() const
	{
		return bIsAlwaysVisible;
	}

	/**
	* Returns whether this proxy is a heterogeneous volume.
	*/
	inline bool IsHeterogeneousVolume() const
	{
		return bIsHeterogeneousVolume;
	}

	/**
	 * Returns true if all meshes drawn by this proxy support GPU scene. 
	 */
	inline bool SupportsGPUScene() const
	{
		return bSupportsGPUScene;
	}

	/**
	 * Returns true if this proxy has any deformable mesh, meaning the mesh is animated e.g., by deforming the vertices through skinning, morphing or some procedural update.
	 * WPO and PDO are not considered here (as they are material effects).
	 */
	inline bool HasDeformableMesh() const
	{
		return bHasDeformableMesh;
	}

	/**
	 *	Returns whether the proxy utilizes custom occlusion bounds or not
	 *
	 *	@return	bool		true if custom occlusion bounds are used, false if not;
	 */
	virtual bool HasCustomOcclusionBounds() const
	{
		return false;
	}

	/**
	 *	Return the custom occlusion bounds for this scene proxy.
	 *	
	 *	@return	FBoxSphereBounds		The custom occlusion bounds.
	 */
	virtual FBoxSphereBounds GetCustomOcclusionBounds() const
	{
		checkf(false, TEXT("GetCustomOcclusionBounds> Should not be called on base scene proxy!"));
		return GetBounds();
	}

	virtual bool HasDistanceFieldRepresentation() const
	{
		return false;
	}

	virtual bool HasDynamicIndirectShadowCasterRepresentation() const
	{
		return false;
	}

	/**
	 * Retrieves the instance draw distance range (mostly only used by objects whose instances are culled on the GPU)
	 * 
	 * @param OutDistanceMinMax	contains the min/max camera distance of the primitive's instances when enabled
	 * @return bool 			true if camera distance culling is enabled for this primitive's instances
	 **/
	virtual bool GetInstanceDrawDistanceMinMax(FVector2f& OutDistanceMinMax) const
	{
		OutDistanceMinMax = FVector2f(0.0f);
		return false;
	}
	
	/**
	 * Note: should not modify the OutInstancingFadeOutParams output unless it actually has a valid output - the default is initialized.
	 */
	virtual bool GetInstancingFadeOutParams(FVector2f& OutInstancingFadeOutParams) const
	{
		return false;
	}

	/**
	 * Retrieves the per-instance world position offset disable distance
	 * 
	 * @param OutWPODisableDistance	contains the distance from the camera at which the primitive's instances disable WPO
	 * @return bool 				true if WPO disable distance is enabled for this primitive's instances
	 **/
	virtual bool GetInstanceWorldPositionOffsetDisableDistance(float& OutWPODisableDistance) const
	{
		OutWPODisableDistance = 0.0f;
		return false;
	}

	/**
	 * return < 0.0f to indicate no cutoff, 0.0f to use the global default and any value > 0.0f to indicate the relative screen space footprint size below which to stop animations.
	 */
	virtual float GetAnimationMinScreenSize() const  { return -1.0f; }

	virtual FRenderCurveResourceData* GetRenderCurveResourceData()
	{
		return nullptr;
	}

	/** 
	 * Drawing helper. Draws nice bouncy line.
	 */
	static ENGINE_API void DrawArc(FPrimitiveDrawInterface* PDI, const FVector& Start, const FVector& End, const float Height, const uint32 Segments, const FLinearColor& Color
		, uint8 DepthPriorityGroup,	const float Thickness = 0.0f, const bool bScreenSpace = false);
	
	static ENGINE_API void DrawArrowHead(FPrimitiveDrawInterface* PDI, const FVector& Tip, const FVector& Origin, const float Size, const FLinearColor& Color
		, uint8 DepthPriorityGroup,	const float Thickness = 0.0f, const bool bScreenSpace = false);


	/**
	 * Shifts primitive position and all relevant data by an arbitrary delta.
	 * Called on world origin changes
	 * @param InOffset - The delta to shift by
	 */
	ENGINE_API virtual void ApplyWorldOffset(FRHICommandListBase& RHICmdList, FVector InOffset);

	/**
	 * Applies a "late in the frame" adjustment to the proxy's existing transform
	 * @param LateUpdateTransform - The post-transform to be applied to the LocalToWorld matrix
	 */
	ENGINE_API virtual void ApplyLateUpdateTransform(FRHICommandListBase& RHICmdList, const FMatrix& LateUpdateTransform);

	/**
	 * Updates the primitive proxy's uniform buffer.
	 */
	ENGINE_API void UpdateUniformBuffer(FRHICommandList& RHICmdList);

	/**
	 * Apply the unform shader parameter settings for the proxy to the builder.
	 */
	ENGINE_API void BuildUniformShaderParameters(FPrimitiveUniformShaderParametersBuilder &Builder) const;

#if ENABLE_DRAW_DEBUG

	struct FDebugMassData
	{
		//Local here just means local to ElemTM which can be different depending on how the component uses the mass data
		FQuat LocalTensorOrientation;
		FVector LocalCenterOfMass;
		FVector MassSpaceInertiaTensor;
		int32 BoneIndex;

		ENGINE_API void DrawDebugMass(class FPrimitiveDrawInterface* PDI, const FTransform& ElemTM) const;
	};

	TArray<FDebugMassData> DebugMassData;

	/** Sets the primitive proxy's mass space to component space. Useful for debugging physics center of mass and inertia tensor*/
	ENGINE_API virtual void SetDebugMassData(const TArray<FDebugMassData>& InDebugMassData);
#endif

#if MESH_DRAW_COMMAND_STATS
	inline FName GetMeshDrawCommandStatsCategory() const { return MeshDrawCommandStatsCategory; }
#endif

	/**
	 * Get the list of LCIs. Used to set the precomputed lighting uniform buffers, which can only be created by the RENDERER_API.
	 */
	typedef TArray<class FLightCacheInterface*, TInlineAllocator<8> > FLCIArray;
	virtual void GetLCIs(FLCIArray& LCIs) {}

#if WITH_EDITORONLY_DATA
	/**
	 * Get primitive distance to view origin for a given LOD-section.
	 * @param LODIndex					LOD index (INDEX_NONE for all) 
	 * @param ElementIndex				Element index (INDEX_NONE for all)
	 * @param PrimitiveDistance (OUT)	LOD-section distance to view
	 * @return							Whether distance was computed or not
	 */
	ENGINE_API virtual bool GetPrimitiveDistance(int32 LODIndex, int32 SectionIndex, const FVector& ViewOrigin, float& PrimitiveDistance) const;

	/**
	 * Get mesh UV density for a LOD-section.
	 * @param LODIndex					LOD index (INDEX_NONE for all) 
	 * @param ElementIndex				Element index (INDEX_NONE for all)
	 * @param WorldUVDensities (OUT)	UV density in world units for each UV channel
	 * @return							Whether the densities were computed or not.
	 */
	ENGINE_API virtual bool GetMeshUVDensities(int32 LODIndex, int32 SectionIndex, FVector4& WorldUVDensities) const;

	/**
	 * Get mesh UV density for a LOD-section.
	 * @param LODIndex					LOD index (INDEX_NONE for all) 
	 * @param ElementIndex				Element index (INDEX_NONE for all)
	 * @param MaterialRenderProxy		Material bound to that LOD-section
	 * @param OneOverScales (OUT)		One over the texture scales (array size = TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL / 4)
	 * @param UVChannelIndices (OUT)	The related index for each (array size = TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL / 4)
	 * @return							Whether scales were computed or not.
	 */
	ENGINE_API virtual bool GetMaterialTextureScales(int32 LODIndex, int32 SectionIndex, const class FMaterialRenderProxy* MaterialRenderProxy, FVector4f* OneOverScales, FIntVector4* UVChannelIndices) const;
#endif

	/**
	* Get the lightmap resolution for this primitive. Used in VMI_LightmapDensity.
	*/
	virtual int32 GetLightMapResolution() const { return 0; }

	/**
	 * Get the lightmap UV coordinate index for this primitive. Used by systems that explicitly fetch lightmap UVs.
	 */
	virtual int32 GetLightMapCoordinateIndex() const { return INDEX_NONE; }

	/** Tell us if this proxy is drawn in game.*/
	inline bool IsDrawnInGame() const { return DrawInGame; }

	UE_DEPRECATED(5.8, "Use SetIsDrawn_RenderThread instead.")
	inline void SetDrawnInGame_RenderThread(bool bInDrawInGame) {}

	/**
	 * Set the DrawInGame and the DrawInEditor flags, from the render thread.
	 * @param bIsDrawn		The new value.
 	 */
	ENGINE_API void SetIsDrawn_RenderThread(bool bIsDrawn);

	/** Tell us if this proxy is drawn in editor.*/
	inline bool IsDrawnInEditor() const { return DrawInEditor; }

	virtual uint8 GetCurrentFirstLODIdx_RenderThread() const { return 0; }

#if !UE_BUILD_SHIPPING
	/** Returns max(0, WantedLODs - ResidentLODs) for streaming deficit debug visualization. */
	virtual uint8 GetStreamingLODWantedDeficit() const { return 0; }
#endif

	virtual FDesiredLODLevel GetDesiredLODLevel_RenderThread(const FSceneView* View) const { return FDesiredLODLevel::CreateFirst(0); }

	/** Returns a scale to apply to ScreenSize used in LOD calculation. */
	virtual float GetLodScreenSizeScale() const { return 1.f; }

	/** Returns the instance radius to use for per instance GPU LOD calculation. Returns 0.f if GPU LOD isn't enabled on the primitive. */
	virtual float GetGpuLodInstanceRadius() const { return 0.f; }

	/** */
	virtual FUintVector2 GetMeshPaintTextureDescriptor() const { return FUintVector2(0, 0); }

	/** Get the descriptor of the material cache, may be invalid if none allocated */
	ENGINE_API virtual uint32 GetMaterialCacheDescriptor() const;

	/** 
	 * Get the custom primitive data for this scene proxy.
	 * @return The payload of custom data that will be set on the primitive and accessible in the material through a material expression.
	 */
	const FCustomPrimitiveData* GetCustomPrimitiveData() const { return &CustomPrimitiveData; }

	EShadowCacheInvalidationBehavior GetShadowCacheInvalidationBehavior() const { return ShadowCacheInvalidationBehavior; }

	enum class EInstanceBufferAccessFlags
	{
		SynchronizeUpdateTask,
		UnsynchronizedAndUnsafe,
	};

	inline bool HasInstanceDataBuffers() const { return InstanceSceneDataBuffersInternal != nullptr; }


	/**
	 * Get the instance data view, which may be null for uninstanced primitives. The pointer must be kept valid for as long as the proxy lives, as a copy is cached in FPrimitiveSceneInfo.
	 */
	ENGINE_API const FInstanceSceneDataBuffers *GetInstanceSceneDataBuffers(EInstanceBufferAccessFlags AccessFlags = EInstanceBufferAccessFlags::SynchronizeUpdateTask) const;

	/**
	 * Return a pointer to a class that can be used to guard access to instance data that is being updated by a task.
	 * The proxy must guarantee the lifetime of this object, and, since it is being used on the render thread be careful about how it is updated.
	 * In general, it must always be updated on the RT if it is visible to the RT.
	 */
	virtual FInstanceDataUpdateTaskInfo *GetInstanceDataUpdateTaskInfo() const { return nullptr; }

	/**
	 */
	ENGINE_API FInstanceDataBufferHeader GetInstanceDataHeader() const;

#if UE_WITH_PSO_PRECACHING
	ENGINE_API void BoostPrecachedPSORequestsOnDraw();
#endif

protected:
	ENGINE_API void SetupInstanceSceneDataBuffers(const FInstanceSceneDataBuffers* InInstanceSceneDataBuffers);

	/** Updates bVisibleInLumen, which indicated whether a primitive should be tracked by Lumen scene. Checks if primitive can be ray traced and if it can by captured by surface cache. */
	ENGINE_API void UpdateVisibleInLumenScene();

	/** Returns true if a primitive can never be rendered outside of a runtime virtual texture. */
	bool IsRuntimeVirtualTextureOnly() const { return bVirtualTextureMainPassDrawNever; }

	/** Returns true if a primitive should currently be hidden because it is drawn only to the runtime virtual texture. The result can depend on the current scene state. */
	bool DrawInRuntimeVirtualTextureOnly(bool bEditor) const;

	/** Allow subclasses to override the primitive name. Used primarily by BSP. */
	void OverrideOwnerName(FName InOwnerName)
	{
		OwnerName = InOwnerName;
	}

	/**
	 * Called to notify the proxy that the level has been fully added to
	 * the world and the primitive will now be rendered.
	 * Only called if bShouldNotifyOnWorldAddRemove is set to true.
	 */
	ENGINE_API void OnLevelAddedToWorld_RenderThread();

	/**
	 * Called to notify the proxy that the level has been fully removed from
	 * the world and the primitive will not be rendered.
	 * Only called if bShouldNotifyOnWorldAddRemove is set to true.
	 */
	ENGINE_API void OnLevelRemovedFromWorld_RenderThread();

	void SetForceHidden(bool bForceHiddenIn)
	{
		if (bForceHidden != bForceHiddenIn)
		{
			bForceHidden = bForceHiddenIn;
			OnForceHiddenChanged();
		}
	}

	virtual void OnForceHiddenChanged() {}

	/** 
	 * Call during setup to set flags to indicate GPU-Scene support for the proxy if GPU-Scene is enabled & supported for the current feature level.
	 */
	ENGINE_API void EnableGPUSceneSupportFlags();

private:
	/**
	 * Internally used function that sets up the parameters that are not handled by the Scene BuildPrimitiveUniformShaderParameters_RenderThread implementations.
     * In general, this should not be expanded on, as the data representation should be kept internal in the renderer.
	 */
	ENGINE_API void BuildUniformShaderParametersInternal(FPrimitiveUniformShaderParametersBuilder& Builder) const;

public:
	/** Thread-safe accessor for this proxy's SimpleStreamableAssetManager (SSAM) handle. Returns
	 *  the default-invalid handle if the proxy was never registered (SSAM disabled, or
	 *  ClearPrimitiveHandle already ran). */
	FSSAMPrimitiveHandle GetSSAMPrimitiveHandle() const { return SSAMPrimitiveHandle; }

private:
	friend class FSimpleStreamableAssetManager;
	/** SSAM handle for this proxy. Paired with UPrimitiveComponent::SSAMPrimitiveHandle when
	 *  a matching UPrimitiveComponent exists. Mutated post-publication by Assign/Clear paths
	 *  (GT/PGT when a UPrimitiveComponent is bound, or true worker threads for proxy-only
	 *  callers like FastGeo's async render state creation) and by the RT FRemovePrimitiveCommand
	 *  teardown; concurrent readers on RT/PGT call GetSSAMPrimitiveHandle(). */
	FSSAMAtomicPrimitiveHandle SSAMPrimitiveHandle;


#if !UE_BUILD_TEST
	FLinearColor WireframeColor;
	FLinearColor PrimitiveColor;
#endif

	friend class FScene;

	/** Custom primitive data */
	FCustomPrimitiveData CustomPrimitiveData;

	/** The translucency sort priority */
	int16 TranslucencySortPriority;

	/** Translucent sort distance offset */
	float TranslucencySortDistanceOffset;

	TEnumAsByte<EComponentMobility::Type> Mobility;
	ELightmapType LightmapType;

	/** Used for dynamic stats */
	TStatId StatId;

	uint8 bIsLocalToWorldDeterminantNegative : 1;
	uint8 DrawInGame : 1;
	uint8 DrawInEditor : 1;
	uint8 bReceivesDecals : 1;
	uint8 bVirtualTextureMainPassDrawAlways : 1;
	uint8 bVirtualTextureMainPassDrawNever : 1;
	uint8 bOnlyOwnerSee : 1;
	uint8 bOwnerNoSee : 1;
	uint8 bOftenMoving : 1;
	/** Parent Actor is selected */
	uint8 bParentSelected : 1;
	/** Component is selected directly */
	uint8 bIndividuallySelected : 1;
	/** Component belongs to an Editing LevelInstance */
	uint8 bLevelInstanceEditingState : 1;
	
	/** true if the mouse is currently hovered over this primitive in a level viewport */
	uint8 bHovered : 1;

	/** true if ViewOwnerDepthPriorityGroup should be used. */
	uint8 bUseViewOwnerDepthPriorityGroup : 1;

	/** DPG this prim belongs to. */
	uint8 StaticDepthPriorityGroup : SDPG_NumBits;

	/** DPG this primitive is rendered in when viewed by its owner. */
	uint8 ViewOwnerDepthPriorityGroup : SDPG_NumBits;

	/** True if the primitive will cache static lighting. */
	uint8 bStaticLighting : 1;

	/** True if the primitive should be visible in reflection captures. */
	uint8 bVisibleInReflectionCaptures : 1;

	/** True if the primitive should be visible in real-time sky light reflection captures. */
	uint8 bVisibleInRealTimeSkyCaptures : 1;

	/** If true, this component will be visible in ray tracing effects. Turning this off will remove it from ray traced reflections, shadows, etc. */
	uint8 bVisibleInRayTracing : 1;

	/** If true, this component will be visible in ray traced reflections. */
	uint8 bVisibleInReflections : 1;

	/** If true this primitive Renders in the depthPass */
	uint8 bRenderInDepthPass : 1;

	/** If true this primitive Renders in the mainPass */
	uint8 bRenderInMainPass : 1;

	/** If true this primitive is hidden (used when level is not yet visible) */
	uint8 bForceHidden : 1;

	/** Whether this component has any collision enabled */
	uint8 bCollisionEnabled : 1;

	/** Whether the primitive should be treated as part of the background for occlusion purposes. */
	uint8 bTreatAsBackgroundForOcclusion : 1;

	friend class FLightPrimitiveInteraction;
	friend class FMaterialCacheSceneExtensionUpdater;

protected:

	/** Whether the proxy supports asynchronously calling GetDynamicMeshElements. If disabled, all calls for various proxies are serialized with respect to each other. */
	uint8 bSupportsParallelGDME : 1;

	/**
	 * Whether to call GetDynamicMeshElements a single time, instead of once per unique FSceneViewFamily.  If set to true, the GetDynamicMeshElements
	 * implementation must take into account that Views may point to different view families, with different EngineShowFlags.  In practice, this means
	 * using View->Family in the loop over Views, especially for accessing EngineShowFlags, rather than the ViewFamily parameter passed in.  The
	 * "AllViews" member, plus certain other members like the frame counter and time, are invariant across view families, and safe to access from
	 * ViewFamily.  Example use cases are features that do global processing or initialize a common shared buffer across views, which are fairly rare.
	 */
	uint8 bSinglePassGDME : 1;

	/** Whether the proxy supports asynchronously calling GetDynamicRayTracingInstances. If disabled, all calls for various proxies are serialized with respect to each other. */
	uint8 bSupportsParallelGDRTI : 1;

	/** Whether this component should be tracked by Lumen Scene. Turning this off will remove it from Lumen Scene and Lumen won't generate surface cache for it. */
	uint8 bVisibleInLumenScene : 1;

	/** Whether this component contains opaque materials (cached once from assigned material). Note: if composed from multiple meshes and materials, it may contain translucent materials. */
	uint8 bOpaqueOrMasked : 1;

	/** Whether this component can skip redundant transform updates where applicable. */
	UE_DEPRECATED(5.8, "Will be removed as it is not generally thread safe.")
	uint8 bCanSkipRedundantTransformUpdates : 1;

	/** Whether this proxy's mesh is unlikely to be constantly changing. */
	uint8 bGoodCandidateForCachedShadowmap : 1;

	/** Whether the primitive should be statically lit but has unbuilt lighting, and a preview should be used. */
	uint8 bNeedsUnbuiltPreviewLighting : 1;

	/** True if the primitive wants to use static lighting, but has invalid content settings to do so. */
	uint8 bHasValidSettingsForStaticLighting : 1;

	/** Can be set to false to skip some work only needed on lit primitives. */
	uint8 bWillEverBeLit : 1;

	/** True if the primitive casts dynamic shadows. */
	uint8 bCastDynamicShadow : 1;

	/** Whether the primitive will be used as an emissive light source. */
	uint8 bEmissiveLightSource : 1;

	/** True if the primitive influences dynamic indirect lighting. */
	uint8 bAffectDynamicIndirectLighting : 1;

	/** True if the primitive should affect indirect lighting even when hidden. */
	uint8 bAffectIndirectLightingWhileHidden : 1;

	uint8 bAffectDistanceFieldLighting : 1;

	uint8 bSupportsMaterialCache : 1;

	/** True if the primitive casts static shadows. */
	uint8 bCastStaticShadow : 1;

	EShadowCacheInvalidationBehavior ShadowCacheInvalidationBehavior;

	/** 
	 * Whether the object should cast a volumetric translucent shadow.
	 * Volumetric translucent shadows are useful for primitives with smoothly changing opacity like particles representing a volume, 
	 * But have artifacts when used on highly opaque surfaces.
	 */
	uint8 bCastVolumetricTranslucentShadow : 1;

	/** Whether the object should cast a contact shadow */
	uint8 bCastContactShadow : 1;

	/** Whether the object should cast a deep shadow */
	uint8 bCastDeepShadow : 1;

	/** Whether the primitive should use capsules for direct shadowing, if present.  Forces inset shadows. */
	uint8 bCastCapsuleDirectShadow : 1;

	/** Whether the primitive should use an inset indirect shadow from capsules or mesh distance fields. */
	uint8 bCastsDynamicIndirectShadow : 1;

	/** True if the primitive casts shadows even when hidden. */
	uint8 bCastHiddenShadow : 1;

	/** Whether this primitive should cast dynamic shadows as if it were a two sided material. */
	uint8 bCastShadowAsTwoSided : 1;

	/** When enabled, the component will only cast a shadow on itself and not other components in the world.  This is especially useful for first person weapons, and forces bCastInsetShadow to be enabled. */
	uint8 bSelfShadowOnly : 1;

	/** Whether this component should create a per-object shadow that gives higher effective shadow resolution. true if bSelfShadowOnly is true. */
	uint8 bCastInsetShadow : 1;

	/** 
	 * Whether this component should create a per-object shadow that gives higher effective shadow resolution. 
	 * Useful for cinematic character shadowing. Assumed to be enabled if bSelfShadowOnly is enabled.
	 */
	uint8 bCastCinematicShadow : 1;

	/* When enabled, the component will be rendering into the distant shadow cascades (only for directional lights). */
	uint8 bCastFarShadow : 1;

	/** 
	 * Whether to light this component and any attachments as a group.  This only has effect on the root component of an attachment tree.
	 * When enabled, attached component shadowing settings like bCastInsetShadow, bCastVolumetricTranslucentShadow, etc, will be ignored.
	 * This is useful for improving performance when multiple movable components are attached together.
	 */
	uint8 bLightAttachmentsAsGroup : 1;

	/** 
	 * Whether the whole component should be shadowed as one from stationary lights, which makes shadow receiving much cheaper.
	 * When enabled shadowing data comes from the volume lighting samples precomputed by Lightmass, which are very sparse.
	 * This is currently only used on stationary directional lights.  
	 */
	uint8 bSingleSampleShadowFromStationaryLights : 1;

	/** 
	 * Whether this proxy always uses UniformBuffer and no other uniform buffers.  
	 * When true, a fast path for updating can be used that does not update static draw lists.
	 */
	uint8 bStaticElementsAlwaysUseProxyPrimitiveUniformBuffer : 1;

	/** 
	 * Whether this proxy ever draws with vertex factories that require a primitive uniform buffer. 
	 * When false, updating the primitive uniform buffer can be skipped since vertex factories always use GPUScene instead.
	 */
	uint8 bVFRequiresPrimitiveUniformBuffer : 1;

	/** Set to true to inform scene update that the MDCs produced may use the (GPU)Scene instance count, and thus don't require recaching if the instance count changed. */
	uint8 bDoesMeshBatchesUseSceneInstanceCount : 1;

	/** Whether this proxy is a static mesh. */
	uint8 bIsStaticMesh : 1;

	/** Whether this proxy is a Nanite mesh. */
	uint8 bIsNaniteMesh : 1;

	/** Whether this proxy is always visible. */
	uint8 bIsAlwaysVisible : 1;

	/** Whether this proxy is a heterogeneous volume. */
	uint8 bIsHeterogeneousVolume : 1;

	/** Whether the primitive is a HierarchicalInstancedStaticMesh. */
	uint8 bIsHierarchicalInstancedStaticMesh : 1;

	/** Whether the primitive is landscape grass. */
	uint8 bIsLandscapeGrass : 1;

	/** True if all meshes (AKA all vertex factories) drawn by this proxy support GPU scene (default is false). */
	uint8 bSupportsGPUScene : 1;

	/** True if the mesh representation is deformable (see HasDeformableMesh() above for more details). Defaults to true to be conservative. */
	uint8 bHasDeformableMesh : 1;

	/** Whether the primitive should evaluate any World Position Offset. */
	uint8 bEvaluateWorldPositionOffset : 1;

	/** Whether the primitive has any materials with World Position Offset, and some conditions around velocity are met. */
	uint8 bHasWorldPositionOffsetVelocity : 1;

	/** Whether the primitive has any materials with World Position Offset. */
	uint8 bAnyMaterialHasWorldPositionOffset : 1;

	/** Whether the primitive has any materials that must ALWAYS evaluate World Position Offset. */
	uint8 bAnyMaterialAlwaysEvaluatesWorldPositionOffset : 1;

	/** Whether the primitive has any materials that must ALWAYS evaluate World Position Offset. */
	uint8 bAnyMaterialHasPixelAnimation : 1;

	/** Whether the primitive has any materials that evaluate temporal Responsiveness. */
	uint8 bAnyMaterialUsesTemporalResponsiveness : 1;

	/** Whether the primitive has any materials that evaluate motion vector offset to modify velocity. */
	uint8 bAnyMaterialUsesMotionVectorWorldOffset : 1;

	/** Whether the primitive should always be considered to have velocities, even if it hasn't moved. */
	uint8 bAlwaysHasVelocity : 1;

	/** Whether the primitive type supports a distance field representation.  Does not mean the primitive has a valid representation. */
	uint8 bSupportsDistanceFieldRepresentation : 1;

	/** Whether the primitive implements GetHeightfieldRepresentation() */
	uint8 bSupportsHeightfieldRepresentation : 1;

	/** If true, a single top-down Lumen card is allocated */
	uint8 bLumenHeightfield : 1;

	/** Whether this is a landscape proxy */
	uint8 bIsLandscapeProxy : 1;
	
	/** Whether this is FLandscapeNaniteSceneProxy and implements GetSourceLandscapeComponentIds(). */
	uint8 bIsLandscapeNaniteProxy : 1;

	/** Whether Lumen cards of this primitive can be shared. */
	uint8 bCompatibleWithLumenCardSharing : 1;

	/** Whether the object support triangles when rendered with translucent material */
	uint8 bSupportsSortedTriangles : 1;

	/** Whether this primitive requires notification when its level is added to the world and made visible for the first time. */
	uint8 bShouldNotifyOnWorldAddRemove : 1;

	/** true by default, if set to false will make given proxy never drawn with selection outline */
	uint8 bWantsSelectionOutline : 1;

	uint8 bVerifyUsedMaterials : 1;

	/** False by default, if true the proxy wants editor-only effects like outlines and overlays. */
    uint8 bWantsEditorEffects : 1;
    
	/** If this is True, this primitive doesn't need exact occlusion info. */
	uint8 bAllowApproximateOcclusion : 1;

	/**
	 * If this is True, this primitive should render black with an alpha of 0, but all secondary effects (shadows, refletions, indirect lighting)
	 * should behave as usual. This feature requires activating the project setting(s) "Alpha Output", and "Support Primitive Alpha Holdout" if using the deferred renderer.
	 */
	uint8 bHoldout : 1;

	/** If this is True, this primitive is a spline mesh */
	uint8 bSplineMesh : 1;

	/** If this is True, this primitive is a skinned mesh */
	uint8 bSkinnedMesh : 1;
	
	/** If this is True, this primitive is a skinned mesh */
	uint8 bInstancedSkinnedMesh : 1;

	/** If this is True, this primitive is to be rendered as a first person view object. Mutually exclusive with bIsFirstPersonWorldSpaceRepresentation. */
	uint8 bIsFirstPerson : 1;

	/** If this is True, this primitive represents another first person primitive in world space. Mutually exclusive with bIsFirstPerson. */
	uint8 bIsFirstPersonWorldSpaceRepresentation : 1;

	/** This is only set true by the constructors of proxy types which support writing to runtime virtual texture. */
	uint8 bSupportsRuntimeVirtualTexture : 1;

	/** This is only set to true by constructors of proxy types which support gathering of UStreamableAssets */
	uint8 bImplementsStreamableAssetGathering : 1;

private:

	/** If this is True, this primitive will be used to occlusion cull other primitives. */
	uint8 bUseAsOccluder:1;

	/** If this is True, this primitive can be selected in the editor. */
	uint8 bSelectable : 1;

	/** If this primitive has per-instance hit proxies. */
	uint8 bHasPerInstanceHitProxies : 1;

	/** Whether this primitive should be composited onto the scene after post processing (editor only) */
	uint8 bUseEditorCompositing : 1;

	/** Whether the component is currently being moved by the editor, even if Mobility is Static. */
	uint8 bIsBeingMovedByEditor : 1;

	/** Whether this primitive receive CSM shadows (Mobile) */
	uint8 bReceiveMobileCSMShadows : 1;

	/** This primitive has bRenderCustomDepth enabled */
	uint8 bRenderCustomDepth : 1;

	/** This primitive is only visible in Scene Capture */
	uint8 bVisibleInSceneCaptureOnly : 1;

	/** This primitive should be hidden in Scene Capture */
	uint8 bHiddenInSceneCapture : 1;

	/** if true, forces mips for streamable assets used by this proxy to be resident when this proxy is created. **/
	uint8 bForceMipStreaming : 1;

	/** This primitive will be available to ray trace as a far field primitive even if hidden. */
	bool bRayTracingFarField : 1;

	/** Optionally write this stencil value during the CustomDepth pass */
	uint8 CustomDepthStencilValue;

	/** When writing custom depth stencil, use this write mask */
	TEnumAsByte<EStencilMask> CustomDepthStencilWriteMask;

	uint8 LightingChannelMask;

	// Run-time groups of proxies
	int32 RayTracingGroupId;
	uint8 RayTracingGroupCullingPriority;

private:
	// Never use directly in descendant or implementation except for very good reason.
	const FInstanceSceneDataBuffers *InstanceSceneDataBuffersInternal = nullptr;

protected:
	/** Cached material relevance available for GetMaterialRelevance call. */
	FMaterialRelevance CombinedMaterialRelevance;

	/** Quality of interpolated indirect lighting for Movable components. */
	TEnumAsByte<EIndirectLightingCacheQuality> IndirectLightingCacheQuality;

	/** Geometry Lod bias when rendering to runtime virtual texture. */
	int8 VirtualTextureLodBias;
	/** Number of low mips to skip when rendering to runtime virtual texture. */
	int8 VirtualTextureCullMips;
	/** Log2 of minimum estimated pixel coverage before culling from runtime virtual texture. */
	int8 VirtualTextureMinCoverage;

	/** Min visibility for capsule shadows. */
	float DynamicIndirectShadowMinVisibility;

	float DistanceFieldSelfShadowBias;

	/**
	 * Maximum distance of World Position Offset used by materials. Values > 0.0 will cause the WPO to be clamped and the primitive's
	 * bounds to be padded to account for it. Value of zero will not clamp the WPO of materials nor pad bounds (legacy behavior)
	 */
	float MaxWPOExtent;

	FVector2f MinMaxMaterialDisplacement;

	/** Array of runtime virtual textures unique ids that this proxy should render to. */
	TArray<int32> RuntimeVirtualTextures;
	/** Set of unique runtime virtual texture material types referenced by RuntimeVirtualTextures. */
	TArray<ERuntimeVirtualTextureMaterialType> RuntimeVirtualTextureMaterialTypes;

public:
	/** Allocated cache render proxies */
	TArray<class FMaterialCacheVirtualTextureRenderProxy*> MaterialCacheRenderProxies;

	/** Allocated cache descriptor */
	FMaterialCacheDescriptor MaterialCacheDescriptor = InvalidMaterialCacheDescriptor;

private:
	/** The hierarchy of owners of this primitive. This array contains the unique id of actors.  */
	TArray<uint32> Owners;

	/** The primitive's local to world transform. */
	FMatrix LocalToWorld;

	/** The primitive's bounds. */
	FBoxSphereBounds Bounds = FBoxSphereBounds(ForceInit);

	/** The primitive's local space bounds. */
	FBoxSphereBounds LocalBounds = FBoxSphereBounds(ForceInit);

	/** The component's actor's position. */
	FVector ActorPosition;

	/** 
	* Id for the component this proxy belongs to.  
	* This will stay the same for the lifetime of the component, so it can be used to identify the component across re-registers.
	*/
	FPrimitiveComponentId PrimitiveComponentId;

	/** The scene the primitive is in. */
	FSceneInterface* Scene;

	/** Pointer back to the PrimitiveSceneInfo that owns this Proxy. */
	FPrimitiveSceneInfo* PrimitiveSceneInfo;

	/** The name of the actor this component is attached to. */
	FName OwnerName;

	/** The name of the resource used by the component. */
	FName ResourceName;

	/** The name of the level the primitive is in. */
	FName LevelName;

#if MESH_DRAW_COMMAND_STATS
	/** Category name for this primitive in the mesh draw stat collection. */
	FName MeshDrawCommandStatsCategory;
#endif

#if WITH_EDITOR
	/** A copy of the actor's group membership for handling per-view group hiding */
	uint64 HiddenEditorViews;

	/** Color to blend over the object as an overlay in the viewport */
	FColor OverlayColor = FColor(EForceInit::ForceInitToZero);

	/** Index of the color to use for the object's outline */
	uint32 SelectionOutlineColorIndex : 8;

	/** Whether this should only draw in any editing mode*/
	uint32 DrawInAnyEditMode : 1;

	uint32 bIsFoliage : 1;
#endif

	/** Used for precomputed visibility */
	int32 VisibilityId;

	/** The primitive's uniform buffer. */
	TUniformBufferRef<FPrimitiveUniformShaderParameters> UniformBuffer;

	/** 
	 * The UPrimitiveComponent this proxy is for, useful for quickly inspecting properties on the corresponding component while debugging.
	 * This should not be dereferenced on the rendering thread.  The game thread can be modifying UObject members at any time.
	 * Use PrimitiveComponentId instead when a component identifier is needed.
	 */
	const UPrimitiveComponent* ComponentForDebuggingOnly;

#if WITH_EDITOR
	/**
	*	How many invalid lights for this primitive, just refer for scene outliner
	*/
	int32 NumUncachedStaticLightingInteractions;

	TArray<UMaterialInterface*> UsedMaterialsForVerification;
#endif
#if UE_WITH_PSO_PRECACHING
	// The pso precache request IDs that will be boosted to the highest priority when drawn.
	TArray<FMaterialPSOPrecacheRequestID> PSOPrecacheRequestsToBoostOnDraw;
#endif
	/**
	 * Updates the primitive proxy's cached transforms, and calls OnUpdateTransform to notify it of the change.
	 * Called in the thread that owns the proxy; game or rendering.
	 * @param InLocalToWorld - The new local to world transform of the primitive.
	 * @param InBounds - The new bounds of the primitive.
	 * @param InLocalBounds - The local space bounds of the primitive.
	 */
	ENGINE_API void SetTransform(FRHICommandListBase& RHICmdList, const FMatrix& InLocalToWorld, const FBoxSphereBounds& InBounds, const FBoxSphereBounds& InLocalBounds, FVector InActorPosition);

	UE_DEPRECATED(5.8, "Will be removed as it is not generally thread safe.")
	ENGINE_API bool WouldSetTransformBeRedundant_AnyThread(const FMatrix& InLocalToWorld, const FBoxSphereBounds& InBounds, const FBoxSphereBounds& InLocalBounds, const FVector& InActorPosition) const;

	ENGINE_API void CreateUniformBuffer();

	/** Updates the hidden editor view visibility map on the render thread */
	void SetHiddenEdViews_RenderThread( uint64 InHiddenEditorViews );

protected:

	/** The primitive's cull distance. */
	float MaxDrawDistance;

	/** The primitive's minimum cull distance. */
	float MinDrawDistance;

	/**
	 * Called on the render thread for a proxy that has an instance data update, the buffers are updated asynchronously so it is unclear what this should do, except update legacy data if needed.
	 * @param InBounds - Primitive world space bounds.
	 * @param InLocalBounds - Primitive local space bounds.
	 */
	ENGINE_API virtual void UpdateInstances_RenderThread(FRHICommandListBase& RHICmdList, const FBoxSphereBounds& InBounds, const FBoxSphereBounds& InLocalBounds);

	/** Updates selection for the primitive proxy. This is called in the rendering thread by SetSelection_GameThread. */
	void SetSelection_RenderThread(const bool bInParentSelected, const bool bInIndividuallySelected);

	/** Updates LevelInstance editing state for the primitive proxy. This is called in the rendering thread. */
	void SetLevelInstanceEditingState_RenderThread(const bool bInLevelInstanceEditingState);

	/** Updates hover state for the primitive proxy. This is called in the rendering thread by SetHovered_GameThread. */
	void SetHovered_RenderThread(const bool bInHovered);

	/** Allows child implementations to do render-thread work when bEvaluateWorldPositionOffset changes */
	virtual void OnEvaluateWorldPositionOffsetChanged_RenderThread() {}

	ENGINE_API FRenderBounds PadInstanceLocalBounds(const FRenderBounds& InBounds);
};

/**
 * Returns if specified mesh command can be cached, or needs to be recreated every frame.
 */
ENGINE_API extern bool SupportsCachingMeshDrawCommands(const FMeshBatch& MeshBatch);

/**
 * Returns if specified mesh command can be cached, or needs to be recreated every frame; this is a slightly slower version
 * used for materials with external textures that need invalidating their PSOs.
 */
ENGINE_API extern bool SupportsCachingMeshDrawCommands(const FMeshBatch& MeshBatch, ERHIFeatureLevel::Type FeatureLevel);

/**
 * Returns if specified mesh can be rendered via Nanite.
 */
ENGINE_API extern bool SupportsNaniteRendering(const FVertexFactory* RESTRICT VertexFactory, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy);

ENGINE_API extern bool SupportsNaniteRendering(const FVertexFactory* RESTRICT VertexFactory, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, const class FMaterialRenderProxy* MaterialRenderProxy, ERHIFeatureLevel::Type FeatureLevel);

ENGINE_API extern bool SupportsNaniteRendering(const class FVertexFactoryType* RESTRICT VertexFactoryType, const class FMaterial& Material, ERHIFeatureLevel::Type FeatureLevel);

// Whether scene proxies will have GetDynamicMeshElements called in parallel.
ENGINE_API bool IsParallelGatherDynamicMeshElementsEnabled();

// Whether scene proxies will have GetDynamicRayTracingInstances called in parallel.
ENGINE_API bool IsParallelGatherDynamicRayTracingInstancesEnabled();

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "SceneView.h"
#endif
