// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Package.h"
#include "VT/RuntimeVirtualTexture.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/Level.h"
#include "Components/PrimitiveComponent.h"

struct FPrimitiveSceneProxyDesc
{	
	ENGINE_API FPrimitiveSceneProxyDesc();
	ENGINE_API FPrimitiveSceneProxyDesc(const UPrimitiveComponent*);
	ENGINE_API virtual ~FPrimitiveSceneProxyDesc();

	ENGINE_API void InitializeFromPrimitiveComponent(const UPrimitiveComponent*);

	UObject* Component = nullptr;
	UObject* Owner = nullptr;
	UWorld* World = nullptr;

	TArrayView<URuntimeVirtualTexture*>  RuntimeVirtualTextures;

	// Only used by actors for now, explicitly intended to be moved to the FPrimitiveSceneProxy
	mutable TArray<uint32> ActorOwners;
	const FCustomPrimitiveData* CustomPrimitiveData = nullptr;
	FSceneInterface* Scene = nullptr;
	const IPrimitiveComponent* PrimitiveComponentInterface = nullptr;

	const UObject* AdditionalStatObjectPtr = nullptr;

	uint64 HiddenEditorViews = 0;

	TStatId StatId;

	uint32 CastShadow : 1;
	uint32 bReceivesDecals : 1;
	uint32 bOnlyOwnerSee : 1;
	uint32 bOwnerNoSee : 1;
	uint32 bLevelInstanceEditingState : 1;
	uint32 bUseViewOwnerDepthPriorityGroup  : 1;
	uint32 bVisibleInReflectionCaptures : 1;
	uint32 bVisibleInRealTimeSkyCaptures : 1;
	uint32 bVisibleInRayTracing : 1;
	uint32 bVisibleInReflections : 1;
	uint32 bRenderInDepthPass : 1;
	uint32 bRenderInMainPass : 1;
	uint32 bTreatAsBackgroundForOcclusion : 1;
	uint32 bCastDynamicShadow : 1;
	uint32 bCastStaticShadow : 1;
	uint32 bEmissiveLightSource : 1;
	uint32 bAffectDynamicIndirectLighting : 1;
	uint32 bAffectIndirectLightingWhileHidden : 1;
	uint32 bAffectDistanceFieldLighting : 1;
	uint32 bCastVolumetricTranslucentShadow : 1;
	uint32 bCastContactShadow : 1;
	uint32 bCastHiddenShadow : 1;
	uint32 bCastShadowAsTwoSided : 1;
	uint32 bSelfShadowOnly : 1;
	uint32 bCastInsetShadow : 1;
	uint32 bCastCinematicShadow : 1;
	uint32 bCastFarShadow : 1;
	uint32 bLightAttachmentsAsGroup : 1;
	uint32 bSingleSampleShadowFromStationaryLights : 1;
	uint32 bUseAsOccluder : 1;
	uint32 bSelectable : 1;
	uint32 bHasPerInstanceHitProxies : 1;
	uint32 bUseEditorCompositing : 1;
	uint32 bIsBeingMovedByEditor : 1;
	uint32 bReceiveMobileCSMShadows : 1;
	uint32 bRenderCustomDepth : 1;
	uint32 bVisibleInSceneCaptureOnly : 1;
	uint32 bHiddenInSceneCapture : 1;
	uint32 bForceMipStreaming : 1;
	uint32 bRayTracingFarField : 1;
	uint32 bHoldout : 1;
	uint32 bIsFirstPerson : 1;
	uint32 bIsFirstPersonWorldSpaceRepresentation : 1;
	uint32 bLumenHeightfield : 1;

	// not mirrored from UPrimitiveComponent
	uint32 bIsVisible : 1;
	uint32 bIsVisibleEditor : 1; 
	uint32 bSelected : 1;
	uint32 bIndividuallySelected : 1;
	uint32 bShouldRenderSelected : 1;
	uint32 bWantsEditorEffects : 1;
	uint32 bCollisionEnabled : 1;
	uint32 bIsHidden : 1;
	uint32 bIsHiddenEd : 1;
	uint32 bSupportsWorldPositionOffsetVelocity : 1;
	uint32 bIsOwnerEditorOnly : 1;
	uint32 bIsInstancedStaticMesh : 1;
	uint32 bHasStaticLighting : 1;
	uint32 bHasValidSettingsForStaticLighting : 1;
	uint32 bIsPrecomputedLightingValid : 1;
	uint32 bShadowIndirectOnly: 1;
	uint32 bUsePSOPrecacheFallbackMaterial:1;	
#if WITH_EDITOR
	uint32 bIsOwnedByFoliage:1;
#endif

	int32 TranslucencySortPriority = 0;
	float TranslucencySortDistanceOffset = 0.0f;
	int32 CustomDepthStencilValue = 0;

	FLightingChannels LightingChannels;
	ERayTracingGroupCullingPriority RayTracingGroupCullingPriority = ERayTracingGroupCullingPriority::CP_4_DEFAULT;
	EShadowCacheInvalidationBehavior ShadowCacheInvalidationBehavior = EShadowCacheInvalidationBehavior::Auto;

	TEnumAsByte<ESceneDepthPriorityGroup> ViewOwnerDepthPriorityGroup = ESceneDepthPriorityGroup::SDPG_World;
	TEnumAsByte<EComponentMobility::Type> Mobility = EComponentMobility::Movable;
	TEnumAsByte<EIndirectLightingCacheQuality> IndirectLightingCacheQuality = EIndirectLightingCacheQuality::ILCQ_Point;
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriorityGroup = ESceneDepthPriorityGroup::SDPG_World;
	
	ELightmapType LightmapType = ELightmapType::Default;
	ERendererStencilMask CustomDepthStencilWriteMask = ERendererStencilMask::ERSM_Default;
	ERuntimeVirtualTextureMainPassType VirtualTextureRenderPassType = ERuntimeVirtualTextureMainPassType::Exclusive;
	int8 VirtualTextureLodBias = 0;
	int8 VirtualTextureMinCoverage = 0;

	int32 VirtualTextureCullMips = 0;
	float VirtualTextureMainPassMaxDrawDistance = 0.0f;

	FPrimitiveComponentId ComponentId;
	int32 VisibilityId = 0;
	float CachedMaxDrawDistance = 0.0f;
	float MinDrawDistance = 0.0f;
	float BoundsScale = 1.0f;
	int32 RayTracingGroupId = FPrimitiveSceneProxy::InvalidRayTracingGroupId;

	ERHIFeatureLevel::Type FeatureLevel;

#if UE_WITH_PSO_PRECACHING
	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDsToBoostOnDraw;
#endif // UE_WITH_PSO_PRECACHING

#if MESH_DRAW_COMMAND_STATS
	FName MeshDrawCommandStatsCategory;
#endif

#if WITH_EDITOR
	FColor OverlayColor = FColor(EForceInit::ForceInitToZero);
#endif

	bool IsVisible() const
	{
		return bIsVisible;
	}

	bool IsVisibleEditor() const
	{
		return bIsVisibleEditor;
	}

	bool ShouldRenderSelected() const
	{
		return bShouldRenderSelected;
	}

	bool IsComponentIndividuallySelected() const
	{
		return bIndividuallySelected;
	}

	ESceneDepthPriorityGroup GetStaticDepthPriorityGroup() const
	{
		return DepthPriorityGroup;
	}

	bool HasStaticLighting() const
	{
		return bHasStaticLighting;
	}

	bool IsCollisionEnabled() const
	{
		return bCollisionEnabled;
	}

	bool IsPrecomputedLightingValid() const
	{
		return false;
	}

	bool HasValidSettingsForStaticLighting() const
	{
		return bHasValidSettingsForStaticLighting;
	}

	bool GetShadowIndirectOnly() const
	{
		return bShadowIndirectOnly;
	}

	bool GetLevelInstanceEditingState() const
	{
		return bLevelInstanceEditingState;
	}

	bool IsHidden() const
	{
		return bIsHidden;
	}

	bool IsOwnerEditorOnly() const
	{
		return bIsOwnerEditorOnly;
	}

	bool UsePSOPrecacheFallbackMaterial() const
	{
		return bUsePSOPrecacheFallbackMaterial;
	}

	bool SupportsWorldPositionOffsetVelocity() const
	{
		return bSupportsWorldPositionOffsetVelocity;
	}

	bool IsFirstPersonRelevant() const
	{
		return bIsFirstPerson || bIsFirstPersonWorldSpaceRepresentation;
	}

	int32 GetRayTracingGroupId() const
	{
		return RayTracingGroupId;
	}

	FSceneInterface* GetScene() const
	{
		return Scene;
	}

	const FCustomPrimitiveData& GetCustomPrimitiveData() const
	{
		check(CustomPrimitiveData);
		return *CustomPrimitiveData;
	}

	TArrayView<URuntimeVirtualTexture*> GetRuntimeVirtualTextures() const
	{
		return RuntimeVirtualTextures;
	}

	ERuntimeVirtualTextureMainPassType GetVirtualTextureRenderPassType() const
	{
		return VirtualTextureRenderPassType;
	}

	float GetVirtualTextureMainPassMaxDrawDistance() const
	{
		return VirtualTextureMainPassMaxDrawDistance;
	}

	const UObject* AdditionalStatObject() const
	{
		return AdditionalStatObjectPtr;
	}

	TStatId GetStatID(bool bForDeferredUse = false) const
	{
		return StatId;
	}

	const IPrimitiveComponent* GetPrimitiveComponentInterface() const
	{
		return PrimitiveComponentInterface;
	}

	UWorld* GetWorld() const
	{
		check(World);
		return World;
	}

	FPrimitiveComponentId GetPrimitiveSceneId() const
	{
		return ComponentId;
	}

	UObject* GetOwner() const
	{
		return Owner;
	}

	template<class T>
	T* GetOwner() const
	{
		return Cast<T>(GetOwner());
	}

	ULevel* GetLevel() const
	{
		return  Owner ? Owner->GetTypedOuter<ULevel>() : nullptr;
	}

	FString GetPathName() const
	{
		return Component->GetPathName();
	}
	
#if WITH_EDITOR
	bool IsHiddenEd() const
	{
		return bIsHiddenEd;
	}

	uint64 GetHiddenEditorViews() const
	{
		return HiddenEditorViews;
	}

	bool IsOwnedByFoliage() const
	{
		return bIsOwnedByFoliage;
	}
#endif

#if MESH_DRAW_COMMAND_STATS
	FName GetMeshDrawCommandStatsCategory() const
	{
		return MeshDrawCommandStatsCategory;
	}
#endif

	ENGINE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const;

	ENGINE_API void SetPSOPrecacheComponentData(const FPSOPrecacheComponentData& PrecacheComponentData);

	UE_DEPRECATED(5.6, "Use FPrimitiveSceneProxyDesc::IsFirstPersonRelevant() instead.")
	bool IsStaticMeshFirstPersonRelevant() const
	{
		return IsFirstPersonRelevant();
	}

	UE_DEPRECATED(5.6, "Use GetLevel instead.")
	ULevel* GetComponentLevel() const
	{
		return GetLevel();
	}
	
	UE_DEPRECATED(5.8, "Use FPrimitiveSceneProxyDesc::UsePSOPrecacheFallbackMaterial() instead.")
	bool ShouldRenderProxyFallbackToDefaultMaterial() const
	{
		return UsePSOPrecacheFallbackMaterial();
	}
};
