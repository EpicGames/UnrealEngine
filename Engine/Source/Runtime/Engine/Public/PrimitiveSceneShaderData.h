// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneDefinitions.h"
#include "Containers/StaticArray.h"
#include "InstanceUniformShaderParameters.h"
#include "LightmapUniformShaderParameters.h"
#include "PrimitiveUniformShaderParameters.h"

class FPrimitiveSceneProxy;

struct FPrimitiveSceneShaderData
{
	static const uint32 DataStrideInFloat4s = PRIMITIVE_SCENE_DATA_STRIDE;

	TStaticArray<FVector4f, DataStrideInFloat4s> Data;

	FPrimitiveSceneShaderData()
		: Data(InPlace, NoInit)
	{
		Setup(GetIdentityPrimitiveParameters());
	}

	explicit FPrimitiveSceneShaderData(const FPrimitiveUniformShaderParameters& PrimitiveUniformShaderParameters)
		: Data(InPlace, NoInit)
	{
		Setup(PrimitiveUniformShaderParameters);
	}

	ENGINE_API FPrimitiveSceneShaderData(const FPrimitiveSceneProxy* RESTRICT Proxy);

	/**
	 * Directly construct the data from the proxy into an output array, removing the need to construct an intermediate.
	 */
	ENGINE_API static void BuildDataFromProxy(const FPrimitiveSceneProxy* RESTRICT Proxy, FVector4f* RESTRICT OutData);

	/**
	 */
	ENGINE_API static void Setup(const FPrimitiveUniformShaderParameters& PrimitiveUniformShaderParameters, FVector4f* RESTRICT OutData);

	ENGINE_API void Setup(const FPrimitiveUniformShaderParameters& PrimitiveUniformShaderParameters);
};

class FSinglePrimitiveStructured : public FRenderResource
{
public:
	FSinglePrimitiveStructured()
	{}

	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	virtual void ReleaseRHI() override
	{
		SkyIrradianceEnvironmentMapRHI.SafeRelease();
		SkyIrradianceEnvironmentMapSRV.SafeRelease();
		PrimitiveSceneDataTextureRHI.SafeRelease();
		PrimitiveSceneDataTextureSRV.SafeRelease();
	}

	ENGINE_API void UploadToGPU(FRHICommandListBase& RHICmdList);

	EShaderPlatform ShaderPlatform=SP_NumPlatforms;

	FBufferRHIRef SkyIrradianceEnvironmentMapRHI;
	FShaderResourceViewRHIRef SkyIrradianceEnvironmentMapSRV;

	FTextureRHIRef PrimitiveSceneDataTextureRHI;
	FShaderResourceViewRHIRef PrimitiveSceneDataTextureSRV;
};

/**
* Default Primitive data buffer.  
* This is used when the VF is used for rendering outside normal mesh passes, where there is no valid scene.
*/
extern ENGINE_API TGlobalResource<FSinglePrimitiveStructured> GIdentityPrimitiveBuffer;
