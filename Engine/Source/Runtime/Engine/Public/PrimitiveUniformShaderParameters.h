// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "Engine/EngineTypes.h"
#include "SceneTypes.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "UniformBuffer.h"

/** 
 * The uniform shader parameters associated with a primitive. 
 * Note: Must match FPrimitiveSceneData in shaders.
 * Note 2: Try to keep this 16 byte aligned. i.e |Matrix4x4|Vector3,float|Vector3,float|Vector4|  _NOT_  |Vector3,(waste padding)|Vector3,(waste padding)|Vector3. Or at least mark out padding if it can't be avoided.
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FPrimitiveUniformShaderParameters,ENGINE_API)
	SHADER_PARAMETER(uint32,		Flags)
	SHADER_PARAMETER(uint32,		InstanceSceneDataOffset)
	SHADER_PARAMETER(uint32,		NumInstanceSceneDataEntries)
	SHADER_PARAMETER(int32,			SingleCaptureIndex)										// Should default to 0 if no reflection captures are provided, as there will be a default black (0,0,0,0) cubemap in that slot
	SHADER_PARAMETER(FVector3f,		PositionHigh)
	SHADER_PARAMETER(uint32,		PrimitiveComponentId)									// TODO: Refactor to use PersistentPrimitiveIndex, ENGINE USE ONLY - will be removed
	SHADER_PARAMETER(FMatrix44f,	LocalToRelativeWorld)									// Always needed
	SHADER_PARAMETER(FMatrix44f,	RelativeWorldToLocal)									// Rarely needed
	SHADER_PARAMETER(FMatrix44f,	PreviousLocalToRelativeWorld)							// Used to calculate velocity
	SHADER_PARAMETER(FMatrix44f,	PreviousRelativeWorldToLocal)							// Rarely used when calculating velocity, if material uses vertex offset along with world->local transform
	SHADER_PARAMETER(FMatrix44f,	WorldToPreviousWorld)									// Used when calculating instance prev local->world for static instances that do not store it (calculated via doubles to resolve precision issues)
	SHADER_PARAMETER_EX(FVector3f,	InvNonUniformScale,  EShaderPrecisionModifier::Half)	// Often needed
	SHADER_PARAMETER(float,			ObjectBoundsX)											// Only needed for editor/development
	SHADER_PARAMETER(FVector4f,		ObjectWorldPositionHighAndRadius)						// Needed by some materials
	SHADER_PARAMETER(FVector3f,		ObjectWorldPositionLow)									// Needed by some materials
	SHADER_PARAMETER(float,			MinMaterialDisplacement)
	SHADER_PARAMETER(FVector3f,		ActorWorldPositionHigh)
	SHADER_PARAMETER(float,			MaxMaterialDisplacement)
	SHADER_PARAMETER(FVector3f,		ActorWorldPositionLow)
	SHADER_PARAMETER(uint32,		LightmapUVIndex)										// Only needed if static lighting is enabled
	SHADER_PARAMETER_EX(FVector3f,	ObjectOrientation,   EShaderPrecisionModifier::Half)
	SHADER_PARAMETER(uint32,		LightmapDataIndex)										// Only needed if static lighting is enabled
	SHADER_PARAMETER_EX(FVector4f,	NonUniformScale,     EShaderPrecisionModifier::Half)
	SHADER_PARAMETER(FVector3f,		PreSkinnedLocalBoundsMin)								// Local space min bounds, pre-skinning
	SHADER_PARAMETER(uint32,		NaniteResourceID)
	SHADER_PARAMETER(FVector3f,		PreSkinnedLocalBoundsMax)								// Local space bounds, pre-skinning
	SHADER_PARAMETER(uint32,		NaniteHierarchyOffset)
	SHADER_PARAMETER(uint32,		NaniteAssemblyTransformOffset)
	SHADER_PARAMETER(uint32,		NaniteAssemblyTransformCount)
	SHADER_PARAMETER(FVector3f,		LocalObjectBoundsMin)									// This is used in a custom material function (ObjectLocalBounds.uasset)
	SHADER_PARAMETER(float,			ObjectBoundsY)											// Only needed for editor/development
	SHADER_PARAMETER(FVector3f,		LocalObjectBoundsMax)									// This is used in a custom material function (ObjectLocalBounds.uasset)
	SHADER_PARAMETER(float,			ObjectBoundsZ)											// Only needed for editor/development
	SHADER_PARAMETER(FVector3f,		InstanceLocalBoundsCenter)
	SHADER_PARAMETER(uint32,		InstancePayloadDataOffset)
	SHADER_PARAMETER(FVector3f,		InstanceLocalBoundsExtent)
	SHADER_PARAMETER(uint32,		InstancePayloadDataStride)
	SHADER_PARAMETER(uint32,		InstancePayloadExtensionSize)
	SHADER_PARAMETER(FVector2f,		WireframeAndPrimitiveColor)								// Only needed for editor/development
	SHADER_PARAMETER(uint32,		PackedNaniteFlags)
	SHADER_PARAMETER(int32,			PersistentPrimitiveIndex)
	SHADER_PARAMETER(FVector2f,		InstanceDrawDistanceMinMaxSquared)
	SHADER_PARAMETER(float,			InstanceWPODisableDistanceSquared)
	SHADER_PARAMETER(float,			PixelProgrammableDistanceSquared)
	SHADER_PARAMETER(float,			MaterialDisplacementFadeOutSize)
	SHADER_PARAMETER(float,			AnimationMinScreenSize)
	SHADER_PARAMETER(FVector2f,		InstancingFadeOutParams)
	SHADER_PARAMETER(FUintVector2,	MeshPaintTextureDescriptor)
	SHADER_PARAMETER(uint32,        MaterialCacheDescriptor)
	SHADER_PARAMETER(uint32,		NaniteRayTracingDataOffset)
	SHADER_PARAMETER(float,			MaxWPOExtent)
	SHADER_PARAMETER(uint32,		CustomStencilValueAndMask)
	SHADER_PARAMETER(uint32,		VisibilityFlags)
	SHADER_PARAMETER_ARRAY(FVector4f, CustomPrimitiveData, [FCustomPrimitiveData::NumCustomPrimitiveDataFloat4s]) // Custom data per primitive that can be accessed through material expression parameters and modified through UStaticMeshComponent
END_GLOBAL_SHADER_PARAMETER_STRUCT()

ENGINE_API TUniformBufferRef<FPrimitiveUniformShaderParameters> CreatePrimitiveUniformBufferImmediate(
	const FMatrix& LocalToWorld,
	const FBoxSphereBounds& WorldBounds,
	const FBoxSphereBounds& LocalBounds,
	const FBoxSphereBounds& PreSkinnedLocalBounds,
	bool bReceivesDecals,
	bool bOutputVelocity
);

ENGINE_API FPrimitiveUniformShaderParameters GetIdentityPrimitiveParameters();

/**
 * Primitive uniform buffer containing only identity transforms.
 */
class FIdentityPrimitiveUniformBuffer : public TUniformBuffer<FPrimitiveUniformShaderParameters>
{
public:
	void InitContents()
	{
		SetContents(FRenderResource::GetImmediateCommandList(), GetIdentityPrimitiveParameters());
	}
};

/** Global primitive uniform buffer resource containing identity transformations. */
extern ENGINE_API TGlobalResource<FIdentityPrimitiveUniformBuffer> GIdentityPrimitiveUniformBuffer;

