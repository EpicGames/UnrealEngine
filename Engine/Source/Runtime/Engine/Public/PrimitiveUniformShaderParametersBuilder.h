// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "Engine/EngineTypes.h"
#include "SceneTypes.h"
#include "ShaderParameters.h"
#include "UniformBuffer.h"
#include "LightmapUniformShaderParameters.h"
#include "NaniteDefinitions.h"
#include "RenderTransform.h"
#include "SceneDefinitions.h"
#include "PrimitiveUniformShaderParameters.h"

struct FPrimitiveUniformShaderParametersBuilder
{
public:
	inline FPrimitiveUniformShaderParametersBuilder& Defaults()
	{
		ObjectRadius = 0.0f;

		// Flags defaulted on
		bCastShadow									= true;
		bCastContactShadow							= true;
		bEvaluateWorldPositionOffset				= true;
		bVisibleInGame								= true;
		bVisibleInEditor							= true;
		bVisibleInMainPass							= true;
		bVisibleInReflectionCaptures				= true;
		bVisibleInRealTimeSkyCaptures				= true;
		bVisibleInRayTracing						= true;
		bVisibleInLumenScene						= true;

		// Flags defaulted off
		bReceivesDecals								= false;
		bUseSingleSampleShadowFromStationaryLights	= false;
		bUseVolumetricLightmap						= false;
		bOutputVelocity								= false;
		bHasRayTracingRepresentation				= false;
		bHasCapsuleRepresentation					= false;
		bHasPreSkinnedLocalBounds					= false;
		bHasPreviousLocalToWorld					= false;
		bHasInstanceLocalBounds						= false;
		bCastHiddenShadow							= false;
		bVisibleInSceneCaptureOnly					= false;
		bHiddenInSceneCapture						= false;
		bForceHidden								= false;
		bHasInstanceDrawDistanceCull				= false;
		bHasWPODisableDistance						= false;
		bHasAlwaysEvaluateWPOMaterials				= false;
		bReverseCulling								= false;
		bHoldout									= false;
		bDisableMaterialInvalidations				= false;
		bSplineMesh									= false;
		bSkinnedMesh								= false;
		bAllowInstanceCullingOcclusionQueries		= false;
		bHasPixelAnimation                          = false;
		bRayTracingFarField							= false;
		bRayTracingHasGroupId						= false;
		bHasPerClusterDisplacementFallbackRaster	= false;
		bIsFirstPerson								= false;
		bEnableObjectWorldPositionFix				= false;

		Parameters.MaxWPOExtent						= 0.0f;
		Parameters.MinMaterialDisplacement			= 0.0f;
		Parameters.MaxMaterialDisplacement			= 0.0f;

		// Default colors
		Parameters.WireframeAndPrimitiveColor			= FVector2f(FMath::AsFloat(0xFFFFFF00), FMath::AsFloat(0xFFFFFF00));

		// Invalid indices
		Parameters.LightmapDataIndex				= 0;
		Parameters.LightmapUVIndex					= INDEX_NONE;
		Parameters.SingleCaptureIndex				= INDEX_NONE;
		Parameters.PersistentPrimitiveIndex			= INDEX_NONE;
		Parameters.PrimitiveComponentId				= ~uint32(0u);

		// Nanite
		Parameters.NaniteResourceID						= INDEX_NONE;
		Parameters.NaniteHierarchyOffset				= INDEX_NONE;
		Parameters.NaniteAssemblyTransformOffset		= INDEX_NONE;
		Parameters.PackedNaniteFlags					= 0;
		Parameters.NaniteRayTracingDataOffset			= INDEX_NONE;

		// Instance data
		Parameters.InstanceSceneDataOffset			= INDEX_NONE;
		Parameters.NumInstanceSceneDataEntries		= 0;
		Parameters.InstancePayloadDataOffset		= INDEX_NONE;
		Parameters.InstancePayloadDataStride		= 0;
		Parameters.InstancePayloadExtensionSize		= 0;

		Parameters.MeshPaintTextureDescriptor		= FUintVector2(0, 0);
		Parameters.MaterialCacheDescriptor		    = 0;

		LightingChannels = GetDefaultLightingChannelMask();

		return CustomPrimitiveData(nullptr);
	}

#define PRIMITIVE_UNIFORM_BUILDER_METHOD(INPUT_TYPE, VARIABLE_NAME) \
	inline FPrimitiveUniformShaderParametersBuilder& VARIABLE_NAME(INPUT_TYPE In##VARIABLE_NAME) { Parameters.VARIABLE_NAME = In##VARIABLE_NAME; return *this; }

#define PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(INPUT_TYPE, VARIABLE_NAME) \
	inline FPrimitiveUniformShaderParametersBuilder& VARIABLE_NAME(INPUT_TYPE In##VARIABLE_NAME) { b##VARIABLE_NAME = In##VARIABLE_NAME; return *this; }

	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			ReceivesDecals);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			HasRayTracingRepresentation);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			HasCapsuleRepresentation);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			HasInstanceLocalBounds);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			CastContactShadow);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			CastHiddenShadow);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			CastShadow);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			UseSingleSampleShadowFromStationaryLights);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			UseVolumetricLightmap);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			OutputVelocity);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			EvaluateWorldPositionOffset);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			VisibleInGame);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			VisibleInEditor);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			ReverseCulling);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			VisibleInMainPass);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			VisibleInReflectionCaptures);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			VisibleInRealTimeSkyCaptures);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			VisibleInRayTracing);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			VisibleInLumenScene);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			VisibleInSceneCaptureOnly);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			HiddenInSceneCapture);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			ForceHidden);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			Holdout);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			DisableMaterialInvalidations);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			SplineMesh);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			SkinnedMesh);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			AllowInstanceCullingOcclusionQueries);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			HasAlwaysEvaluateWPOMaterials);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			HasPixelAnimation);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			RayTracingFarField);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			RayTracingHasGroupId);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			HasPerClusterDisplacementFallbackRaster);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			IsFirstPerson);
	PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD(bool,			EnableObjectWorldPositionFix);

	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			InstanceSceneDataOffset);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			NumInstanceSceneDataEntries);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			InstancePayloadDataOffset);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			InstancePayloadDataStride);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			InstancePayloadExtensionSize);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(int32,				SingleCaptureIndex);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(int32,				PersistentPrimitiveIndex);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			PrimitiveComponentId);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			NaniteResourceID);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			NaniteHierarchyOffset);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			NaniteRayTracingDataOffset);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			LightmapUVIndex);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,			LightmapDataIndex);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(float,				MaterialDisplacementFadeOutSize);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(FUintVector2,		MeshPaintTextureDescriptor);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(uint32,	        MaterialCacheDescriptor);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(float,				AnimationMinScreenSize);
	PRIMITIVE_UNIFORM_BUILDER_METHOD(FVector2f,			InstancingFadeOutParams);

#undef PRIMITIVE_UNIFORM_BUILDER_FLAG_METHOD
#undef PRIMITIVE_UNIFORM_BUILDER_METHOD

	inline FPrimitiveUniformShaderParametersBuilder& LightingChannelMask(uint32 InLightingChannelMask)
	{
		LightingChannels = InLightingChannelMask;
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& MaxWorldPositionOffsetExtent(float InMaxExtent)
	{
		Parameters.MaxWPOExtent = InMaxExtent;
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& MinMaxMaterialDisplacement(const FVector2f& InMinMaxDisplacement)
	{
		Parameters.MinMaterialDisplacement = InMinMaxDisplacement.X;
		Parameters.MaxMaterialDisplacement = InMinMaxDisplacement.Y;
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& ObjectBounds(const FVector3f& InObjectBounds)
	{
		Parameters.ObjectBoundsX = InObjectBounds.X;
		Parameters.ObjectBoundsY = InObjectBounds.Y;
		Parameters.ObjectBoundsZ = InObjectBounds.Z;
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& WorldBounds(const FBoxSphereBounds& InWorldBounds)
	{
		AbsoluteObjectWorldPosition = InWorldBounds.Origin;
		ObjectRadius = static_cast<float>(InWorldBounds.SphereRadius);											//LWC_TODO: Precision loss
		Parameters.ObjectBoundsX = static_cast<float>(InWorldBounds.BoxExtent.X);
		Parameters.ObjectBoundsY = static_cast<float>(InWorldBounds.BoxExtent.Y);
		Parameters.ObjectBoundsZ = static_cast<float>(InWorldBounds.BoxExtent.Z);
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& LocalBounds(const FBoxSphereBounds& InLocalBounds)
	{
		Parameters.LocalObjectBoundsMin = FVector3f(InLocalBounds.GetBoxExtrema(0)); // 0 == minimum		//LWC_TODO: Precision loss
		Parameters.LocalObjectBoundsMax = FVector3f(InLocalBounds.GetBoxExtrema(1)); // 1 == maximum		//LWC_TODO: Precision loss
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& PreSkinnedLocalBounds(const FBoxSphereBounds& InPreSkinnedLocalBounds)
	{
		bHasPreSkinnedLocalBounds = true;
		Parameters.PreSkinnedLocalBoundsMin = FVector3f(InPreSkinnedLocalBounds.GetBoxExtrema(0)); // 0 == minimum		//LWC_TODO: Precision loss
		Parameters.PreSkinnedLocalBoundsMax = FVector3f(InPreSkinnedLocalBounds.GetBoxExtrema(1)); // 1 == maximum		//LWC_TODO: Precision loss
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& ActorWorldPosition(const FVector& InActorWorldPosition)
	{
		AbsoluteActorWorldPosition = InActorWorldPosition;
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& LocalToWorld(const FMatrix& InLocalToWorld)
	{
		AbsoluteLocalToWorld = InLocalToWorld;
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& InstanceLocalBounds(const FRenderBounds& InInstanceLocalBounds)
	{
		bHasInstanceLocalBounds = true;
		Parameters.InstanceLocalBoundsCenter = InInstanceLocalBounds.GetCenter();
		Parameters.InstanceLocalBoundsExtent = InInstanceLocalBounds.GetExtent();
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& PreviousLocalToWorld(const FMatrix& InPreviousLocalToWorld)
	{
		bHasPreviousLocalToWorld = true;
		AbsolutePreviousLocalToWorld = InPreviousLocalToWorld;
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& EditorColors(const FLinearColor& InWireframeColor, const FLinearColor& InPrimitiveColor)
	{
		FColor WireframeColor = InWireframeColor.QuantizeRound();
		FColor PrimitiveColor = InPrimitiveColor.QuantizeRound();
		Parameters.WireframeAndPrimitiveColor = FVector2f(FMath::AsFloat(WireframeColor.ToPackedRGBA()), FMath::AsFloat(PrimitiveColor.ToPackedRGBA()));
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& CustomPrimitiveData(const FCustomPrimitiveData* InCustomPrimitiveData)
	{
		// If this primitive has custom primitive data, set it
		if (InCustomPrimitiveData && !InCustomPrimitiveData->Data.IsEmpty())
		{
			// Copy at most up to the max supported number of floats for safety
			FMemory::Memcpy(
				&Parameters.CustomPrimitiveData,
				InCustomPrimitiveData->Data.GetData(),
				InCustomPrimitiveData->Data.GetTypeSize() * FMath::Min(InCustomPrimitiveData->Data.Num(),
				FCustomPrimitiveData::NumCustomPrimitiveDataFloats)
			);

			bHasCustomData = true;
		}
		else
		{
			// Clear to 0
			FMemory::Memzero(Parameters.CustomPrimitiveData);
			bHasCustomData = false;
		}

		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& NaniteFilterFlags(uint32 FilterFlags)
	{
		Parameters.PackedNaniteFlags = FilterFlags;
		return *this;
	}

	inline FPrimitiveUniformShaderParametersBuilder& CustomDepthStencil(uint8 StencilValue, EStencilMask StencilWriteMask)
	{
		// Translate the enum to a mask that can be consumed by the GPU with fewer operations
		uint32 GPUStencilMask;
		switch (StencilWriteMask)
		{		
		case SM_Default:
			GPUStencilMask = 0u; // Use zero to mean replace
			break;
		case SM_255:
			GPUStencilMask = 0xFFu;
			break;
		default:
			GPUStencilMask = (1u << uint32(StencilWriteMask - SM_1)) & 0xFFu;
			break;
		}
		Parameters.CustomStencilValueAndMask = (GPUStencilMask << 8u) | StencilValue;

		return *this;
	}

	ENGINE_API FPrimitiveUniformShaderParametersBuilder& InstanceDrawDistance(FVector2f DistanceMinMax);

	ENGINE_API FPrimitiveUniformShaderParametersBuilder& InstanceWorldPositionOffsetDisableDistance(float WPODisableDistance);

	ENGINE_API FPrimitiveUniformShaderParametersBuilder& PixelProgrammableDistance(float PixelProgrammableDistance);
	
	ENGINE_API FPrimitiveUniformShaderParametersBuilder& NaniteAssemblyInfo(uint32 TransformOffset, uint32 TransformCount);

	inline const FPrimitiveUniformShaderParameters& Build()
	{
		const FDFVector3 AbsoluteWorldPosition(AbsoluteLocalToWorld.GetOrigin());
		const FVector PositionHigh(AbsoluteWorldPosition.High);

		Parameters.PositionHigh = AbsoluteWorldPosition.High;

		{
			// Inverse on FMatrix44f can generate NaNs if the source matrix contains large scaling, so do it in double precision.
			// Also use double precision to calculate WorldToPreviousWorld to prevent precision issues at far distances

			FMatrix LocalToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrixDouble(PositionHigh, AbsoluteLocalToWorld);

			FMatrix RelativeWorldToLocal = LocalToRelativeWorld.Inverse();

			Parameters.LocalToRelativeWorld = FMatrix44f(LocalToRelativeWorld);
			Parameters.RelativeWorldToLocal = FMatrix44f(RelativeWorldToLocal);

			if (bHasPreviousLocalToWorld)
			{
				FMatrix PrevLocalToRelativeWorld = FDFMatrix::MakeClampedToRelativeWorldMatrixDouble(PositionHigh, AbsolutePreviousLocalToWorld);
				Parameters.PreviousLocalToRelativeWorld = FMatrix44f(PrevLocalToRelativeWorld);
				Parameters.PreviousRelativeWorldToLocal = FMatrix44f(PrevLocalToRelativeWorld.Inverse());
				Parameters.WorldToPreviousWorld = FMatrix44f(RelativeWorldToLocal * PrevLocalToRelativeWorld);
			}
			else
			{
				Parameters.PreviousLocalToRelativeWorld = Parameters.LocalToRelativeWorld;
				Parameters.PreviousRelativeWorldToLocal = Parameters.RelativeWorldToLocal;
				Parameters.WorldToPreviousWorld = FMatrix44f::Identity;
			}
		}

		if (GPrimitiveHasTileOffsetData)
		{
			const FLargeWorldRenderPosition AbsoluteActorWorldPositionTO { AbsoluteActorWorldPosition };
			Parameters.ActorWorldPositionHigh = AbsoluteActorWorldPositionTO.GetTile();
			Parameters.ActorWorldPositionLow = AbsoluteActorWorldPositionTO.GetOffset();
			const FLargeWorldRenderPosition ObjectWorldPositionTO { AbsoluteObjectWorldPosition };
			Parameters.ObjectWorldPositionHighAndRadius = FVector4f(ObjectWorldPositionTO.GetTile(), ObjectRadius);
			Parameters.ObjectWorldPositionLow = ObjectWorldPositionTO.GetOffset();
		}
		else
		{
			const FDFVector3 AbsoluteActorWorldPositionDF { AbsoluteActorWorldPosition };
			Parameters.ActorWorldPositionHigh = AbsoluteActorWorldPositionDF.High;
			Parameters.ActorWorldPositionLow = AbsoluteActorWorldPositionDF.Low;
			const FDFVector3 ObjectWorldPosition { AbsoluteObjectWorldPosition };
			Parameters.ObjectWorldPositionHighAndRadius = FVector4f(ObjectWorldPosition.High, ObjectRadius);
			Parameters.ObjectWorldPositionLow = ObjectWorldPosition.Low;
		}

		if (!bHasInstanceLocalBounds)
		{
			FRenderBounds InstanceLocalBounds(Parameters.LocalObjectBoundsMin, Parameters.LocalObjectBoundsMax);
			Parameters.InstanceLocalBoundsCenter = InstanceLocalBounds.GetCenter();
			Parameters.InstanceLocalBoundsExtent = InstanceLocalBounds.GetExtent();
		}

		if (!bHasPreSkinnedLocalBounds)
		{
			Parameters.PreSkinnedLocalBoundsMin = Parameters.LocalObjectBoundsMin;
			Parameters.PreSkinnedLocalBoundsMax = Parameters.LocalObjectBoundsMax;
		}

		Parameters.ObjectOrientation = Parameters.LocalToRelativeWorld.GetUnitAxis(EAxis::Z);

		{
			// Extract per axis scales from LocalToWorld transform
			FVector4f WorldX = FVector4f(Parameters.LocalToRelativeWorld.M[0][0], Parameters.LocalToRelativeWorld.M[0][1], Parameters.LocalToRelativeWorld.M[0][2], 0);
			FVector4f WorldY = FVector4f(Parameters.LocalToRelativeWorld.M[1][0], Parameters.LocalToRelativeWorld.M[1][1], Parameters.LocalToRelativeWorld.M[1][2], 0);
			FVector4f WorldZ = FVector4f(Parameters.LocalToRelativeWorld.M[2][0], Parameters.LocalToRelativeWorld.M[2][1], Parameters.LocalToRelativeWorld.M[2][2], 0);
			float ScaleX = FVector3f(WorldX).Size();
			float ScaleY = FVector3f(WorldY).Size();
			float ScaleZ = FVector3f(WorldZ).Size();
			Parameters.NonUniformScale = FVector4f(ScaleX, ScaleY, ScaleZ, FMath::Max3(FMath::Abs(ScaleX), FMath::Abs(ScaleY), FMath::Abs(ScaleZ)));
			Parameters.InvNonUniformScale = FVector3f(
				ScaleX > UE_KINDA_SMALL_NUMBER ? 1.0f / ScaleX : 0.0f,
				ScaleY > UE_KINDA_SMALL_NUMBER ? 1.0f / ScaleY : 0.0f,
				ScaleZ > UE_KINDA_SMALL_NUMBER ? 1.0f / ScaleZ : 0.0f);
		}

		// If SingleCaptureIndex is invalid, set it to 0 since there will be a default cubemap at that slot
		Parameters.SingleCaptureIndex = FMath::Max(Parameters.SingleCaptureIndex, 0);

		Parameters.Flags = 0;
		Parameters.Flags |= bReceivesDecals ? PRIMITIVE_SCENE_DATA_FLAG_DECAL_RECEIVER : 0u;
		Parameters.Flags |= bHasRayTracingRepresentation ? PRIMITIVE_SCENE_DATA_FLAG_HAS_RAY_TRACING_REPRESENTATION : 0u;
		Parameters.Flags |= bHasCapsuleRepresentation ? PRIMITIVE_SCENE_DATA_FLAG_HAS_CAPSULE_REPRESENTATION : 0u;
		Parameters.Flags |= bUseSingleSampleShadowFromStationaryLights ? PRIMITIVE_SCENE_DATA_FLAG_USE_SINGLE_SAMPLE_SHADOW_SL : 0u;
		Parameters.Flags |= (bUseVolumetricLightmap && bUseSingleSampleShadowFromStationaryLights) ? PRIMITIVE_SCENE_DATA_FLAG_USE_VOLUMETRIC_LM_SHADOW_SL : 0u;
		Parameters.Flags |= bOutputVelocity ? PRIMITIVE_SCENE_DATA_FLAG_OUTPUT_VELOCITY : 0u;
		Parameters.Flags |= bEvaluateWorldPositionOffset ? PRIMITIVE_SCENE_DATA_FLAG_EVALUATE_WORLD_POSITION_OFFSET : 0u;
		Parameters.Flags |= (Parameters.LocalToRelativeWorld.RotDeterminant() < 0.0f) ? PRIMITIVE_SCENE_DATA_FLAG_DETERMINANT_SIGN : 0u;
		Parameters.Flags |= bHasCustomData ? PRIMITIVE_SCENE_DATA_FLAG_HAS_PRIMITIVE_CUSTOM_DATA : 0u;
		Parameters.Flags |= ((LightingChannels & 0x1) != 0) ? PRIMITIVE_SCENE_DATA_FLAG_LIGHTING_CHANNEL_0 : 0u;
		Parameters.Flags |= ((LightingChannels & 0x2) != 0) ? PRIMITIVE_SCENE_DATA_FLAG_LIGHTING_CHANNEL_1 : 0u;
		Parameters.Flags |= ((LightingChannels & 0x4) != 0) ? PRIMITIVE_SCENE_DATA_FLAG_LIGHTING_CHANNEL_2 : 0u;
		Parameters.Flags |= bHasInstanceLocalBounds ? PRIMITIVE_SCENE_DATA_FLAG_HAS_INSTANCE_LOCAL_BOUNDS : 0u;
		Parameters.Flags |= bCastShadow ? PRIMITIVE_SCENE_DATA_FLAG_CAST_SHADOWS : 0u;
		Parameters.Flags |= bCastContactShadow ? PRIMITIVE_SCENE_DATA_FLAG_HAS_CAST_CONTACT_SHADOW : 0u;
		Parameters.Flags |= bHasInstanceDrawDistanceCull ? PRIMITIVE_SCENE_DATA_FLAG_INSTANCE_DRAW_DISTANCE_CULL : 0u;
		Parameters.Flags |= bHasWPODisableDistance ? PRIMITIVE_SCENE_DATA_FLAG_WPO_DISABLE_DISTANCE : 0u;
		Parameters.Flags |= bHasAlwaysEvaluateWPOMaterials ? PRIMITIVE_SCENE_DATA_FLAG_HAS_ALWAYS_EVALUATE_WPO_MATERIALS : 0u;
#if SUPPORT_REVERSE_CULLING_IN_NANITE
		Parameters.Flags |= bReverseCulling ? PRIMITIVE_SCENE_DATA_FLAG_REVERSE_CULLING : 0u;
#endif
		Parameters.Flags |= bHoldout ? PRIMITIVE_SCENE_DATA_FLAG_HOLDOUT : 0u;
		Parameters.Flags |= bDisableMaterialInvalidations ? PRIMITIVE_SCENE_DATA_FLAG_DISABLE_MATERIAL_INVALIDATIONS : 0u;
		Parameters.Flags |= bSplineMesh ? PRIMITIVE_SCENE_DATA_FLAG_SPLINE_MESH : 0u;
		Parameters.Flags |= bSkinnedMesh ? PRIMITIVE_SCENE_DATA_FLAG_SKINNED_MESH : 0u;
		Parameters.Flags |= bAllowInstanceCullingOcclusionQueries ? PRIMITIVE_SCENE_DATA_FLAG_INSTANCE_CULLING_OCCLUSION_QUERIES: 0u;
		Parameters.Flags |= bHasPixelAnimation ? PRIMITIVE_SCENE_DATA_FLAG_HAS_PIXEL_ANIMATION : 0u;
		Parameters.Flags |= bRayTracingFarField ? PRIMITIVE_SCENE_DATA_FLAG_RAYTRACING_FAR_FIELD : 0u;
		Parameters.Flags |= bRayTracingHasGroupId ? PRIMITIVE_SCENE_DATA_FLAG_RAYTRACING_HAS_GROUPID : 0u;
		Parameters.Flags |= bHasPerClusterDisplacementFallbackRaster ? PRIMITIVE_SCENE_DATA_FLAG_PER_CLUSTER_DISPLACEMENT_FALLBACK_RASTER : 0u;
		Parameters.Flags |= bIsFirstPerson ? PRIMITIVE_SCENE_DATA_FLAG_IS_FIRST_PERSON : 0u;
		// TODO: Rename the flag later - not done now to avoid affecting more shaders than needed
		Parameters.Flags |= bEnableObjectWorldPositionFix ? PRIMITIVE_SCENE_DATA_FLAG_UNUSED_2 : 0u;
		
		Parameters.VisibilityFlags = 0;
		Parameters.VisibilityFlags |= bCastHiddenShadow ? PRIMITIVE_VISIBILITY_FLAG_CAST_HIDDEN_SHADOW : 0u;
		Parameters.VisibilityFlags |= bVisibleInGame ? PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_GAME : 0u;
	#if WITH_EDITOR
		Parameters.VisibilityFlags |= bVisibleInEditor ? PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_EDITOR : 0u;
	#endif
		Parameters.VisibilityFlags |= bVisibleInMainPass ? PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_MAIN_PASS : 0u;
		Parameters.VisibilityFlags |= bVisibleInReflectionCaptures ? PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_REFLECTION_CAPTURES : 0u;
		Parameters.VisibilityFlags |= bVisibleInRealTimeSkyCaptures ? PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_REAL_TIME_SKY_CAPTURES : 0u;
		Parameters.VisibilityFlags |= bVisibleInRayTracing ? PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_RAY_TRACING : 0u;
		Parameters.VisibilityFlags |= bVisibleInLumenScene ? PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_LUMEN_SCENE : 0u;
		Parameters.VisibilityFlags |= bVisibleInSceneCaptureOnly ? PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_SCENE_CAPTURE_ONLY : 0u;
		Parameters.VisibilityFlags |= bHiddenInSceneCapture ? PRIMITIVE_VISIBILITY_FLAG_HIDDEN_IN_SCENE_CAPTURE : 0u;
		Parameters.VisibilityFlags |= bForceHidden ? PRIMITIVE_VISIBILITY_FLAG_FORCE_HIDDEN : 0u;

		return Parameters;
	}

private:
	FPrimitiveUniformShaderParameters Parameters;
	
	FMatrix AbsoluteLocalToWorld;
	FMatrix AbsolutePreviousLocalToWorld;
	FVector AbsoluteObjectWorldPosition;
	FVector AbsoluteActorWorldPosition;

	float ObjectRadius;

	uint32 LightingChannels : 3;
	uint32 bReceivesDecals : 1;
	uint32 bUseSingleSampleShadowFromStationaryLights : 1;
	uint32 bUseVolumetricLightmap : 1;
	uint32 bOutputVelocity : 1;
	uint32 bEvaluateWorldPositionOffset : 1;
	uint32 bCastShadow : 1;
	uint32 bCastContactShadow : 1;
	uint32 bCastHiddenShadow : 1;
	uint32 bHasRayTracingRepresentation : 1;
	uint32 bHasCapsuleRepresentation : 1;
	uint32 bHasPreSkinnedLocalBounds : 1;
	uint32 bHasInstanceLocalBounds : 1;
	uint32 bHasCustomData : 1;
	uint32 bHasPreviousLocalToWorld : 1;
	uint32 bVisibleInGame : 1;
	uint32 bVisibleInEditor : 1;
	uint32 bVisibleInMainPass : 1;
	uint32 bVisibleInReflectionCaptures : 1;
	uint32 bVisibleInRealTimeSkyCaptures : 1;
	uint32 bVisibleInRayTracing : 1;
	uint32 bVisibleInLumenScene : 1;
	uint32 bVisibleInSceneCaptureOnly : 1;
	uint32 bHiddenInSceneCapture : 1;
	uint32 bForceHidden : 1;
	uint32 bHasInstanceDrawDistanceCull : 1;
	uint32 bHasWPODisableDistance : 1;
	uint32 bHasAlwaysEvaluateWPOMaterials : 1;
	uint32 bReverseCulling: 1;
	uint32 bHoldout : 1;
	uint32 bDisableMaterialInvalidations : 1;
	uint32 bSplineMesh : 1;
	uint32 bSkinnedMesh : 1;
	uint32 bAllowInstanceCullingOcclusionQueries : 1;
	uint32 bHasPixelAnimation : 1;
	uint32 bRayTracingFarField : 1;
	uint32 bRayTracingHasGroupId : 1;
	uint32 bHasPerClusterDisplacementFallbackRaster : 1;
	uint32 bIsFirstPerson : 1;
	uint32 bEnableObjectWorldPositionFix : 1;
};
