// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialRelevance.h"

/**
 * The different types of relevance a primitive scene proxy can declare towards a particular scene view.
 * the class is only storing bits, and has an |= operator
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct FPrimitiveViewRelevance : public FMaterialRelevance
{
	// Warning: This class is memzeroed externally as 0 is assumed a
	// valid value for all members meaning 'not relevant'. If this
	// changes existing class usage should be re-evaluated

	/** The primitive's static elements are rendered for the view. */
	uint32 bStaticRelevance : 1; 
	/** The primitive's dynamic elements are rendered for the view. */
	uint32 bDynamicRelevance : 1;
	/** The primitive is drawn. */
	uint32 bDrawRelevance : 1;
	/** The primitive is casting a shadow. */
	uint32 bShadowRelevance : 1;
	/** The primitive should render velocity. */
	uint32 bVelocityRelevance : 1;
	/** The primitive should render to the custom depth pass. */
	uint32 bRenderCustomDepth : 1;
	/** The primitive should render to the depth prepass even if it's not rendered in the main pass. */
	uint32 bRenderInDepthPass : 1;
	/** The primitive should render to the base pass / normal depth / velocity rendering. */
	uint32 bRenderInMainPass : 1;
	/** The primitive is drawn only in the editor and composited onto the scene after post processing */
	uint32 bEditorPrimitiveRelevance : 1;
	/** The primitive's elements belong to a LevelInstance which is being edited and rendered again in the visualize LevelInstance pass */
	uint32 bEditorVisualizeLevelInstanceRelevance : 1;
	/** The primitive's static elements are selected and rendered again in the selection outline pass*/
	uint32 bEditorStaticSelectionRelevance : 1;
	/** The primitive is drawn only in the editor and composited onto the scene after post processing using no depth testing */
	uint32 bEditorNoDepthTestPrimitiveRelevance : 1;
	/** The primitive should have GatherSimpleLights called on the proxy when gathering simple lights. */
	uint32 bHasSimpleLights : 1;
	/** Whether the primitive uses non-default lighting channels. */
	uint32 bUsesLightingChannels : 1;
	/** Whether the primitive has materials that use volumetric translucent self shadow. */
	uint32 bTranslucentSelfShadow : 1;
	/** Whether the primitive should be rendered in the second stage depth only pass. */
	uint32 bRenderInSecondStageDepthPass : 1;

	/** 
	 * Whether this primitive view relevance has been initialized this frame.  
	 * Primitives that have not had ComputeRelevanceForView called on them (because they were culled) will not be initialized,
	 * But we may still need to render them from other views like shadow passes, so this tracks whether we can reuse the cached relevance or not.
	 */
	uint32 bInitializedThisFrame : 1;

	bool HasTranslucency() const 
	{
		return bNormalTranslucency || bSeparateTranslucency || bTranslucencyModulate || bPostMotionBlurTranslucency;
	}

	bool HasVelocity() const
	{
		return bVelocityRelevance || bOutputsTranslucentVelocity;
	}

	/** Default constructor */
	FPrimitiveViewRelevance()
	{
		// the class is only storing bits, the following avoids code redundancy
		uint8 * RESTRICT p = (uint8*)this;
		for(uint32 i = 0; i < sizeof(*this); ++i)
		{
			*p++ = 0;
		}

		// only exceptions (bugs we need to fix?):

		bOpaque = true;
		// without it BSP doesn't render
		bRenderInMainPass = true;
	}

	/** Bitwise OR operator.  Sets any relevance bits which are present in either. */
	FPrimitiveViewRelevance& operator|=(const FPrimitiveViewRelevance& B)
	{
		// the class is only storing bits, the following avoids code redundancy
		const uint8 * RESTRICT s = (const uint8*)&B;
		uint8 * RESTRICT d = (uint8*)this;
		for(uint32 i = 0; i < sizeof(*this); ++i)
		{
			*d = *d | *s; 
			++s;++d;
		}
		return *this;
	}
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
