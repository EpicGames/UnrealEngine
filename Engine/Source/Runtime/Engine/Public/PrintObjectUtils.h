// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"			// For TCHAR
#include "Misc/Build.h"				// For UE_BUILD_SHIPPING
#include "Misc/CoreMiscDefines.h"	// For UE_INTERNAL
#include "Misc/EnumClassFlags.h"

#if !UE_BUILD_SHIPPING

class UObject;
class UStruct;
class UClass;
class FOutputDevice;

/**
 *	Utility functions and console commands that print UObject state information.
 *	Can be used in a debugger as well.
**/

namespace UE
{

/**
 * Flags for specifying optional behaviors.
 */
enum class EPrintObjectFlag : uint32
{
	/** No optional behaviors. */
	None = 0,
	/** Include verbose information. */
	Verbose = (1 << 0),
	/** Include the initialization state for properties (only relevant for functions that print properties). */
	PropertyInitializationState = (1 << 1),
	/** Show the full archetype hierarchy (only relevant for functions that print archetypes). */
	FullArchetypeChain = (1 << 2),
};

ENUM_CLASS_FLAGS(EPrintObjectFlag);

/**
 * Lists all objects under a specified parent.
 * 
 * @param Object The outer object to use.
 * @param OutputDevice The output device to print to (can be NULL).
 */
UE_INTERNAL void PrintObjectsInOuter(UObject* Object, EPrintObjectFlag Flags = EPrintObjectFlag::None, FOutputDevice* OutputDevice = nullptr);

/**
 * Lists all objects with a given name.
 *
 * @param ObjectName The name to search for.
 * @param Flags For specifying optional behaviors.
 * @param OutputDevice The output device to print to (can be NULL).
 */
UE_INTERNAL void PrintObjectsWithName(const TCHAR* ObjectName, EPrintObjectFlag Flags = EPrintObjectFlag::None, FOutputDevice* OutputDevice = nullptr);

/**
 * Lists the properties of a UStruct instance.
 *
 * @param Struct The type of the UStruct instance.
 * @param StructData The data of the UStruct instance.
 * @param Flags For specifying optional behaviors.
 * @param OutputDevice The output device to print to (can be NULL).
 */
UE_INTERNAL void PrintStructProperties(UStruct* Struct, void* StructData, EPrintObjectFlag Flags = EPrintObjectFlag::None, FOutputDevice* OutputDevice = nullptr);

/**
 * Lists the property values of an object.
 *
 * @param Object The object to use.
 * @param Flags For specifying optional behaviors.
 * @param OutputDevice The output device to print to (can be NULL).
 */
UE_INTERNAL void PrintObjectProperties(UObject* Object, EPrintObjectFlag Flags = EPrintObjectFlag::None, FOutputDevice* OutputDevice = nullptr);

/**
 * Lists any overridable serialization overrides for a given object.
 *
 * @param Object The object to use.
 * @param Flags For specifying optional behaviors.
 * @param OutputDevice The output device to print to (can be NULL).
 */
UE_INTERNAL void PrintObjectOverrides(UObject* Object, EPrintObjectFlag Flags = EPrintObjectFlag::None, FOutputDevice* OutputDevice = nullptr);

/**
 * Outputs an object's archetype.
 *
 * @param Object The object to use.
 * @param Flags For specifying optional behaviors.
 * @param OutputDevice The output device to print to (can be NULL).
 */
UE_INTERNAL void PrintObjectArchetype(UObject* Object, EPrintObjectFlag Flags = EPrintObjectFlag::None, FOutputDevice* OutputDevice = nullptr);

/**
 * Outputs an object's IDO (Instance Data Object).
 *
 * @param Object The object to use.
 * @param Flags For specifying optional behaviors.
 * @param OutputDevice The output device to print to (can be NULL).
 */
UE_INTERNAL void PrintObjectIDO(UObject* Object, EPrintObjectFlag Flags = EPrintObjectFlag::None, FOutputDevice* OutputDevice = nullptr);

/**
 * Outputs a class' Class Default Object.
 *
 * @param Class The class to use.
 * @param Flags For specifying optional behaviors.
 * @param OutputDevice The output device to print to (can be NULL).
 */
UE_INTERNAL void PrintClassDefaultObject(const UClass* Class, EPrintObjectFlag Flags = EPrintObjectFlag::None, FOutputDevice* OutputDevice = nullptr);

/**
 * Outputs a struct's type hierarchy.
 *
 * @param Struct The struct to use.
 * @param Flags For specifying optional behaviors.
 * @param OutputDevice The output device to print to (can be NULL).
 */
UE_INTERNAL void PrintStructHierarchy(const UStruct* Struct, EPrintObjectFlag Flags = EPrintObjectFlag::None, FOutputDevice* OutputDevice = nullptr);

} // namespace UE

#endif // !UE_BUILD_SHIPPING
