// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "HealthSnapshot.generated.h"

class FPerformanceTrackingChart;

UCLASS(MinimalAPI)
class UHealthSnapshotBlueprintLibrary : public UBlueprintFunctionLibrary
{

	GENERATED_UCLASS_BODY()

	/**
	* Begins capturing FPS charts that can be used to include performance data in a HealthSnapshot. If snapshots are already running clears all accumulated performance data
	*
	*/
	UFUNCTION(Exec, BlueprintCallable, Category = "Performance | HealthSnapshot")
	static ENGINE_API void StartPerformanceSnapshots();

	/**
	* Stops capturing FPS charts only if StartHealthSnapshotChart has first been called. Does nothing if FPS charts are not running. HealthSnapshots captured after this is called will not include performance stats.
	*/
	UFUNCTION(Exec, BlueprintCallable, Category = "Performance | HealthSnapshot")
	static ENGINE_API void StopPerformanceSnapshots();


	/**
	* Writes a snapshot to the log. Captures memory stats by default. Also captures performance stats if called after StartHealthSnapshotChart and before SopHealthSnapshotChart.
	*
	* @param	SnapshotTitle			The name to be given to the new HealthSnapshot.
	*/
	UFUNCTION(Exec, BlueprintCallable, Category = "Performance | HealthSnapshot")
	static ENGINE_API void LogPerformanceSnapshot(const FString SnapshotTitle, bool bResetStats=true);

	/* The performance chart we register with the engine for tracking */
	static ENGINE_API TSharedPtr<FPerformanceTrackingChart> PerformanceChart;
};

/**

	Describes a health snapshot about the game at the current time including memory stats.
	If given a FPerformanceTrackingChart with MeasuredPerfTime > 0, this will also include a basic summary of the active FPS charting session as performance stats.
	You can create HealthSnapshots simply by creating a new object with the constructor.
	
	See UHealthSnapshotBlueprintLibrary for static helpers that can manage a performance chart if one does not already exist

	Snapshots can be dumped to logs or any other FOutputDevice.
*/
class FHealthSnapshot
{
public:

	/**
	* Create a snapshot of the current game health. Captures only memory stats.
	*
	* @param: Title The name of the new snapshot.
	*/
	ENGINE_API FHealthSnapshot(const TCHAR* Title);

	/**
	* Create a snapshot of the current game health. Captures both memory and performance stats.
	*
	* @param: Title The name of the new snapshot.
	* @param: GameplayFPSChart The chart which has been updated with performance stats to be included in the snapshot.
	*/
	ENGINE_API FHealthSnapshot(const TCHAR* Title, const FPerformanceTrackingChart* GameplayFPSChart);

	virtual ~FHealthSnapshot() = default;

	/**
	* Dumps a text blob describing all stats captured by the snapshot to the given output device. Outputs to the LogHealthSnapshot log category.
	*
	* @param: Ar The output device that will print the snapshot text blob. Use *GLog to simply write to the log.
	* @return: void
	*/
	ENGINE_API void Dump(FOutputDevice& Ar);

protected:

	/** Snapshots current memory stats */
	ENGINE_API virtual void CaptureMemoryStats();

	/** Snapshots performance stats if the given tracking chart is filled with FPS charting data (MeasuredPerfTime > 0)*/
	ENGINE_API virtual void CapturePerformanceStats(const FPerformanceTrackingChart* GameplayFPSChart);

	/* Dump a text blob describing all stats captured by the snapshot to the given output device with the given log category. */
	ENGINE_API virtual void DumpStats(FOutputDevice& Ar, FName CategoryName);

public:

	// Helper class that can describe memory in a system. Some systems may not provide Used/Peak values.
	template<typename T>
	struct FMemoryStat
	{
		FMemoryStat()
			: Size(T())
			, Used(T())
			, Peak(T())
		{}

		T Size;
		T Used;
		T Peak;
	};

	struct FThreadStat
	{
		FThreadStat()
			: PercentFramesBound(0)
			, HitchesPerMinute(0)
			, AvgTime(0)
		{}

		float PercentFramesBound;
		float HitchesPerMinute;
		float AvgTime;
	};

	template <typename T>
	struct FMmaStat
	{
		FMmaStat()
			: Min()
			, Max()
			, Avg()
		{}

		T Min;
		T Max;
		T Avg;
	};

	/** Memory data */
	// General "how much memory is used"
	FMemoryStat<float> CPUMemoryMB;
	FMemoryStat<float> StreamingMemoryMB;
	// System level info
	FMemoryStat<float> PhysicalMemoryMB;
	float LLMTotalMemoryMB;
	FMmaStat<uint64> PhysicalMemory; // more detailed physical memory stats obtained more frequently
	FMmaStat<uint64> VirtualMemory;
	TArray<typename FPlatformMemoryStats::FPlatformSpecificStat> PlatformMemoryStats;

	/** Performance data */
	double MeasuredPerfTime; // Duration of time the following performance values came from
	FThreadStat GameThread;
	FThreadStat RenderThread;
	FThreadStat RHIThread;
	FThreadStat GPU;
	double HitchesPerMinute;
	double AvgHitchTime;
	double MVP;
	double AvgFPS;
	FMmaStat<int> DrawCalls;
	FMmaStat<int> PrimitivesDrawn;
	FMmaStat<double> FrameTime;
	FMmaStat<double> DynamicResolution;

	/** Title of the Snapshot */
	FString Title;
};
