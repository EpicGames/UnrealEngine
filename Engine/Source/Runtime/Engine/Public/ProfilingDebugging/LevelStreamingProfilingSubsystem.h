// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Subsystems/WorldSubsystem.h"
#include "Tasks/Task.h"
#include "UObject/ObjectKey.h"
#include "UObject/UObjectAnnotation.h"

#include "LevelStreamingProfilingSubsystem.generated.h"

class AGameStateBase;
class UWorld;
class ULevel;
class ULevelStreaming;

enum class ELevelStreamingState : uint8;
enum class ELevelStreamingTargetState : uint8;

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, LevelStreaming);

/** 
 * This subsystem captures level streaming operations for a specified time and outputs a .tsv (tab separated values) file to the 
 * profiling directory containing the amount of time spent loading levels and the time levels spent queuing for theability to load.
 */
UCLASS(MinimalAPI)
class ULevelStreamingProfilingSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	// WorldSubsystem interface
	ENGINE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	ENGINE_API virtual void PostInitialize() override;
	ENGINE_API virtual void Deinitialize() override;

	ENGINE_API ULevelStreamingProfilingSubsystem(const FObjectInitializer&);
	ENGINE_API ~ULevelStreamingProfilingSubsystem();

	// Begin recording timings for level streaming events.
	ENGINE_API void StartTracking();
	// Top recording timings for level streaming events and output a .tsv (tab separated values) file to the Profiling directory.
	ENGINE_API void StopTrackingAndReport();

	inline bool IsTracking() const { return bIsTracking; }

	// Gives child classes an opportunity to clean up after a report is produced.
	virtual void PostReport() { }

	// Access to tuning values set by cvars for other systems
	/* Returns the squared distance (e.g. from world partition cell bounds) at which a level is considered to have streamed in too late. */
	static ENGINE_API double GetLateStreamingDistanceSquared();

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)
	ENGINE_API void RecordLevelStreamingCountsToCSV();
#endif

protected:
	// WorldSubsystem interface
	ENGINE_API virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const;

	enum class ELevelState;
	struct FLevelStats;

	/** 
	 * Gives child classes an opportunity to track additional data along with the main report.
	 * Levels usually progress linearly through the ELevelState values, but may skip values depending on the initial state of the world
	 * when tracking begins.
	 * @param TrackingIndex Integer parameter tracking the row of the report data is being collected for (may be non-contiguous).
	 * @param StreamingLevel The ULevelStreaming controlling the streaming for this level.
	 * @param PreviousState The last state recorded for the level.
	 * @param NewState The new state for the level.
	 */
	virtual void UpdateTrackingData(
		int32 TrackingIndex, 
		FLevelStats& BaseStats,
		const ULevelStreaming* StreamingLevel, 
		ELevelState PreviousState, 
		ELevelState NewState) {}

	/** 
	 * Gives child classes an opportunity to add additional data to the final report.
	 * They should prepend anything they append with \t and separate each field they add with \t.
	 * @param Builder String builder to write to.
	 */
	virtual void AugmentReportHeader(FUtf8StringBuilderBase& Builder) {}

	/** 
	 * Gives child classes an opportunity to add additional data to the final report.
	 * They should prepend anything they append with \t and separate each field they add with \t.
	 * @param Builder String builder to write to.
	 * @param TrackingIndex The tracking index from a previous call to UpdateTrackingData to append data for.
	 */
	virtual void AugmentReportRow(FUtf8StringBuilderBase& Builder, int32 TrackingIndex) {}

	ENGINE_API static const TCHAR* EnumToString(ULevelStreamingProfilingSubsystem::ELevelState State);

	ENGINE_API TConstArrayView<FLevelStats> GetLevelStats() const;
	
	enum class ELevelState
	{
		None,
		QueuedForLoading,
		Loading,
		Loaded,
		QueuedForAddToWorld,
		AddingToWorld,
		AddedToWorld,
		QueuedForRemoveFromWorld,
		RemovingFromWorld,
		RemovedFromWorld,
	};

	struct FActiveLevel
	{
		explicit FActiveLevel(int32 InStatsIndex) 
		: StatsIndex(InStatsIndex)
		{
		}

		~FActiveLevel() = default;

		ELevelState State = ELevelState::QueuedForLoading;
		double StateStartTime = 0.0; // world time we entered the current state
		TOptional<double> TimeAddedToWorld; // time we were last added to the world for calculating how long the level was streamed in for 
		int32 StatsIndex = INDEX_NONE; // Index into subsystem's stats array to update 
	};

	struct FLevelStats
	{
		FLevelStats() = default;
		~FLevelStats() = default;

		FName PackageNameOnDisk = {}; // Could consider using FPackagePath here?
		FName PackageNameInMemory = {};

		FBox CellBounds = FBox(ForceInit), ContentBounds = FBox(ForceInit);

		// All time units in seconds
		TOptional<double> TimeQueuedForLoading;
		TOptional<double> TimeLoading;
		TOptional<double> TimeQueueudForAddToWorld;
		TOptional<double> TimeAddingToWorld;
		TOptional<double> TimeInWorld;
		TOptional<double> TimeQueuedForRemoveFromWorld;
		TOptional<double> TimeRemovingFromWorld;

		// Distance at which this level was fully streamed in 
		TOptional<double> FinalStreamInDistance_Cell;
		TOptional<double> FinalStreamInDistance_Content;

		// Location of streaming source (e.g. player) when level was fully streamed in 
		TOptional<FVector> FinalStreamInLocation;

		// Is this level a hierarchical LOD representation of more detailed content
		bool bIsHLOD = false;
		// If a level is tracked but never starts loading before being removed, we don't include it in the results 
		bool bValid = false;
	};

private:
	// Callbacks for level streaming status updates
	void OnLevelStreamingTargetStateChanged(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LevelIfLoaded, ELevelStreamingState CurrentState, ELevelStreamingTargetState PrevTarget, ELevelStreamingTargetState NewTarget);
	void OnLevelStreamingStateChanged(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LevelIfLoaded, ELevelStreamingState PrevState, ELevelStreamingState NewState);

	void OnLevelQueuedForLoading(UWorld* World, const ULevelStreaming* StreamingLevel);
	void OnLevelUnqueuedForLoading(UWorld* World, const ULevelStreaming* StreamingLevel);
	void OnLevelStartedAsyncLoading(UWorld* World, const ULevelStreaming* StreamingLevel);
	void OnLevelFinishedAsyncLoading(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);
	void OnLevelQueuedForAddToWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);
	void OnLevelUnqueuedForAddToWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);
	void OnLevelStartedAddToWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);
	void OnLevelFinishedAddToWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);
	void OnLevelQueuedForRemoveFromWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);
	void OnLevelUnqueuedForRemoveFromWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);
	void OnLevelStartedRemoveFromWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);
	void OnLevelFinishedRemoveFromWorld(UWorld* World, const ULevelStreaming* StreamingLevel, ULevel* LoadedLevel);
	void OnStreamingLevelRemoved(UWorld* World, const ULevelStreaming* StreamingLevel);

	TUniquePtr<FActiveLevel> MakeActiveLevel(const ULevelStreaming* StreamingLevel, ELevelState InitialState, ULevel* LoadedLevel=nullptr);
	void UpdateLevelState(FActiveLevel& Level, const ULevelStreaming* StreamingLevel, ELevelState NewState, double Time);

	// Handles for registered callbacks
	FDelegateHandle Handle_OnLevelStreamingTargetStateChanged;
	FDelegateHandle Handle_OnLevelStreamingStateChanged;
	FDelegateHandle Handle_OnLevelBeginAddToWorld;
	FDelegateHandle Handle_OnLevelBeginRemoveFromWorld;

	bool bIsTracking = false;

	// A level currently pending or in the process of streaming
	TMap<TObjectKey<const ULevelStreaming>, TUniquePtr<FActiveLevel>> ActiveLevels; // Can remove indirection when TMap supports forward declared values

	// Stats for levels which have been at least partially loaded. 
	// May contain duplicates for a given level if it streamed out and back in again.
	TArray<FLevelStats> LevelStats;

	// Possibly-executing task dependend on our recorded stats. Cannot start recording again until this task is complete.
	UE::Tasks::TTask<void> ReportWritingTask;

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)
	struct FWorldLevelStreamingStats
	{
		void Remove(const ULevelStreaming* StreamingLevel)
		{
			LevelsLoading.Remove(StreamingLevel);
			LevelsMakingVisible.Remove(StreamingLevel);
			LevelsMakingInvisible.Remove(StreamingLevel);
		}
		TSet<const ULevelStreaming*> LevelsLoading;
		TSet<const ULevelStreaming*> LevelsMakingVisible;
		TSet<const ULevelStreaming*> LevelsMakingInvisible;
	};

	FWorldLevelStreamingStats WorldLevelStreamingStats;
#endif
};


