// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Engine.h"
#include "HAL/MallocLeakDetection.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Containers/Ticker.h"


/**

	FMallocLeakReporter is a helper class that works alongside FMallocLeakDetection to provide easy access to
	reports of low-level memory leaks.

	The reporter can be operated from the console where it can be started/stopped and used to periodically
	or manually report leaks, or it can be called from game code to implement this functionality at desired times.

	Example Console Usage:

	"mallocleak.start report=300"	- start tracking leaks and generate a report every 300 secs
	"mallocleak report"				- report allocations and leaks
	"mallocleak stop"				- stop tracking leaks

	Example Code usage

	FMallocLeakReporter::Get().Start(0, 300) - start tracking allocs > 0 bytes and generate a report every 300 secs
	FMallocLeakReporter::Get().WriteReports() - Writes reports and returns the number of suspected leaks
	FMallocLeakReporter::Get().WriteReport(TEXT("BigAllocs"), Options); - Write custom report of allocations that matches "Options"
	FMallocLeakReporter::Get().Stop() - stop tracking allocations


	Reports are written to the profiling dir, e,g GameName/Saved/Profiling/SessionName

 */
class FMallocLeakReporter
{
	DECLARE_MULTICAST_DELEGATE_TwoParams(FMallocLeakReportDelegate, const int32, const int32);

public:
	struct EReportOption
	{
		enum
		{
			ReportLeaks = (1 << 0),
			ReportAllocs = (1 << 1),
			ReportMemReport = (1 << 2),

			ReportAll = ReportLeaks | ReportAllocs | ReportMemReport,
		};
	};

	/** Return singleton instance */
	static ENGINE_API FMallocLeakReporter& Get();

	/**
	 * Starts tracking allocations.
	 *
	 * @param: 	FilterSize		Only track allocations >= this value in bytes. Higher values affect performance less
	 * @param: 	ReportOnTime	Write out a report every N seconds
	 * @return: void		
	*/ENGINE_API void		Start(int32 FilterSize = 0, float ReportOnTime = 0.0f);

	/**
	 * Stop tracking leaks
	*/
	ENGINE_API void		Stop();

	/**
	 * Clears all accumulated data
	*/
	ENGINE_API void		Clear();

	/**
	* Returns our enabled state
	*
	* @return: bool
	*/
	bool		IsEnabled() const { return Enabled; }

	/**
	 *	Writes out a set of reports according to our defaults
	 */
	ENGINE_API int32		WriteReports(const uint32 ReportFlags= EReportOption::ReportAll);

	/**
	 *	Writes out a report according to the passed in options
	 */
	ENGINE_API int32		WriteReport(const TCHAR* ReportName, const FMallocLeakReportOptions& Options);

	/**
	 * Sets default options for what are considered memory leaks
	 *
	 * @param: 	Options		
	 * @return: void		
	*/
	void SetDefaultLeakReportOptions(const FMallocLeakReportOptions& Options)
	{
		DefaultLeakReportOptions = Options;
	}

	/**
	 * Sets default options for reporting allocations
	 *
	 * @param: 	Options		
	 * @return: void		
	*/
	void SetDefaultAllocReportOptions(const FMallocLeakReportOptions& Options)
	{
		DefaultAllocReportOptions = Options;
	}

protected:

	/**
	 *	Private constructor
	 */
	ENGINE_API FMallocLeakReporter();

	/**
	 *	Called internally to generate rate checkpoints
	 */
	ENGINE_API void		Checkpoint();

	bool						Enabled;
	int32						ReportCount;
	FTSTicker::FDelegateHandle	CheckpointTicker;
	FTSTicker::FDelegateHandle	ReportTicker;
	FMallocLeakReportDelegate	ReportDelegate;

	// Report vars
	FMallocLeakReportOptions	DefaultLeakReportOptions;
	FMallocLeakReportOptions	DefaultAllocReportOptions;
};
