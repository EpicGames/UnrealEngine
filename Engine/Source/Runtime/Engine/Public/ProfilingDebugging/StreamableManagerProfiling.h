// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "ProfilingDebugging/CountersTrace.h"

#ifndef UE_STREAMABLE_MANAGER_PROFILING_ENABLED
#define UE_STREAMABLE_MANAGER_PROFILING_ENABLED (COUNTERSTRACE_ENABLED || CSV_PROFILER_STATS) && !UE_AUTORTFM
#endif //  UE_STREAMABLE_MANAGER_PROFILING_ENABLED

#if UE_STREAMABLE_MANAGER_PROFILING_ENABLED
/**
 * A class used for profiling streamable manager requests
 */
class FStreamableManagerProfiling
{
public:
	static void TryInitialize();

	static void OnStreamableQueuedForAsyncLoad();
	static void OnStreamableCompleted(double Time);
	static void OnStreamablesCancelled(int32 NumCancelled);
	static void OnJITLoaderCreated();
	static void OnJITLoaderDestroyed();


public:
	static void OnAsyncLoadingFlushed();
	static void HandlePreLoadMap(const FString& MapName);

private:
	static void Reset();

private:
	static FDelegateHandle OnAsyncLoadingFlushedHandle;
	static FDelegateHandle OnPreMapLoadHandle;
	static std::atomic<bool> bIsInitialized;

	static std::atomic<int32> JITLoadersActive;
	static std::atomic<int32> StreamablesInFlight;
	static std::atomic<int32> TotalStreamablesQueued;
	static std::atomic<int32> TotalStreamablesCancelled;
	static std::atomic<double> TotalStreamableProcessingTime;
	static std::atomic<int32> TotalStreamableProcessingTimeCount;	
};
#endif // UE_STREAMABLE_MANAGER_PROFILING_ENABLED
