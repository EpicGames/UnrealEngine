// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/World.h"

#if !defined(UE_SCREENSHOT_TRACE_ENABLED)
	#define UE_SCREENSHOT_TRACE_ENABLED UE_TRACE_ENABLED
#endif

#if UE_SCREENSHOT_TRACE_ENABLED

class ULevel;

class FTraceScreenshot
{
public:

// In no logging configurations all log categories are of type FNoLoggingCategory, which has no relation with
// FLogCategoryBase. In order to not need to conditionally set the argument alias the type here.
#if NO_LOGGING
	typedef FNoLoggingCategory FLogCategoryAlias;
#else
	typedef FLogCategoryBase FLogCategoryAlias;
#endif

	static ENGINE_API void RequestScreenshot(FString Name, bool bShowUI, const FLogCategoryAlias& LogCategory = LogCore);

	/* 
	* Add the provided screenshot to the trace.
	* @param InSizeX - The width of the screenshot.
	* @param InSizeY - The heigth of the screenshot.
	* @param InImageData - The data of the screenshot.
	* @param InScreenshotName - The name of the screenshot.
	* @param DesiredX - Optionally resize the image to the desired width before tracing. Aspect ratio is preserved.
	*/
	static ENGINE_API void TraceScreenshot(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData, const FString& InScreenshotName, int32 DesiredX = -1);
	static ENGINE_API void TraceScreenshot(int32 InSizeX, int32 InSizeY, const TArray<FLinearColor>& InImageData, const FString& InScreenshotName, int32 DesiredX = -1);

	/**
	* Returns true if the screenshot should not be writted to a file.
	*/
	static bool ShouldSuppressWritingToFile() { return bSuppressWritingToFile; }

	/**
	* Reset the internal state of the TraceScreenshot system.
	*/
	static ENGINE_API void Reset();

private:
	ENGINE_API FTraceScreenshot();
	FTraceScreenshot(const FTraceScreenshot& Other) {}
	FTraceScreenshot(const FTraceScreenshot&& Other) {}
	void operator =(const FTraceScreenshot& Other) {}
	ENGINE_API ~FTraceScreenshot();
	ENGINE_API void Unbind();

private:
	static ENGINE_API bool bSuppressWritingToFile;
};

#endif // UE_SCREENSHOT_TRACE_ENABLED