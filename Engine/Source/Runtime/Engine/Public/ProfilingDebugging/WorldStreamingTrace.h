// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Subsystems/WorldSubsystem.h"

#include "WorldStreamingTrace.generated.h"

// EXPERIMENTAL: The compile guards (UE_TRACE_WORLD_STREAMING_*_ENABLED) and trace channels (WorldStreamingChannel, WorldStreamingPriorityChannel, WorldStreamingDependenciesChannel) declared in this header are subject to change without deprecation.
// Code, build scripts, or capture commands depending on these names may need updates between engine versions.

#ifndef UE_TRACE_WORLD_STREAMING_ENABLED
	#define UE_TRACE_WORLD_STREAMING_ENABLED (UE_TRACE_MINIMAL_ENABLED && !UE_BUILD_SHIPPING)
#endif // UE_TRACE_WORLD_STREAMING_ENABLED

// Separate guard for world streaming containers priority tracing — gates the per-frame priority sort over all streaming cells.
#ifndef UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
	#define UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED (UE_TRACE_WORLD_STREAMING_ENABLED && !UE_BUILD_SHIPPING)
#endif // UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED

// Separate guard for world streaming containers dependency tracing — gates per-cell emission of transitive package dependencies via PackageStore DFS.
#ifndef UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
	#define UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED (UE_TRACE_WORLD_STREAMING_ENABLED && !UE_BUILD_SHIPPING)
#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED

#if UE_TRACE_WORLD_STREAMING_ENABLED
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogWorldStreamingTrace, Log, All);
UE_TRACE_MINIMAL_CHANNEL_EXTERN(WorldStreamingChannel, ENGINE_API)
#endif

#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
UE_TRACE_MINIMAL_CHANNEL_EXTERN(WorldStreamingPriorityChannel, ENGINE_API)
#endif

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
UE_TRACE_MINIMAL_CHANNEL_EXTERN(WorldStreamingDependenciesChannel, ENGINE_API)
#endif

namespace UE::WorldStreamingTrace
{
	enum class UE_EXPERIMENTAL(5.8, "World streaming trace API is experimental and subject to change.") EStreamingContainerState : uint8
	{
		Unloaded,
		Loading,
		Loaded,
		Activating,
		Active,
		Deactivating
	};
}

UCLASS(MinimalAPI)
class UE_EXPERIMENTAL(5.8, "World streaming trace API is experimental and subject to change.") UWorldStreamingTraceSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:

	//~ Begin UWorldSubsystem Interface.
	ENGINE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	ENGINE_API virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override;
	//~ End UWorldSubsystem Interface.

#if UE_TRACE_WORLD_STREAMING_ENABLED
	//~ Begin USubsystem Interface.
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;
	//~ End USubsystem Interface.

	// Containers
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	void TraceContainerStateChange(uint64 InContainerId, UE::WorldStreamingTrace::EStreamingContainerState InState);
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	void EnsureContainerDescribed(uint64 InContainerId, uint64 InParentId, FStringView InName, FStringView InPackageName, const FBox& InBounds, const TArray<uint64>& InTags);
	bool IsContainerDescribed(uint64 InContainerId) const;

	// Streaming Sources
	void EnsureStreamingSourceDescribed(uint64 InStreamingSourceId, FStringView InName);
	bool IsStreamingSourceDescribed(uint64 InStreamingSourceId) const;
	void TraceStreamingSourceUpdate(uint64 InStreamingSourceId, const FVector& InLocation);
	// Call once per frame after all TraceStreamingSourceUpdate calls. Deactivates sources not updated this frame.
	void FlushInactiveStreamingSources();

	// Tags
	void EnsureTagGroupDescribed(uint64 InTagGroupId, FStringView InTagGroupName);
	void EnsureTagDescribed(uint64 InTagId, uint64 InTagGroupId, uint64 InParentId, FStringView InName);

	// Container priority
#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
	// Called on GT to decide whether the per-update priority sort in CaptureDebugCellsPriorityTraceData is worth doing this frame.
	// Returns true if the subsystem exists for InWorld, the channel is enabled, and the throttle interval (Trace.WorldStreaming.PriorityUpdateInterval) has elapsed since the last emit.
	static bool ShouldTraceContainerPriorityThisFrame(const UWorld* InWorld);
	void TraceContainerPriorityBatch(TArrayView<const uint64> InContainerIds, TArrayView<const float> InPriorities);
#endif // UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED

	// Container dependencies
#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
	void TraceDependenciesForPackage(uint64 InContainerId, FName InPackageName);
#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED

private:
	void TraceContainerDescription(uint64 InContainerId, uint64 InParentId, FStringView InName, FStringView InPackageName, const FBox& InBounds, const TArray<uint64>& InTags);
	void TraceStreamingSourceDescription(uint64 InStreamingSourceId, FStringView InName);
	void TraceTagGroupDescription(uint64 InTagGroupId, FStringView InTagGroupName);
	void TraceTagDescription(uint64 InTagId, uint64 InTagGroupId, uint64 InParentId, FStringView InName);

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
	void TraceContainerDependencies(uint64 InContainerId, TArrayView<const uint64> InDependencyIds);
	void ResolveAndEnsurePackageNamesMapped(TArrayView<const uint64> InPackageIds);
#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED

	uint64 WorldId = 0;

#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
	double LastPriorityEmitTime = 0.0;
#endif // UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED

	TSet<uint64> KnownContainers;
	TSet<uint64> KnownStreamingSources;
	TSet<uint64> KnownTagGroups;
	TSet<uint64> KnownTags;
	TMap<uint64, FVector> StreamingSourceLocations;
	TSet<uint64> ActiveStreamingSources;

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
	TSet<uint64> KnownPackageMappings;
	TArray<uint64> PendingNameMappingPackageIds;

	void ResolveAndTracePackageNames(TArrayView<const uint64> InPackageIds);
	void OnNameMappingBuildComplete();
#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
#endif // UE_TRACE_WORLD_STREAMING_ENABLED
};