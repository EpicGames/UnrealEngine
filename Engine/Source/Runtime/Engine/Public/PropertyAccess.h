// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/Object.h"

#include "PropertyAccess.generated.h"

struct FPropertyAccessSystem;
struct FPropertyAccessLibrary;
enum class EPropertyAccessType : uint8;

// The various types of property copy
UENUM()
enum class EPropertyAccessCopyBatch : uint8
{
	// A copy of internal->internal data, unbatched
	InternalUnbatched,

	// A copy of external->internal data, unbatched
	ExternalUnbatched,

	// A copy of internal->internal data, batched
	InternalBatched,

	// A copy of external->internal data, batched
	ExternalBatched,

	Count
};

namespace PropertyAccess
{
	/** Batch ID - organisation/batching strategy determined by client systems via compilation */
	struct FCopyBatchId
	{
		explicit FCopyBatchId(int32 InId)
			: Id(InId)
		{}
		
		int32 Id;
	};


	UE_DEPRECATED(5.0, "Please use PatchPropertyOffsets instead")
	ENGINE_API extern void PostLoadLibrary(FPropertyAccessLibrary& InLibrary);

	/** 
	 * Called to patch up library after it is loaded or linked
	 * This converts all FName-based paths into node-based paths that provide an optimized way of accessing properties.
	 */
	ENGINE_API extern void PatchPropertyOffsets(FPropertyAccessLibrary& InLibrary);

	/** 
	 * Process a 'tick' of a property access instance. 
	 * Note internally allocates via FMemStack and pushes its own FMemMark
	 */
	ENGINE_API extern void ProcessCopies(UObject* InObject, const FPropertyAccessLibrary& InLibrary, const FCopyBatchId& InBatchId);

	UE_DEPRECATED(5.0, "Please use the signature that takes a FCopyBatchId BatchId")
	ENGINE_API extern void ProcessCopies(UObject* InObject, const FPropertyAccessLibrary& InLibrary, EPropertyAccessCopyBatch InBatchType);

	/** 
	 * Process a single copy 
	 * Note that this can potentially allocate via FMemStack, so inserting FMemMark before a number of these calls is recommended
	 */
	ENGINE_API extern void ProcessCopy(UObject* InObject, const FPropertyAccessLibrary& InLibrary, const FCopyBatchId& InBatchId, int32 InCopyIndex, TFunctionRef<void(const FProperty*, void*)> InPostCopyOperation);

	UE_DEPRECATED(5.0, "Please use the signature that takes a FCopyBatchId BatchId")
	ENGINE_API extern void ProcessCopy(UObject* InObject, const FPropertyAccessLibrary& InLibrary, EPropertyAccessCopyBatch InBatchType, int32 InCopyIndex, TFunctionRef<void(const FProperty*, void*)> InPostCopyOperation);

	UE_DEPRECATED(5.0, "Property Access Events are no longer supported")
	ENGINE_API extern void BindEvents(UObject* InObject, const FPropertyAccessLibrary& InLibrary);

	UE_DEPRECATED(5.0, "Property Access Events are no longer supported")
	ENGINE_API extern int32 GetEventId(const UClass* InClass, TArrayView<const FName> InPath);

	/** Gets the property and address of the specified access, minimally-resolving all compiled-in indirections */
	ENGINE_API extern void GetAccessAddress(UObject* InObject, const FPropertyAccessLibrary& InLibrary, int32 InAccessIndex, TFunctionRef<void(const FProperty*, void*)> InFunction);
}

// The type of an indirection
UENUM()
enum class EPropertyAccessIndirectionType : uint8
{
	// Access node is a simple basePtr + offset
	Offset,

	// Access node needs to dereference an object at its current address
	Object,

	// Access node indexes a dynamic array
	Array,

	// Access node calls a script function to get a value
	ScriptFunction,

	// Access node calls a native function to get a value
	NativeFunction,
};

// For object nodes, we need to know what type of object we are looking at so we can cast appropriately
UENUM()
enum class EPropertyAccessObjectType : uint8
{
	// Access is not an object
	None,

	// Access is an object
	Object,

	// Access is a weak object
	WeakObject,

	// Access is a soft object
	SoftObject,
};

// Runtime-generated access node.
// Represents:
// - An offset within an object 
// - An indirection to follow (object, array, function)
USTRUCT()
struct FPropertyAccessIndirection
{
	GENERATED_BODY()

	FPropertyAccessIndirection() = default;

private:
	friend struct ::FPropertyAccessSystem;

	// Property of this indirection. Used for arrays and functions (holds the return value property for functions)
	UPROPERTY()
	TFieldPath<FProperty> Property;

	// Function if this is a script of native function indirection
	UPROPERTY()
	TObjectPtr<UFunction> Function = nullptr;

	// Return buffer size if this is a script of native function indirection
	UPROPERTY()
	int32 ReturnBufferSize = 0;

	// Return buffer alignment if this is a script of native function indirection
	UPROPERTY()
	int32 ReturnBufferAlignment = 0;

	// Array index if this is an array indirection
	UPROPERTY()
	int32 ArrayIndex = INDEX_NONE;

	// Offset of this indirection within its containing object
	UPROPERTY()
	uint32 Offset = 0;

	// Object type if this is an object indirection
	UPROPERTY()
	EPropertyAccessObjectType ObjectType = EPropertyAccessObjectType::None;

	// The type of this indirection
	UPROPERTY()
	EPropertyAccessIndirectionType Type = EPropertyAccessIndirectionType::Offset;
};

// A single property access list. This is a list of FPropertyAccessIndirection
USTRUCT()
struct FPropertyAccessIndirectionChain
{
	GENERATED_BODY()

	FPropertyAccessIndirectionChain() = default;

private:
	friend struct ::FPropertyAccessSystem;

	// Leaf property
	UPROPERTY()
	TFieldPath<FProperty> Property = nullptr;

	// Index of the first indirection of a property access
	UPROPERTY()
	int32 IndirectionStartIndex = INDEX_NONE;

	// Index of the last indirection of a property access
	UPROPERTY()
	int32 IndirectionEndIndex = INDEX_NONE;
};

// Flags for a segment of a property access path
// Note: NOT an UENUM as we dont support mixing flags and values properly in UENUMs, e.g. for serialization.
enum class EPropertyAccessSegmentFlags : uint16
{
	// Segment has not been resolved yet, we don't know anything about it
	Unresolved = 0,

	// Segment is a struct property
	Struct,

	// Segment is a leaf property
	Leaf,

	// Segment is an object
	Object,

	// Segment is a weak object
	WeakObject,

	// Segment is a soft object
	SoftObject,

	// Segment is a dynamic array. If the index is INDEX_NONE, then the entire array is referenced.
	Array,

	// Segment is a dynamic array of structs. If the index is INDEX_NONE, then the entire array is referenced.
	ArrayOfStructs,

	// Segment is a dynamic array of objects. If the index is INDEX_NONE, then the entire array is referenced.
	ArrayOfObjects,

	// Entries before this are exclusive values
	LastExclusiveValue = ArrayOfObjects,

	// Segment is a function
	Function		= (1 << 15),

	// All modifier flags
	ModifierFlags = (Function),
};

ENUM_CLASS_FLAGS(EPropertyAccessSegmentFlags);

// A segment of a 'property path' used to access an object's properties from another location
USTRUCT()
struct FPropertyAccessSegment
{
	GENERATED_BODY()

	FPropertyAccessSegment() = default;

private:
	friend struct FPropertyAccessSystem;
	friend struct FPropertyAccessEditorSystem;

	/** The sub-component of the property path, a single value between .'s of the path */
	UPROPERTY()
	FName Name = NAME_None;

	/** The Class or ScriptStruct that was used last to resolve Name to a property. */
	UPROPERTY()
	TObjectPtr<UStruct> Struct = nullptr;

	/** The cached property on the Struct that this Name resolved to at compile time. If this is a Function segment, then this is the return property of the function. */
	UPROPERTY()
	TFieldPath<FProperty> Property;

	/** If this segment is a function, EPropertyAccessSegmentFlags::Function flag will be present and this value will be valid */
	UPROPERTY()
	TObjectPtr<UFunction> Function = nullptr;

	/** The optional array index. */
	UPROPERTY()
	int32 ArrayIndex = INDEX_NONE;

	/** @see EPropertyAccessSegmentFlags */
	UPROPERTY()
	uint16 Flags = (uint16)EPropertyAccessSegmentFlags::Unresolved;
};

// A property access path. References a string of property access segments.
// These are resolved at load time to create corresponding FPropertyAccess entries
USTRUCT() 
struct FPropertyAccessPath
{
	GENERATED_BODY()

	FPropertyAccessPath()
		: PathSegmentStartIndex(INDEX_NONE)
		, PathSegmentCount(INDEX_NONE)
	{
	}

private:
	friend struct FPropertyAccessSystem;
	friend struct FPropertyAccessEditorSystem;

	// Index into the library's path segments. Used to provide a starting point for a path resolve
	UPROPERTY()
	int32 PathSegmentStartIndex = INDEX_NONE;

	// The count of the path segments.
	UPROPERTY()
	int32 PathSegmentCount = INDEX_NONE;
};

UENUM()
enum class EPropertyAccessCopyType : uint8
{
	// No copying
	None,

	// For plain old data types, we do a simple memcpy.
	Plain,

	// For more complex data types, we need to call the properties copy function
	Complex,

	// Read and write properties using bool property helpers, as source/dest could be bitfield or boolean
	Bool,
	
	// Use struct copy operation, as this needs to correctly handle CPP struct ops
	Struct,

	// Read and write properties using object property helpers, as source/dest could be regular/weak/soft etc.
	Object,

	// FName needs special case because its size changes between editor/compiler and runtime.
	Name,

	// Array needs special handling for fixed size arrays
	Array,

	// Promote the type during the copy
	// Bool promotions
	PromoteBoolToByte,
	PromoteBoolToInt32,
	PromoteBoolToInt64,
	PromoteBoolToFloat,
	PromoteBoolToDouble,

	// Byte promotions
	PromoteByteToInt32,
	PromoteByteToInt64,
	PromoteByteToFloat,
	PromoteByteToDouble,

	// Int32 promotions
	PromoteInt32ToInt64,
	PromoteInt32ToFloat,		// This is strictly sketchy because of potential data loss, but it is usually OK in the general case
	PromoteInt32ToDouble,

	// Float promotions		// LWC_TODO: Float/double should become synonyms?
	PromoteFloatToDouble,
	DemoteDoubleToFloat,	// LWC_TODO: This should not ship!

	PromoteArrayFloatToDouble,
	DemoteArrayDoubleToFloat,

	PromoteMapValueFloatToDouble,
	DemoteMapValueDoubleToFloat,
};

// A property copy, represents a one-to-many copy operation
USTRUCT() 
struct FPropertyAccessCopy
{
	GENERATED_BODY()

	FPropertyAccessCopy() = default;

private:
	friend struct FPropertyAccessSystem;
	friend struct FPropertyAccessEditorSystem;

	// Index into the library's Accesses
	UPROPERTY()
	int32 AccessIndex = INDEX_NONE;

	// Index of the first of the library's DescAccesses
	UPROPERTY()
	int32 DestAccessStartIndex = INDEX_NONE;

	// Index of the last of the library's DescAccesses
	UPROPERTY()
	int32 DestAccessEndIndex = INDEX_NONE;

	UPROPERTY()
	EPropertyAccessCopyType Type = EPropertyAccessCopyType::Plain;
};

USTRUCT()
struct FPropertyAccessCopyBatch
{
	GENERATED_BODY()

	FPropertyAccessCopyBatch() = default;

private:
	friend struct FPropertyAccessSystem;
	friend struct FPropertyAccessEditorSystem;

	UPROPERTY()
	TArray<FPropertyAccessCopy> Copies;
};

/** A library of property paths used within a specific context (e.g. a class) */
USTRUCT()
struct FPropertyAccessLibrary
{
	GENERATED_BODY()

	FPropertyAccessLibrary() = default;

	ENGINE_API FPropertyAccessLibrary(const FPropertyAccessLibrary& Other);
	ENGINE_API const FPropertyAccessLibrary& operator =(const FPropertyAccessLibrary& Other);

private:
	friend struct FPropertyAccessSystem;
	friend struct FPropertyAccessEditorSystem;

	// All path segments in this library.
	UPROPERTY()
	TArray<FPropertyAccessSegment> PathSegments;

	// All source paths
	UPROPERTY()
	TArray<FPropertyAccessPath> SrcPaths;

	// All destination paths
	UPROPERTY()
	TArray<FPropertyAccessPath> DestPaths;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FPropertyAccessCopyBatch CopyBatches_DEPRECATED[(uint8)EPropertyAccessCopyBatch::Count];
#endif

	// All copy operations
	UPROPERTY()
	TArray<FPropertyAccessCopyBatch> CopyBatchArray;
	
	// All source property accesses
	TArray<FPropertyAccessIndirectionChain> SrcAccesses;

	// All destination accesses (that are copied to our instances).
	TArray<FPropertyAccessIndirectionChain> DestAccesses;

	// Indirections
	TArray<FPropertyAccessIndirection> Indirections;
	
	// Whether this library has been post-loaded
	bool bHasBeenPostLoaded = false;
};
