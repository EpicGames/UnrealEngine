// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Misc/StringBuilder.h"
#include "Templates/OverrideVoidReturnInvoker.h"
#include "UObject/NameTypes.h"

/** A simple name/value pairs map */
class FPropertyPairsMap
{
public:
	/**
	 * Add a property/value pair to the map. This will replace any existing property with the new value.
	 *
	 * @param InName Name of the property.
	 * @param InValue Optional value to be associated with the property.
	 */
	inline void AddProperty(FName InName, FName InValue = NAME_None)
	{
		Properties.Add(InName, InValue);
	}

	/**
	 * Test if the provided property exists in the map.
	 *
	 * @param InName Name of the property.
	 * @returns True if the property exist in the map.
	 */
	inline bool HasProperty(FName InName) const
	{
		return GetProperty(InName);
	}

	inline bool HasProperty(const TCHAR* InName) const
	{
		FName PropertyName(InName, FNAME_Find);
		return !PropertyName.IsNone() && GetProperty(PropertyName);
	}

	/**
	 * Gets the value associated with the provided property.
	 *
	 * @param InName Name of the property.
	 * @out OutValue Value associated with the provided property.
	 * @returns True if the property exist in the map.
	 */
	inline bool GetProperty(FName InName, FName* OutValue = nullptr) const
	{
		if (const FName* ValuePtr = Properties.Find(InName))
		{
			if (OutValue)
			{
				*OutValue = *ValuePtr;
			}
			return true;
		}
		return false;
	}

	inline bool GetProperty(const TCHAR* InName, FName* OutValue = nullptr) const
	{
		FName PropertyName(InName, FNAME_Find);
		return !PropertyName.IsNone() && GetProperty(PropertyName, OutValue);
	}

	/**
	 * Compare this map with another for equality, ignoring ordering.
	 *
	 * @param InOther Other property map to compare against.
	 * @returns True if the property maps contains the same properties and values.
	 */
	bool operator==(const FPropertyPairsMap& InOther) const
	{
		return Properties.OrderIndependentCompareEqual(InOther.Properties);
	}

	bool operator!=(const FPropertyPairsMap& InOther) const
	{
		return !(*this == InOther);
	}

	/**
	 * Serialize this property map to the provided archive.
	 *
	 * @param Ar Archive to serialize to.
	 * @param InPropertyPairsMap Property map to serialize.
	 * @returns Archive used to serialize the property map.
	 */
	inline friend FArchive& operator<<(FArchive& InAr, FPropertyPairsMap& InPropertyPairsMap)
	{
		return InAr << InPropertyPairsMap.Properties;
	}

	/**
	 * Iterate through the property map and invoke a functor that can optionally break iteration by returning false.
	 *
	 * @param InFunc Function to invoke on the properties of the map.
	 */
	template <class Func>
	inline void ForEachProperty(Func InFunc)
	{
		TOverrideVoidReturnInvoker Invoker(true, InFunc);

		for (auto [Name, Value] : Properties)
		{
			if (!Invoker(Name, Value))
			{
				return;
			}
		}
	}

	/** @returns True if the property map is empty. */
	inline bool IsEmpty() const
	{
		return Properties.IsEmpty();
	}

	/** @return The number of properties in the map. */
	inline int32 Num() const
	{
		return Properties.Num();
	}

	/**
	 * Converts this property map to a string representation.
	 *
	 * @return The string representation.
	 */
	inline FString ToString() const
	{
		TStringBuilder<1024> StringBuilder;
		
		StringBuilder.AppendChar(TEXT('('));

		if (Properties.Num())
		{
			for (auto [Name, Value] : Properties)
			{
				StringBuilder.AppendChar(TEXT('('));
				StringBuilder.Append(Name.ToString());

				if (!Value.IsNone())
				{
					StringBuilder.AppendChar(TEXT(','));
					StringBuilder.Append(Value.ToString());
				}

				StringBuilder.AppendChar(TEXT(')'));
				StringBuilder.AppendChar(TEXT(','));
			}
			StringBuilder.RemoveSuffix(1);
		}

		StringBuilder.AppendChar(TEXT(')'));

		return StringBuilder.ToString();
	}

protected:
	TMap<FName, FName> Properties;
};