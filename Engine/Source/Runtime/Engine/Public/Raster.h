// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Raster.h: Generic triangle rasterization code.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

//
//	FTriangleRasterizer - A generic 2d triangle rasterizer which linearly interpolates vertex parameters and calls a virtual function for each pixel.
//

template<class RasterPolicyType> class FTriangleRasterizer : public RasterPolicyType
{
public:

	typedef typename RasterPolicyType::InterpolantType InterpolantType;

	void DrawTriangle(const InterpolantType& I0, const InterpolantType& I1, const InterpolantType& I2, const FVector2D& P0, const FVector2D& P1, const FVector2D& P2, bool BackFacing)
	{
		InterpolantType	Interpolants[3] = { I0, I1, I2 };
		FVector2f		Points[3] = { FVector2f(P0), FVector2f(P1), FVector2f(P2) };

		// Find the top point.

		if (Points[1].Y < Points[0].Y && Points[1].Y <= Points[2].Y)
		{
			Exchange(Points[0], Points[1]);
			Exchange(Interpolants[0], Interpolants[1]);
		}
		else if (Points[2].Y < Points[0].Y && Points[2].Y <= Points[1].Y)
		{
			Exchange(Points[0], Points[2]);
			Exchange(Interpolants[0], Interpolants[2]);
		}

		// Find the bottom point.

		if (Points[1].Y > Points[2].Y)
		{
			Exchange(Points[2], Points[1]);
			Exchange(Interpolants[2], Interpolants[1]);
		}

		// Avoid any division by zero
		float PointDiffY_1_0 = (Points[1].Y - Points[0].Y);
		if (FMath::IsNearlyZero(PointDiffY_1_0))
		{
			PointDiffY_1_0 = PointDiffY_1_0 >= 0.f ? UE_SMALL_NUMBER : -UE_SMALL_NUMBER;
		}
		float PointDiffY_2_0 = (Points[2].Y - Points[0].Y);
		if (FMath::IsNearlyZero(PointDiffY_2_0))
		{
			PointDiffY_2_0 = PointDiffY_2_0 >= 0.f ? UE_SMALL_NUMBER : -UE_SMALL_NUMBER;
		}
		float PointDiffY_2_1 = (Points[2].Y - Points[1].Y);
		if (FMath::IsNearlyZero(PointDiffY_2_1))
		{
			PointDiffY_2_1 = PointDiffY_2_1 >= 0.f ? UE_SMALL_NUMBER : -UE_SMALL_NUMBER;
		}

		// Calculate the edge gradients.

		float			TopMinDiffX = (Points[1].X - Points[0].X) / PointDiffY_1_0,
						TopMaxDiffX = (Points[2].X - Points[0].X) / PointDiffY_2_0;
		InterpolantType	TopMinDiffInterpolant = (Interpolants[1] - Interpolants[0]) / PointDiffY_1_0,
						TopMaxDiffInterpolant = (Interpolants[2] - Interpolants[0]) / PointDiffY_2_0;

		float			BottomMinDiffX = (Points[2].X - Points[1].X) / PointDiffY_2_1,
						BottomMaxDiffX = (Points[2].X - Points[0].X) / PointDiffY_2_0;
		InterpolantType	BottomMinDiffInterpolant = (Interpolants[2] - Interpolants[1]) / PointDiffY_2_1,
						BottomMaxDiffInterpolant = (Interpolants[2] - Interpolants[0]) / PointDiffY_2_0;

		DrawTriangleTrapezoid(
			Interpolants[0],
			TopMinDiffInterpolant,
			Interpolants[0],
			TopMaxDiffInterpolant,
			Points[0].X,
			TopMinDiffX,
			Points[0].X,
			TopMaxDiffX,
			Points[0].Y,
			Points[1].Y,
			BackFacing
			);

		DrawTriangleTrapezoid(
			Interpolants[1],
			BottomMinDiffInterpolant,
			Interpolants[0] + TopMaxDiffInterpolant * PointDiffY_1_0,
			BottomMaxDiffInterpolant,
			Points[1].X,
			BottomMinDiffX,
			Points[0].X + TopMaxDiffX * PointDiffY_1_0,
			BottomMaxDiffX,
			Points[1].Y,
			Points[2].Y,
			BackFacing
			);
	}

	FTriangleRasterizer(const RasterPolicyType& InRasterPolicy): RasterPolicyType(InRasterPolicy) {}

private:

	void DrawTriangleTrapezoid(
		const InterpolantType& TopMinInterpolant,
		const InterpolantType& DeltaMinInterpolant,
		const InterpolantType& TopMaxInterpolant,
		const InterpolantType& DeltaMaxInterpolant,
		float TopMinX,
		float DeltaMinX,
		float TopMaxX,
		float DeltaMaxX,
		float InMinY,
		float InMaxY,
		bool BackFacing
		)
	{
		int32	IntMinY = FMath::Clamp(FMath::CeilToInt(InMinY),RasterPolicyType::GetMinY(),RasterPolicyType::GetMaxY() + 1),
			IntMaxY = FMath::Clamp(FMath::CeilToInt(InMaxY),RasterPolicyType::GetMinY(),RasterPolicyType::GetMaxY() + 1);

		for(int32 IntY = IntMinY;IntY < IntMaxY;IntY++)
		{
			float			Y = IntY - InMinY;
			float			LocalMinX = TopMinX + DeltaMinX * Y;
			float			LocalMaxX = TopMaxX + DeltaMaxX * Y;
			InterpolantType	MinInterpolant = TopMinInterpolant + DeltaMinInterpolant * Y,
							MaxInterpolant = TopMaxInterpolant + DeltaMaxInterpolant * Y;

			if(LocalMinX > LocalMaxX)
			{
				Exchange(LocalMinX,LocalMaxX);
				Exchange(MinInterpolant,MaxInterpolant);
			}

			if(LocalMaxX > LocalMinX)
			{
				int32				IntMinX = FMath::Clamp(FMath::CeilToInt(LocalMinX),RasterPolicyType::GetMinX(),RasterPolicyType::GetMaxX() + 1),
								IntMaxX = FMath::Clamp(FMath::CeilToInt(LocalMaxX),RasterPolicyType::GetMinX(),RasterPolicyType::GetMaxX() + 1);
				InterpolantType	DeltaInterpolant = (MaxInterpolant - MinInterpolant) / (LocalMaxX - LocalMinX);

				for(int32 X = IntMinX;X < IntMaxX;X++)
				{
					RasterPolicyType::ProcessPixel(X,IntY,MinInterpolant + DeltaInterpolant * (X - LocalMinX),BackFacing);
				}
			}
		}
	}
};
