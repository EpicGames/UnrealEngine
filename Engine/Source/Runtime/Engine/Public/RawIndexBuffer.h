// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RawIndexBuffer.h: Raw index buffer definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RenderResource.h"
#include "Containers/DynamicRHIResourceArray.h"


class FRawIndexBuffer : public FIndexBuffer
{
public:

	TArray<uint16> Indices;

	/**
	 * Orders a triangle list for better vertex cache coherency.
	 */
	void CacheOptimize();

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	// Serialization.
	friend FArchive& operator<<(FArchive& Ar,FRawIndexBuffer& I);
};


class FRawIndexBuffer16or32 : public FIndexBuffer
{
public:
	FRawIndexBuffer16or32()
		: b32Bit(true)
	{
	}

	TArray<uint32> Indices;

	/**
	 * Orders a triangle list for better vertex cache coherency.
	 */
	void CacheOptimize();

	/**
	 * Computes whether index buffer should be 32 bit
	 */
	void ComputeIndexWidth();

	/**
	 * Forces (or not) usage of 32 bits indices. No validation is made as to whether Indices can all be stored in 16 bits indices (if bIn32Bit == false) or not : 
	 *  use only if you know the max value in Indices, otherwise, use ComputeIndexWidth
	 */
	void ForceUse32Bit(bool bIn32Bit) { b32Bit = bIn32Bit; }

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	// Serialization.
	friend FArchive& operator<<(FArchive& Ar,FRawIndexBuffer16or32& I);

private:
	bool b32Bit;
};

/**
 * Desired stride when creating a static index buffer.
 */
namespace EIndexBufferStride
{
	enum Type
	{
		/** Forces all indices to be 16-bit. */
		Force16Bit = 1,
		/** Forces all indices to be 32-bit. */
		Force32Bit = 2,
		/** Use 16 bits unless an index exceeds MAX_uint16. */
		AutoDetect = 3
	};
}

/**
 * An array view in to a static index buffer. Allows access to the underlying
 * indices regardless of their type without a copy.
 */
class FIndexArrayView
{
public:
	/** Default constructor. */
	FIndexArrayView()
		: UntypedIndexData(nullptr)
		, NumIndices(0)
		, b32Bit(false)
	{
	}

	/**
	 * Initialization constructor.
	 * @param InIndexData	A pointer to untyped index data.
	 * @param InNumIndices	The number of indices stored in the untyped index data.
	 * @param bIn32Bit		True if the data is stored as an array of uint32, false for uint16.
	 */
	FIndexArrayView(const void* InIndexData, int32 InNumIndices, bool bIn32Bit)
		: UntypedIndexData(InIndexData)
		, NumIndices(InNumIndices)
		, b32Bit(bIn32Bit)
	{
	}

	/** Common array access semantics. */
	uint32 operator[](int32 i) { return (uint32)(b32Bit ? ((const uint32*)UntypedIndexData)[i] : ((const uint16*)UntypedIndexData)[i]); }
	uint32 operator[](int32 i) const { return (uint32)(b32Bit ? ((const uint32*)UntypedIndexData)[i] : ((const uint16*)UntypedIndexData)[i]); }
	inline int32 Num() const { return NumIndices; }

private:
	/** Pointer to the untyped index data. */
	const void* UntypedIndexData;
	/** The number of indices stored in the untyped index data. */
	int32 NumIndices;
	/** True if the data is stored as an array of uint32, false for uint16. */
	bool b32Bit;
};

class FRawStaticIndexBuffer : public FIndexBuffer
{
public:	
	/**
	 * Initialization constructor.
	 * @param InNeedsCPUAccess	True if resource array data should be accessible by the CPU.
	 */
	ENGINE_API FRawStaticIndexBuffer(bool InNeedsCPUAccess=false);
	ENGINE_API virtual ~FRawStaticIndexBuffer();

	/**
	 * Copy everything, keeping reference to the same RHI resources.
	 * Returns true if input value for bAllowCPUAccess will be honored:
	 *		- if bAllowCPUAccess is true then no indices have already been cached or IndexStorage is not empty
	 *		- or if bAllowCPUAccess is false
	 */
	bool TrySetAllowCPUAccess(bool bAllowCPUAccess)
	{
		IndexStorage.SetAllowCPUAccess(bAllowCPUAccess);

		return !bAllowCPUAccess || CachedNumIndices == 0 || IndexStorage.Num() > 0;
	}

	/**
	 * Sets a single index value.  Consider using SetIndices() instead if you're setting a lot of indices.
	 * @param	At	The index of the index to set
	 * @param	NewIndexValue	The index value
	 */
	inline void SetIndex( const uint32 At, const uint32 NewIndexValue )
	{
		check( At >= 0 && At < (uint32)IndexStorage.Num() );

		if( b32Bit )
		{
			uint32* Indices32Bit = (uint32*)IndexStorage.GetData();
			Indices32Bit[ At ] = NewIndexValue;
		}
		else
		{
			uint16* Indices16Bit = (uint16*)IndexStorage.GetData();
			Indices16Bit[ At ] = (uint16)NewIndexValue;
		}
	}

	/**
	 * Set the indices stored within this buffer.
	 * @param InIndices		The new indices to copy in to the buffer.
	 * @param DesiredStride	The desired stride (16 or 32 bits).
	 */
	ENGINE_API void SetIndices(const TArray<uint32>& InIndices, EIndexBufferStride::Type DesiredStride);

	/**
	 * Insert indices at the given position in the buffer
	 * @param	At					Index to insert at
	 * @param	IndicesToAppend		Pointer to the array of indices to insert
	 * @param	NumIndicesToAppend	How many indices are in the IndicesToAppend array
	 */
	ENGINE_API void InsertIndices( const uint32 At, const uint32* IndicesToAppend, const uint32 NumIndicesToAppend );

	/**
	 * Append indices to the end of the buffer
	 * @param	IndicesToAppend		Pointer to the array of indices to add to the end
	 * @param	NumIndicesToAppend	How many indices are in the IndicesToAppend array
	 */
	ENGINE_API void AppendIndices( const uint32* IndicesToAppend, const uint32 NumIndicesToAppend );

	/** @return Gets a specific index value */
	inline uint32 GetIndex( const uint32 At ) const
	{
		check( At >= 0 && At < (uint32)IndexStorage.Num() );
		uint32 IndexValue;
		if( b32Bit )
		{
			const uint32* SrcIndices32Bit = (const uint32*)IndexStorage.GetData();
			IndexValue = SrcIndices32Bit[ At ];
		}
		else
		{
			const uint16* SrcIndices16Bit = (const uint16*)IndexStorage.GetData();
			IndexValue = SrcIndices16Bit[ At ];
		}

		return IndexValue;
	}


	/**
	 * Removes indices from the buffer
	 *
	 * @param	At	The index of the first index to remove
	 * @param	NumIndicesToRemove	How many indices to remove
	 */
	ENGINE_API void RemoveIndicesAt( const uint32 At, const uint32 NumIndicesToRemove );

	/**
	 * Retrieve a copy of the indices in this buffer. Only valid if created with
	 * NeedsCPUAccess set to true or the resource has not yet been initialized.
	 * @param OutIndices	Array in which to store the copy of the indices.
	 */
	ENGINE_API void GetCopy(TArray<uint32>& OutIndices) const;

	/** Expands the 16bit index buffer to 32bit */
	void ExpandTo32Bit();

	/**
	 * Get the direct read access to index data 
	 * Only valid if NeedsCPUAccess = true and indices are 16 bit
	 */
	ENGINE_API const uint16* AccessStream16() const;

	/**
	 * Get the direct read access to index data
	 * Only valid if NeedsCPUAccess = true and indices are 32 bit
	 */
	ENGINE_API const uint32* AccessStream32() const;

	/**
	 * Retrieves an array view in to the index buffer. The array view allows code
	 * to retrieve indices as 32-bit regardless of how they are stored internally
	 * without a copy. The array view is valid only if:
	 *		The buffer was created with NeedsCPUAccess = true
	 *		  OR the resource has not yet been initialized
	 *		  AND SetIndices has not been called since.
	 */
	ENGINE_API FIndexArrayView GetArrayView() const;

	/**
	 * Computes the number of indices stored in this buffer.
	 */
	inline int32 GetNumIndices() const
	{
		return CachedNumIndices >= 0 ? CachedNumIndices : (b32Bit ? (IndexStorage.Num()/4) : (IndexStorage.Num()/2));
	}

	/**
	 * Computes the amount of memory allocated to store the indices.
	 */
	inline SIZE_T GetAllocatedSize() const
	{
		return IndexStorage.GetAllocatedSize();
	}

	inline bool GetAllowCPUAccess() const
	{
		return IndexStorage.GetAllowCPUAccess();
	}

	/** == GetNumIndices() * (b32Bit ? 4 : 2) */
	int32 GetIndexDataSize() const { return IndexStorage.Num(); }

	/** Create an RHI index buffer with CPU data. CPU data may be discarded after creation (see TResourceArray::Discard) */
	FBufferRHIRef CreateRHIBuffer(FRHICommandListBase& RHICmdList);

	/** Take over ownership of IntermediateBuffer */
	void InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher);

	/** Release any GPU resource owned by the RHI object */
	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

	/**
	 * Serialization.
	 * @param	Ar				Archive to serialize with
	 * @param	bNeedsCPUAccess	Whether the elements need to be accessed by the CPU
	 */
	ENGINE_API void Serialize(FArchive& Ar, bool bNeedsCPUAccess);

	/** Serialize only meta data (e.g. number of indices) but not the actual index data */
	void SerializeMetaData(FArchive& Ar);

	void ClearMetaData();

    /**
     * Discard
     * discards the serialized data when it is not needed
     */
    void Discard();
    
	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	inline bool Is32Bit() const { return b32Bit; }

private:
	/** Storage for indices. */
	TResourceArray<uint8, INDEXBUFFER_ALIGNMENT> IndexStorage;

	/** If >= 0, represents the number of indices in this IB. Needed in cooked build since InitRHI may discard CPU data */
	int32 CachedNumIndices;

	/** 32bit or 16bit? */
	bool b32Bit;
};

/**
 * Virtual interface for the FRawStaticIndexBuffer16or32 class
 */
class FRawStaticIndexBuffer16or32Interface : public FIndexBuffer
{
public:
	virtual void Serialize( FArchive& Ar ) = 0;

	virtual void SerializeMetaData(FArchive& Ar) = 0;

	virtual void SetMetaData(int32 InCachedNumIndices) = 0;

	/**
	 * The following methods are basically just accessors that allow us
	 * to hide the implementation of FRawStaticIndexBuffer16or32 by making
	 * the index array a private member
	 */
	virtual bool GetNeedsCPUAccess() const = 0;
	// number of indices (e.g. 4 triangles would result in 12 elements)
	virtual int32 Num() const = 0;
	virtual int32 AddItem(uint32 Val) = 0;
	virtual uint32 Get(uint32 Idx) const = 0;
	virtual void* GetPointerTo(uint32 Idx) = 0;
	virtual void Insert(int32 Idx, int32 Num = 1) = 0;
	virtual void Remove(int32 Idx, int32 Num = 1) = 0;
	virtual void Empty(int32 Slack = 0) = 0;
	virtual int32 GetResourceDataSize() const = 0;

	// @param guaranteed only to be valid if the vertex buffer is valid and the buffer was created with the SRV flags
	FRHIShaderResourceView* GetSRV() const
	{
		return SRVValue;
	}

	// Allows for creating SRV on demand
	// Note: consider cache the SRV locally if multiple compute tasks all need access to one in a single frame
	virtual FShaderResourceViewRHIRef GetOrCreateSRV(FRHICommandListBase& RHICmdList) const = 0;

protected:
	ENGINE_API bool IsSRVCreatedByDefault(bool bAllowCPUAccess) const;
	ENGINE_API bool IsShaderResource(bool bAllowCPUAccess) const;

	UE_DEPRECATED(5.7, "Use IsSRVCreatedByDefault or IsShaderResource instead, which gives more granular control over platform specific behavior")	
	ENGINE_API bool IsSRVNeeded(bool bAllowCPUAccess) const;
	
	/** Similar to Init/ReleaseRHI but only update existing SRV so references to the SRV stays valid */
	void InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, size_t IndexSize, FRHIResourceReplaceBatcher& Batcher);
	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

	static ENGINE_API FBufferRHIRef CreateRHIIndexBufferInternal(
		FRHICommandListBase& RHICmdList,
		const TCHAR* InDebugName,
		const FName& InOwnerName,
		int32 IndexCount,
		size_t IndexSize,
		FResourceArrayInterface* ResourceArray,
		bool bIsShaderResource
	);

	// guaranteed only to be valid if the vertex buffer is valid and the buffer was created with the SRV flags
	FShaderResourceViewRHIRef SRVValue;
};

template <typename INDEX_TYPE>
class FRawStaticIndexBuffer16or32 : public FRawStaticIndexBuffer16or32Interface
{
public:	
	/**
	* Constructor
	* @param InNeedsCPUAccess - true if resource array data should be CPU accessible
	*/
	FRawStaticIndexBuffer16or32(bool InNeedsCPUAccess=false)
		: Indices(InNeedsCPUAccess)
		, CachedNumIndices(0)
	{
		static_assert(sizeof(INDEX_TYPE) == 2 || sizeof(INDEX_TYPE) == 4, "FRawStaticIndexBuffer16or32 must have a stride of 2 or 4 bytes.");
	}

	/**
	* Create the index buffer RHI resource and initialize its data
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		IndexBufferRHI = CreateRHIBuffer(RHICmdList);

		if (IsSRVCreatedByDefault(Indices.GetAllowCPUAccess()))
		{
			SRVValue = GetOrCreateSRV(RHICmdList);
		}
	}
	
	virtual void ReleaseRHI() override
	{
		FRawStaticIndexBuffer16or32Interface::ReleaseRHI();

		SRVValue.SafeRelease();
	}

	virtual FShaderResourceViewRHIRef GetOrCreateSRV(FRHICommandListBase& RHICmdList) const override
	{
		if (SRVValue.IsValid())
		{
			return SRVValue;
		}
		
		if (!IndexBufferRHI)
		{
			return {};
		}
		
		if (!IsShaderResource(Indices.GetAllowCPUAccess()))
		{
			return {};
		}
		
		FShaderResourceViewRHIRef ExternalSRV = RHICmdList.CreateShaderResourceView(
				IndexBufferRHI,
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(sizeof(INDEX_TYPE) == 2 ? PF_R16_UINT : PF_R32_UINT));
		
		return ExternalSRV;
	}
	
	/**
	* Serializer for this class
	* @param Ar - archive to serialize to
	* @param I - data to serialize
	*/
	virtual void Serialize( FArchive& Ar ) override
	{
		Indices.BulkSerialize(Ar);
		CachedNumIndices = Indices.Num();
	}

	virtual void SerializeMetaData(FArchive& Ar) override
	{
		Ar << CachedNumIndices;
	}

	virtual void SetMetaData(int32 InCachedNumIndices) override
	{
		CachedNumIndices = InCachedNumIndices;
	}

	/**
	* Orders a triangle list for better vertex cache coherency.
	*/
	void CacheOptimize();

	
	/**
	 * The following methods are basically just accessors that allow us
	 * to hide the implementation by making the index array a private member
	 */
	virtual bool GetNeedsCPUAccess() const override { return Indices.GetAllowCPUAccess(); }

	virtual int32 Num() const override
	{
		return CachedNumIndices;
	}

	virtual int32 AddItem(uint32 Val) override
	{
		++CachedNumIndices;
		return Indices.Add((INDEX_TYPE)Val);
	}

	virtual uint32 Get(uint32 Idx) const override
	{
		return (uint32)Indices[Idx];
	}

	virtual void* GetPointerTo(uint32 Idx) override
	{
		return (void*)(&Indices[Idx]);
	}

	virtual void Insert(int32 Idx, int32 Num) override
	{
		CachedNumIndices += Num;
		Indices.InsertUninitialized(Idx, Num);
		check(CachedNumIndices == Indices.Num());
	}

	virtual void Remove(int32 Idx, int32 Num) override
	{
		CachedNumIndices -= Num;
		Indices.RemoveAt(Idx, Num);
		check(CachedNumIndices == Indices.Num());
	}

	virtual void Empty(int32 Slack) override
	{
		Indices.Empty(Slack);
		CachedNumIndices = 0;
	}

	virtual int32 GetResourceDataSize() const override
	{
		return Indices.GetResourceDataSize();
	}

	virtual void AssignNewBuffer(const TArray<INDEX_TYPE>& Buffer)
	{
		using IndexBufferType = typename TResourceArray<INDEX_TYPE, INDEXBUFFER_ALIGNMENT>::Super;
		Indices = IndexBufferType(Buffer);
		CachedNumIndices = Indices.Num();
	}

	/** Create an RHI index buffer with CPU data. CPU data may be discarded after creation (see TResourceArray::Discard) */
	FBufferRHIRef CreateRHIBuffer(FRHICommandListBase& RHICmdList)
	{
		if (CachedNumIndices)
		{
			// Need to cache number of indices from the source array *before* RHICreateBuffer is called
			// because it will empty the source array.
			CachedNumIndices = Indices.Num();

			return CreateRHIIndexBufferInternal(
				RHICmdList,
				sizeof(INDEX_TYPE) == 4 ? TEXT("FRawStaticIndexBuffer32") : TEXT("FRawStaticIndexBuffer16"),
				GetOwnerName(),
				Indices.Num(),
				sizeof(INDEX_TYPE),
				&Indices,
				IsShaderResource(Indices.GetAllowCPUAccess())
			);
		}
		return nullptr;
	}

	/** Similar to Init/ReleaseRHI but only update existing SRV so references to the SRV stays valid */
	void InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher)
	{
		FRawStaticIndexBuffer16or32Interface::InitRHIForStreaming(IntermediateBuffer, sizeof(INDEX_TYPE), Batcher);
	}

	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
	{
		FRawStaticIndexBuffer16or32Interface::ReleaseRHIForStreaming(Batcher);
	}

private:
	TResourceArray<INDEX_TYPE,INDEXBUFFER_ALIGNMENT> Indices;

	int32 CachedNumIndices;

};
