// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRayTracingVisualizationData
{
public:
	enum class FModeType : uint8
	{
		Overview,
		Standard,
		Performance,
		Timing,
		Other,
	};

	/** Describes a single available visualization mode. */
	struct FModeRecord
	{
		FString   ModeString;
		FName     ModeName;
		FText     ModeText;
		FText     ModeDesc;
		FModeType ModeType;
		int32     ModeID;

		bool      bTonemapped;
		bool      bHiddenInEditor;
	};

	/** Mapping of FName to a visualization mode record. */
	typedef TMultiMap<FName, FModeRecord> TModeMap;

public:
	FRayTracingVisualizationData();

	ENGINE_API bool HasOverrides() const;
	ENGINE_API FName ApplyOverrides(const FName& InModeName) const;

	/** Get the display name of a named mode from the available mode map. **/
	ENGINE_API FText GetModeDisplayName(const FName& InModeName) const;

	ENGINE_API int32 GetModeID(const FName& InModeName) const;

	ENGINE_API bool GetModeTonemapped(const FName& InModeName) const;

	inline const TModeMap& GetModeMap() const
	{
		return ModeMap;
	}

private:

	void AddVisualizationMode(
		const TCHAR* ModeString,
		const FText& ModeText,
		const FModeType ModeType,
		int32 ModeID,
		bool bTonemapped,
		bool bHiddenInEditor = false
	);

	/** Internal helper function for creating the visualization system console commands. */
	void ConfigureConsoleCommand();

private:
	/** The name->mode mapping table */
	TModeMap ModeMap;
	
	/** Storage for console variable documentation strings. **/
	FString ConsoleDocumentationVisualizationMode;
};

ENGINE_API FRayTracingVisualizationData& GetRayTracingVisualizationData();
