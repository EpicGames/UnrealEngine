// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "RendererInterface.h"

struct FRaytracingVFUVData
{
	FRHIShaderResourceView* BufferSRV = nullptr;

	uint32 NumCoordinates = 0;
};

struct FRaytracingVFSectionData
{
	FRaytracingVFUVData UV;
};

struct FRaytracingVFAttributeData
{
	TArray<FRaytracingVFSectionData, TInlineAllocator<4>> Sections;
};
