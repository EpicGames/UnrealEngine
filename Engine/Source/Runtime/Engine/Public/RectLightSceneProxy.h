// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RectLightSceneProxy.h: FRectLightSceneProxy definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Components/RectLightComponent.h"
#include "LocalLightSceneProxy.h"
#include "SceneManagement.h"

struct FRectLightSceneProxyDesc;

class FRectLightSceneProxy : public FLocalLightSceneProxy
{
public:
	float		SourceWidth;
	float		SourceHeight;
	float		BarnDoorAngle;
	float		BarnDoorLength;
	UTexture*	SourceTexture;
	uint32		RectAtlasId;
	float		LightFunctionConeAngleTangent;	// Use Ortho projection if 0
	FVector4f	SourceTextureScaleOffset;

	FRectLightSceneProxy(const URectLightComponent* Component);
	ENGINE_API FRectLightSceneProxy(const FRectLightSceneProxyDesc& Desc);
	virtual ~FRectLightSceneProxy();

	//~Begin FLightSceneProxy Interface
	virtual bool IsRectLight() const override;
	virtual bool HasSourceTexture() const override;
	virtual void GetLightShaderParameters(FLightRenderParameters& OutLightParameters, uint32 Flags=0) const override;
	virtual bool GetWholeSceneProjectedShadowInitializer(const FSceneViewFamily& ViewFamily, TArray<FWholeSceneProjectedShadowInitializer, TInlineAllocator<6> >& OutInitializers) const override;
	//~End FLightSceneProxy Interface
};

