// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LocalLightSceneProxyDesc.h"

struct FRectLightSceneProxyDesc : public FLocalLightSceneProxyDesc
{
	FRectLightSceneProxyDesc() = default;
	virtual ~FRectLightSceneProxyDesc() = default;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** Width of light source rect. */
	float SourceWidth = 0.0f;

	/** Height of light source rect. */
	float SourceHeight = 0.0f;

	/** Angle of barn door attached to the light source rect. */
	float BarnDoorAngle = 0.0f;

	/** Length of barn door attached to the light source rect. */
	float BarnDoorLength = 0.0f;

	/** Aperture of cone angle for the perspective projection of the light function material. If 0, an orthographic projection is used instead. */
	float LightFunctionConeAngle = 0.0f;

	/** Texture mapped to the light source rectangle */
	TObjectPtr<class UTexture> SourceTexture = nullptr;

	/** Scales the source texture. Value in 0..1. */
	FVector2f SourceTextureScale = FVector2f(1.0f, 1.0f);

	/** Offsets the source texture. Value in 0..1. */
	FVector2f SourceTextureOffset = FVector2f::ZeroVector;
};