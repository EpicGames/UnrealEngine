// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 *	Creates a list of object clusters based on the provided references.
 *
 *	@param	InObjects Array of object GUIDs with their references. Each pair represents the GUID of an
 *			object and its list of references. References are expected to be part of the root object
 *			list, forming a closed set of n clusters.
 *
 *	@return	Array of reference clusters (e.g. group of objects referencing each other). For example, an
 *			input of n objects without any references will generate n clusters of a single object each.
 */
ENGINE_API TArray<TArray<FGuid>> GenerateObjectsClusters(const TArray<TPair<FGuid, TArray<FGuid>>>& InObjects);