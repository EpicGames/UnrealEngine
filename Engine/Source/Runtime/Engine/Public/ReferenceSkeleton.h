// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoneIndices.h"

class USkeleton;
struct FReferenceSkeleton;

// This contains Reference-skeleton related info
// Bone transform is saved as FTransform array
struct FMeshBoneInfo
{
	// Bone's name.
	FName Name;

	// INDEX_NONE if this is the root bone. 
	int32 ParentIndex;

#if WITH_EDITORONLY_DATA
	// Name used for export (this should be exact as FName may mess with case) 
	FString ExportName;
#endif

	FMeshBoneInfo() : Name(NAME_None), ParentIndex(INDEX_NONE) {}

	FMeshBoneInfo(const FName& InName, const FString& InExportName, int32 InParentIndex)
		: Name(InName)
		, ParentIndex(InParentIndex)
#if WITH_EDITORONLY_DATA
		, ExportName(InExportName)
#endif
	{}

	bool operator==(const FMeshBoneInfo& B) const
	{
		return(Name == B.Name);
	}

	friend FArchive &operator<<(FArchive& Ar, FMeshBoneInfo& F);
};

// Cached Virtual Bone data from USkeleton
struct FVirtualBoneRefData
{
	int32 VBRefSkelIndex;
	int32 SourceRefSkelIndex;
	int32 TargetRefSkelIndex;

	FVirtualBoneRefData(int32 InVBRefSkelIndex, int32 InSourceRefSkelIndex, int32 InTargetRefSkelIndex)
		: VBRefSkelIndex(InVBRefSkelIndex)
		, SourceRefSkelIndex(InSourceRefSkelIndex)
		, TargetRefSkelIndex(InTargetRefSkelIndex)
	{
	}
};

class USkeleton;

struct FReferenceSkeleton;

// Allow modifications to a reference skeleton while guaranteeing that virtual bones remain valid.
struct FReferenceSkeletonModifier
{
private:
	FReferenceSkeleton& RefSkeleton;
	const USkeleton*	Skeleton;
public:
	FReferenceSkeletonModifier(FReferenceSkeleton& InRefSkel, const USkeleton* InSkeleton) : RefSkeleton(InRefSkel), Skeleton(InSkeleton) {}
	ENGINE_API FReferenceSkeletonModifier(USkeleton* InSkeleton);
	ENGINE_API ~FReferenceSkeletonModifier();

	// Update the reference pose transform of the specified bone
	ENGINE_API void UpdateRefPoseTransform(const int32 BoneIndex, const FTransform& BonePose);

	// Add a new bone. BoneName must not already exist! ParentIndex must be valid.
	ENGINE_API void Add(const FMeshBoneInfo& BoneInfo, const FTransform& BonePose, const bool bAllowMultipleRoots = false);

	// Remove a bone. BoneName must be valid.
	ENGINE_API void Remove(const FName& BoneName, const bool bRemoveChildren);

	// Rename a bone. InOldName must be valid and InNewName not already a bone name.
	ENGINE_API void Rename(const FName& InOldName, const FName& InNewName);

	// Change bone's parent. InBoneName must be valid and InParentName can be Name_NONE to unparent.
	ENGINE_API int32 SetParent(const FName& InBoneName, const FName& InParentName, const bool bAllowMultipleRoots = false);
	
	/** Find Bone Index from BoneName. Precache as much as possible in speed critical sections! */
	ENGINE_API int32 FindBoneIndex(const FName& BoneName) const;

	/** Accessor to private data. Const so it can't be changed recklessly. */
	ENGINE_API const TArray<FMeshBoneInfo> & GetRefBoneInfo() const;

	const FReferenceSkeleton& GetReferenceSkeleton() const { return RefSkeleton; }
};

/** Reference Skeleton **/
struct FReferenceSkeleton
{
	FReferenceSkeleton(bool bInOnlyOneRootAllowed = true)
		:bOnlyOneRootAllowed(bInOnlyOneRootAllowed)
	{}

private:
	//RAW BONES: Bones that exist in the original asset
	/** Reference bone related info to be serialized **/
	TArray<FMeshBoneInfo>	RawRefBoneInfo;
	/** Reference bone transform **/
	TArray<FTransform>		RawRefBonePose;

	//FINAL BONES: Bones for this skeleton including user added virtual bones
	/** Reference bone related info to be serialized **/
	TArray<FMeshBoneInfo>	FinalRefBoneInfo;
	/** Reference bone transform **/
	TArray<FTransform>		FinalRefBonePose;

	/** TMap to look up bone index from bone name. */
	TMap<FName, int32>		RawNameToIndexMap;
	TMap<FName, int32>		FinalNameToIndexMap;

	// cached data to allow virtual bones to be built into poses
	TArray<FBoneIndexType>  RequiredVirtualBones;
	TArray<FVirtualBoneRefData> UsedVirtualBoneData;

	/** Whether this skeleton is limited to one root or not 
	 *	Multi root is not supported in general Skeleton/SkeletalMesh
	 *	But there are other components that can use this and support multi root - i.e. ControlRig
	 *	This sturct is used in draw code, the long term plan may be detach this from draw code
	 *	and use interface struct
	 */
	bool bOnlyOneRootAllowed;

	/** One index per-bone. Lazy-filled on request. Stores the last element of the branch below the bone.
	 * You can iterate between in the indices stored here and the bone in question to iterate over all children recursively */
	mutable TArray<int32> CachedEndOfBranchIndicesRaw;
	static constexpr int32 BRANCH_CACHE_INVALID_INDEX = -2;
	void InvalidateEndOfBranchCache() const
	{
		CachedEndOfBranchIndicesRaw.Init(BRANCH_CACHE_INVALID_INDEX, RawRefBonePose.Num());
	}

	/** Removes the specified bone, so long as it has no children. Returns whether we removed the bone or not */
	bool RemoveIndividualBone(int32 BoneIndex, TArray<int32>& OutBonesRemoved)
	{
		bool bRemoveThisBone = true;

		// Make sure we have no children
		for(int32 CurrBoneIndex=BoneIndex+1; CurrBoneIndex < GetRawBoneNum(); CurrBoneIndex++)
		{
			if( RawRefBoneInfo[CurrBoneIndex].ParentIndex == BoneIndex )
			{
				bRemoveThisBone = false;
				break;
			}
		}

		if(bRemoveThisBone)
		{
			// Update parent indices of bones further through the array
			for(int32 CurrBoneIndex=BoneIndex+1; CurrBoneIndex < GetRawBoneNum(); CurrBoneIndex++)
			{
				FMeshBoneInfo& Bone = RawRefBoneInfo[CurrBoneIndex];
				if( Bone.ParentIndex > BoneIndex )
				{
					Bone.ParentIndex -= 1;
				}
			}

			OutBonesRemoved.Add(BoneIndex);
			RawRefBonePose.RemoveAt(BoneIndex, 1);
			RawRefBoneInfo.RemoveAt(BoneIndex, 1);
			
			InvalidateEndOfBranchCache();
		}
		return bRemoveThisBone;
	}

	int32 GetParentIndexInternal(const int32 BoneIndex, const TArray<FMeshBoneInfo>& BoneInfo) const
	{
		if (!BoneInfo.IsValidIndex(BoneIndex))
		{
			return INDEX_NONE;
		}
		const int32 ParentIndex = BoneInfo[BoneIndex].ParentIndex;

		// Parent must be valid. Either INDEX_NONE for Root, or before children for non root bones.
		checkSlow(!bOnlyOneRootAllowed || 
			(((BoneIndex == 0) && (ParentIndex == INDEX_NONE))
			|| ((BoneIndex > 0) && BoneInfo.IsValidIndex(ParentIndex) && (ParentIndex < BoneIndex))));

		return ParentIndex;
	}

	void UpdateRefPoseTransform(const int32 BoneIndex, const FTransform& BonePose)
	{
		RawRefBonePose[BoneIndex] = BonePose;
	}

	/** Add a new bone.
	* BoneName must not already exist! ParentIndex must be valid. */
	void Add(const FMeshBoneInfo& BoneInfo, const FTransform& BonePose)
	{
		// Adding a bone that already exists is illegal
		check(FindRawBoneIndex(BoneInfo.Name) == INDEX_NONE);

		// Make sure our arrays are in sync.
		checkSlow((RawRefBoneInfo.Num() == RawRefBonePose.Num()) && (RawRefBoneInfo.Num() == RawNameToIndexMap.Num()));

		const int32 BoneIndex = RawRefBoneInfo.Add(BoneInfo);
		RawRefBonePose.Add(BonePose);
		RawNameToIndexMap.Add(BoneInfo.Name, BoneIndex);

		// Normalize Quaternion to be safe.
		RawRefBonePose[BoneIndex].NormalizeRotation();

		// Parent must be valid. Either INDEX_NONE for Root, or before children for non root bones.
		check(!bOnlyOneRootAllowed || 
			(((BoneIndex == 0) && (BoneInfo.ParentIndex == INDEX_NONE))
			|| ((BoneIndex > 0) && RawRefBoneInfo.IsValidIndex(BoneInfo.ParentIndex) && (BoneInfo.ParentIndex < BoneIndex))));
	}

	/** Remove InBoneName and its children if bRemoveChildren == true. */
	void Remove(const FName InBoneName, const bool bRemoveChildren);

	/** Rename InBoneName with InNewName. */
	void Rename(const FName InBoneName, const FName InNewName);

	/** Set InParentName as InBoneName's parent. */
	int32 SetParent(const FName InBoneName, const FName InParentName);

	// Help us translate a virtual bone source into a raw bone source (for evaluating virtual bone transform)
	int32 GetRawSourceBoneIndex(const USkeleton* Skeleton, const FName& SourceBoneName) const;

	// very slow search function for all children (raw or final)
	int32 GetChildrenInternal(int32 InParentBoneIndex, TArray<int32>& OutChildren, const bool bRaw) const;

	// returns the index of the bone at the end of the branch belonging to the given one (uses cached result if there is one)
	int32 GetCachedEndOfBranchIndex(const int32 InBoneIndex) const;
	
public:
	ENGINE_API void RebuildRefSkeleton(const USkeleton* Skeleton, bool bRebuildNameMap);

	/** Returns number of bones in Skeleton. */
	int32 GetNum() const
	{
		return FinalRefBoneInfo.Num();
	}

	/** Returns number of raw bones in Skeleton. These are the original bones of the asset */
	int32 GetRawBoneNum() const
	{
		return RawRefBoneInfo.Num();
	}

	const TArray<FBoneIndexType>& GetRequiredVirtualBones() const { return RequiredVirtualBones; }

	const TArray<FVirtualBoneRefData>& GetVirtualBoneRefData() const { return UsedVirtualBoneData; }

	/** Accessor to private data. These include the USkeletons virtual bones. Const so it can't be changed recklessly. */
	const TArray<FMeshBoneInfo> & GetRefBoneInfo() const
	{
		return FinalRefBoneInfo;
	}

	/** Accessor to private data. These include the USkeletons virtual bones. Const so it can't be changed recklessly. */
	const TArray<FTransform> & GetRefBonePose() const
	{
		return FinalRefBonePose;
	}

	/** Accessor to private data. Raw relates to original asset. Const so it can't be changed recklessly. */
	const TArray<FMeshBoneInfo> & GetRawRefBoneInfo() const
	{
		return RawRefBoneInfo;
	}

	/** Accessor to private data. Raw relates to original asset. Const so it can't be changed recklessly. */
	const TArray<FTransform> & GetRawRefBonePose() const
	{
		return RawRefBonePose;
	}

	const TMap<FName, int32>& GetRawNameToIndexMap() const { return RawNameToIndexMap; }
	
	/** Returns an array of raw bone names. */
	TArray<FName> GetRawRefBoneNames() const
	{
		TArray<FName> BoneNames;
		BoneNames.Reserve(RawRefBoneInfo.Num());

		for (const FMeshBoneInfo& BoneInfo : RawRefBoneInfo)
		{
			BoneNames.Add(BoneInfo.Name);
		}

		return BoneNames;
	}
	
	void Empty(int32 Size=0)
	{
		RawRefBoneInfo.Empty(Size);
		RawRefBonePose.Empty(Size);

		FinalRefBoneInfo.Empty(Size);
		FinalRefBonePose.Empty(Size);

		RawNameToIndexMap.Empty(Size);
		FinalNameToIndexMap.Empty(Size);

		CachedEndOfBranchIndicesRaw.Empty(Size);
	}

	/** Find Bone Index from BoneName. Precache as much as possible in speed critical sections! */
	int32 FindBoneIndex(const FName& BoneName) const
	{
		checkSlow(FinalRefBoneInfo.Num() == FinalNameToIndexMap.Num());
		int32 BoneIndex = INDEX_NONE;
		if( BoneName != NAME_None )
		{
			const int32* IndexPtr = FinalNameToIndexMap.Find(BoneName);
			if( IndexPtr )
			{
				BoneIndex = *IndexPtr;
			}
		}
		return BoneIndex;
	}

	/** Find Bone Index from BoneName. Precache as much as possible in speed critical sections! */
	int32 FindRawBoneIndex(const FName& BoneName) const
	{
		checkSlow(RawRefBoneInfo.Num() == RawNameToIndexMap.Num());
		int32 BoneIndex = INDEX_NONE;
		if (BoneName != NAME_None)
		{
			const int32* IndexPtr = RawNameToIndexMap.Find(BoneName);
			if (IndexPtr)
			{
				BoneIndex = *IndexPtr;
			}
		}
		return BoneIndex;
	}

	FName GetBoneName(const int32 BoneIndex) const
	{
		return FinalRefBoneInfo[BoneIndex].Name;
	}

	int32 GetParentIndex(const int32 BoneIndex) const
	{
		return GetParentIndexInternal(BoneIndex, FinalRefBoneInfo);
	}

	int32 GetRawParentIndex(const int32 BoneIndex) const
	{
		return GetParentIndexInternal(BoneIndex, RawRefBoneInfo);
	}

	bool IsValidIndex(int32 Index) const
	{
		return (FinalRefBoneInfo.IsValidIndex(Index));
	}

	bool IsValidRawIndex(int32 Index) const
	{
		return (RawRefBoneInfo.IsValidIndex(Index));
	}

	/** 
	 * Returns # of Depth from BoneIndex to ParentBoneIndex
	 * This will return 0 if BoneIndex == ParentBoneIndex;
	 * This will return -1 if BoneIndex isn't child of ParentBoneIndex
	 */
	int32 GetDepthBetweenBones(const int32 BoneIndex, const int32 ParentBoneIndex) const
	{
		if (BoneIndex >= ParentBoneIndex)
		{
			int32 CurBoneIndex = BoneIndex;
			int32 Depth = 0;

			do
			{
				// if same return;
				if (CurBoneIndex == ParentBoneIndex)
				{
					return Depth;
				}

				CurBoneIndex = FinalRefBoneInfo[CurBoneIndex].ParentIndex;
				++Depth;

			} while (CurBoneIndex!=INDEX_NONE);
		}

		return INDEX_NONE;
	}
	
	bool BoneIsChildOf(const int32 ChildBoneIndex, const int32 ParentBoneIndex) const
	{
		if (ParentBoneIndex != INDEX_NONE)
		{
			// Bones are in strictly increasing order.
			// So child must have an index greater than its parent.
			if (ChildBoneIndex > ParentBoneIndex)
			{
				int32 BoneIndex = GetParentIndex(ChildBoneIndex);
				do
				{
					if (BoneIndex == ParentBoneIndex)
					{
						return true;
					}
					BoneIndex = GetParentIndex(BoneIndex);

				} while (BoneIndex != INDEX_NONE);
			}
		}

		return false;
	}

	void RemoveDuplicateBones(const UObject* Requester, TArray<FBoneIndexType> & DuplicateBones);

	/** Removes the supplied bones from the skeleton, unless they have children that aren't also going to be removed */
	TArray<int32> RemoveBonesByName(USkeleton* Skeleton, const TArray<FName>& BonesToRemove)
	{
		TArray<int32> BonesRemoved;

		const int32 NumBones = GetRawBoneNum();
		for(int32 BoneIndex=NumBones-1; BoneIndex>=0; BoneIndex--)
		{
			FMeshBoneInfo& Bone = RawRefBoneInfo[BoneIndex];

			if(BonesToRemove.Contains(Bone.Name))
			{
				RemoveIndividualBone(BoneIndex, BonesRemoved);
			}
		}

		const bool bRebuildNameMap = true;
		RebuildRefSkeleton(Skeleton, bRebuildNameMap);
		return BonesRemoved;
	}

	void RebuildNameToIndexMap();

	/** Ensure parent exists in the given input sorted array. Insert parent to the array. The result should be sorted. */
	ENGINE_API void EnsureParentsExist(TArray<FBoneIndexType>& InOutBoneSortedArray) const;

	/** Ensure parent exists in the given input array. Insert parent to the array. The result should be sorted. */
	ENGINE_API void EnsureParentsExistAndSort(TArray<FBoneIndexType>& InOutBoneUnsortedArray) const;

	SIZE_T GetDataSize() const;

	/** Return all the non-virtual bone poses as absolute transforms (i.e. relative to the origin, rather than the parent bone).
	  * This function will resize the array to store the transforms, with no re-allocation if shrinking */
	ENGINE_API void GetRawBoneAbsoluteTransforms(TArray<FTransform>& OutAbsoluteTransforms) const;
	
	/** Return all the bone poses as absolute transforms (i.e. relative to the origin, rather than the parent bone).
	  * This function will resize the array to store the transforms, with no re-allocation if shrinking */
	ENGINE_API void GetBoneAbsoluteTransforms(TArray<FTransform>& OutAbsoluteTransforms) const;

	/** Return a single non-virtual bone's pose as absolute transform (i.e. relative to the origin, rather than the parent bone). */
	ENGINE_API FTransform GetRawBoneAbsoluteTransform(const int32 InRawBoneIndex) const;
	
	/** Return a single bone's pose as absolute transform (i.e. relative to the origin, rather than the parent bone). */
	ENGINE_API FTransform GetBoneAbsoluteTransform(const int32 InBoneIndex) const;

	// lazy-cached for fast access to children and recursive children
	void GetRawChildrenIndicesCached(const int32 BoneIndex, TArray<int32>& OutChildren) const;
	void GetRawChildrenIndicesRecursiveCached(const int32 BoneIndex, TArray<int32>& OutChildren) const;

	// very slow search function for all children
	ENGINE_API int32 GetDirectChildBones(int32 ParentBoneIndex, TArray<int32> & Children) const;
	ENGINE_API int32 GetRawDirectChildBones(int32 ParentBoneIndex, TArray<int32> & Children) const;
	ENGINE_API friend FArchive & operator<<(FArchive & Ar, FReferenceSkeleton & F);
	friend FReferenceSkeletonModifier;
};
