// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#ifndef WITH_REMOTE_CONSOLE_SERVER
#define WITH_REMOTE_CONSOLE_SERVER (!UE_BUILD_SHIPPING) // Remote console is a development-only feature and is expected to be compiled-out in shipping configs.
#endif

#if WITH_REMOTE_CONSOLE_SERVER
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "Containers/StringFwd.h"

/**
 * Bidirectional JSON-line remote console.
 *
 * NOTE: this server does not perform any access control. This can be handled at the higher level by ITransport.
 * Additionally, the server is expected to be compiled out in shipping builds.
 * 
 * Platform code implements ITransport and calls HandleConnection() on a worker thread.
 * Handshake (hello) quickly distinguishes JSON clients from legacy fire-and-forget senders based on a heuristic IsHandshake().
 * Hello packet is then fully parsed and validated in HandleConnection(). After that point, it will be treated as an error / invalid op.
 *
 * After the connection is established, the client is automatically subscribed to the console log stream.
 *
 * Example exchange:
 *   -> {"id":1,"op":"hello","version":1}\n          (required handshake)
 *   <- {"id":1,"op":"hello","version":1}\n
 *   -> {"id":2,"op":"exec","cmd":"stat unit"}\n
 *   <- {"id":2,"op":"exec","ok":true}\n
 *   <- {"op":"log","cat":"LogTemp","v":"Log","line":"..."}\n
 *   -> {"id":3,"op":"complete","q":"r.Nanite","offset":0,"limit":50}\n
 *   -> {"id":4,"op":"getvar","n":"r.RHI.Name"}\n
 */
class FRemoteConsoleServer
{
public:
	class ITransport
	{
	public:
		virtual ~ITransport() = default;
		virtual bool Send(const char* Buffer, int32 BufferSize) = 0;

		/** Returns bytes read, 0 for timeout/no-data, -1 for error/disconnect. */
		virtual int32 Recv(char* Buffer, int32 BufferSize, int32 TimeoutMs = -1) = 0;
	};

	/** Returns true if the data looks like a JSON-line handshake.
	 *  NOTE: Callers pass the first recv() buffer directly. This assumes the full
	 *  handshake (~35 bytes) arrives in one read, which holds on LAN devkit connections
	 *  where it is sent as a single write(). Not robust against TCP fragmentation. */
	static ENGINE_API bool IsHandshake(FAnsiStringView Data);

	/** Blocks calling thread for the connection lifetime. */
	static ENGINE_API void HandleConnection(ITransport& Transport);

private:
	/** Buffered reader on top of ITransport. */
	struct FRecvBuffer
	{
		TArray<char> Data;
		int32 Pos = 0;
		int32 Len = 0;
		bool bDisconnected = false;
		bool bLineTooLong = false;

		static constexpr int32 MaxLineLength = 1024 * 1024; // 1 MB

		bool Fill(ITransport& Transport, int32 TimeoutMs = -1);
		bool ReadLine(TAnsiStringBuilder<512>& OutLine, ITransport& Transport, int32 TimeoutMs = -1);
		bool HasData(ITransport& Transport);
	};

	static bool SendLine(ITransport& Transport, FStringBuilderBase& Builder);

	struct FCompletionCache
	{
		FString Prefix;
		TArray<FString> SortedNames;
		double Timestamp = 0.0;
		static constexpr double CacheTTL = 5.0;

		bool IsValid(const FString& ForPrefix) const
		{
			return !Prefix.IsEmpty()
				&& ForPrefix.StartsWith(Prefix)
				&& (FPlatformTime::Seconds() - Timestamp) < CacheTTL;
		}
	};

	static void HandleExec(const FString& Command, FStringBuilderBase& Response);
	static void HandleComplete(const FString& Prefix, int32 Offset, int32 Limit,
		FCompletionCache& Cache, FStringBuilderBase& Response);
	static void HandleGetVar(const FString& VarName, FStringBuilderBase& Response);
};

#endif // WITH_REMOTE_CONSOLE_SERVER
