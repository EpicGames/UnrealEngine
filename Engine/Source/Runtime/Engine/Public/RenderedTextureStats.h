// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Lightweight struct used to list the MIP levels of rendered assets.
 */
struct FRenderedTextureStats
{
	int32 MaxMipLevelShown;
	FString TextureGroup;
};
