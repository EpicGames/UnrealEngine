// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"
#include "StaticMeshVertexData.h"
#include "RHI.h"

struct FStaticMeshBuildVertex;
struct FConstMeshBuildVertexView;

/**
* A vertex buffer of colors.
*/
class FColorVertexBuffer : public FVertexBuffer
{
public:
	enum class NullBindStride
	{
		FColorSizeForComponentOverride, //when we want to bind null buffer with the expectation that it must be overridden.  Stride must be correct so override binds correctly
		ZeroForDefaultBufferBind, //when we want to bind the null color buffer for it to actually be used for draw.  Stride must be 0 so the IA gives correct data for all verts.
	};

	/** Default constructor. */
	ENGINE_API FColorVertexBuffer();

	/** Destructor. */
	ENGINE_API ~FColorVertexBuffer();

	/** Delete existing resources */
	ENGINE_API void CleanUp();

	ENGINE_API void Init(uint32 InNumVertices, bool bNeedsCPUAccess = true);

	/**
	* Initializes the buffer with the given vertices, used to convert legacy layouts.
	* @param InVertices - The vertices to initialize the buffer with.
	*/
	ENGINE_API void Init(const TArray<FStaticMeshBuildVertex>& InVertices, bool bNeedsCPUAccess = true);
	ENGINE_API void Init(const FConstMeshBuildVertexView& InVertices, bool bNeedsCPUAccess = true);

	/**
	* Initializes this vertex buffer with the contents of the given vertex buffer.
	* @param InVertexBuffer - The vertex buffer to initialize from.
	*/
	void Init(const FColorVertexBuffer& InVertexBuffer, bool bNeedsCPUAccess = true);
	
	/**
	 * Appends the specified vertices to the end of the buffer
	 *
	 * @param	Vertices	The vertex data to be appended.  Must not be nullptr.
	 * @param	NumVerticesToAppend		How many vertices should be added
	 * @return	true if append operation is successful
	 */
	ENGINE_API bool AppendVertices( const FStaticMeshBuildVertex* Vertices, const uint32 NumVerticesToAppend );
	
	/**
	* Serializer
	*
	* @param	Ar					Archive to serialize with
	* @param	bInNeedsCPUAccess	Whether the elements need to be accessed by the CPU
	*/
	ENGINE_API void Serialize(FArchive& Ar, bool bNeedsCPUAccess);

	void SerializeMetaData(FArchive& Ar);

	ENGINE_API void SetMetaData(uint32 InStride, uint32 InNumVertices);

	void ClearMetaData();

	/**
	* Export the data to a string, used for editor Copy&Paste.
	* The method must not be called if there is no data.
	*/
	void ExportText(FString &ValueStr) const;

	/**
	* Export the data from a string, used for editor Copy&Paste.
	* @param SourceText - must not be 0
	*/
	void ImportText(const TCHAR* SourceText);

	/**
	* Specialized assignment operator, only used when importing LOD's.
	*/
	ENGINE_API void operator=(const FColorVertexBuffer &Other);

	inline FColor& VertexColor(uint32 VertexIndex)
	{
		checkSlow(VertexIndex < GetNumVertices());
		return *(FColor*)(Data + VertexIndex * Stride);
	}

	inline const FColor& VertexColor(uint32 VertexIndex) const
	{
		checkSlow(VertexIndex < GetNumVertices());
		return *(FColor*)(Data + VertexIndex * Stride);
	}

	// Other accessors.
	inline uint32 GetStride() const
	{
		return Stride;
	}
	inline uint32 GetNumVertices() const
	{
		return NumVertices;
	}
	ENGINE_API bool GetAllowCPUAccess() const;

	/** Useful for memory profiling. */
	ENGINE_API uint32 GetAllocatedSize() const;

	/**
	* Gets all vertex colors in the buffer
	*
	* @param OutColors	The populated list of colors
	*/
	ENGINE_API void GetVertexColors(TArray<FColor>& OutColors) const;

	/**
	* Load from a array of colors
	* @param InColors - must not be 0
	* @param Count - must be > 0
	* @param Stride - in bytes, usually sizeof(FColor) but can be 0 to use a single input color or larger.
	*/
	ENGINE_API void InitFromColorArray(const FColor *InColors, uint32 Count, uint32 Stride = sizeof(FColor), bool bNeedsCPUAccess = true);

	/**
	* Load from raw color array.
	* @param InColors - InColors must not be empty
	*/
	void InitFromColorArray(const TArray<FColor> &InColors)
	{
		InitFromColorArray(InColors.GetData(), InColors.Num());
	}

	/**
	* Load from single color.
	* @param Count - must be > 0
	*/
	void InitFromSingleColor(const FColor &InColor, uint32 Count)
	{
		InitFromColorArray(&InColor, Count, 0);
	}

	/** Create an RHI vertex buffer with CPU data. CPU data may be discarded after creation (see TResourceArray::Discard) */
	FBufferRHIRef CreateRHIBuffer(FRHICommandListBase& RHICmdList);

	/** Similar to Init/ReleaseRHI but only update existing SRV so references to the SRV stays valid */
	void InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher);
	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;
	virtual FString GetFriendlyName() const override { return TEXT("ColorOnly Mesh Vertices"); }

	ENGINE_API void BindColorVertexBuffer(const class FVertexFactory* VertexFactory, struct FStaticMeshDataType& StaticMeshData) const;
	ENGINE_API static void BindDefaultColorVertexBuffer(const class FVertexFactory* VertexFactory, struct FStaticMeshDataType& StaticMeshData, NullBindStride BindStride);

	inline FRHIShaderResourceView* GetColorComponentsSRV() const { return ColorComponentsSRV; }

	void* GetVertexData() { return Data; }
	const void* GetVertexData() const { return Data; }

private:

	/** The vertex data storage type */
	class FColorVertexData* VertexData;

	FShaderResourceViewRHIRef ColorComponentsSRV;

	/** The cached vertex data pointer. */
	uint8* Data;

	/** The cached vertex stride. */
	uint32 Stride;

	/** The cached number of vertices. */
	uint32 NumVertices;

	bool NeedsCPUAccess = true;

	/** Allocates the vertex data storage type. */
	void AllocateData(bool bNeedsCPUAccess = true);

	/** Purposely hidden */
	ENGINE_API FColorVertexBuffer(const FColorVertexBuffer &rhs);
};
