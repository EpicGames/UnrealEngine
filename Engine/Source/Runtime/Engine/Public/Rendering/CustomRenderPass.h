// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CustomRenderPass.h: Custom render pass implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphFwd.h"
#include "Engine/EngineTypes.h"

// ----------------------------------------------------------------------------------
/** Base interface to implement for FCustomRenderPassBase */
class ICustomRenderPass
{
public:
	virtual ~ICustomRenderPass() = default;

	/** Bare-bones RTTI method to allow users to differentiate / downcast the custom render pass they're interested into */
	virtual const FName& GetTypeName() const PURE_VIRTUAL(ICustomRenderPass::GetTypeName, static FName Name; return Name;);
};

/** Use this macro to implement the RTTI method for custom render passes. Should only be used on final classes (e.g. if C derives from B which derives from A, we won't be able 
  *  to query a C* for whether it's a "B", in other words, A and B should be abstract and C, final) */
#define IMPLEMENT_CUSTOM_RENDER_PASS(TypeName) const FName& GetTypeName() const override { return GetTypeNameStatic(); } \
	static const FName& GetTypeNameStatic() { static FName Name(TEXT(#TypeName)); return Name; }


// ----------------------------------------------------------------------------------
/** Base interface to implement for attaching user data to a FCustomRenderPassBase */
class ICustomRenderPassUserData
{
public:
	virtual ~ICustomRenderPassUserData() = default;
	/** Bare-bones RTTI method to allow users to differentiate / downcast the custom render pass user data they're interested into */
	virtual const FName& GetTypeName() const PURE_VIRTUAL(ICustomRenderPassUserData::GetTypeName, static FName Name; return Name;);
};

/** Use this macro to implement the RTTI method for custom render pass user data. Should only be used on final classes (e.g. if C derives from B which derives from A, we won't be able
  *  to query a C* for whether it's a "B", in other words, A and B should be abstract and C, final) */
#define IMPLEMENT_CUSTOM_RENDER_PASS_USER_DATA(TypeName) const FName& GetTypeName() const override { return GetTypeNameStatic(); } \
	static const FName& GetTypeNameStatic() { static FName Name(TEXT(#TypeName)); return Name; }


// ----------------------------------------------------------------------------------
/** Base class of the custom render pass. Create a derived class to provide custom behavior. 
	Custom render pass is rendered as part of the main render loop. */
class FCustomRenderPassBase : public ICustomRenderPass
{
public:
	friend class FSceneRenderer;
	friend class FDeferredShadingSceneRenderer;
	friend class FMobileSceneRenderer;

	/** Which render passes are needed for the custom render pass. */
	enum class ERenderMode : uint8
	{
		/** Render depth pre-pass only. */
		DepthPass,
		/** Render depth pre-pass and base pass. */
		DepthAndBasePass
	};

	/** The output type into the render target. What type is valid depends on ERenderMode. */
	enum class ERenderOutput : uint8
	{
		/** Used with ERenderMode::DepthPass. */
		SceneDepth,
		DeviceDepth,
		/** Used with ERenderMode::DepthAndBasePass. */
		SceneColorAndDepth,
		SceneColorAndAlpha, // The alpha is inverted opacity (throughput).
		SceneColorNoAlpha,
		BaseColor,
		Normal,
		MAX
	};

	enum class ERenderCaptureType
	{
		/** Don't trigger a render capture */
		NoCapture, 
		/** Start a render capture on PreRender, stop it on PostRender */
		Capture,
		/** Start a render capture on PreRender (this must be followed by another pass with EndCapture). Use it when the render capture encapsulates several custom passes */
		BeginCapture,
		/** Stop a render capture on PostRender (this must be preceded by another pass with BeginCapture). Use it when the render capture encapsulates several custom passes */
		EndCapture,
	};

	FCustomRenderPassBase() = delete;
	ENGINE_API FCustomRenderPassBase(const FString& InDebugName, ERenderMode InRenderMode, ERenderOutput InRenderOutput, const FIntPoint& InRenderTargetSize);

	virtual ~FCustomRenderPassBase() = default;
	
	/** Called before PreRender on render thread (only really useful for properly scoping render capture names) */
	ENGINE_API void BeginPass(FRDGBuilder& GraphBuilder);
	/** Called before the pass is rendered on render thread, use it for prep tasks such as allocating render target. */
	ENGINE_API void PreRender(FRDGBuilder& GraphBuilder);
	/** Called after the pass is rendered on render thread, use it for post processing tasks such as applying additional shader effects to the render target. */
	ENGINE_API void PostRender(FRDGBuilder& GraphBuilder);
	/** Called after PostRender on render thread (only really useful for properly scoping render capture names) */
	ENGINE_API void EndPass(FRDGBuilder& GraphBuilder);

	/** Convert output type into corresponding scene capture source type. This is because we use CopySceneCaptureComponentToTarget helper function
		to copy the custom render pass render result from scene texture into the pass's render target. */
	ENGINE_API ESceneCaptureSource GetSceneCaptureSource() const;

	/** Perform a render capture when this pass runs if a render capture interface is registered */
	ENGINE_API void PerformRenderCapture(ERenderCaptureType InRenderCaptureType, const FString& InFileName = FString());

	const FString& GetDebugName() const { return DebugName; }
	ERenderMode GetRenderMode() const { return RenderMode; }
	ERenderOutput GetRenderOutput() const { return RenderOutput; }
	FRDGTextureRef GetRenderTargetTexture() const { return RenderTargetTexture; }
	const FIntPoint& GetRenderTargetSize() const { return RenderTargetSize; }

	/** Returns true if translucency is enabled AND an output uses SceneColor.  We can skip translucency if SceneColor (where translucency is written) is unused. */
	bool IsTranslucentIncluded() const
	{
		return bSceneColorWithTranslucent &&
			(RenderOutput == FCustomRenderPassBase::ERenderOutput::SceneColorAndAlpha ||
			 RenderOutput == FCustomRenderPassBase::ERenderOutput::SceneColorNoAlpha ||
			 RenderOutput == FCustomRenderPassBase::ERenderOutput::SceneColorAndDepth ||
			 bSceneColorIsUserSceneTexture);
	}

	ENGINE_API void SetUserData(TUniquePtr<ICustomRenderPassUserData>&& InUserData);
	ENGINE_API ICustomRenderPassUserData* GetUserData(const FName& InTypeName) const;

	template <typename UserDataType>
	typename TEnableIf<TPointerIsConvertibleFromTo<UserDataType, ICustomRenderPassUserData>::Value, UserDataType>::Type* GetUserDataTyped() const
	{
		if (ICustomRenderPassUserData* UserData = GetUserData(UserDataType::GetTypeNameStatic()))
		{
			return reinterpret_cast<UserDataType*>(UserData);
		}
		return nullptr;
	}

protected:
	virtual void OnBeginPass(FRDGBuilder& GraphBuilder) {}
	virtual void OnPreRender(FRDGBuilder& GraphBuilder) {}
	virtual void OnPostRender(FRDGBuilder& GraphBuilder) {}
	virtual void OnEndPass(FRDGBuilder& GraphBuilder) {}

	/** Used internally during scene render to generate multiple outputs from a single custom render pass */
	void OverrideRenderOutput(ERenderOutput InRenderOutput)
	{
		RenderOutput = InRenderOutput;
	}

protected:
	FString DebugName;
	ERenderMode RenderMode = ERenderMode::DepthPass;
	ERenderOutput RenderOutput = ERenderOutput::SceneDepth;
	bool bSceneColorWithTranslucent = false;

	// If the SceneColor generated by this Custom Render Pass is referenced as a UserSceneTexture, translucency should render if enabled,
	// even if RenderOutput is not SceneColor.  Otherwise, translucency is disabled with the assumption it's wasteful to render translucency
	// to a SceneColor buffer that's not used.  See IsTranslucentIncluded() function above.
	bool bSceneColorIsUserSceneTexture = false;

	FRDGTextureRef RenderTargetTexture = nullptr;
	FIntPoint RenderTargetSize = FIntPoint(ForceInit);

	/** The views created for the custom render pass. */
	TArray<class FViewInfo*> Views;

	/** Optional user data providing a hook into the engine to override settings for this render pass.
		The user data should derive from ICustomRenderPassUserData and implement GetTypeName() to be retrieved.
		Use SetUserData/GetUserData */
	TMap<FName, TUniquePtr<ICustomRenderPassUserData>> UserDatas;

private:
	ERenderCaptureType RenderCaptureType = ERenderCaptureType::NoCapture;
	FString RenderCaptureFileName;
};
