// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExponentialHeightFogData.generated.h"

/**
*	Data for an individual fog line integral.
*	This is the data which is not shared between fogs when multiple fogs are set up on a single UExponentialHeightFogComponent
*/
USTRUCT(BlueprintType)
struct FExponentialHeightFogData
{
	GENERATED_USTRUCT_BODY()

	/** Global density factor for this fog. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=ExponentialHeightFogComponent, meta = (UIMin = "0", UIMax = ".05"))
	float FogDensity = 0.02f;

	/**
	* Height density factor, controls how the density increases as height decreases.
	* Smaller values make the visible transition larger.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=ExponentialHeightFogComponent, meta = (UIMin = "0.001", UIMax = "2"))
	float FogHeightFalloff = 0.2f;

	/** Height offset, relative to the actor position Z. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category=ExponentialHeightFogComponent)
	float FogHeightOffset = 0.0f;

	/** Clamp to valid ranges. This might be different from the UI clamp. */
	void ClampToValidRanges()
	{
		FogDensity = FMath::Clamp(FogDensity, 0.0f, 10.0f);
		FogHeightFalloff = FMath::Clamp(FogHeightFalloff, 0.0f, 2.0f);
	}
};
