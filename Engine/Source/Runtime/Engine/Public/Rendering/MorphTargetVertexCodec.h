// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Containers/ContainersFwd.h"
#include "Math/IntVector.h"

struct FMorphTargetDelta;
class UMorphTarget;
class FDefaultBitArrayAllocator;


namespace UE::MorphTargetVertexCodec
{

inline constexpr uint32 BatchSizeBits = 6u;
inline constexpr uint32 BatchSize = 1u << BatchSizeBits;

inline constexpr uint32 NumBatchHeaderDwords = 10u;


struct FQuantizedDelta
{
	FIntVector	Position;
	FIntVector	TangentZ;
	uint32		Index;

	bool operator==(const FQuantizedDelta&) const = default; 
};

struct FDeltaBatchHeader
{
	uint32		DataOffset;
	uint32		NumElements;

	bool		bTangents;
	int8		IndexBits;
	Math::TIntVector3<int8>	PositionBits;
	Math::TIntVector3<int8>	TangentZBits;
	
	uint32		IndexMin;
	FIntVector	PositionMin;
	FIntVector	TangentZMin;

	bool operator==(const FDeltaBatchHeader&) const = default; 

	friend FArchive& operator<<(FArchive& Ar, FDeltaBatchHeader& InHeader)
	{
		Ar << InHeader.DataOffset;
		Ar << InHeader.NumElements;
		Ar << InHeader.bTangents;
		Ar << InHeader.IndexBits;
		Ar << InHeader.PositionBits;
		Ar << InHeader.TangentZBits;
		Ar << InHeader.IndexMin;
		Ar << InHeader.PositionMin;
		Ar << InHeader.TangentZMin;
		
		return Ar;
	}
};


inline float ComputePositionPrecision(float InTargetPositionErrorTolerance)
{
	constexpr float UnrealUnitPerMeter = 100.0f;
	return InTargetPositionErrorTolerance * 2.0f * 1e-6f * UnrealUnitPerMeter;	// * 2.0 because correct rounding guarantees error is at most half of the cell size.
}

inline float ComputeTangentPrecision()
{
	return 1.0f / 2048.0f;	// Object scale irrelevant here. Let's assume ~12bits per component is plenty.
}

/** Encodes the morph deltas, using the given precision, into a pair of arrays -- one to store the header
  * data that describes a bit-packed batch of vertices and another that stores the actual bit packed vertices
  * themselves.
  * Note that due to the compressor throwing away deltas that are under the precision threshold, the number of 
  * output deltas may not match the number of deltas that end up getting packed.
  * @param InMorphDeltas The list of morph deltas that will be compressed.
  * @param InVertexNeedsTangents An optional bit array of vertices that require tangents to be present. If nullptr,
  *    then all vertices are assumed to require tangents. Used to ignore compressing tangents on sections that compute
  *    them automatically at render time.
  * @param InPositionPrecision The precision required for the compression the position data. The greater this value
  *    the fewer bits are used for storing the resulting position values.
  * @param InTangentZPrecision The precision required for the compression the tangent data. The greater this value
  *    the fewer bits are used for storing the resulting tangent values.
  * @param OutBatchHeaders The list of block headers for each compressed block of morph deltas. Multiple headers are
  *    required since the final amount of bits required is dependent on the spread of position/tangent values within
  *    each block.
  * @param OutCompressedVertices The actual compressed data. 
  */
ENGINE_API void Encode(
	TConstArrayView<FMorphTargetDelta> InMorphDeltas,
	const TBitArray<>* InVertexNeedsTangents,
	const float InPositionPrecision,
	const float InTangentZPrecision,
	TArray<FDeltaBatchHeader>& OutBatchHeaders,
	TArray<uint32>& OutCompressedVertices
	);

/** Decode all the vertices given by the two arrays. */ 
ENGINE_API void Decode(
	TConstArrayView<FDeltaBatchHeader> InBatchHeaders,
	TConstArrayView<uint32> InCompressedVertices,
	const float InPositionPrecision,
	const float InTangentZPrecision,
	TArray<FMorphTargetDelta>& OutMorphDeltas
	);

/** Iteratively decode a single morph target delta entry from the data stream. The InOutNextItemToken is an opaque value that should
 *  be initialized to zero for the first entry. It will be adjusted to point to the next entry after a successful decode. */ 
ENGINE_API bool IterativeDecode(
	uint64& InOutNextItemToken,
	TConstArrayView<FDeltaBatchHeader> InBatchHeaders,
	TConstArrayView<uint32> InCompressedVertices,
	const float InPositionPrecision,
	const float InTangentZPrecision,
	FMorphTargetDelta& OutMorphTargetDelta
	);

/** Reads and decodes OutBatchHeader bits from InData. */
ENGINE_API void ReadHeader(FDeltaBatchHeader& OutBatchHeader, const TArrayView<const uint32> InData);

/** Encodes and writes InBatchHeader bits to OutData. */
ENGINE_API void WriteHeader(const FDeltaBatchHeader& InBatchHeader, TArrayView<uint32> OutData);

/** Encodes and writes the batch quantized data from InQuantizeDeletas to OutData. 
 *  OutData must be sufficiently large to hold the number of Dwords specified by InBatchHeader. 
 */
ENGINE_API void WriteQuantizedDeltas(
	TConstArrayView<FQuantizedDelta> InQuantizedDeltas, 
	const FDeltaBatchHeader& InBatchHeader, 
	TArrayView<uint32> OutData
	);

/** Reads and decodes the batch quantized data from InData to OutQuantizeDeletas.
 *  InData must be sufficiently large to hold the number of Dwords specified by InBatchHeader. 
 */
ENGINE_API void ReadQuantizedDeltas(
	TArrayView<FQuantizedDelta> OutQuantizedDeltas, 
	const FDeltaBatchHeader& InBatchHeader, 
	TConstArrayView<uint32> InData
	);

/** Calculates the number of Dwords the batch data has from BatchHeader. */
ENGINE_API uint32 CalculateBatchDwords(const FDeltaBatchHeader& BatchHeader);

/** Quantize FMorphTargetDelta using the given precision. */
ENGINE_API void QuantizeDelta(
	const FMorphTargetDelta& InDelta, 
	const bool bInNeedsTangent, 
	FQuantizedDelta& OutQuantizedDelta, 
	const float InPositionPrecision, 
	const float InTangentZPrecision
	);

/* Dequantize FMorphTargetDelta using the given precision. */
ENGINE_API void DequantizeDelta(
	FMorphTargetDelta& OutDelta, 
	const bool bInNeedsTangent, 
	const FQuantizedDelta& InQuantizedDelta, 
	const float InPositionPrecision, 
	const float InTangentZPrecision
	);

}
