// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Math/Vector4.h"

struct FSkelMeshRenderSection;
struct FMorphTargetLODModel;
class UMorphTarget;

#if !WITH_EDITOR
struct FMorphTargetCompressedLODModel;
#endif

class FMorphTargetVertexInfoBuffers : public FRenderResource
{
public:
	ENGINE_API FMorphTargetVertexInfoBuffers();
	ENGINE_API ~FMorphTargetVertexInfoBuffers();

	ENGINE_API void InitMorphResources(EShaderPlatform ShaderPlatform, const TArray<FSkelMeshRenderSection>& RenderSections, const TArray<UMorphTarget*>& MorphTargets, int32 NumVertices, int32 LODIndex, float TargetPositionErrorTolerance);

	ENGINE_API void InitMorphResourcesStreaming(EShaderPlatform ShaderPlatform, const TArray<FSkelMeshRenderSection>& RenderSections, TConstArrayView<const FMorphTargetLODModel*> MorphTargets, int32 NumVertices, float TargetPositionErrorTolerance);

#if !WITH_EDITOR
	ENGINE_API void InitMorphResourcesStreaming(EShaderPlatform ShaderPlatform, TConstArrayView<const FMorphTargetCompressedLODModel*> MorphTargets, TConstArrayView<FVector4f> InMinimumValuePerMorph, TConstArrayView<FVector4f> InMaximumValuesPerMorph);
#endif

	inline bool IsMorphResourcesInitialized() const { return bResourcesInitialized; }
	inline bool IsRHIInitialized() const { return bRHIInitialized; }
	inline bool IsMorphCPUDataValid() const{ return bIsMorphCPUDataValid; }
	
	bool GetEmptyMorphCPUDataOnInitRHI() const { return bEmptyMorphCPUDataOnInitRHI; }
	void SetEmptyMorphCPUDataOnInitRHI(bool bEmpty) { bEmptyMorphCPUDataOnInitRHI = bEmpty; }

	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;

	UE_DEPRECATED(5.2, "GetMaximumThreadGroupSize will be removed as it is no longer used.")
	static ENGINE_API uint32 GetMaximumThreadGroupSize();

	uint32 GetNumBatches(uint32 index = UINT_MAX) const
	{
		check(index == UINT_MAX || index < (uint32)BatchesPerMorph.Num());
		return index != UINT_MAX ? BatchesPerMorph[index] : NumTotalBatches;
	}

	uint32 GetNumMorphs() const
	{
		return BatchesPerMorph.Num();
	}

	uint32 GetBatchStartOffset(uint32 Index) const
	{
		check(Index < (uint32)BatchStartOffsetPerMorph.Num());
		return BatchStartOffsetPerMorph[Index];
	}

	const FVector4f& GetMaximumMorphScale(uint32 Index) const
	{
		check(Index < (uint32)MaximumValuePerMorph.Num());
		return MaximumValuePerMorph[Index];
	}

	const FVector4f& GetMinimumMorphScale(uint32 Index) const
	{
		check(Index < (uint32)MinimumValuePerMorph.Num());
		return MinimumValuePerMorph[Index];
	}

	float GetPositionPrecision() const
	{
		return PositionPrecision;
	}

	float GetTangentZPrecision() const
	{
		return TangentZPrecision;
	}

	const uint32* GetData() const
	{
		return MorphData.GetData();
	}
	
	uint64 GetMorphDataSizeInBytes() const 
	{ 
		return MorphData.Num() * sizeof(uint32);
	}
	
	static ENGINE_API bool IsPlatformShaderSupported(EShaderPlatform ShaderPlatform);
	static ENGINE_API bool NeedsTangents(bool bRecomputeTangents);

	FBufferRHIRef MorphDataBuffer;
	FShaderResourceViewRHIRef MorphDataSRV;

protected:
	ENGINE_API void ResetCPUData();

	ENGINE_API void ValidateVertexBuffers(bool bMorphTargetsShouldBeValid);
	ENGINE_API void Serialize(FArchive& Ar);

	void Compress(const TArray<FSkelMeshRenderSection>& RenderSections, TConstArrayView<const FMorphTargetLODModel*> MorphTargets, const TBitArray<>& UsesBuiltinMorphTargetCompression, int32 NumVertices, float TargetPositionErrorTolerance);

#if !WITH_EDITOR
	void InitDataFromCompressedMorphs(TConstArrayView<const FMorphTargetCompressedLODModel*> MorphTargets, TConstArrayView<FVector4f> InMinimumValuePerMorph, TConstArrayView<FVector4f> MaximumValuesPerMorph);
#endif
	// Transient data. Gets deleted as soon as the GPU resource has been initialized (unless excplicitly disabled by bEmptyMorphCPUDataOnInitRHI).
	TResourceArray<uint32> MorphData;

	//x,y,z separate for position and shared w for tangent

	TArray<FVector4f> MaximumValuePerMorph;
	TArray<FVector4f> MinimumValuePerMorph;
	TArray<uint32> BatchStartOffsetPerMorph;
	TArray<uint32> BatchesPerMorph;
	
	uint32 NumTotalBatches = 0;
	float PositionPrecision = 0.0f;
	float TangentZPrecision = 0.0f;

	bool bIsMorphCPUDataValid = false;
	bool bResourcesInitialized = false;
	bool bRHIInitialized = false;
	bool bEmptyMorphCPUDataOnInitRHI = true;

	friend class FSkeletalMeshLODRenderData;
	ENGINE_API friend FArchive& operator<<(FArchive& Ar, FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers);
};

ENGINE_API FArchive& operator<<(FArchive& Ar, FMorphTargetVertexInfoBuffers& MorphTargetVertexInfoBuffers);
