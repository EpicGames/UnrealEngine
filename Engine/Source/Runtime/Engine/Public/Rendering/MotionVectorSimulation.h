// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Optional.h"
#include "Tickable.h"
#include "UObject/UObjectArray.h"
#include "Math/Transform.h"
#include "HAL/IConsoleManager.h"

class UObjectBase;
class USceneComponent;

/**
 * Singleton class used for optionally overriding previous transforms for motion vector computation.
 * This predominantly used by Sequencer on camera cut frames to forcibly inform the renderer of simulated trajectories for primitive components and cameras.
 * Transforms are stored in an unintrusive sparse map per-component to avoid paying a memory cost for all components.
 * Simulated transforms are only valid for the frame that they were added, and are removed on a subsequent tick.
 */
class FMotionVectorSimulation : public FTickableGameObject, public FUObjectArray::FUObjectDeleteListener
{
public:

	/**
	 * Access the singleton instance for motion vector simulation
	 * @return the singleton instance
	 */
	static ENGINE_API FMotionVectorSimulation& Get();


	/**
	 * Check whether motion vector simulation is enabled. When disabled, no transforms will be returned.
	 * @return Whether motion vector simulation is currently enabled or not.
	 */
	static ENGINE_API bool IsEnabled();

public:


	/**
	 * Check if the specified scene component has a simulated transform, setting the specified transform if so.
	 * @note: Previous simulated transforms are only valid for the frame on which they were added, and are removed on the next frame
	 *
	 * @param Object         The Object to retrieve a previous transform for
	 * @param OutTransform   A valid (non-null) pointer to a transform to receive the simulated transform if possible.
	 * @return True if OutTransform was overwritten with a valid transform, false otherwise.
	 */
	ENGINE_API bool GetPreviousTransform(UObject* Object, FTransform* OutTransform) const;

	/**
	 * Check if the specified uboject has a simulated transform and return the result as an optional transform.
	 * @note: Previous simulated transforms are only valid for the frame on which they were added, and are removed on the next frame
	 *
	 * @param Object      The object to retrieve a previous transform for
	 * @return An optional transform that is set if the specified component has a previous simulated transform, otherwise an empty optional.
	 */
	ENGINE_API TOptional<FTransform> GetPreviousTransform(UObject* Object) const;	

	/**
	 * Assign a simulated previous frame transform for the specified object. Overwrites any existing simulated transform.
	 * @note: Previous simulated transforms are only valid for the frame on which they were added, and are removed on the next frame
	 *
	 * @param Object                      The Object to assign the previous transform to
	 * @param SimulatedPreviousTransform  The simulated transform that this component had on the last frame
	 */
	ENGINE_API void SetPreviousTransform(UObject* Object, const FTransform& SimulatedPreviousTransform);
	
	/**
	 * Clear the simulated previous transform for the specified component
	 *
	 * @param Object The Object to clear the previous transform for
	 */
	ENGINE_API void ClearPreviousTransform(UObject* Component);

	virtual SIZE_T GetAllocatedSize() const override
	{
		return SimulatedTransforms.GetAllocatedSize();
	}

private:

	//~ FTickableGameObject interface - used for removing transforms that are no longer valid
	virtual bool IsTickable() const { return SimulatedTransforms.Num() > 0; }
	virtual bool IsTickableInEditor() const override { return true; }
	ENGINE_API virtual void Tick( float DeltaTime ) override;
	ENGINE_API virtual TStatId GetStatId() const override;

	//~ FUObjectDeleteListener interface - used for removing transforms for components that are no longer valid
	ENGINE_API virtual void NotifyUObjectDeleted(const UObjectBase* Object, int32 Index) override;
	ENGINE_API virtual void OnUObjectArrayShutdown() override;
	
	//~ Private constructor/destructor to prevent non-singleton use
	FMotionVectorSimulation();
	~FMotionVectorSimulation();

	struct FSimulatedTransform
	{
		/** The simulated transform for the component from the last frame */
		FTransform Transform;
		/** The frame number that this simulated transform relates to */
		uint64 FrameNumber;
	};
	mutable FCriticalSection MapCriticalSection;
	TMap<const UObjectBase *, FSimulatedTransform> SimulatedTransforms;
};
