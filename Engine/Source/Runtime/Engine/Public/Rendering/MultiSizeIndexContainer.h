// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RawIndexBuffer.h"

struct FMultiSizeIndexContainerData
{
	TArray<uint32> Indices;
	uint32 DataTypeSize;
};

/**
* Skeletal mesh index buffers are 16 bit by default and 32 bit when called for.
* This class adds a level of abstraction on top of the index buffers so that we can treat them all as 32 bit.
*/
class FMultiSizeIndexContainer
{
public:
	FMultiSizeIndexContainer()
		: DataTypeSize(sizeof(uint16))
		, IndexBuffer(NULL)
	{
	}

	ENGINE_API ~FMultiSizeIndexContainer();

	/**
	* SetOwnerName should be called before BeginInitResources for the owner name to be successfully tracked.
	*/
	ENGINE_API void SetOwnerName(const FName& OwnerName);

	/**
	* Initialize the index buffer's render resources.
	*/
	void InitResources();

	/**
	* Releases the index buffer's render resources.
	*/
	void ReleaseResources();

	/**
	* Serialization.
	* @param Ar				The archive with which to serialize.
	* @param bNeedsCPUAccess	If true, the loaded data will remain in CPU memory
	*							even after the RHI index buffer has been initialized.
	*/
	ENGINE_API void Serialize(FArchive& Ar, bool bNeedsCPUAccess);

	void SerializeMetaData(FArchive& Ar, bool bNeedsCPUAccess);

	/**
	* Creates a new index buffer
	*/
	ENGINE_API void CreateIndexBuffer(uint8 DataTypeSize);

	/**
	* Repopulates the index buffer
	*/
	ENGINE_API void RebuildIndexBuffer(uint8 InDataTypeSize, const TArray<uint32>& NewArray);

	/**
	* Returns a 32 bit version of the index buffer
	*/
	ENGINE_API void GetIndexBuffer(TArray<uint32>& OutArray) const;

	/**
	* Populates the index buffer with a new set of indices
	*/
	ENGINE_API void CopyIndexBuffer(const TArray<uint32>& NewArray);

	bool IsIndexBufferValid() const { return IndexBuffer != NULL; }

	/**
	* Accessors
	*/
	uint8 GetDataTypeSize() const { return DataTypeSize; }
	FRawStaticIndexBuffer16or32Interface* GetIndexBuffer()
	{
		check(IndexBuffer != NULL);
		return IndexBuffer;
	}
	const FRawStaticIndexBuffer16or32Interface* GetIndexBuffer() const
	{
		check(IndexBuffer != NULL);
		return IndexBuffer;
	}

#if WITH_EDITOR
	/**
	* Retrieves index buffer related data
	*/
	ENGINE_API void GetIndexBufferData(FMultiSizeIndexContainerData& OutData) const;

	ENGINE_API FMultiSizeIndexContainer(const FMultiSizeIndexContainer& Other);
	ENGINE_API FMultiSizeIndexContainer& operator=(const FMultiSizeIndexContainer& Buffer);
#endif

	friend FArchive& operator<<(FArchive& Ar, FMultiSizeIndexContainer& Buffer);

	ENGINE_API FBufferRHIRef CreateRHIBuffer(FRHICommandListBase& RHICmdList);

	void InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher);
	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

private:
	/** Size of the index buffer's index type (should be 2 or 4 bytes) */
	uint8 DataTypeSize;
	/** The vertex index buffer */
	FRawStaticIndexBuffer16or32Interface* IndexBuffer;
};