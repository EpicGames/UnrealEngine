// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "RenderResource.h"
#include "RHIFwd.h"

class FPositionVertexData;
struct FStaticMeshBuildVertex;
struct FConstMeshBuildVertexView;

/** A vertex that stores just position. */
struct FPositionVertex
{
	FVector3f	Position;

	friend FArchive& operator<<(FArchive& Ar, FPositionVertex& V)
	{
		Ar << V.Position;
		return Ar;
	}
};

/** A vertex buffer of positions. */
class FPositionVertexBuffer : public FVertexBuffer
{
public:

	/** Default constructor. */
	ENGINE_API FPositionVertexBuffer();

	/** Destructor. */
	ENGINE_API ~FPositionVertexBuffer();

	/** Delete existing resources */
	ENGINE_API void CleanUp();

	void ENGINE_API Init(uint32 NumVertices, bool bInNeedsCPUAccess = true);

	/**
	* Initializes the buffer with the given vertices, used to convert legacy layouts.
	* @param InVertices - The vertices to initialize the buffer with.
	*/
	ENGINE_API void Init(const TArray<FStaticMeshBuildVertex>& InVertices, bool bInNeedsCPUAccess = true);
	ENGINE_API void Init(const FConstMeshBuildVertexView& InVertices, bool bInNeedsCPUAccess = true);
	
	/**
	* Initializes this vertex buffer with the contents of the given vertex buffer.
	* @param InVertexBuffer - The vertex buffer to initialize from.
	*/
	ENGINE_API void Init(const FPositionVertexBuffer& InVertexBuffer, bool bInNeedsCPUAccess = true);

	ENGINE_API void Init(const TArray<FVector3f>& InPositions, bool bInNeedsCPUAccess = true);

	/**
	 * Appends the specified vertices to the end of the buffer
	 *
	 * @param	Vertices	The vertex data to be appended.  Must not be nullptr.
	 * @param	NumVerticesToAppend		How many vertices should be added
	 * @return	true if append operation is successful
	 */
	ENGINE_API bool AppendVertices( const FStaticMeshBuildVertex* Vertices, const uint32 NumVerticesToAppend );

	/**
	* Serializer
	*
	* @param	Ar					Archive to serialize with
	* @param	bInNeedsCPUAccess	Whether the elements need to be accessed by the CPU
	*/
	ENGINE_API void Serialize(FArchive& Ar, bool bInNeedsCPUAccess);

	void SerializeMetaData(FArchive& Ar);

	ENGINE_API void SetMetaData(uint32 InStride, uint32 InNumVertices);

	void ClearMetaData();

	/**
	* Specialized assignment operator, only used when importing LOD's.
	*/
	ENGINE_API void operator=(const FPositionVertexBuffer &Other);

	// Vertex data accessors.
	inline FVector3f& VertexPosition(uint32 VertexIndex)
	{
		checkSlow(VertexIndex < GetNumVertices());
		return ((FPositionVertex*)(Data + VertexIndex * Stride))->Position;
	}
	inline const FVector3f& VertexPosition(uint32 VertexIndex) const
	{
		checkSlow(VertexIndex < GetNumVertices());
		return ((FPositionVertex*)(Data + VertexIndex * Stride))->Position;
	}
	// Other accessors.
	inline uint32 GetStride() const
	{
		return Stride;
	}
	inline uint32 GetNumVertices() const
	{
		return NumVertices;
	}
	ENGINE_API bool GetAllowCPUAccess() const;
	
	inline SIZE_T GetAllocatedSize() const { return (Data != nullptr) ? Stride * NumVertices : 0; }

	/** Create an RHI vertex buffer with CPU data. CPU data may be discarded after creation (see TResourceArray::Discard) */
	FBufferRHIRef CreateRHIBuffer(FRHICommandListBase& RHICmdList);

	/** Similar to Init/ReleaseRHI but only update existing SRV so references to the SRV stays valid */
	void InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher);
	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;
	virtual FString GetFriendlyName() const override { return TEXT("PositionOnly Static-mesh vertices"); }

	ENGINE_API void BindPositionVertexBuffer(const class FVertexFactory* VertexFactory, struct FStaticMeshDataType& Data) const;

	void* GetVertexData() { return Data; }
	const void* GetVertexData() const { return Data; }

	FRHIShaderResourceView* GetSRV() const { return PositionComponentSRV; }

private:

	FShaderResourceViewRHIRef PositionComponentSRV;

	/** The vertex data storage type */
	FPositionVertexData* VertexData;

	/** The cached vertex data pointer. */
	uint8* Data;

	/** The cached vertex stride. */
	uint32 Stride;

	/** The cached number of vertices. */
	uint32 NumVertices;

	bool bNeedsCPUAccess = true;

	/** Allocates the vertex data storage type. */
	void AllocateData(bool bInNeedsCPUAccess = true);
};


