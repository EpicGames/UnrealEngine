// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RayTracingGeometryManagerInterface.h"

#include "Containers/SparseArray.h"
#include "Containers/Map.h"
#include "IO/IoBuffer.h"
#include "Serialization/BulkData.h"
#include "Tasks/Task.h"

#if RHI_RAYTRACING

class FPrimitiveSceneProxy;
class UStaticMesh;
class FRayTracingStreamableAsset;

class FRayTracingGeometryManager : public IRayTracingGeometryManager
{
public:

	ENGINE_API FRayTracingGeometryManager();
	ENGINE_API virtual ~FRayTracingGeometryManager();

	ENGINE_API virtual RayTracing::FBuildRequestHandle RequestBuildAccelerationStructure(FRayTracingGeometry* InGeometry, ERTAccelerationStructureBuildPriority InPriority, EAccelerationStructureBuildMode InBuildMode) override;

	ENGINE_API virtual void RemoveBuildRequest(RayTracing::FBuildRequestHandle RequestHandle) override;
	ENGINE_API virtual void BoostPriority(RayTracing::FBuildRequestHandle RequestHandle, float InBoostValue) override;
	ENGINE_API virtual void ForceBuildIfPending(FRHIComputeCommandList& InCmdList, const TArrayView<const FRayTracingGeometry*> InGeometries) override;
	ENGINE_API virtual void ProcessBuildRequests(FRHIComputeCommandList& InCmdList, bool bInBuildAll = false) override;

	ENGINE_API virtual RayTracing::FGeometryHandle RegisterRayTracingGeometry(FRayTracingGeometry* InGeometry) override;
	ENGINE_API virtual void ReleaseRayTracingGeometryHandle(RayTracing::FGeometryHandle Handle) override;

	ENGINE_API virtual RayTracing::FGeometryGroupHandle RegisterRayTracingGeometryGroup(uint32 NumLODs, uint32 CurrentFirstLODIdx = 0) override;
	ENGINE_API virtual void ReleaseRayTracingGeometryGroup(RayTracing::FGeometryGroupHandle Handle) override;

	ENGINE_API virtual void RefreshRegisteredGeometry(RayTracing::FGeometryHandle Handle) override;

	ENGINE_API void SetRayTracingGeometryStreamingData(const FRayTracingGeometry* Geometry, FRayTracingStreamableAsset& StreamableAsset);
	ENGINE_API void SetRayTracingGeometryGroupCurrentFirstLODIndex(FRHICommandListBase& RHICmdList, RayTracing::FGeometryGroupHandle Handle, uint8 CurrentFirstLODIdx);

	ENGINE_API virtual void PreRender() override;
	ENGINE_API virtual void Update(FRHICommandList& RHICmdList) override;
	ENGINE_API virtual void Tick(FRHICommandList& RHICmdList) override;

	ENGINE_API void RegisterProxyWithCachedRayTracingState(FPrimitiveSceneProxy* Proxy, RayTracing::FGeometryGroupHandle InRayTracingGeometryGroupHandle);
	ENGINE_API void UnregisterProxyWithCachedRayTracingState(FPrimitiveSceneProxy* Proxy, RayTracing::FGeometryGroupHandle InRayTracingGeometryGroupHandle);

	ENGINE_API virtual void RequestUpdateCachedRenderState(RayTracing::FGeometryGroupHandle InRayTracingGeometryGroupHandle) override;

	ENGINE_API void AddReferencedGeometry(const FRayTracingGeometry* Geometry);
	ENGINE_API void AddReferencedGeometries(const TSet<RayTracing::FGeometryHandle>& Geometries);
	ENGINE_API void AddReferencedGeometryGroups(const TSet<RayTracing::FGeometryGroupHandle>& GeometryGroups);
	ENGINE_API void AddReferencedGeometryGroupsForDynamicUpdate(const TSet<RayTracing::FGeometryGroupHandle>& GeometryGroups);

#if DO_CHECK
	ENGINE_API bool IsGeometryReferenced(const FRayTracingGeometry* Geometry) const;
	ENGINE_API bool IsGeometryGroupReferenced(RayTracing::FGeometryGroupHandle GeometryGroup) const;
#endif

	ENGINE_API virtual bool IsGeometryVisible(RayTracing::FGeometryHandle GeometryHandle) const override;
	ENGINE_API virtual void AddVisibleGeometry(RayTracing::FGeometryHandle GeometryHandle) override;
	ENGINE_API void ResetVisibleGeometries(); // TODO: Temp - is it needed?

private:

	struct FBuildRequest
	{
		FRayTracingGeometry* Owner = nullptr;

		RayTracing::FBuildRequestHandle RequestHandle;

		float BuildPriority = 0.0f;
		EAccelerationStructureBuildMode BuildMode;
		bool bReleaseBuffersAfterBuild = false;
	};

	void SetupBuildParams(const FBuildRequest& InBuildRequest, TArray<FRayTracingGeometryBuildParams>& InBuildParams, TArray<FRayTracingGeometry*>& InReleaseBuffers);

	void ReleaseRayTracingGeometryGroupReference(RayTracing::FGeometryGroupHandle Handle);

	bool RequestRayTracingGeometryStreamIn(FRHICommandList& RHICmdList, RayTracing::FGeometryHandle GeometryHandle);
	void ProcessCompletedStreamingRequests(FRHICommandList& RHICmdList);

	void UpdateReferenceBasedResidencyState();
	void ProcessReferenceBasedResidency(FRHICommandList& RHICmdList);

	FCriticalSection RequestCS;

	TSparseArray<FBuildRequest> GeometryBuildRequests;
	TArray<uint16> GeometryBuildRequestsVersions;

	// Working array with all active build build params in the RHI
	TArray<FBuildRequest> SortedRequests;
	TArray<FRayTracingGeometryBuildParams> BuildParams;

	// Operations such as registering geometry/groups can be done from different render command pipes (eg: SkeletalMesh)
	// so need to use critical section in relevant functions
	FCriticalSection MainCS;

	struct FRayTracingGeometryGroup
	{
		TArray<RayTracing::FGeometryHandle, TInlineAllocator<2>> GeometryHandles;
		TSet<RayTracing::FGeometryHandle> AdditionalGeometryHandles;

		TSet<FPrimitiveSceneProxy*> ProxiesWithCachedRayTracingState;

		int32 CurrentFirstLODIdx = INDEX_NONE;

		// Due to the way we batch release FRenderResource and SceneProxies, 
		// ReleaseRayTracingGeometryGroup(...) can end up being called before all FRayTracingGeometry and SceneProxies are actually released.
		// To deal with this, we keep track of whether the group is still referenced and only release the group handle once all references are released.
		uint32 NumReferences = 0;
	};

	struct FRegisteredGeometry
	{
		FRayTracingGeometry* Geometry = nullptr;
		uint32 Size = 0;

		FRayTracingStreamableAsset* StreamableAsset = nullptr;
		uint32 StreamableBVHSize = 0;
		uint32 StreamableBuffersSize = 0;

		int16 StreamingRequestIndex = INDEX_NONE;

		bool bEvicted : 1 = false;
		bool bAlwaysResident : 1 = false;
	};

	struct FStreamingRequest
	{
		FIoBuffer RequestBuffer;
		FBulkDataBatchRequest Request;

		RayTracing::FGeometryHandle GeometryHandle;
		uint32 GeometrySize = 0;

		bool bBuffersOnly = false;

		bool bCancelled = false;

		bool IsValid() const { return !GeometryHandle.IsNull() || bCancelled; }

		void Cancel()
		{
			GeometryHandle = {};
			bCancelled = true;

			if (Request.IsPending())
			{
				Request.Cancel();
			}
		}

		void Reset()
		{
			GeometryHandle = {};
			bCancelled = false;

			checkf(!Request.IsPending(), TEXT("Can't cancel ray tracing geometry streaming request that is still in-flight."));
			Request.Reset();
			RequestBuffer = {};
		}
	};

	void DoesRayTracingGeometryRequireStreaming(const FRegisteredGeometry& RegisteredGeometry, bool& bOutStreamBVH, bool& bOutStreamBuffers);

	void CancelStreamingRequest(FRegisteredGeometry& RegisteredGeometry);

	// Helper function to stream out geometry
	// Must call this instead of FRayTracingGeometry::ReleaseRHIForStreaming() directly
	void StreamOutGeometry(FRHIResourceReplaceBatcher& Batcher, FRegisteredGeometry& RegisteredGeometry);

	// Helper function to make geometry resident + necessary management
	// Must call this instead of FRayTracingGeometry::MakeResident() directly
	void MakeGeometryResident(FRHICommandList& RHICmdList, FRegisteredGeometry& RegisteredGeometry);

	// Helper function to evict geometry + necessary streaming management
	// Must call this instead of FRayTracingGeometry::Evict() directly
	void EvictGeometry(FRHICommandListBase& RHICmdList, FRegisteredGeometry& RegisteredGeometry);

	void MakeResidentAllGeometries(FRHICommandList& RHICmdList);
	void EvictAllGeometries(FRHICommandList& RHICmdList);

	void RefreshAlwaysResidentRayTracingGeometries(FRHICommandList& RHICmdList);

#if DO_CHECK
	void CheckIntegrity() const;
#endif // DO_CHECK

#if !UE_BUILD_SHIPPING
	bool DumpRayTracingGeometries(TArrayView<RayTracing::FGeometryHandle> Geometries, const FString& Filename) const;
	void DumpUnreferencedAlwaysResidentRayTracingGeometries() const;
#endif // !UE_BUILD_SHIPPING

	FBuildRequest& GetBuildRequest(RayTracing::FBuildRequestHandle Handle);

	bool IsHandleValid(RayTracing::FGeometryHandle Handle) const;
	bool IsHandleValid(RayTracing::FGeometryGroupHandle Handle) const;

	uint32 GetRegisteredGeometryIndex(RayTracing::FGeometryHandle Handle) const;
	uint32 GetRegisteredGroupIndex(RayTracing::FGeometryGroupHandle Handle) const;

	FRegisteredGeometry& GetRegisteredGeometry(RayTracing::FGeometryHandle Handle);
	const FRegisteredGeometry& GetRegisteredGeometry(RayTracing::FGeometryHandle Handle) const;
	FRayTracingGeometryGroup& GetRegisteredGroup(RayTracing::FGeometryGroupHandle Handle);
	const FRayTracingGeometryGroup& GetRegisteredGroup(RayTracing::FGeometryGroupHandle Handle) const;

	static bool IsAlwaysResidentGeometry(const FRayTracingGeometry* InGeometry, const FRayTracingGeometryGroup& Group);

	TSparseArray<FRayTracingGeometryGroup> RegisteredGroups;
	TArray<uint16> RegisteredGroupsVersions;

	// Used for keeping track of geometries when ray tracing is dynamic
	TSparseArray<FRegisteredGeometry> RegisteredGeometries;
	TArray<uint16> RegisteredGeometriesVersions;

	TSet<RayTracing::FGeometryHandle> ResidentGeometries;
	int64 TotalResidentSize = 0;

	TSet<RayTracing::FGeometryHandle> AlwaysResidentGeometries;
	int64 TotalAlwaysResidentSize = 0;

	TSet<RayTracing::FGeometryHandle> ReferencedGeometries;
	TSet<RayTracing::FGeometryGroupHandle> ReferencedGeometryGroups;

	TSet<RayTracing::FGeometryGroupHandle> ReferencedGeometryGroupsForDynamicUpdate;

	TSet<RayTracing::FGeometryHandle> PendingStreamingRequests;

	TArray<FStreamingRequest> StreamingRequests;
	int32 NumStreamingRequests = 0;
	int32 NextStreamingRequestIndex = 0;

	// Total size of the BLASes currently being streamed.
	// Used to keep track of how much TotalResidentSize will increase when the requests complete.
	int64 TotalStreamingSize = 0;

	// Feedback
	TSet<RayTracing::FGeometryHandle> VisibleGeometryHandles;

	bool bRenderedFrame = false;

	TSet<RayTracing::FGeometryHandle> ReleasedGeometries;
	TSet<RayTracing::FGeometryGroupHandle> ReleasedGroups;

	TSet<RayTracing::FGeometryHandle> ModifiedGeometries;
	TSet<RayTracing::FGeometryGroupHandle> ModifiedGroups;

	TPimplPtr<struct FReferenceBasedResidencyState> ReferenceBasedResidencyState;

	UE::Tasks::FTask ReferenceBasedResidencyTask = {};

	friend struct FReferenceBasedResidencyState;
};

#endif // RHI_RAYTRACING
