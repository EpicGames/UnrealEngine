// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Memory/MemoryFwd.h"
#include "CoreMinimal.h"

class FBulkDataBatchRequest;
class FIoBuffer;
class FRHICommandListBase;
class FRHIResourceReplaceBatcher;

enum EAsyncIOPriorityAndFlags : int32;

class FRayTracingStreamableAsset
{
public:
	virtual uint32 GetRequestSize() const = 0;
	virtual uint32 GetRequestSizeBVH() const = 0;
	virtual uint32 GetRequestSizeBuffers() const = 0;

	virtual bool AreBuffersStreamedIn() const = 0;
	virtual bool IsBVHStreamedIn() const = 0;

	virtual void IssueRequest(FBulkDataBatchRequest& Request, FIoBuffer& RequestBuffer, EAsyncIOPriorityAndFlags Priority, bool bBuffersOnly = false) = 0;
	virtual void InitWithStreamedData(FRHICommandListBase& RHICmdList, FMemoryView StreamedData, bool bBuffersOnly = false) = 0;
	virtual void ReleaseForStreaming(FRHIResourceReplaceBatcher& Batcher) = 0;
	virtual void ReleaseBVHForStreaming(FRHIResourceReplaceBatcher& Batcher) = 0;
	virtual void ReleaseBuffersForStreaming(FRHIResourceReplaceBatcher& Batcher) = 0;

	UE_DEPRECATED(5.8, "Please specify EAsyncIOPriorityAndFlags")
	virtual void IssueRequest(FBulkDataBatchRequest& Request, FIoBuffer& RequestBuffer, bool bBuffersOnly = false) = 0;
};
