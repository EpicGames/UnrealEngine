// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderingThread.h"

DECLARE_RENDER_COMMAND_PIPE(SkeletalMesh, ENGINE_API);