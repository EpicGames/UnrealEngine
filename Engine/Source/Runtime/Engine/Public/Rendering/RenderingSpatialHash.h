// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace RenderingSpatialHash
{

/**
 * Describes an object location in the rendering hierarchical spatial hash grid.
 * The location cosists of an integer 3D coordinate and a Level which is derived such that the size of the bounds are at most 1 (integer) unit at that level.
 * Put differently, the Level is calculated as the FloorLog2(Size).
 */
template <typename ScalarType>
struct TLocation
{
	using FIntVector3 = UE::Math::TIntVector3<ScalarType>;
	FIntVector3 Coord;
	int32 Level;

	inline TLocation() {};

	template <typename ScalarBType>
	inline explicit TLocation(const TLocation< ScalarBType>& InLoc)
		: Coord(UE::Math::TIntVector3<ScalarType>(InLoc.Coord))
		, Level(InLoc.Level)
	{
	}

	template <typename ScalarBType>
	inline explicit TLocation(const UE::Math::TIntVector3< ScalarBType>& InCoord, int32 InLevel)
		: Coord(UE::Math::TIntVector3<ScalarType>(InCoord))
		, Level(InLevel)
	{
	}

	inline friend uint32 GetTypeHash(const TLocation& CellLocation)
	{
		using UScalarType = std::make_unsigned_t<ScalarType>;

		return uint32((UScalarType)CellLocation.Coord.X * 1150168907 + (UScalarType)CellLocation.Coord.Y * 1235029793 + (UScalarType)CellLocation.Coord.Z * 1282581571 + (UScalarType)CellLocation.Level * 1264559321);
	}
	
	inline bool operator == (const TLocation& Other) const
	{
		return Level == Other.Level && Coord == Other.Coord;
	}

	inline bool operator!=(const TLocation& Other) const
	{
		return Level != Other.Level || Coord != Other.Coord;
	}

	inline TLocation operator+(const TLocation& RHS) const
	{
		checkSlow(Level == RHS.Level);
		TLocation Result;
		Result.Coord = Coord + RHS.Coord;
		Result.Level = Level;
		return Result;
	}


	inline TLocation operator-(const TLocation& RHS) const
	{
		checkSlow(Level == RHS.Level);
		TLocation Result;
		Result.Coord = Coord - RHS.Coord;
		Result.Level = Level;
		return Result;
	}

	friend inline FArchive& operator<<(FArchive& Ar, TLocation& Item)
	{
		Ar << Item.Coord;
		Ar << Item.Level;
		return Ar;
	}
};

using FLocation64 = TLocation<int64>;
using FLocation32 = TLocation<int32>;
using FLocation8 = TLocation<int8>;

inline int32 CalcLevel(double Size)
{
	// TODO: using integer Log2 breaks down for small scales, where the level would need to go negative. It is however far faster and not expected to be useful.
	return int32(FMath::FloorLog2(uint32(Size)));
};

inline int32 CalcLevel(float Size)
{
	// TODO: using integer Log2 breaks down for small scales, where the level would need to go negative. It is however far faster and not expected to be useful.
	return int32(FMath::FloorLog2(uint32(Size)));
};

inline int32 CalcLevelFromRadius(double Radius)
{
	return CalcLevel(Radius * 2.0);
};

inline double GetCellSize(int32 Level)
{
	checkSlow(Level >= 0);
	return double(1ull << uint32(Level + 1));
};

inline double GetRecCellSize(int32 Level)
{
	return 1.0 / GetCellSize(Level);
}

inline FLocation64 ToCellLoc(int32 Level, const FVector& WorldPos)
{
	FLocation64 Result;
	double RecLevelCellSize = GetRecCellSize(Level);
	Result.Level = Level;
	FVector LevelGridPos = WorldPos * RecLevelCellSize;
	Result.Coord = FLocation64::FIntVector3(FMath::FloorToInt(LevelGridPos.X), FMath::FloorToInt(LevelGridPos.Y), FMath::FloorToInt(LevelGridPos.Z));
	return Result;
};

inline FLocation64 CalcLevelAndLocation(const FBoxSphereBounds& BoxSphereBounds)
{
	// Can't be lower than this, or the footprint might be larger than 2x2x2, globally the same, can pre-calc.
	int32 Level = CalcLevelFromRadius(BoxSphereBounds.SphereRadius);
	return ToCellLoc(Level, BoxSphereBounds.Origin);
};

inline FLocation64 CalcLevelAndLocation(const FVector4d& Sphere)
{
	// Can't be lower than this, or the footprint might be larger than 2x2x2, globally the same, can pre-calc.
	int32 Level = CalcLevelFromRadius(Sphere.W);
	return ToCellLoc(Level, FVector(Sphere));
};

inline FLocation64 CalcLevelAndLocationClamped(const FVector3d& Center, double Radius, int32 FirstLevel)
{
	// Can't be lower than this, or the footprint might be larger than 2x2x2, globally the same, can pre-calc.
	int32 Level = CalcLevelFromRadius(Radius);
	Level = FMath::Max(Level, FirstLevel);
	return ToCellLoc(Level, Center);
};

inline FVector3d CalcWorldPosition(const FLocation64& Loc)
{
	return FVector3d(Loc.Coord) * GetCellSize(Loc.Level);
}

inline FVector3d CalcWorldCellCenter(const FLocation64& Loc)
{
	double LevelCellSize = GetCellSize(Loc.Level);
	FVector Center = FVector3d(Loc.Coord) * LevelCellSize + LevelCellSize * 0.5;
	return Center;
}

inline FLocation64 ToLevel(const FLocation64& Loc, int32 Level)
{
	int32 LevelDelta = Level - Loc.Level;
	FLocation64 ResLoc = Loc;
	ResLoc.Level = Level;
	if (LevelDelta > 0)
	{
		ResLoc.Coord = ResLoc.Coord >> LevelDelta;
	}
	else if (LevelDelta < 0)
	{
		ResLoc.Coord = ResLoc.Coord << -LevelDelta;
	}
	return ResLoc;
};

};
