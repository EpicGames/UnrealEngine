// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "GPUSkinVertexFactory.h"
#include "SkeletalMeshVertexBuffer.h"


class FSkeletalMeshAttributeVertexBuffer :
	public FRenderResource
{
public:
	virtual ~FSkeletalMeshAttributeVertexBuffer() override;
	
	void Init(
		const EPixelFormat InPixelFormat,
		const int32 InNumVertices,
		const int32 InComponentCount,
		TConstArrayView<float> InValues
		);

	FRHIShaderResourceView* GetSRV() const
	{
		return ValueBuffer.VertexBufferSRV;
	}

	int32 GetResourceSize() const;
	void CleanUp();

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
	virtual FString GetFriendlyName() const override { return TEXT("SkeletalMeshAttributeVertexBuffer"); }

	friend FArchive& operator<<(FArchive& Ar, FSkeletalMeshAttributeVertexBuffer& Data)
	{
		Data.Serialize(Ar);
		return Ar;
	}

	void Serialize(FArchive& Ar);

	struct FCPUData
	{
		FSkeletalMeshVertexDataInterface* Data = nullptr;
		int32 PixelFormat = PF_Unknown;
	};

	FCPUData GetCPUData() const
	{
		FCPUData Out;
		Out.Data = ValueData;
		Out.PixelFormat = PixelFormat;
		return Out;
	}
	
private:
	void Allocate();
	void SetData(TConstArrayView<float> InValues);
	
	FVertexBufferAndSRV ValueBuffer;
	FSkeletalMeshVertexDataInterface *ValueData = nullptr;

	int32 ComponentCount = 0;
	int32 PixelFormat = PF_Unknown;
	int32 ComponentStride = 0;
};
