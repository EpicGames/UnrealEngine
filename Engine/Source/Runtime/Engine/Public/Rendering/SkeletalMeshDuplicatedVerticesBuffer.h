// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GPUSkinVertexFactory.h"
#include "RenderResource.h"
#include "Rendering/SkeletalMeshVertexBuffer.h"

struct FIndexLengthPair
{
    uint32 Length;
    uint32 Index;

    /**
    * Serializer
    *
    * @param Ar - archive to serialize with
    * @param V - vertex to serialize
    * @return archive that was used
    */
    friend FArchive& operator<<(FArchive& Ar, FIndexLengthPair& V)
    {
        Ar << V.Length
            << V.Index;
        return Ar;
    }
};
template<> struct TCanBulkSerialize<FIndexLengthPair> { enum { Value = true }; };

class FDuplicatedVerticesBuffer : public FRenderResource
{
public:
    FVertexBufferAndSRV DuplicatedVerticesIndexBuffer;
    FVertexBufferAndSRV LengthAndIndexDuplicatedVerticesIndexBuffer;

    TSkeletalMeshVertexData<uint32> DupVertData;
    TSkeletalMeshVertexData<FIndexLengthPair> DupVertIndexData;

    bool bHasOverlappingVertices;

    FDuplicatedVerticesBuffer() 
#if WITH_EDITOR
	// Keep CPU copy in editor for geometry operations
	: DupVertData(true), DupVertIndexData(true)
#else
	: DupVertData(false), DupVertIndexData(false)
#endif
	, bHasOverlappingVertices(false) {}
	
    /** Destructor. */
	virtual ~FDuplicatedVerticesBuffer() = default;

    void Init(const int32 NumVertices, const TMap<int32, TArray<int32>>& OverlappingVertices)
    {
        int32 NumDups = 0;
        for (auto Iter = OverlappingVertices.CreateConstIterator(); Iter; ++Iter)
        {
            const TArray<int32>& Array = Iter.Value();
            NumDups += Array.Num();
        }

        DupVertData.ResizeBuffer(NumDups ? NumDups : 1);
        DupVertIndexData.ResizeBuffer(NumVertices ? NumVertices : 1);

        uint8* VertData = DupVertData.GetDataPointer();
        uint32 VertStride = DupVertData.GetStride();
        check(VertStride == sizeof(uint32));

        uint8* IndexData = DupVertIndexData.GetDataPointer();
        uint32 IndexStride = DupVertIndexData.GetStride();
        check(IndexStride == sizeof(FIndexLengthPair));

        if (NumDups)
        {
            int32 SubIndex = 0;
            for (int32 Index = 0; Index < NumVertices; ++Index)
            {
                const TArray<int32>* Array = OverlappingVertices.Find(Index);
                FIndexLengthPair NewEntry;
                NewEntry.Length = (Array) ? Array->Num() : 0;
                NewEntry.Index = SubIndex;
                *((FIndexLengthPair*)(IndexData + Index * IndexStride)) = NewEntry;
                if (Array)
                {
                    for (const int32 OverlappingVert : *Array)
                    {
                        *((uint32*)(VertData + SubIndex * VertStride)) = OverlappingVert;
                        SubIndex++;
                    }
                }
            }
            check(SubIndex == NumDups);
            bHasOverlappingVertices = true;
        }
        else
        {
            FMemory::Memzero(IndexData, NumVertices * sizeof(FIndexLengthPair));
            FMemory::Memzero(VertData, sizeof(uint32));
        }
    }

	void ReleaseCPUResources()
	{
		DupVertData.Discard();
		DupVertIndexData.Discard();
	}

    virtual void InitRHI(FRHICommandListBase& RHICmdList) override
    {
		const static FLazyName DuplicatedVerticesBuffeName(TEXT("FDuplicatedVerticesBuffer"));
        {
            FResourceArrayInterface* ResourceArray = DupVertData.GetResourceArray();
            check(ResourceArray->GetResourceDataSize() > 0);

			const FRHIBufferCreateDesc Desc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("DuplicatedVerticesIndexBuffer"), ResourceArray->GetResourceDataSize())
				.AddUsage(BUF_Static | BUF_ShaderResource)
				.SetClassName(DuplicatedVerticesBuffeName)
				.SetOwnerName(GetOwnerName())
				.SetInitActionResourceArray(ResourceArray)
				.DetermineInitialState();

			DuplicatedVerticesIndexBuffer.VertexBufferRHI = RHICmdList.CreateBuffer(Desc);
            DuplicatedVerticesIndexBuffer.VertexBufferSRV = RHICmdList.CreateShaderResourceView(
				DuplicatedVerticesIndexBuffer.VertexBufferRHI,
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
        }

        {
            FResourceArrayInterface* ResourceArray = DupVertIndexData.GetResourceArray();
            check(ResourceArray->GetResourceDataSize() > 0);

			const FRHIBufferCreateDesc Desc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("LengthAndIndexDuplicatedVerticesIndexBuffer"), ResourceArray->GetResourceDataSize())
				.AddUsage(BUF_Static | BUF_ShaderResource)
				.SetClassName(DuplicatedVerticesBuffeName)
				.SetOwnerName(GetOwnerName())
				.SetInitActionResourceArray(ResourceArray)
				.DetermineInitialState();

            LengthAndIndexDuplicatedVerticesIndexBuffer.VertexBufferRHI = RHICmdList.CreateBuffer(Desc);
            LengthAndIndexDuplicatedVerticesIndexBuffer.VertexBufferSRV = RHICmdList.CreateShaderResourceView(
				LengthAndIndexDuplicatedVerticesIndexBuffer.VertexBufferRHI,
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
        }
    }

	// FRenderResource interface.
	virtual void ReleaseRHI() override
	{
		DuplicatedVerticesIndexBuffer.SafeRelease();
		LengthAndIndexDuplicatedVerticesIndexBuffer.SafeRelease();
	}
	virtual FString GetFriendlyName() const override { return TEXT("FDuplicatedVerticesBuffer"); }

	friend FArchive& operator<<(FArchive& Ar, FDuplicatedVerticesBuffer& Data)
	{
		Ar << Data.DupVertData;
		Ar << Data.DupVertIndexData;

		return Ar;
	}
};
