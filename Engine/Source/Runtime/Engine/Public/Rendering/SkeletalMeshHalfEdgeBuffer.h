// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"
#include "Containers/DynamicRHIResourceArray.h"

class FSkeletalMeshLODRenderData;
class FRHIShaderResourceView;
class FRHIResourceReplaceBatcher;

namespace SkeletalMeshHalfEdgeUtility
{
	void ENGINE_API BuildHalfEdgeBuffers(const FSkeletalMeshLODRenderData& InLodRenderData, TResourceArray<int32>& OutVertexToEdge, TResourceArray<int32>& OutEdgeToTwinEdge);
}

/*
 * Render resource containing the half edge buffers. 
 */ 
class FSkeletalMeshHalfEdgeBuffer : public FRenderResource
{
public:
	void ENGINE_API Init(const FSkeletalMeshLODRenderData& InLodRenderData);

	struct FRHIInfo
	{
		FBufferRHIRef VertexToEdgeBufferRHI;
		FBufferRHIRef EdgeToTwinEdgeBufferRHI;
	};

	FRHIInfo CreateRHIBuffer(FRHICommandListBase& RHICmdList);

	void InitRHIForStreaming(FRHIInfo RHIInfo, FRHIResourceReplaceBatcher& Batcher);
	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);
	
	void ENGINE_API InitRHI(FRHICommandListBase& RHICmdList) override;

	void ENGINE_API ReleaseRHI() override;

	bool ENGINE_API IsCPUDataValid() const;

	bool ENGINE_API IsReadyForRendering() const;
	
	void ENGINE_API CleanUp();

	int32 GetResourceSize() const;
	
	void Serialize(FArchive& Ar);
	
	friend class FSkeletalMeshLODRenderData;
	friend FArchive& operator<<(FArchive& Ar, FSkeletalMeshHalfEdgeBuffer& MorphTargetVertexInfoBuffers);

	FRHIShaderResourceView* GetVertexToEdgeBufferSRV() const
	{
		return VertexToEdgeBufferSRV;
	}

	FRHIShaderResourceView* GetEdgeToTwinEdgeBufferSRV() const
	{
		return EdgeToTwinEdgeBufferSRV;
	}

	ENGINE_API FSkeletalMeshHalfEdgeBuffer();
	ENGINE_API ~FSkeletalMeshHalfEdgeBuffer();

private:
	uint32 GetMinBufferSize() const;
	
	TResourceArray<int32> VertexToEdgeData;
	TResourceArray<int32> EdgeToTwinEdgeData;
	
	FBufferRHIRef VertexToEdgeBufferRHI;
	FShaderResourceViewRHIRef VertexToEdgeBufferSRV;		
	FBufferRHIRef EdgeToTwinEdgeBufferRHI;
	FShaderResourceViewRHIRef EdgeToTwinEdgeBufferSRV;	
};