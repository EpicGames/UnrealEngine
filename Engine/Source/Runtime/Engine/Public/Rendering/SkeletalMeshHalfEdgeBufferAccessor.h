// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "SkeletalMeshHalfEdgeBufferAccessor.generated.h"

namespace SkeletalMeshHalfEdgeBufferAccessor
{
	// Asset implementing this interface should add this tag to the asset tags
	// if half edge buffer is required
	ENGINE_API	FName GetHalfEdgeRequirementAssetTagName();
	bool IsHalfEdgeRequired(TSoftObjectPtr<UObject> InAssetSoftPtr);	
}

UINTERFACE()
class USkeletalMeshHalfEdgeBufferAccessor : public UInterface
{
	GENERATED_BODY()
};

class ISkeletalMeshHalfEdgeBufferAccessor
{
	GENERATED_BODY()

public:
	
	virtual bool IsSkeletalMeshHalfEdgeBufferRequired() const = 0;
};


