// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MeshDescription.h"

#if WITH_EDITOR

#include "Engine/EngineTypes.h"
#include "BoneIndices.h"
#include "Serialization/BulkData.h"
#include "Components.h"
#include "Math/GenericOctree.h"
#include "Animation/MorphTarget.h"
#include "Templates/DontCopy.h"

struct FImportedSkinWeightProfileData;
struct FMeshDescription;
class FSkeletalMeshLODModel;


enum class ESkeletalMeshGeoImportVersions : uint8
{
	Before_Versionning = 0,
	SkeletalMeshBuildRefactor,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

enum class ESkeletalMeshSkinningImportVersions : uint8
{
	Before_Versionning = 0,
	SkeletalMeshBuildRefactor,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};


namespace SkeletalMeshImportData
{
	/**
	* Some information per individual mesh, as appearing in the source asset.
	* This could store the data for say meshes "UpperBody", "Legs", "Hat", etc.
	*/
	struct FMeshInfo
	{
		FName Name;	// The name of the mesh.
		int32 NumVertices = 0;	// The number of imported (dcc) vertices that are part of this mesh. This is a value of 8 for a cube. So NOT the number of render vertices.
		int32 StartImportedVertex = 0;	// The first index of imported (dcc) vertices in the mesh. So this NOT an index into the render vertex buffer. In range of 0..7 for a cube.
	};

	struct FMeshWedge
	{
		uint32			iVertex;			// Vertex index.
		FVector2f		UVs[MAX_TEXCOORDS];	// UVs.
		FColor			Color;			// Vertex color.
		friend FArchive &operator<<(FArchive& Ar, FMeshWedge& T)
		{
			Ar << T.iVertex;
			for (int32 UVIdx = 0; UVIdx < MAX_TEXCOORDS; ++UVIdx)
			{
				Ar << T.UVs[UVIdx];
			}
			Ar << T.Color;
			return Ar;
		}
	};

	struct FMeshFace
	{
		// Textured Vertex indices.
		uint32		iWedge[3];
		// Source Material (= texture plus unique flags) index.
		uint16		MeshMaterialIndex;

		FVector3f	TangentX[3];
		FVector3f	TangentY[3];
		FVector3f	TangentZ[3];

		// 32-bit flag for smoothing groups.
		uint32   SmoothingGroups;
	};

	// A bone: an orientation, and a position, all relative to their parent.
	struct FJointPos
	{
		::FTransform3f	Transform;	// LWC_TODO: UE::Geometry namespace issues

		// For collision testing / debug drawing...
		float       Length;
		float       XSize;
		float       YSize;
		float       ZSize;

		friend FArchive &operator<<(FArchive& Ar, FJointPos& F)
		{
			Ar << F.Transform;
			return Ar;
		}
	};

	// Textured triangle.
	struct FTriangle
	{
		// Point to three vertices in the vertex list.
		uint32   WedgeIndex[3];
		// Materials can be anything.
		uint16    MatIndex;
		// Second material from exporter (unused)
		uint8    AuxMatIndex;
		// 32-bit flag for smoothing groups.
		uint32   SmoothingGroups;

		FVector3f	TangentX[3];
		FVector3f	TangentY[3];
		FVector3f	TangentZ[3];


		friend FArchive &operator<<(FArchive& Ar, FTriangle& F)
		{
			Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

			if (Ar.IsLoading() && Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::SkeletalMeshSourceDataSupport16bitOfMaterialNumber)
			{
				uint8 TempMatIndex = 0;
				Ar << TempMatIndex;
				F.MatIndex = TempMatIndex;
			}
			else
			{
				Ar << F.MatIndex;
			}
			Ar << F.AuxMatIndex;
			Ar << F.SmoothingGroups;
			
			Ar << F.WedgeIndex[0];
			Ar << F.WedgeIndex[1];
			Ar << F.WedgeIndex[2];

			Ar << F.TangentX[0];
			Ar << F.TangentX[1];
			Ar << F.TangentX[2];

			Ar << F.TangentY[0];
			Ar << F.TangentY[1];
			Ar << F.TangentY[2];

			Ar << F.TangentZ[0];
			Ar << F.TangentZ[1];
			Ar << F.TangentZ[2];
			return Ar;
		}
	};

	struct FVertInfluence
	{
		float Weight;
		uint32 VertIndex;
		FBoneIndexType BoneIndex;
		friend FArchive &operator<<(FArchive& Ar, FVertInfluence& F)
		{
			Ar << F.Weight << F.VertIndex << F.BoneIndex;
			return Ar;
		}
	};

	// Raw data material.
	struct FMaterial
	{
		/** The actual material created on import or found among existing materials, this member is not serialize, importer can found back the material */
		TWeakObjectPtr<UMaterialInterface> Material;
		/** The material name found by the importer */
		FString MaterialImportName;

		friend FArchive &operator<<(FArchive& Ar, FMaterial& F)
		{
			Ar << F.MaterialImportName;
			return Ar;
		}
	};


	// Raw data bone.
	struct FBone
	{
		FString		Name;     //
							  //@ todo FBX - Flags unused?
		uint32		Flags;        // reserved / 0x02 = bone where skin is to be attached...	
		int32 		NumChildren;  // children  // only needed in animation ?
		int32       ParentIndex;  // 0/NULL if this is the root bone.  
		FJointPos	BonePos;      // reference position

		friend FArchive &operator<<(FArchive& Ar, FBone& F)
		{
			Ar << F.Name;
			Ar << F.Flags;
			Ar << F.NumChildren;
			Ar << F.ParentIndex;
			Ar << F.BonePos;
			return Ar;
		}
	};


	// Raw data bone influence.
	struct FRawBoneInfluence // just weight, vertex, and Bone, sorted later....
	{
		float Weight;
		int32   VertexIndex;
		int32   BoneIndex;

		friend FArchive &operator<<(FArchive& Ar, FRawBoneInfluence& F)
		{
			Ar << F.Weight;
			Ar << F.VertexIndex;
			Ar << F.BoneIndex;
			return Ar;
		}
	};

	// Vertex with texturing info, akin to Hoppe's 'Wedge' concept - import only.
	struct FVertex
	{
		uint32	VertexIndex; // Index to a vertex.
		FVector2f UVs[MAX_TEXCOORDS];        // Scaled to BYTES, rather...-> Done in digestion phase, on-disk size doesn't matter here.
		FColor	Color;		 // Vertex colors
		uint8    MatIndex;    // At runtime, this one will be implied by the face that's pointing to us.
		uint8    Reserved;    // Top secret.

		FVertex()
		{
			FMemory::Memzero(this, sizeof(FVertex));
		}

		bool operator==(const FVertex& Other) const
		{
			bool Equal = true;

			Equal &= (VertexIndex == Other.VertexIndex);
			Equal &= (MatIndex == Other.MatIndex);
			Equal &= (Color == Other.Color);
			Equal &= (Reserved == Other.Reserved);

			bool bUVsEqual = true;
			for (uint32 UVIdx = 0; UVIdx < MAX_TEXCOORDS; ++UVIdx)
			{
				if (!UVs[UVIdx].Equals(Other.UVs[UVIdx], UE_SMALL_NUMBER))
				{
					bUVsEqual = false;
					break;
				}
			}

			Equal &= bUVsEqual;

			return Equal;
		}

		friend uint32 GetTypeHash(const FVertex& Vertex)
		{
			return FCrc::MemCrc_DEPRECATED(&Vertex, sizeof(FVertex));
		}

		friend FArchive &operator<<(FArchive& Ar, FVertex& F)
		{
			Ar << F.VertexIndex;
			Ar << F.Color;
			Ar << F.MatIndex;
			Ar << F.Reserved;

			for (uint32 UVIdx = 0; UVIdx < MAX_TEXCOORDS; ++UVIdx)
			{
				Ar << F.UVs[UVIdx];
			}

			return Ar;
		}
	};


	// Points: regular FVectors (for now..)
	struct FPoint
	{
		FVector3f	Point; // Change into packed integer later IF necessary, for 3x size reduction...
		
		friend FArchive &operator<<(FArchive& Ar, FPoint& F)
		{
			Ar << F.Point;
			return Ar;
		}
	};

	struct FVertexAttribute
	{
		FVertexAttribute() = default;
		FVertexAttribute(TArray<float>&& InAttributeValues, int32 InComponentCount) :
			AttributeValues(InAttributeValues), ComponentCount(InComponentCount)
		{}
		FVertexAttribute(const FVertexAttribute&) = default;
		FVertexAttribute(FVertexAttribute&&) = default;
		
		TArray<float> AttributeValues;
		int32 ComponentCount;
		
		friend FArchive &operator<<(FArchive& Ar, FVertexAttribute& A)
		{
			Ar << A.AttributeValues;
			Ar << A.ComponentCount;

			return Ar;
		}
	};
}

template <> struct TIsPODType<SkeletalMeshImportData::FMeshWedge> { enum { Value = true }; };
template <> struct TIsPODType<SkeletalMeshImportData::FMeshFace> { enum { Value = true }; };
template <> struct TIsPODType<SkeletalMeshImportData::FJointPos> { enum { Value = true }; };
template <> struct TIsPODType<SkeletalMeshImportData::FTriangle> { enum { Value = true }; };
template <> struct TIsPODType<SkeletalMeshImportData::FVertInfluence> { enum { Value = true }; };

/**
* Container and importer for skeletal mesh (FBX file) data
**/
class FSkeletalMeshImportData
{
public:
	TArray <SkeletalMeshImportData::FMaterial> Materials;
	TArray <FVector3f> Points;
	TArray <SkeletalMeshImportData::FVertex> Wedges;
	TArray <SkeletalMeshImportData::FTriangle> Faces;
	TArray <SkeletalMeshImportData::FBone> RefBonesBinary;
	TArray <SkeletalMeshImportData::FRawBoneInfluence> Influences;
	TArray <SkeletalMeshImportData::FMeshInfo> MeshInfos;
	TArray <int32> PointToRawMap;	// Mapping from current point index to the original import point index
	uint32 NumTexCoords; // The number of texture coordinate sets
	uint32 MaxMaterialIndex; // The max material index found on a triangle
	bool bHasVertexColors; // If true there are vertex colors in the imported file
	bool bHasNormals; // If true there are normals in the imported file
	bool bHasTangents; // If true there are tangents in the imported file

	// Morph targets imported(i.e. FBX) data. The name is the morph target name
	TArray<FSkeletalMeshImportData> MorphTargets;
	TArray<TSet<uint32>> MorphTargetModifiedPoints;
	TArray<FString> MorphTargetNames;
	
	// Alternate influence imported(i.e. FBX) data. The name is the alternate skinning profile name
	TArray<FSkeletalMeshImportData> AlternateInfluences;
	TArray<FString> AlternateInfluenceProfileNames;
	
	TArray<SkeletalMeshImportData::FVertexAttribute> VertexAttributes;
	TArray<FString> VertexAttributeNames;

	//////////////////////////////////////////////////////////////////////////

	FSkeletalMeshImportData()
		: NumTexCoords(0)
		, MaxMaterialIndex(0)
		, bHasVertexColors(false)
		, bHasNormals(false)
		, bHasTangents(false)
	{

	}

	/*
	 * Copy only unnecessary array data from the structure to build the morph target (this will save a lot of memory)
	 */
	ENGINE_API void CopyDataNeedByMorphTargetImport(FSkeletalMeshImportData& Other) const;

	/*
	 * Remove all unnecessary array data from the structure (this will save a lot of memory)
	 * We only need Points, Influences and RefBonesBinary arrays
	 */
	ENGINE_API void KeepAlternateSkinningBuildDataOnly();

	/**
	* Copy mesh data for importing a single LOD
	*
	* @param LODPoints - vertex data.
	* @param LODWedges - wedge information to static LOD level.
	* @param LODFaces - triangle/ face data to static LOD level.
	* @param LODInfluences - weights/ influences to static LOD level.
	*/
	ENGINE_API void CopyLODImportData(
		TArray<FVector3f>& LODPoints,
		TArray<SkeletalMeshImportData::FMeshWedge>& LODWedges,
		TArray<SkeletalMeshImportData::FMeshFace>& LODFaces,
		TArray<SkeletalMeshImportData::FVertInfluence>& LODInfluences,
		TArray<int32>& LODPointToRawMap) const;

	static ENGINE_API FString FixupBoneName(FString InBoneName);

	/**
	* Removes all import data
	*/
	void Empty()
	{
		Materials.Empty();
		Points.Empty();
		Wedges.Empty();
		Faces.Empty();
		RefBonesBinary.Empty();
		Influences.Empty();
		PointToRawMap.Empty();
		MeshInfos.Empty();
	}

	static ENGINE_API bool ReplaceSkeletalMeshGeometryImportData(const USkeletalMesh* SkeletalMesh, FSkeletalMeshImportData* ImportData, int32 LodIndex);
	static ENGINE_API bool ReplaceSkeletalMeshRigImportData(const USkeletalMesh* SkeletalMesh, FSkeletalMeshImportData* ImportData, int32 LodIndex);

	//Fit another rig data on this one
	ENGINE_API bool ApplyRigToGeo(FSkeletalMeshImportData& Other);

	/*
	 * Use the faces corner normals to create the face smooth groups data
	 */
	ENGINE_API void ComputeSmoothGroupFromNormals();

	/*
	 * Add morph target data from UMorphTarget in case there was none on the mesh itself.
	 */
	ENGINE_API void AddMorphTarget(FName InMorphTargetName, const FMorphTargetLODModel& InMorphTargetModel, const TArray<uint32>& InVertexMap);

	/*
	 * Add alternate skin profile from FImportedSkinWeightProfileData
	 */
	ENGINE_API void AddSkinWeightProfile(FName InProfileName, const FImportedSkinWeightProfileData& InProfileData, const TArray<int32>& InVertexMap, const TArray<FBoneIndexType>& InBoneIndexMap);
	
	
	/**
	 * Returns a mesh description from the import data. If logging on failures is required, pass in a pointer 
	 * to the owning skeletal mesh. Otherwise, leave as a \c nullptr. 
	 */
	ENGINE_API bool GetMeshDescription(const USkeletalMesh* InSkeletalMesh, const FSkeletalMeshBuildSettings* InBuildSettings, FMeshDescription& OutMeshDescription) const;

	/**
	 * @note MeshDescription always contains color, normal and tangent data by default. Therefore, while iterating
	 * over the vertices, we check if at least one normal/tangent vector is not a zero vector and the vertex color is
	 * not pure white, then we set the corresponding bHasNormals/bHasTangent/bHasVertexColors flags to true.
	 */
	static ENGINE_API FSkeletalMeshImportData CreateFromMeshDescription(const FMeshDescription& InMeshDescription);

private:
	void CopySkinWeightsToMeshDescription(
		const USkeletalMesh* InSkeletalMesh,
		const FName InSkinWeightName,
		const FSkeletalMeshImportData& InSkinWeightMesh,
		const TArray<FVertexID>& InVertexIDMap,
		FMeshDescription& OutMeshDescription
		) const;

	void CleanUpUnusedMaterials();
	void SplitVerticesBySmoothingGroups();
	
	friend FArchive& operator<<(FArchive& Ar, FSkeletalMeshImportData& RawMesh);
};

/**
* Bulk data storage for raw ImportModel. This structure is deprecated, we now only store the original vertex and triangle count, see FInlineReductionCacheData.
*/
struct FReductionBaseSkeletalMeshBulkData
{
	/** Internally store bulk data as bytes. */
	FByteBulkData BulkData;

	//The custom version when this was load
	FCustomVersionContainer SerializeLoadingCustomVersionContainer;
	FPackageFileVersion UEVersion;
	int32 LicenseeUEVersion = 0;
	bool bUseSerializeLoadingCustomVersion = false;

	uint32 CacheLODVertexNumber = MAX_uint32;
	uint32 CacheLODTriNumber = MAX_uint32;

	/*
	 * Caching those value since this is a slow operation to load the bulk data to retrieve the original geometry information
	 */
	void CacheGeometryInfo(const FSkeletalMeshLODModel& SourceLODModel);

public:
	/** Default constructor. */
	ENGINE_API FReductionBaseSkeletalMeshBulkData();

	/*Static function helper to serialize an array of reduction data. */
	ENGINE_API static void Serialize(FArchive& Ar, TArray<FReductionBaseSkeletalMeshBulkData*>& ReductionBaseSkeletalMeshDatas, UObject* Owner);

	/** Serialization. */
	ENGINE_API void Serialize(class FArchive& Ar, class UObject* Owner);

	/** Store a new raw mesh in the bulk data. */
	ENGINE_API void SaveReductionData(FSkeletalMeshLODModel& BaseLODModel, TMap<FString, TArray<FMorphTargetDelta>>& BaseLODMorphTargetData, UObject* Owner);

	/** Load the raw mesh from bulk data. */
	ENGINE_API void LoadReductionData(FSkeletalMeshLODModel& BaseLODModel, TMap<FString, TArray<FMorphTargetDelta>>& BaseLODMorphTargetData, UObject* Owner);

	/** Return the number of vertices and triangles store in this bulk data. */
	ENGINE_API void GetGeometryInfo(uint32& LODVertexNumber, uint32& LODTriNumber, UObject* Owner);

	FByteBulkData& GetBulkData() { return BulkData; }

	/** Empty the bulk data. */
	void EmptyBulkData() { BulkData.RemoveBulkData(); }

	/** Returns true if no bulk data is available for this mesh. */
	inline bool IsEmpty() const { return BulkData.GetBulkDataSize() == 0; }
};

struct FInlineReductionCacheData
{
	uint32 CacheLODVertexCount = MAX_uint32;
	uint32 CacheLODTriCount = MAX_uint32;

	/*
	 * Caching those value since this is a slow operation to load the bulk data to retrieve the original geometry information
	 */
	ENGINE_API void SetCacheGeometryInfo(const FSkeletalMeshLODModel& SourceLODModel);

	ENGINE_API void SetCacheGeometryInfo(uint32 LODVertexCount, uint32 LODTriCount);

	/** Return the cache count of vertices and triangles. */
	ENGINE_API void GetCacheGeometryInfo(uint32& LODVertexCount, uint32& LODTriCount) const;
};

inline FArchive& operator<<(FArchive& Ar, FInlineReductionCacheData& InlineReductionCacheData)
{
	Ar << InlineReductionCacheData.CacheLODVertexCount;
	Ar << InlineReductionCacheData.CacheLODTriCount;
	return Ar;
}

/**
* Bulk data storage for raw meshes.
*/
class FRawSkeletalMeshBulkData
{
#if WITH_EDITOR
	/** Protects simultaneous access to BulkData */
	TDontCopy<FRWLock> BulkDataLock;
#endif
	/** Internally store bulk data as bytes. */
	FByteBulkData BulkData;
	/** GUID associated with the data stored herein. */
	FGuid Guid;
	/** If true, the GUID is actually a hash of the contents. */
	bool bGuidIsHash;

	//The custom version when this was load
	FCustomVersionContainer SerializeLoadingCustomVersionContainer;
	FPackageFileVersion UEVersion;
	int32 LicenseeUEVersion = 0;
	bool bUseSerializeLoadingCustomVersion = false;

public:
	/*
	 * The last geo imported version, we use this flag to know if we have some data or not.
	 * This flag must be updated every time we import a new geometry
	 */
	ESkeletalMeshGeoImportVersions GeoImportVersion;

	/*
	 * The last skinning imported version, we use this flag to know if we have some data or not.
	 * This flag must be updated every time we import the skinning
	 */
	ESkeletalMeshSkinningImportVersions SkinningImportVersion;

	/** Default constructor. */
	ENGINE_API FRawSkeletalMeshBulkData();

	/*Static function helper to serialize an array of Raw source data. */
	ENGINE_API static void Serialize(FArchive& Ar, TArray<TSharedRef<FRawSkeletalMeshBulkData>>& RawSkeltalMeshBulkDatas, UObject* Owner);
	
	/** Serialization. */
	ENGINE_API void Serialize(class FArchive& Ar, class UObject* Owner);

	/** Store a new raw mesh in the bulk data. */
	ENGINE_API void SaveRawMesh(FSkeletalMeshImportData& InMesh);

	/** Load the raw mesh from bulk data. */
	ENGINE_API void LoadRawMesh(FSkeletalMeshImportData& OutMesh);

	/** Retrieve a string uniquely identifying the contents of this bulk data. */
	ENGINE_API FString GetIdString() const;

	/** Uses a hash as the GUID, useful to prevent creating new GUIDs on load for legacy assets. */
	ENGINE_API void UseHashAsGuid(class UObject* Owner);

	ENGINE_API FByteBulkData& GetBulkData();
	
	ENGINE_API const FByteBulkData& GetBulkData() const;
	
	/** Empty the bulk data. */
	void EmptyBulkData()
	{
		//Clear all the data
		BulkData.RemoveBulkData();
		Guid.Invalidate();
		bGuidIsHash = false;
		SerializeLoadingCustomVersionContainer.Empty();
		bUseSerializeLoadingCustomVersion = false;
		GeoImportVersion = ESkeletalMeshGeoImportVersions::Before_Versionning;
		SkinningImportVersion = ESkeletalMeshSkinningImportVersions::Before_Versionning;
	}

	/** Returns true if no bulk data is available for this mesh. */
	inline bool IsEmpty() const { return BulkData.GetBulkDataSize() == 0; }

	/** Returns true if the last import version is enough to use the new build system. WE cannot rebuild asset if we did not previously store the data*/
	bool IsBuildDataAvailable() const
	{
		return GeoImportVersion >= ESkeletalMeshGeoImportVersions::SkeletalMeshBuildRefactor &&
			SkinningImportVersion >= ESkeletalMeshSkinningImportVersions::SkeletalMeshBuildRefactor;
	}

private:
	ENGINE_API void UpdateRawMeshFormat();
};

namespace FWedgePositionHelper
{
	inline bool PointsEqual(const FVector3f& V1, const FVector3f& V2, float ComparisonThreshold)
	{
		if (FMath::Abs(V1.X - V2.X) > ComparisonThreshold
			|| FMath::Abs(V1.Y - V2.Y) > ComparisonThreshold
			|| FMath::Abs(V1.Z - V2.Z) > ComparisonThreshold)
		{
			return false;
		}
		return true;
	}

	/** Helper struct for building acceleration structures. */
	struct FIndexAndZ
	{
		float Z;
		int32 Index;

		/** Default constructor. */
		FIndexAndZ() {}

		/** Initialization constructor. */
		FIndexAndZ(int32 InIndex, FVector3f V)
		{
			Z = 0.30f * V.X + 0.33f * V.Y + 0.37f * V.Z;
			Index = InIndex;
		}
	};

	/** Sorting function for vertex Z/index pairs. */
	struct FCompareIndexAndZ
	{
		inline bool operator()(FIndexAndZ const& A, FIndexAndZ const& B) const { return A.Z < B.Z; }
	};
}

struct FWedgeInfo
{
	FVector Position;
	int32 WedgeIndex;
};

/** Helper struct for the mesh component vert position octree */
struct FWedgeInfoOctreeSemantics
{
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	/**
	* Get the bounding box of the provided octree element. In this case, the box
	* is merely the point specified by the element.
	*
	* @param	Element	Octree element to get the bounding box for
	*
	* @return	Bounding box of the provided octree element
	*/
	inline static FBoxCenterAndExtent GetBoundingBox(const FWedgeInfo& Element)
	{
		return FBoxCenterAndExtent(Element.Position, FVector::ZeroVector);
	}

	/**
	* Determine if two octree elements are equal
	*
	* @param	A	First octree element to check
	* @param	B	Second octree element to check
	*
	* @return	true if both octree elements are equal, false if they are not
	*/
	inline static bool AreElementsEqual(const FWedgeInfo& A, const FWedgeInfo& B)
	{
		return (A.Position == B.Position && A.WedgeIndex == B.WedgeIndex);
	}

	/** Ignored for this implementation */
	inline static void SetElementId(const FWedgeInfo& Element, FOctreeElementId2 Id)
	{
	}
};
typedef TOctree2<FWedgeInfo, FWedgeInfoOctreeSemantics> TWedgeInfoPosOctree;

class FOctreeQueryHelper
{
public:
	FOctreeQueryHelper(const TWedgeInfoPosOctree *InWedgePosOctree)
		: WedgePosOctree(InWedgePosOctree)
	{}
	/*
	* Find the nearest wedge indexes to SearchPosition.
	*
	* SearchPosition: The reference vertex position use to search the wedges
	* OutNearestWedges: The nearest wedge indexes to SearchPosition
	*/
	ENGINE_API void FindNearestWedgeIndexes(const FVector3f& SearchPosition, TArray<FWedgeInfo>& OutNearestWedges);
private:
	const TWedgeInfoPosOctree *WedgePosOctree;
};

struct FWedgePosition
{
	FWedgePosition()
	{
		WedgePosOctree = nullptr;
	}

	~FWedgePosition()
	{
		if (WedgePosOctree != nullptr)
		{
			delete WedgePosOctree;
			WedgePosOctree = nullptr;
		}
	}

	const TWedgeInfoPosOctree *GetOctree() const
	{
		return WedgePosOctree;
	}

	/*
	 * Find all wedges index that match exactly the vertex position. OutResult will be empty if there is no match
	 * 
	 * Position: The reference vertex position use to search the wedges
	 * RefPoints: The array of position that was use when calling FillWedgePosition
	 * RefWedges: The array of wedges that was use when calling FillWedgePosition
	 * OutResults: The wedge indexes that fit the Position parameter
	 * ComparisonThreshold: The threshold use to exactly match the Position. Not use when bExactMatch is false
	 */
	void FindMatchingPositionWegdeIndexes(const FVector3f &Position, float ComparisonThreshold, TArray<int32>& OutResults);

	// Fill the data:
	// Create the SortedPosition use to find exact match (position)
	// Create the wedge position octree to find the closest position, we use this when there is no exact match
	// The targetPositions is use to to max out the octree bounding box to both the source and target geometry
	static void FillWedgePosition(
		FWedgePosition& OutOverlappingPosition,
		const TArray<FVector3f>& Positions,
		const TArray<SkeletalMeshImportData::FVertex> Wedges,
		const TArray<FVector3f>& TargetPositions,
		float ComparisonThreshold);

private:
	TArray<FWedgePositionHelper::FIndexAndZ> SortedPositions;
	TWedgeInfoPosOctree *WedgePosOctree;
	TArray<FVector3f> Points;
	TArray<SkeletalMeshImportData::FVertex> Wedges;
};



#endif // WITH_EDITOR
