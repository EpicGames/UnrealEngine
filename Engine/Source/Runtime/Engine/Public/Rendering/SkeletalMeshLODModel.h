// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Animation/SkeletalMeshVertexAttribute.h"
#include "Rendering/SkinWeightProfile.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "BoneIndices.h"
#include "Components.h"
#include "GPUSkinPublicDefs.h"
#include "HAL/CriticalSection.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Serialization/BulkData.h"
#include "SkeletalMeshTypes.h"

/**
* A set of skeletal mesh triangles which use the same material
*/
struct FSkelMeshSection
{
	/** Material (texture) used for this section. */
	uint16 MaterialIndex;

	/** The offset of this section's indices in the LOD's index buffer. */
	uint32 BaseIndex;

	/** The number of triangles in this section. */
	uint32 NumTriangles;

	/** Is this mesh selected? */
	uint8 bSelected : 1;

	/** This section will recompute tangent in runtime */
	bool bRecomputeTangent;

	/** Vertex color channel to mask recompute tangents. R=0,G=1,B=2,A=None=3 */
	ESkinVertexColorChannel RecomputeTangentsVertexMaskChannel;

	/** This section will cast shadow */
	bool bCastShadow;
	
	/** If true, this section will be visible in ray tracing effects. Turning this off will remove it from ray traced reflections, shadows, etc. */
	bool bVisibleInRayTracing;

	/** This is set for old 'duplicate' sections that need to get removed on load */
	bool bLegacyClothingSection_DEPRECATED;

	/** Corresponding Section Index will be enabled when this section is disabled
	because corresponding cloth section will be shown instead of this
	or disabled section index when this section is enabled for cloth simulation
	*/
	int16 CorrespondClothSectionIndex_DEPRECATED;

	/** The offset into the LOD's vertex buffer of this section's vertices. */
	uint32 BaseVertexIndex;

	/** The soft vertices of this section. */
	TArray<FSoftSkinVertex> SoftVertices;

	/**
	 * The cloth deformer mapping data to each required cloth LOD.
	 * Raytracing may require a different deformer LOD to the one being simulated/rendered.
	 * The outer array indexes the LOD bias to this LOD. The inner array indexes the vertex mapping data.
	 * For example, if this LODModel is LOD3, then ClothMappingDataLODs[1] will point to defomer data that are using cloth LOD2.
	 * Then ClothMappingDataLODs[2] will point to defomer data that are using cloth LOD1, ...etc.
	 * ClothMappingDataLODs[0] will always point to defomer data that are using the same cloth LOD as this section LOD,
	 * this is convenient for cases where the cloth LOD bias is not known/required.
	 */
	TArray<TArray<FMeshToMeshVertData>> ClothMappingDataLODs;

	/** The bones which are used by the vertices of this section. Indices of bones in the USkeletalMesh::RefSkeleton array */
	TArray<FBoneIndexType> BoneMap;

	/** Number of vertices in this section (size of SoftVertices array). Available in non-editor builds. */
	int32 NumVertices;

	/** max # of bones used to skin the vertices in this section */
	int32 MaxBoneInfluences;

	/** whether to store bone indices as 16 bit or 8 bit in vertex buffer for rendering. */
	bool bUse16BitBoneIndex;

	// INDEX_NONE if not set
	int16 CorrespondClothAssetIndex;

	/** Clothing data for this section, clothing is only present if ClothingData.IsValid() returns true */
	FClothingSectionData ClothingData;

	/** Map between a vertex index and all vertices that share the same position **/
	TMap<int32, TArray<int32>> OverlappingVertices;

	/** If disabled, we won't render this section */
	bool bDisabled;

	/*
	 * The LOD index at which any generated lower quality LODs will include this section.
	 * A value of -1 mean the section will always be include when generating a LOD
	 */
	int32 GenerateUpToLodIndex;

	/*
	 * This represent the original section index in the imported data. The original data is chunk per material,
	 * we use this index to store user section modification. The user cannot change a BONE chunked section data,
	 * since the BONE chunk can be per-platform. Do not use this value to index the Sections array, only the user
	 * section data should be index by this value.
	 */
	int32 OriginalDataSectionIndex;

	/*
	 * If this section was produce because of BONE chunking, the parent section index will be valid.
	 * If the section is not the result of skin vertex chunking, this value will be INDEX_NONE.
	 * Use this value to know if the section was BONE chunked:
	 * if(ChunkedParentSectionIndex != INDEX_NONE) will be true if the section is BONE chunked
	 */
	int32 ChunkedParentSectionIndex;



	FSkelMeshSection()
		: MaterialIndex(0)
		, BaseIndex(0)
		, NumTriangles(0)
		, bSelected(false)
		, bRecomputeTangent(false)
		, RecomputeTangentsVertexMaskChannel(ESkinVertexColorChannel::None)
		, bCastShadow(true)
		, bVisibleInRayTracing(true)
		, bLegacyClothingSection_DEPRECATED(false)
		, CorrespondClothSectionIndex_DEPRECATED(-1)
		, BaseVertexIndex(0)
		, NumVertices(0)
		, MaxBoneInfluences(4)
		, bUse16BitBoneIndex(false)
		, CorrespondClothAssetIndex(INDEX_NONE)
		, bDisabled(false)
		, GenerateUpToLodIndex(INDEX_NONE)
		, OriginalDataSectionIndex(INDEX_NONE)
		, ChunkedParentSectionIndex(INDEX_NONE)
	{}


	/**
	* @return total num rigid verts for this section
	*/
	inline int32 GetNumVertices() const
	{
		// Either SoftVertices should be empty, or size should match NumVertices
		check(SoftVertices.Num() == 0 || SoftVertices.Num() == NumVertices);
		return NumVertices;
	}

	/**
	* @return starting index for rigid verts for this section in the LOD vertex buffer
	*/
	inline int32 GetVertexBufferIndex() const
	{
		return BaseVertexIndex;
	}

	/**
	* @return TRUE if we have cloth data for this section
	*/
	inline bool HasClothingData() const
	{
		constexpr int32 ClothLODBias = 0;  // Must at least have the mapping for the matching cloth LOD
		return ClothMappingDataLODs.Num() && ClothMappingDataLODs[ClothLODBias].Num();
	}

	/**
	* Calculate max # of bone influences used by this skel mesh section
	*/
	ENGINE_API void CalcMaxBoneInfluences();

	inline int32 GetMaxBoneInfluences() const
	{
		return MaxBoneInfluences;
	}

	/**
	* Calculate if this skel mesh section needs 16-bit bone indices
	*/
	ENGINE_API void CalcUse16BitBoneIndex();

	inline bool Use16BitBoneIndex() const
	{
		return bUse16BitBoneIndex;
	}

	// Serialization.
	friend FArchive& operator<<(FArchive& Ar, FSkelMeshSection& S);
	static void DeclareCustomVersions(FArchive& Ar);
};

class FSkelMeshSectionArray : public TArray<FSkelMeshSection, TInlineAllocator<1>>
{
	using Super = TArray<FSkelMeshSection, TInlineAllocator<1>>;
public:
	using Super::Super;
};

/**
* Structure containing all the section data a user can change.
*
* Some section data also impact dependent generated LOD, those member should be add to the DDC Key
* and trig a rebuild if they are change.
*/
struct FSkelMeshSourceSectionUserData
{
	/** This section will recompute tangent in runtime */
	bool bRecomputeTangent;
	
	/** Vertex color channel to use to mask recompute tangent */
	ESkinVertexColorChannel RecomputeTangentsVertexMaskChannel;

	/** This section will cast shadow */
	bool bCastShadow;

	/** If true, this section will be visible in ray tracing effects. Turning this off will remove it from ray traced reflections, shadows, etc. */
	bool bVisibleInRayTracing;

	// INDEX_NONE if not set
	int16 CorrespondClothAssetIndex;

	/** Clothing data for this section, clothing is only present if ClothingData.IsValid() returns true */
	FClothingSectionData ClothingData;


	//////////////////////////////////////////////////////////////////////////
	// Skeletal mesh DDC key members, Add sections member that impact generated skel mesh here

	/** If disabled, we won't render this section */
	bool bDisabled;

	/*
	 * The LOD index at which any generated lower quality LODs will include this section.
	 * A value of -1 mean the section will always be include when generating a LOD
	 */
	int32 GenerateUpToLodIndex;

	// End DDC members
	//////////////////////////////////////////////////////////////////////////



	FSkelMeshSourceSectionUserData()
		: bRecomputeTangent(false)
		, RecomputeTangentsVertexMaskChannel(ESkinVertexColorChannel::None)
		, bCastShadow(true)
		, bVisibleInRayTracing(true)
		, CorrespondClothAssetIndex(INDEX_NONE)
		, bDisabled(false)
		, GenerateUpToLodIndex(INDEX_NONE)
	{}

	/**
	* @return TRUE if we have cloth data for this section
	*/
	inline bool HasClothingData() const
	{
		return (ClothingData.AssetGuid.IsValid());
	}

	static FSkelMeshSourceSectionUserData& GetSourceSectionUserData(TMap<int32, FSkelMeshSourceSectionUserData>& UserSectionsData, const FSkelMeshSection& Section)
	{
		FSkelMeshSourceSectionUserData* UserSectionData = UserSectionsData.Find(Section.OriginalDataSectionIndex);
		if (!UserSectionData)
		{
			//If the UserSectionData do not exist add it and copy from the section data
			UserSectionData = &UserSectionsData.Add(Section.OriginalDataSectionIndex);
			UserSectionData->bCastShadow = Section.bCastShadow;
			UserSectionData->bVisibleInRayTracing = Section.bVisibleInRayTracing;			
			UserSectionData->bDisabled = Section.bDisabled;
			UserSectionData->bRecomputeTangent = Section.bRecomputeTangent;
			UserSectionData->RecomputeTangentsVertexMaskChannel = Section.RecomputeTangentsVertexMaskChannel;
			UserSectionData->GenerateUpToLodIndex = Section.GenerateUpToLodIndex;
			UserSectionData->CorrespondClothAssetIndex = Section.CorrespondClothAssetIndex;
			UserSectionData->ClothingData.AssetGuid = Section.ClothingData.AssetGuid;
			UserSectionData->ClothingData.AssetLodIndex = Section.ClothingData.AssetLodIndex;
		}
		check(UserSectionData);
		return *UserSectionData;
	}
	// Serialization.
	friend FArchive& operator<<(FArchive& Ar, FSkelMeshSourceSectionUserData& S);
};

/**
* All data to define a certain LOD model for a skeletal mesh.
*/
class FSkeletalMeshLODModel
{
public:
	/** Sections. */
	TArray<FSkelMeshSection> Sections;

	/*
	 * When user change section data in the UI, we store it here to be able to regenerate the changes
	 * Note: the key (int32) is the original imported section data, because of BONE chunk the size of
	 * this array is not the same as the Sections array. Use the section's OriginalDataSectionIndex to
	 * index it.
	 */
	TMap<int32, FSkelMeshSourceSectionUserData> UserSectionsData;

	uint32						NumVertices;
	/** The number of unique texture coordinate sets in this lod */
	uint32						NumTexCoords;

	/** Index buffer, covering all sections */
	TArray<uint32> IndexBuffer;

	/**
	* Bone hierarchy subset active for this LOD.
	* This is a map between the bones index of this LOD (as used by the vertex structs) and the bone index in the reference skeleton of this SkeletalMesh.
	*/
	TArray<FBoneIndexType> ActiveBoneIndices;

	/**
	* Bones that should be updated when rendering this LOD. This may include bones that are not required for rendering.
	* All parents for bones in this array should be present as well - that is, a complete path from the root to each bone.
	* For bone LOD code to work, this array must be in strictly increasing order, to allow easy merging of other required bones.
	*/
	TArray<FBoneIndexType> RequiredBones;

	/** Set of skin weight profile, identified by a FName which matches FSkinWeightProfileInfo.Name in the owning Skeletal Mesh*/
	TMap<FName, FImportedSkinWeightProfileData> SkinWeightProfiles;

	/** Set of vertex attributes identified by an FName */
	TMap<FName, FSkeletalMeshModelVertexAttribute> VertexAttributes;

	/** For each point in this LOD model that maps to its source point in the FMeshDescription that produced it.
	 *  I.e. a linear map mapping the result of GetVertices() to its matching FVertexId entry in the original FMeshDescription.
	 */
	TArray<int32>				MeshToImportVertexMap;
	
	/** The max index in MeshToImportVertexMap, ie. the number of imported (raw) verts. */
	int32						MaxImportVertex;

	/** Accessor for the RawPointIndice which is editor only data: array of the original point (wedge) indices for each of the vertices in a FSkeletalMeshLODModel */
	const TArray<uint32>& GetRawPointIndices() const
	{
		return RawPointIndices2;
	}

	/** Accessor for the RawPointIndice which is editor only data: array of the original point (wedge) indices for each of the vertices in a FSkeletalMeshLODModel */
	TArray<uint32>& GetRawPointIndices()
	{
		return RawPointIndices2;
	}
	
	UE_DEPRECATED(5.0, "Please do not access this function anymore. This data is not use anymore.")
	FRawSkeletalMeshBulkData& GetRawSkeletalMeshBulkData_DEPRECATED()
	{
		return RawSkeletalMeshBulkData_DEPRECATED;
	}

	/** This ID is use to create the DDC key, it must be set when we save the FRawSkeletalMeshBulkData. */
	FString						RawSkeletalMeshBulkDataID;
	bool						bIsBuildDataAvailable;
	bool						bIsRawSkeletalMeshBulkDataEmpty;

	/** Constructor (default) */
	FSkeletalMeshLODModel()
		: NumVertices(0)
		, NumTexCoords(0)
		, MaxImportVertex(-1)
		, RawSkeletalMeshBulkDataID(TEXT(""))
		, bIsBuildDataAvailable(false)
		, bIsRawSkeletalMeshBulkDataEmpty(true)
		, BuildStringID(TEXT(""))
	{
		//Sice this ID is part of the DDC Key, we have to set it to an empty GUID not an empty string
		RawSkeletalMeshBulkDataID = FGuid().ToString();
		//Allocate the private mutex
		BulkDataReadMutex = new FCriticalSection();
	}

	~FSkeletalMeshLODModel()
	{
		//Release the allocate resources
		if(BulkDataReadMutex != nullptr)
		{
			delete BulkDataReadMutex;
			BulkDataReadMutex = nullptr;
		}
	}

	FSkeletalMeshLODModel(FSkeletalMeshLODModel&& InOther)
	{
		Sections = MoveTemp(InOther.Sections);
		UserSectionsData = MoveTemp(InOther.UserSectionsData);
		NumVertices = InOther.NumVertices;
		NumTexCoords = InOther.NumTexCoords;
		IndexBuffer = MoveTemp(InOther.IndexBuffer);
		ActiveBoneIndices = MoveTemp(InOther.ActiveBoneIndices);
		RequiredBones = MoveTemp(InOther.RequiredBones);
		SkinWeightProfiles = MoveTemp(InOther.SkinWeightProfiles);
		VertexAttributes = MoveTemp(InOther.VertexAttributes);
		MeshToImportVertexMap = MoveTemp(InOther.MeshToImportVertexMap);
		MaxImportVertex = InOther.MaxImportVertex;
		RawSkeletalMeshBulkDataID = MoveTemp(InOther.RawSkeletalMeshBulkDataID);
		bIsBuildDataAvailable = InOther.bIsBuildDataAvailable;
		bIsRawSkeletalMeshBulkDataEmpty = InOther.bIsRawSkeletalMeshBulkDataEmpty;
		ImportedMeshInfos = MoveTemp(InOther.ImportedMeshInfos);
		RawPointIndices2 = MoveTemp(InOther.RawPointIndices2);
		BulkDataReadMutex = InOther.BulkDataReadMutex;
		InOther.BulkDataReadMutex = nullptr;
		BuildStringID = InOther.BuildStringID;
	}

	/*Empty the skeletal mesh LOD model. Empty copy a default constructed FSkeletalMeshLODModel but will not copy the BulkDataReadMutex which will be the same after*/
	void Empty()
	{
		FCriticalSection* BackupBulkDataReadMutex = BulkDataReadMutex;
		*this = FSkeletalMeshLODModel();
		BulkDataReadMutex = BackupBulkDataReadMutex;
	}

private:
	friend struct FSkeletalMeshSourceModel;
	
	struct FSkelMeshImportedMeshInfo
	{
		FName Name;	// The name of the mesh.
		int32 NumVertices;	// The number of imported (dcc) vertices that are part of this mesh. This is a value of 8 for a cube. So NOT the number of render vertices.
		int32 StartImportedVertex;	// The first index of imported (dcc) vertices in the mesh. So this NOT an index into the render vertex buffer. In range of 0..7 for a cube.

		void Serialize(FArchive& Ar)
		{
			Ar << Name;
			Ar << NumVertices;
			Ar << StartImportedVertex;
		}
	};
	friend FArchive& operator<<(FArchive& Ar, FSkelMeshImportedMeshInfo& Info)
	{
		Info.Serialize(Ar);
		return Ar;
	}

	TArray<FSkelMeshImportedMeshInfo> ImportedMeshInfos;

	/** For each point in this LOD model that maps to its source point in FSkeletalMeshImportData.
	 *  I.e. a linear map mapping the result of GetVertices() to its matching FSkeletalMeshImportData::Points entry.
	 *  This can be used to map back to alternative skin weight profiles and vertex attributes on the import data.
	 */
	TArray<uint32>				RawPointIndices2;

	FIntBulkData				RawPointIndices_DEPRECATED;
	FWordBulkData				LegacyRawPointIndices_DEPRECATED;
	FRawSkeletalMeshBulkData	RawSkeletalMeshBulkData_DEPRECATED;

	//Mutex use by the CopyStructure function. It's a pointer because FCriticalSection privatize the operator= function, which will prevent this class operator= to use the default.
	//We want to avoid having a custom equal operator that will get deprecated if dev forget to add the new member in this class
	//The CopyStructure function will copy everything but make sure the destination mutex is set to a new mutex pointer.
	FCriticalSection* BulkDataReadMutex;

	//Use the static FSkeletalMeshLODModel::CopyStructure function to copy from one instance to another
	//The reason is we want the copy to be multithread safe and use the BulkDataReadMutex.
	FSkeletalMeshLODModel& operator=(const FSkeletalMeshLODModel& Other) = default;

	//Use the static FSkeletalMeshLODModel::CreateCopy function to copy from one instance to another
	//The reason is we want the copy to be multithread safe and use the BulkDataReadMutex.
	FSkeletalMeshLODModel(const FSkeletalMeshLODModel& Other) = delete;

public:
	/**
	* Special serialize function passing the owning UObject along as required by FUnytpedBulkData
	* serialization.
	*
	* @param	Ar		Archive to serialize with
	* @param	Owner	UObject this structure is serialized within
	* @param	Idx		Index of current array entry being serialized
	*/
	void Serialize(FArchive& Ar, UObject* Owner, int32 Idx);
	static void DeclareCustomVersions(FArchive& Ar);

	/**
	* Fill array with vertex position and tangent data from skel mesh chunks.
	*
	* @param Vertices Array to fill.
	*/
	ENGINE_API void GetVertices(TArray<FSoftSkinVertex>& Vertices, const FSkeletalMeshUnifiedBoneIndices* UnifiedBoneIndices = nullptr) const;

	/**
	* Fill array with APEX cloth mapping data.
	*
	* @param MappingData Array to fill.
	*/
	void GetClothMappingData(TArray<FMeshToMeshVertData>& MappingData, TArray<FClothBufferIndexMapping>& OutClothIndexMapping) const;



	/** Utility for finding the section that a particular vertex is in. */
	ENGINE_API void GetSectionFromVertexIndex(int32 InVertIndex, int32& OutSectionIndex, int32& OutVertIndex) const;

	/**
	* @return true if any chunks have cloth data.
	*/
	ENGINE_API bool HasClothData() const;

	ENGINE_API int32 NumNonClothingSections() const;

	ENGINE_API int32 GetNumNonClothingVertices() const;

	/**
	* Similar to GetVertices but ignores vertices from clothing sections
	* to avoid getting duplicate vertices from clothing sections if not needed
	*
	* @param OutVertices Array to fill
	*/
	ENGINE_API void GetNonClothVertices(TArray<FSoftSkinVertex>& OutVertices, const FSkeletalMeshUnifiedBoneIndices* UnifiedBoneIndices = nullptr) const;

	ENGINE_API int32 GetMaxBoneInfluences() const;
	ENGINE_API bool DoSectionsUse16BitBoneIndex() const;

	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const;

	/**
	* Make sure user section data is present for every original sections
	*/
	ENGINE_API void SyncronizeUserSectionsDataArray(bool bResetNonUsedSection = false);

	// Offers protected access while supporting copy.
	struct FThreadSafeBuildStringID
	{
	private:
		FString    BuildStringID;
		mutable UE::FMutex Mutex;
	public:
		FThreadSafeBuildStringID() = default;
		FThreadSafeBuildStringID(const TCHAR* InText)
			: BuildStringID(InText)
		{
		}

		FThreadSafeBuildStringID(const FThreadSafeBuildStringID& Other)
		{
			UE::TUniqueLock OtherLock(Other.Mutex);
			UE::TUniqueLock Lock(Mutex);
			BuildStringID = Other.BuildStringID;
		}

		FThreadSafeBuildStringID& operator=(const FThreadSafeBuildStringID& Other)
		{
			if (&Other != this)
			{
				UE::TUniqueLock OtherLock(Other.Mutex);
				UE::TUniqueLock Lock(Mutex);
				BuildStringID = Other.BuildStringID;
			}
			return *this;
		}

		bool operator==(const FString& Other) const
		{
			UE::TUniqueLock Lock(Mutex);
			return BuildStringID == Other;
		}

		bool operator==(const FThreadSafeBuildStringID& Other) const
		{
			if (&Other == this)
			{
				return true;
			}

			UE::TUniqueLock OtherLock(Other.Mutex);
			UE::TUniqueLock Lock(Mutex);
			return BuildStringID == Other.BuildStringID;
		}

		void operator=(const FString& Other)
		{
			UE::TUniqueLock Lock(Mutex);
			BuildStringID = Other;
		}

		operator FString() const
		{
			UE::TUniqueLock Lock(Mutex);
			return BuildStringID;
		}

		FThreadSafeBuildStringID(FThreadSafeBuildStringID&&) = delete;
		FThreadSafeBuildStringID& operator=(FThreadSafeBuildStringID&&) = delete;
	};

	//Temporary build String ID
	//We use this string to store the LOD model data so we can know if the LOD need to be rebuild
	//This GUID is set when we Cache the render data (build function)
	mutable FThreadSafeBuildStringID BuildStringID;

	/**
	* Build a derive data key with the user section data (UserSectionsData) and the original bulk data
	*/
	ENGINE_API FString GetLODModelDeriveDataKey() const;

	/**
	* This function will update the chunked information for each section. Only old data before the 
	* skeletal mesh build refactor should need to call this function.
	*/
	ENGINE_API void UpdateChunkedSectionInfo(const FString& SkeletalMeshName);

	/**
	* Copy one structure to the other, make sure all bulk data is unlock and the data can be read before copying.
	*
	* It also use a private mutex to make sure it's thread safe to copy the same source multiple time in multiple thread.
	*/
	static ENGINE_API void CopyStructure(FSkeletalMeshLODModel* Destination, const FSkeletalMeshLODModel* Source);

	/**
	* Create a new FSkeletalMeshLODModel on the heap. Copy data from the "FSkeletalMeshLODModel* Other" to the just created LODModel return the heap allocated LODModel.
	* This function is thread safe since its use the thread safe CopyStructure function to copy the data from Other.
	*/
	static FSkeletalMeshLODModel* CreateCopy(const FSkeletalMeshLODModel* Other)
	{
		FSkeletalMeshLODModel* Destination = new FSkeletalMeshLODModel();
		FSkeletalMeshLODModel::CopyStructure(Destination, Other);
		return Destination;
	}

	/**
	 * Fills in a representation of this model into the given mesh description object. Existing
	 * mesh description data is emptied.
	 */
	void ENGINE_API GetMeshDescription(const USkeletalMesh *InSkeletalMesh, const int32 InLODIndex, FMeshDescription& OutMeshDescription) const;
};

#endif // WITH_EDITOR
