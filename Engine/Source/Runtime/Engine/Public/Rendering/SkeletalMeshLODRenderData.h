// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/MultiSizeIndexContainer.h"
#include "Rendering/SkeletalMeshVertexBuffer.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "Rendering/ColorVertexBuffer.h"
#include "Rendering/SkeletalMeshDuplicatedVerticesBuffer.h"
#include "Rendering/SkeletalMeshVertexClothBuffer.h"
#include "Rendering/MorphTargetVertexInfoBuffers.h"
#include "Rendering/SkeletalMeshHalfEdgeBuffer.h"
#include "SkeletalMeshTypes.h"
#include "BoneIndices.h"
#include "StaticMeshResources.h"
#include "GPUSkinVertexFactory.h"
#include "Animation/SkeletalMeshVertexAttribute.h"
#include "Rendering/SkinWeightProfile.h"

class USkinnedAsset;
#if WITH_EDITOR
class FSkeletalMeshLODModel;
struct FSkeletalMeshVertexAttributeInfo;
#endif // WITH_EDITOR

struct FSkelMeshRenderSection
{
	/** Material (texture) used for this section. */
	uint16 MaterialIndex;

	/** The offset of this section's indices in the LOD's index buffer. */
	uint32 BaseIndex;

	/** The number of triangles in this section. */
	uint32 NumTriangles;

	/** This section will recompute tangent in runtime */
	bool bRecomputeTangent;

	/** This section will cast shadow */
	bool bCastShadow;

	/** If true, this section will be visible in ray tracing effects. Turning this off will remove it from ray traced reflections, shadows, etc. */
	bool bVisibleInRayTracing;

	/** If true, this section contains a unified bone map, meaning all other sections contain the same map with the union of all bones. */
	bool bUnifiedBoneMap;

	/** Which channel for masking the recompute tangents */
	ESkinVertexColorChannel RecomputeTangentsVertexMaskChannel;

	/** The offset into the LOD's vertex buffer of this section's vertices. */
	uint32 BaseVertexIndex;

	UE_DEPRECATED(5.8, "Use ClothMappingDataLODs instead.")
	TArray<FMeshToMeshVertData> ClothMappingData_DEPRECATED;

	/**
	 * The cloth deformer mapping data to each of the required cloth LOD.
	 * Raytracing may require a different deformer LOD to the one being simulated/rendered.
	 * The outer array index represents the LOD bias. The inner array indexes the vertex data.
	 * If this LODModel is LOD3, ClothMappingDataLODs[1] will point to defomer data using LOD2,
	 * and ClothMappingDataLODs[2] will point to defomer data that are using cloth LOD1, ...etc.
	 * ClothMappingDataLODs[0] always point to defomer data of the same cloth LOD, this is
	 * convenient for cases where the cloth LOD bias is not known or required.
	 */
	TArray<TArray<FMeshToMeshVertData>> ClothMappingDataLODs;

	/** The bones which are used by the vertices of this section. Indices of bones in the USkeletalMesh::RefSkeleton array */
	TArray<FBoneIndexType> BoneMap;

	/** The number of vertices in this section. */
	uint32 NumVertices;

	/** max # of bones used to skin the vertices in this section */
	int32 MaxBoneInfluences;

	// INDEX_NONE if not set
	int16 CorrespondClothAssetIndex;

	/** Clothing data for this section, clothing is only present if ClothingData.IsValid() returns true */
	FClothingSectionData ClothingData;

	/** Index Buffer containting all duplicated vertices in the section and a buffer containing which indices into the index buffer are relevant per vertex **/
	FDuplicatedVerticesBuffer DuplicatedVerticesBuffer;

	/** Disabled sections will not be collected when rendering, controlled from the source section in the skeletal mesh asset */
	bool bDisabled;

	FSkelMeshRenderSection()
		: MaterialIndex(0)
		, BaseIndex(0)
		, NumTriangles(0)
		, bRecomputeTangent(false)
		, bCastShadow(true)
		, bVisibleInRayTracing(true)
		, bUnifiedBoneMap(false)
		, RecomputeTangentsVertexMaskChannel(ESkinVertexColorChannel::None)
		, BaseVertexIndex(0)
		, NumVertices(0)
		, MaxBoneInfluences(4)
		, CorrespondClothAssetIndex(-1)
		, bDisabled(false)
	{}

	inline bool HasUnifiedBoneMap() const
	{
		return bUnifiedBoneMap;
	}

	inline bool HasClothingData() const
	{
		constexpr int32 ClothLODBias = 0;  // Must at least have the mapping for the matching cloth LOD
		return ClothMappingDataLODs.Num() && ClothMappingDataLODs[ClothLODBias].Num();
	}

	inline int32 GetVertexBufferIndex() const
	{
		return BaseVertexIndex;
	}

	inline int32 GetNumVertices() const
	{
		return NumVertices;
	}

	inline bool IsValid() const
	{
		return NumTriangles > 0 && !bDisabled;
	}

	ENGINE_API friend FArchive& operator<<(FArchive& Ar, FSkelMeshRenderSection& S);
};

class FSkeletalMeshLODRenderData : public UE::Private::FQueryableRefCountedObject
{
public:
	UE_DEPRECATED(5.8, "Temporary fix. Do not use in production code")
	mutable FTransactionallySafeCriticalSection RenderDataMutex;

	/** Info about each section of this LOD for rendering */
	TArray<FSkelMeshRenderSection>	RenderSections;

	// Index Buffer (MultiSize: 16bit or 32bit)
	FMultiSizeIndexContainer	MultiSizeIndexContainer;

	/** static vertices from chunks for skinning on GPU */
	FStaticMeshVertexBuffers	StaticVertexBuffers;

	/** Skin weights for skinning */
	FSkinWeightVertexBuffer		SkinWeightVertexBuffer;

	/** A buffer for cloth mesh-mesh mapping */
	FSkeletalMeshVertexClothBuffer	ClothVertexBuffer;

	/** GPU friendly access data for MorphTargets for an LOD */
	FMorphTargetVertexInfoBuffers	MorphTargetVertexInfoBuffers;

	/** Skin weight profile data structures, can contain multiple profiles and their runtime FSkinWeightVertexBuffer */
	FSkinWeightProfilesData SkinWeightProfilesData;

	FSkeletalMeshVertexAttributeRenderData VertexAttributeBuffers;

	/** GPU buffer for half edge data of an LOD, useful for mesh deformers */
	FSkeletalMeshHalfEdgeBuffer HalfEdgeBuffer;

	TArray<FBoneIndexType> ActiveBoneIndices;

	TArray<FBoneIndexType> RequiredBones;

	/** When populated, all sections share the same unified bone map across this LOD. */
	TConstArrayView<FBoneIndexType> UnifiedBoneMap;

	uint32 BuffersSize;

	/** Precooked ray tracing geometry. Used as source data to build skeletal mesh instance ray tracing geometry. */
	FRayTracingGeometry SourceRayTracingGeometry;

	FByteBulkData StreamingBulkData;

	/** Whether buffers of this LOD is inlined (i.e. stored in .uexp instead of .ubulk) */
	uint32 bStreamedDataInlined : 1;
	/** Whether this LOD is below MinLod */
	uint32 bIsLODOptional : 1;

#if RHI_RAYTRACING
	/** Game thread reference counter of static skeletal mesh objects referencing this render data */
	int32 NumReferencingStaticSkeletalMeshObjects = 0;
	/** Same as NumReferencingStaticSkeletalMeshObjects, but on render thread to determine lifetime of resources in mesh streaming */
	bool bReferencedByStaticSkeletalMeshObjects_RenderThread = false;
	/** Static ray tracing geometry, only initialized when bRenderStatic = true on any skeletal mesh scene proxy*/
	FRayTracingGeometry StaticRayTracingGeometry;
#endif

	/**
	* Initialize the LOD's render resources.
	*
	* @param Parent Parent mesh
	*/
	void InitResources(bool bNeedsVertexColors, int32 LODIndex, USkinnedAsset* Owner);

	void InitMorphResources();

	/**
	* Releases the LOD's render resources.
	*/
	ENGINE_API void ReleaseResources();

	/**
	* Releases the LOD's CPU render resources.
	*/
	void ReleaseCPUResources(bool bForStreaming = false);

	/** Constructor (default) */
	FSkeletalMeshLODRenderData(bool bAddRef = true)
		: BuffersSize(0)
		, bStreamedDataInlined(true)
		, bIsLODOptional(false)
		, bIsDataReady(false)
	{
		if (bAddRef)
		{
			AddRef();
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	inline ~FSkeletalMeshLODRenderData()
	{
		check(GetRefCount() == 0);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS


	/**
	* Special serialize function passing the owning UObject along as required by FUnytpedBulkData
	* serialization.
	*
	* @param	Ar		Archive to serialize with
	* @param	Owner	UObject this structure is serialized within
	* @param	Idx		Index of current array entry being serialized
	*/
	void Serialize(FArchive& Ar, UObject* Owner, int32 Idx);

	/**
	* Serialize the portion of data that might be streamed
	* @param bNeedsCPUAcces - Whether something requires keeping CPU copy of resources (see ShouldKeepCPUResources)
	* @param bForceKeepCPUResources - Whether we want to force keeping CPU resources
	* @return Size of streamed LOD data in bytes
	*/
	ENGINE_API void SerializeStreamedData(FArchive& Ar, USkinnedAsset* Owner, int32 LODIdx, uint8 ClassStripFlags, bool bNeedsCPUAccess, bool bForceKeepCPUResources);

	void SerializeAvailabilityInfo(FArchive& Ar, int32 LODIdx, bool bHasAdjacencyData, bool bNeedsCPUAccess);

#if WITH_EDITOR

	struct FBuildSettings
	{
		FBuildSettings() {}

		TConstArrayView<uint16> UnifiedBoneMap;
		ESkeletalMeshVertexFlags BuildFlags = ESkeletalMeshVertexFlags::None;
		bool bBuildHalfEdgeBuffers = false;
	};
	
	/**
	 * Initialize render data (e.g. vertex buffers) from model info
	 * @param InLODModel The model to build the render data from.
	 * @param InVertexAttributeInfos The vertex attributes to possibly include and their stored data type.
	 * @param InBuildSettings Forwards relevant settings from LodInfo, See ESkeletalMeshVertexFlags.
	 */
	void ENGINE_API BuildFromLODModel(
		const FSkeletalMeshLODModel* InLODModel,
		TConstArrayView<FSkeletalMeshVertexAttributeInfo> InVertexAttributeInfos = {},
		const FBuildSettings& InBuildSettings = FBuildSettings{}
		);
#endif // WITH_EDITOR

	inline bool HasUnifiedBoneMap() const
	{
		return !UnifiedBoneMap.IsEmpty();
	}

	inline TConstArrayView<FBoneIndexType> GetUnifiedBoneMap() const
	{
		return UnifiedBoneMap;
	}

	uint32 GetNumVertices() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE::TScopeLock Lock(RenderDataMutex);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		return StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
	}

	uint32 GetVertexBufferMaxBoneInfluences() const
	{
		return SkinWeightVertexBuffer.GetMaxBoneInfluences();
	}

	bool DoesVertexBufferUse16BitBoneIndex() const
	{
		return SkinWeightVertexBuffer.Use16BitBoneIndex();
	}

	uint32 GetNumTexCoords() const
	{
		return StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	}

	/** Checks whether or not the skin weight buffer has been overridden 'by default' and if not return the original Skin Weight buffer */
	FSkinWeightVertexBuffer* GetSkinWeightVertexBuffer() 
	{
		FSkinWeightVertexBuffer* OverrideBuffer = SkinWeightProfilesData.GetDefaultOverrideBuffer();
		return OverrideBuffer != nullptr ? OverrideBuffer : &SkinWeightVertexBuffer;
	}
	
	/** Checks whether or not the skin weight buffer has been overridden 'by default' and if not return the original Skin Weight buffer */
	const FSkinWeightVertexBuffer* GetSkinWeightVertexBuffer() const
	{
		FSkinWeightVertexBuffer* OverrideBuffer = SkinWeightProfilesData.GetDefaultOverrideBuffer();
		return OverrideBuffer != nullptr ? OverrideBuffer : &SkinWeightVertexBuffer;
	}
	
	FSkinWeightVertexBuffer* GetSkinWeightProfileVertexBuffer(const FSkinWeightProfileStack& InProfileStack);

	/** Utility function for returning total number of faces in this LOD. */
	ENGINE_API int32 GetTotalFaces() const;

	/**
	* @return true if any chunks have cloth data.
	*/
	ENGINE_API bool HasClothData() const;

	/** Utility for finding the section that a particular vertex is in. */
	ENGINE_API void GetSectionFromVertexIndex(int32 InVertIndex, int32& OutSectionIndex, int32& OutVertIndex) const;

	/**
	* Get Resource Size
	*/
	ENGINE_API void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const;

	/** Get the estimated memory overhead of buffers marked as NeedsCPUAccess. */
	SIZE_T GetCPUAccessMemoryOverhead() const;

	// O(1)
	// @return -1 if not found
	uint32 FindSectionIndex(const FSkelMeshRenderSection& Section) const;

	ENGINE_API int32 NumNonClothingSections() const;

	void IncrementMemoryStats(bool bNeedsVertexColors);
	void DecrementMemoryStats();

	static bool ShouldForceKeepCPUResources();
	static bool ShouldKeepCPUResources(const USkinnedAsset* SkinnedAsset, int32 LODIdx, bool bForceKeep);

	/** 
	* @return Returns true if data is not being streamed/serialized in and is ready for use. If data is being serialized i.e. in midst 
	* of Serialize(...) where data is being loaded in, especially from a different thread, this will return false. This is an atomic
	* load and may not be suitable for perf-critical paths. 
	*/
	bool IsDataReady() const { return bIsDataReady.Load(); }

private:
	enum EClassDataStripFlag : uint8
	{
		CDSF_AdjacencyData_DEPRECATED = 1,
		CDSF_MinLodData = 2
	};

	static uint8 GenerateClassStripFlags(FArchive& Ar, const USkinnedAsset* OwnerMesh, int32 LODIdx);

	static bool IsLODCookedOut(const ITargetPlatform* TargetPlatform, const USkinnedAsset* SkinnedAsset, bool bIsBelowMinLOD);

	static bool IsLODInlined(const ITargetPlatform* TargetPlatform, const USkinnedAsset* SkinnedAsset, int32 LODIdx, bool bIsBelowMinLOD);

	static int32 GetNumOptionalLODsAllowed(const ITargetPlatform* TargetPlatform, const USkinnedAsset* SkinnedAsset);

	inline void SetIsDataReady(bool bReady) { bIsDataReady.Store(bReady); }

	/** Atomic bool to indicate if required data has been streamed/serialized in. */
	TAtomic<bool> bIsDataReady;

	friend class FSkeletalMeshRenderData;
};
