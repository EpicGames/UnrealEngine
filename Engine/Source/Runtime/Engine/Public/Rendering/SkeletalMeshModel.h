// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "Rendering/SkeletalMeshLODModel.h"

class FSkeletalMeshLODModel;
class USkeletalMesh;
class USkinnedAsset;
class FArchive;

/**
* Imported resources for a skeletal mesh.
*/
class FSkeletalMeshModel
{
public:
	/** GUID to indicate this version of data. Needs to change when modifications are made to model, so that derived data is regenerated. */
	FGuid SkeletalMeshModelGUID;
	/** If this GUID was generated from a hash of the data, or randomly generated when it was changed */
	bool bGuidIsHash;

	/** Per-LOD data. */
	TIndirectArray<FSkeletalMeshLODModel> LODModels;

	/** Default constructor. */
	ENGINE_API FSkeletalMeshModel();

#if WITH_EDITOR
	/** Creates a new GUID for this Model */
	ENGINE_API void GenerateNewGUID();

	/** 
	 *	Util to regenerate a GUID for this Model based on hashing its data 
	 *	Used by old content, rather than a random new GUID.
	 */
	void GenerateGUIDFromHash(USkinnedAsset* Owner);

	/** Get current GUID Id as a string, for DDC key */
	ENGINE_API FString GetIdString() const;

	ENGINE_API void SyncronizeLODUserSectionsData();

	ENGINE_API FString GetLODModelIdString() const;

	ENGINE_API void EmptyOriginalReductionSourceMeshData();
	
	/* When user reduce an imported LOD with itself (BaseLOD == TargetLOD), we need to store some imported model data so we can reduce again from the same data.*/
	/* We do not need to store such a data, since we can use the USkeletalMesh::MeshEditorDataObject which have the imported data */
	TArray<FReductionBaseSkeletalMeshBulkData*> OriginalReductionSourceMeshData_DEPRECATED;

	/* When user reduce an imported LOD with itself (BaseLOD == TargetLOD), we need to store the geometry count (vertex count and triangle count).
	We use this data to query if the reduction is active when the reduction criterion is an absolute vertex/triangle number*/
	TArray<FInlineReductionCacheData> InlineReductionCacheDatas;
#endif

	/** Serialize to/from the specified archive.. */
	ENGINE_API void Serialize(FArchive& Ar, USkinnedAsset* Owner);

	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);
};

#endif // WITH_EDITOR