// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"
#include "Containers/IndirectArray.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/NaniteInterface.h"

struct FMeshUVChannelInfo;
class USkeletalMesh;
class USkinnedAsset;
struct FSkeletalMaterial;
struct FResourceSizeEx;

class FSkeletalMeshRenderData
{
public:
	/** Per-LOD render data. */
	TIndirectArray<FSkeletalMeshLODRenderData> LODRenderData;

	/** Nanite resource data. */
	TPimplPtr<Nanite::FResources> NaniteResourcesPtr;

	/** True if rhi resources are initialized */
	bool bReadyForStreaming;

	/** Const after serialization. */
	uint8 NumInlinedLODs;

	/** Const after serialization. */
	uint8 NumNonOptionalLODs;

	/** [RenderThread] Index of the most detailed valid LOD. */
	uint8 CurrentFirstLODIdx;

	/** [GameThread/RenderThread] Future value of CurrentFirstLODIdx. */
	uint8 PendingFirstLODIdx;

	/** Runtime LOD bias modifier for this skeletal mesh */
	uint8 LODBiasModifier;

	/** Whether ray tracing acceleration structures should be created for this mesh. Derived from owner USkinnedAsset. */
	bool bSupportRayTracing;

	/** Optional, name of the owning skeletal mesh */
	FName OwnerName = NAME_None;

#if RHI_RAYTRACING
	RayTracing::FGeometryGroupHandle RayTracingGeometryGroupHandle;
#endif

#if WITH_EDITORONLY_DATA
	/** UV data used for streaming accuracy debug view modes. In sync for rendering thread */
	TArray<FMeshUVChannelInfo> UVChannelDataPerMaterial;

	/** The derived data key associated with this render data. */
	FString DerivedDataKey;

	/** The next cached derived data in the list. */
	TUniquePtr<class FSkeletalMeshRenderData> NextCachedRenderData;
#endif

	ENGINE_API FSkeletalMeshRenderData();
	ENGINE_API ~FSkeletalMeshRenderData();

#if WITH_EDITOR
	ENGINE_API void Cache(const ITargetPlatform* TargetPlatform, USkinnedAsset* Owner, class FSkinnedAssetCompilationContext* ContextPtr);
	FString GetDerivedDataKey(const ITargetPlatform* TargetPlatform, USkinnedAsset* Owner);

	ENGINE_API void SyncUVChannelData(const TArray<FSkeletalMaterial>& ObjectData);
#endif

	/** Serialize to/from the specified archive.. */
	ENGINE_API void Serialize(FArchive& Ar, USkinnedAsset* Owner);

	/** Initializes rendering resources. */
	ENGINE_API void InitResources(bool bNeedsVertexColors, USkinnedAsset* Owner);

	/** Releases rendering resources. */
	ENGINE_API void ReleaseResources();

	/** Return the resource size */
	ENGINE_API void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);

	/** Initializes the static ray tracing geometry for the specified LOD (if necessary). */
	ENGINE_API void InitStaticRayTracingGeometry(int32 LODIndex);

	/** Releases the static ray tracing geometry for the specified LOD (if it's the last reference). */
	ENGINE_API void ReleaseStaticRayTracingGeometry(int32 LODIndex);

	/** Get the estimated memory overhead of buffers marked as NeedsCPUAccess. */
	ENGINE_API SIZE_T GetCPUAccessMemoryOverhead() const;

	UE_DEPRECATED(5.8, "Always returns false. All platforms support 16-bit bone indices and sections are no longer split by bone count.")
	ENGINE_API bool RequiresCPUSkinning(ERHIFeatureLevel::Type FeatureLevel) const;

	UE_DEPRECATED(5.8, "Always returns false. All platforms support 16-bit bone indices and sections are no longer split by bone count.")
	ENGINE_API bool RequiresCPUSkinning(ERHIFeatureLevel::Type FeatureLevel, int32 MinLODIdx) const;

	/** Returns the number of bone influences per vertex. */
	ENGINE_API uint32 GetNumBoneInfluences() const;

	/** Returns the number of bone influences per vertex starting at MinLODIdx. */
	ENGINE_API uint32 GetNumBoneInfluences(int32 MinLODIdx) const;
	
	UE_DEPRECATED(5.8, "Sections are no longer split by bone count. Use BoneMap.Num() on individual sections if needed.")
	ENGINE_API int32 GetMaxBonesPerSection() const;

	UE_DEPRECATED(5.8, "Sections are no longer split by bone count. Use BoneMap.Num() on individual sections if needed.")
	ENGINE_API int32 GetMaxBonesPerSection(int32 MinLODIdx) const;
	
	/** Return first valid LOD index starting at MinLODIdx. */
	ENGINE_API int32 GetFirstValidLODIdx(int32 MinLODIdx) const;

	/** Return the pending first LODIdx that can be used. */
	inline int32 GetPendingFirstLODIdx(int32 MinLODIdx) const
	{
		return GetFirstValidLODIdx(FMath::Max<int32>(PendingFirstLODIdx, MinLODIdx));
	}

	/** Check if any render section casts shadows */
	ENGINE_API bool AnyRenderSectionCastsShadows(int32 MinLODIdx) const;

	/** 
	 * Return the pending first LOD that can be used for rendering starting at MinLODIdx.
	 * This takes into account the streaming status from PendingFirstLODIdx, 
	 * and MinLODIdx is expected to be USkeletalMesh::MinLOD, which is platform specific.
	 */
	inline const FSkeletalMeshLODRenderData* GetPendingFirstLOD(int32 MinLODIdx) const
	{
		const int32 PendingFirstIdx = GetPendingFirstLODIdx(MinLODIdx);
		return PendingFirstIdx == INDEX_NONE ? nullptr : &LODRenderData[PendingFirstIdx];
	}

	bool IsInitialized() const
	{
		return bInitialized;
	}

	bool HasUnifiedBoneMap() const
	{
		return !UnifiedBoneMap.IsEmpty();
	}

	TConstArrayView<FBoneIndexType> GetUnifiedBoneMap() const
	{
		return UnifiedBoneMap;
	}

	/** Initialize the render data's unified bone map from the last LOD's storage. Call after LOD render data is fully built. */
	void InitUnifiedBoneMapFromLODs()
	{
		UnifiedBoneMap = !LODRenderData.IsEmpty() ? LODRenderData.Last().GetUnifiedBoneMap() : TConstArrayView<FBoneIndexType>{};
	}

	ENGINE_API bool HasValidNaniteData() const;

	ENGINE_API bool IsLODDataReady() const;

private:
	/** When populated, all sections share the same unified bone map across LODs. */
	TConstArrayView<FBoneIndexType> UnifiedBoneMap;

	/** Count the number of LODs that are inlined and not streamable. Starting from the last LOD and stopping at the first non inlined LOD. */
	int32 GetNumNonStreamingLODs() const;
	/** Count the number of LODs that not optional and guarantied to be installed. Starting from the last LOD and stopping at the first optional LOD. */
	int32 GetNumNonOptionalLODs() const;

	/** True if the resource has been initialized. */
	bool bInitialized = false;
};