// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PackedNormal.h"
#include "Components.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Rendering/StaticMeshVertexBuffer.h"
#include "RenderMath.h"

struct FSoftSkinVertex;

/**
* Base vertex data for GPU skinned skeletal meshes
*	make sure to update GpuSkinCacheCommon.usf if the member sizes/order change!
*/
struct TGPUSkinVertexBase
{
	// Tangent, U-direction
	FPackedNormal	TangentX;
	// Normal
	FPackedNormal	TangentZ;

	inline FVector3f GetTangentY() const
	{
		return FVector3f(GenerateYAxis(TangentX, TangentZ));
	}

	/** Serializer */
	void Serialize(FArchive& Ar);
};

/**
* 16 bit UV version of skeletal mesh vertex
*	make sure to update GpuSkinCacheCommon.usf if the member sizes/order change!
*/
template<uint32 InNumTexCoords>
struct TGPUSkinVertexFloat16Uvs : public TGPUSkinVertexBase
{
	constexpr static uint32 NumTexCoords = InNumTexCoords;
	constexpr static EStaticMeshVertexUVType StaticMeshVertexUVType = EStaticMeshVertexUVType::Default;

	/** full float position **/
	FVector3f			Position;
	/** half float UVs */
	FVector2DHalf	UVs[NumTexCoords];

	/**
	* Serializer
	*
	* @param Ar - archive to serialize with
	* @param V - vertex to serialize
	* @return archive that was used
	*/
	friend FArchive& operator<<(FArchive& Ar, TGPUSkinVertexFloat16Uvs& V)
	{
		V.Serialize(Ar);
		Ar << V.Position;

		for (uint32 UVIndex = 0; UVIndex < NumTexCoords; UVIndex++)
		{
			Ar << V.UVs[UVIndex];
		}
		return Ar;
	}
};

/**
* 32 bit UV version of skeletal mesh vertex
*	make sure to update GpuSkinCacheCommon.usf if the member sizes/order change!
*/
template<uint32 InNumTexCoords>
struct TGPUSkinVertexFloat32Uvs : public TGPUSkinVertexBase
{
	constexpr static uint32 NumTexCoords = InNumTexCoords;
	constexpr static EStaticMeshVertexUVType StaticMeshVertexUVType = EStaticMeshVertexUVType::HighPrecision;

	/** full float position **/
	FVector3f			Position;
	/** full float UVs */
	FVector2f UVs[NumTexCoords];

	/**
	* Serializer
	*
	* @param Ar - archive to serialize with
	* @param V - vertex to serialize
	* @return archive that was used
	*/
	friend FArchive& operator<<(FArchive& Ar, TGPUSkinVertexFloat32Uvs& V)
	{
		V.Serialize(Ar);
		Ar << V.Position;

		for (uint32 UVIndex = 0; UVIndex < NumTexCoords; UVIndex++)
		{
			Ar << V.UVs[UVIndex];
		}
		return Ar;
	}
};

/** An interface to the skel-mesh vertex data storage type. */
class FSkeletalMeshVertexDataInterface
{
public:

	/** Virtual destructor. */
	virtual ~FSkeletalMeshVertexDataInterface() = default;

	/**
	* Resizes the vertex data buffer, discarding any data which no longer fits.
	* @param NumVertices - The number of vertices to allocate the buffer for.
	*/
	virtual void ResizeBuffer(uint32 NumVertices) = 0;

	/** @return The stride of the vertex data in the buffer. */
	virtual uint32 GetStride() const = 0;

	/** @return A pointer to the data in the buffer. */
	virtual uint8* GetDataPointer() = 0;

	/** @return number of vertices in the buffer */
	virtual uint32 GetNumVertices() = 0;

	/** @return A pointer to the FResourceArrayInterface for the vertex data. */
	virtual FResourceArrayInterface* GetResourceArray() = 0;

	/** Serializer. */
	virtual void Serialize(FArchive& Ar) = 0;
};


/** The implementation of the skeletal mesh vertex data storage type. */
template<typename VertexDataType>
class TSkeletalMeshVertexData :
	public FSkeletalMeshVertexDataInterface,
	public TResourceArray<VertexDataType, VERTEXBUFFER_ALIGNMENT>
{
public:
	typedef TResourceArray<VertexDataType, VERTEXBUFFER_ALIGNMENT> ArrayType;

	/**
	* Constructor
	* @param InNeedsCPUAccess - true if resource array data should be CPU accessible
	*/
	TSkeletalMeshVertexData(bool InNeedsCPUAccess = false)
		: TResourceArray<VertexDataType, VERTEXBUFFER_ALIGNMENT>(InNeedsCPUAccess)
	{
	}

	/**
	* Resizes the vertex data buffer, discarding any data which no longer fits.
	*
	* @param NumVertices - The number of vertices to allocate the buffer for.
	*/
	virtual void ResizeBuffer(uint32 NumVertices)
	{
		if ((uint32)ArrayType::Num() < NumVertices)
		{
			// Enlarge the array.
			ArrayType::Reserve(NumVertices);
			ArrayType::AddUninitialized(NumVertices - ArrayType::Num());
		}
		else if ((uint32)ArrayType::Num() > NumVertices)
		{
			// Shrink the array.
			ArrayType::RemoveAt(NumVertices, ArrayType::Num() - NumVertices);
		}
	}
	/**
	* @return stride of the vertex type stored in the resource data array
	*/
	virtual uint32 GetStride() const
	{
		return sizeof(VertexDataType);
	}
	/**
	* @return uint8 pointer to the resource data array
	*/
	virtual uint8* GetDataPointer()
	{
		return (uint8*)&(*this)[0];
	}
	/**
	* @return number of vertices stored in the resource data array
	*/
	virtual uint32 GetNumVertices()
	{
		return ArrayType::Num();
	}
	/**
	* @return resource array interface access
	*/
	virtual FResourceArrayInterface* GetResourceArray()
	{
		return this;
	}
	/**
	* Serializer for this class
	*
	* @param Ar - archive to serialize to
	* @param B - data to serialize
	*/
	virtual void Serialize(FArchive& Ar)
	{
		ArrayType::BulkSerialize(Ar);
	}
	/**
	* Assignment operator. This is currently the only method which allows for
	* modifying an existing resource array
	*/
	TSkeletalMeshVertexData<VertexDataType>& operator=(const TArray<VertexDataType>& Other)
	{
		ArrayType::operator=(TArray<VertexDataType, TAlignedHeapAllocator<VERTEXBUFFER_ALIGNMENT> >(Other));
		return *this;
	}
};


/**
* Vertex buffer with static lod chunk vertices for use with GPU skinning
*/
class FDummySkeletalMeshVertexBuffer
{
public:
	~FDummySkeletalMeshVertexBuffer();

	/**
	* Serializer for this class
	* @param Ar - archive to serialize to
	* @param B - data to serialize
	*/
	friend FArchive& operator<<(FArchive& Ar, FDummySkeletalMeshVertexBuffer& VertexBuffer);

protected:
	// guaranteed only to be valid if the vertex buffer is valid
	FShaderResourceViewRHIRef SRVValue;

private:
	/** Corresponds to USkeletalMesh::bUseFullPrecisionUVs. if true then 32 bit UVs are used */
	bool bUseFullPrecisionUVs = false;
	/** true if this vertex buffer will be used with CPU skinning. Resource arrays are set to cpu accessible if this is true */
	bool bNeedsCPUAccess = false;
	/** The vertex data storage type */
	FSkeletalMeshVertexDataInterface* VertexData = nullptr;
	/** The cached vertex data pointer. */
	uint8* Data = nullptr;
	/** The cached vertex stride. */
	uint32 Stride = 0;
	/** The cached number of vertices. */
	uint32 NumVertices = 0;
	/** The number of unique texture coordinate sets in this buffer */
	uint32 NumTexCoords = 0;

	/**
	* Allocates the vertex data storage type. Based on UV precision needed
	*/
	void AllocateData();

	void CleanUp();
};
