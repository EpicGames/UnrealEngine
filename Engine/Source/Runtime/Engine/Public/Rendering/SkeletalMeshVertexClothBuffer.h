// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"
#include "SkeletalMeshTypes.h"

class FSkeletalMeshVertexDataInterface;

/**
* A vertex buffer for holding skeletal mesh clothing information only.
* This buffer sits along side the other skeletal mesh buffers per LOD
*/
class FSkeletalMeshVertexClothBuffer : public FVertexBuffer
{
public:
	/**
	* Constructor
	*/
	ENGINE_API FSkeletalMeshVertexClothBuffer();

	/**
	* Destructor
	*/
	ENGINE_API virtual ~FSkeletalMeshVertexClothBuffer();

	/**
	* Assignment. Assumes that vertex buffer will be rebuilt
	*/
	ENGINE_API FSkeletalMeshVertexClothBuffer& operator=(const FSkeletalMeshVertexClothBuffer& Other);

	/**
	* Constructor (copy)
	*/
	ENGINE_API FSkeletalMeshVertexClothBuffer(const FSkeletalMeshVertexClothBuffer& Other);

	/**
	* Delete existing resources
	*/
	void CleanUp();

	void ClearMetaData();

	/**
	* Initializes the buffer with the given vertices.
	* @param InVertices - The vertices to initialize the buffer with.
	* @param InClothIndexMapping - The cloth deformer mapping data offset for a specific section/LODBias.
	*/
	ENGINE_API void Init(const TArrayView<const FMeshToMeshVertData>& InMappingData, const TArrayView<const FClothBufferIndexMapping>& InClothIndexMapping);


	/**
	* Serializer for this class
	* @param Ar - archive to serialize to
	* @param B - data to serialize
	*/
	ENGINE_API friend FArchive& operator<<(FArchive& Ar, FSkeletalMeshVertexClothBuffer& VertexBuffer);

	void SerializeMetaData(FArchive& Ar);

	//~ Begin FRenderResource interface.

	/**
	* Initialize the RHI resource for this vertex buffer
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/**
	* Release the RHI resource for this vertex buffer
	*/
	virtual void ReleaseRHI() override;

	/**
	* @return text description for the resource type
	*/
	virtual FString GetFriendlyName() const override;

	//~ End FRenderResource interface.

	//~ Vertex data accessors.

	inline FMeshToMeshVertData& MappingData(uint32 VertexIndex)
	{
		checkSlow(VertexIndex < GetNumVertices());
		return *((FMeshToMeshVertData*)(Data + VertexIndex * Stride));
	}
	inline const FMeshToMeshVertData& MappingData(uint32 VertexIndex) const
	{
		checkSlow(VertexIndex < GetNumVertices());
		return *((FMeshToMeshVertData*)(Data + VertexIndex * Stride));
	}

	/**
	* @return number of vertices in this vertex buffer
	*/
	inline uint32 GetNumVertices() const
	{
		return NumVertices;
	}

	/**
	* @return cached stride for vertex data type for this vertex buffer
	*/
	inline uint32 GetStride() const
	{
		return Stride;
	}
	/**
	* @return total size of data in resource array
	*/
	inline uint32 GetVertexDataSize() const
	{
		return NumVertices * Stride;
	}

	inline FShaderResourceViewRHIRef GetSRV() const
	{
		return VertexBufferSRV;
	}

	inline const TArray<FClothBufferIndexMapping>& GetClothIndexMapping() const
	{
		return ClothIndexMapping;
	}

	/** Create an RHI vertex buffer with CPU data. CPU data may be discarded after creation (see TResourceArray::Discard) */
	FBufferRHIRef CreateRHIBuffer(FRHICommandListBase& RHICmdList);

	/** Similar to Init/ReleaseRHI but only update existing SRV so references to the SRV stays valid */
	void InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher);
	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

	ENGINE_API void SetMetadata(const TArray<FClothBufferIndexMapping>& ClothIndexMapping, uint32 Stride, uint32 NumVertices);
	
private:
	FShaderResourceViewRHIRef VertexBufferSRV;

	/** The Cloth deformer mapping to the simulation LODs. */
	TArray<FClothBufferIndexMapping> ClothIndexMapping;

	/** The vertex data storage type */
	FSkeletalMeshVertexDataInterface* VertexData;
	/** The cached vertex data pointer. */
	uint8* Data;
	/** The cached vertex stride. */
	uint32 Stride;
	/** The cached number of vertices. */
	uint32 NumVertices;

	/**
	* Allocates the vertex data storage type
	*/
	void AllocateData();
};