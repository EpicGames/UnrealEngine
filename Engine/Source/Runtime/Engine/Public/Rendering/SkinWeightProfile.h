// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneIndices.h"
#include "Misc/StringBuilder.h"
#include "Misc/RWSpinLock.h"
#include "RHIGPUReadback.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "RenderingThread.h"
#include "String/Join.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/PerPlatformProperties.h"

#include "SkinWeightProfile.generated.h"

class USkeletalMesh;
class UDebugSkelMeshComponent;

namespace SkeletalMeshImportData
{
	struct FVertInfluence;
}

extern ENGINE_API int32 GSkinWeightProfilesLoadByDefaultMode;
extern ENGINE_API int32 GSkinWeightProfilesDefaultLODOverride;
extern ENGINE_API int32 GSkinWeightProfilesAllowedFromLOD;

extern ENGINE_API FAutoConsoleVariableRef CVarSkinWeightsLoadByDefaultMode;
extern ENGINE_API FAutoConsoleVariableRef CVarSkinWeightProfilesDefaultLODOverride;
extern ENGINE_API FAutoConsoleVariableRef CVarSkinWeightProfilesAllowedFromLOD;

/** Structure storing user facing properties, and is used to identify profiles at the SkeletalMesh level*/
USTRUCT()
struct FSkinWeightProfileInfo
{
	GENERATED_BODY()

	/** Name of the Skin Weight Profile*/
	UPROPERTY(EditAnywhere, Category = SkinWeights)
	FName Name;
	
	/** Whether or not this Profile should be considered the Default loaded for specific LODs rather than the original Skin Weights of the Skeletal Mesh */
	UPROPERTY(EditAnywhere, Category = SkinWeights)
	FPerPlatformBool DefaultProfile;

	/** When DefaultProfile is set any LOD below this LOD Index will override the Skin Weights of the Skeletal Mesh with the Skin Weights from this Profile */
	UPROPERTY(EditAnywhere, Category = SkinWeights, meta=(EditCondition="DefaultProfile", ClampMin=0, DisplayName = "Default Profile from LOD Index"))
	FPerPlatformInt DefaultProfileFromLODIndex;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Category = SkinWeights)
	TMap<int32, FString> PerLODSourceFiles;
#endif
};

#if WITH_EDITORONLY_DATA

/** Editor only skin weight representation */
struct FRawSkinWeight
{
	// MAX_TOTAL_INFLUENCES for now
	FBoneIndexType InfluenceBones[MAX_TOTAL_INFLUENCES];
	uint16 InfluenceWeights[MAX_TOTAL_INFLUENCES];

	friend FArchive& operator<<(FArchive& Ar, FRawSkinWeight& OverrideEntry);
};

/** Editor only representation of a Skin Weight profile, stored as part of FSkeletalMeshLODModel, used as a base for generating the runtime version (FSkeletalRenderDataSkinWeightProfilesData) */
struct FImportedSkinWeightProfileData
{
	TArray<FRawSkinWeight> SkinWeights;

	//This is the result of the imported data before the chunking
	//We use this data every time we need to re-chunk the skeletal mesh
	TArray<SkeletalMeshImportData::FVertInfluence> SourceModelInfluences;

	friend FArchive& operator<<(FArchive& Ar, FImportedSkinWeightProfileData& ProfileData);
};

#endif // WITH_EDITORONLY_DATA

/** Runtime structure containing the set of override weights and the associated vertex indices */
struct FRuntimeSkinWeightProfileData
{
	/** Structure containing per Skin Weight offset and length */
	struct FSkinWeightOverrideInfo
	{
		/** Offset into FRuntimeSkinWeightOverrideData.Weights */
		uint32 InfluencesOffset;
#if WITH_EDITORONLY_DATA
		/** Number of influences to be read from FRuntimeSkinWeightOverrideData.Weights */
		uint8 NumInfluences_DEPRECATED;
#endif
		friend FArchive& operator<<(FArchive& Ar, FSkinWeightOverrideInfo& OverrideInfo);
	};

	void ApplyOverrides(FSkinWeightVertexBuffer* OverrideBuffer) const;	
	void ApplyDefaultOverride(FSkinWeightVertexBuffer* Buffer) const;

#if WITH_EDITORONLY_DATA
	/** Per skin weight offset into Weights array and number of weights stored */
	TArray<FSkinWeightOverrideInfo> OverridesInfo_DEPRECATED;
	/** Bulk data containing all Weights, stored as bone id in upper and weight in lower (8) bits */
	TArray<uint16> Weights_DEPRECATED;	
#endif 

	// Either contains FBoneIndexType or uint8 bone indices
	TArray<uint8> BoneIDs;
	TArray<uint8> BoneWeights;
	/** Map between Vertex Indices and the influence offset into BoneIDs/BoneWeights (DEPRECATED and entries of OverridesInfo) */
	TMap<uint32, uint32> VertexIndexToInfluenceOffset;

	uint8 NumWeightsPerVertex;
	bool b16BitBoneIndices;
	
	friend FArchive& operator<<(FArchive& Ar, FRuntimeSkinWeightProfileData& OverrideData);
};

struct FSkinWeightReadbackData
{
	mutable FCriticalSection Mutex;

	TUniquePtr<FRHIGPUBufferReadback> BufferReadback;
	TArray<uint8> ReadbackData;
	uint32 ReadbackFinishedFrameIndex = std::numeric_limits<uint32>::max();

	// Source VB size snapshotted at EnqueueCopy; mismatch vs ReadbackData.Num() means LOD restreamed mid-readback.
	// 0 = no readback in flight.
	uint32 CapturedSize = 0;

	// Debug: incremented each time a readback was skipped + retried due to size mismatch; logged on next successful readback.
	uint32 RetryCount = 0;
};

/** An identifier to identify a skin weight profile layer stack. 
  * Currently limited to two layers, but can be extended if needed.
  */
struct FSkinWeightProfileStack 
{
	static constexpr int32 MaxLayerCount = 2;
	TStaticArray<FName, MaxLayerCount> Layers;
	
	FSkinWeightProfileStack() :
		Layers{InPlace, NAME_None}
	{}

	static const FSkinWeightProfileStack& GetEmpty()
	{
		static const FSkinWeightProfileStack EmptyStack{};
		return EmptyStack;
	}

	// A fancy way of specifying a function with FName argument count between 1 and MaxLayerCount
	template <
		typename... ArgTypes
		UE_REQUIRES((sizeof...(ArgTypes) > 0 && sizeof...(ArgTypes) <= MaxLayerCount) && UE::Core::Private::TCanBeConvertedToFromAll_V<FName, ArgTypes...>)
	>
	explicit FSkinWeightProfileStack(ArgTypes&&... Args)
		: Layers{Forward<ArgTypes>(Args)...}
	{
		// Make sure that we mark any remaining elements are NAME_None to avoid corrupted names.  
		for (int32 LayerIndex = sizeof...(ArgTypes); LayerIndex < MaxLayerCount; ++LayerIndex)
		{
			Layers[LayerIndex] = NAME_None;
		}
	}

	explicit FSkinWeightProfileStack(const FName InProfileNames[MaxLayerCount])
	{
		for (int32 LayerIndex = 0; LayerIndex < MaxLayerCount; ++LayerIndex)
		{
			Layers[LayerIndex] = InProfileNames[LayerIndex];
		}
	}

	bool operator==(const FSkinWeightProfileStack& InProfileStack) const
	{
		return Layers == InProfileStack.Layers;
	}

	bool operator!=(const FSkinWeightProfileStack& InProfileStack) const
	{
		return Layers != InProfileStack.Layers;
	}

	FName operator[](int32 InLayerIndex) const
	{
		return Layers[InLayerIndex];
	}

	FName& operator[](int32 InLayerIndex)
	{
		return Layers[InLayerIndex];
	}

	bool IsEmpty() const
	{
		for (int32 LayerIndex = 0; LayerIndex < MaxLayerCount; ++LayerIndex)
		{
			if (!Layers[LayerIndex].IsNone())
			{
				return false;
			}
		}
		return true;
	}

	void CopyIntoArray(FName OutProfileNames[MaxLayerCount]) const
	{
		for (int32 LayerIndex = 0; LayerIndex < MaxLayerCount; ++LayerIndex)
		{
			OutProfileNames[LayerIndex] = Layers[LayerIndex];
		}
	}
	
	/** Returns a normalized layer stack, such that any empty layers are collapsed. */
	FSkinWeightProfileStack Normalized() const
	{
		FSkinWeightProfileStack NormalizedLayers;
		
		for (int32 ReadLayer = 0, WriteLayer = 0; ReadLayer < MaxLayerCount; ++ReadLayer)
		{
			if (!Layers[ReadLayer].IsNone())
			{
				NormalizedLayers.Layers[WriteLayer++] = Layers[ReadLayer];
			}
		}
		return NormalizedLayers;
	}
	
	FString GetUniqueId() const
	{
		TStringBuilder<256> IdString(InPlace, UE::String::JoinBy(Layers, [](FName InLayerName) { return InLayerName.ToString(); }, TEXT("-")));
		return IdString.ToString();
	}

	friend uint32 GetTypeHash(const FSkinWeightProfileStack& InProfileStack)
	{
		uint32 Hash = 0;
		for (int32 LayerIndex = 0; LayerIndex < MaxLayerCount; ++LayerIndex)
		{
			Hash = HashCombineFast(Hash, GetTypeHash(InProfileStack[LayerIndex]));
		}
		return Hash;
	}
};


/** Runtime structure for keeping track of skin weight profile(s) and the associated buffer */
struct FSkinWeightProfilesData
{
	ENGINE_API void Init(FSkinWeightVertexBuffer* InBaseBuffer);

	ENGINE_API ~FSkinWeightProfilesData();
	
	DECLARE_DELEGATE_RetVal_ThreeParams(int32 /** Index into Profiles ArrayView */, FOnPickOverrideSkinWeightProfile, const USkeletalMesh* /** Skeletal Mesh to pick the profile for */, const TArrayView<const FSkinWeightProfileInfo> /** Available skin weight profiles to pick from */, int32 /** LOD Index */);
	static ENGINE_API FOnPickOverrideSkinWeightProfile OnPickOverrideSkinWeightProfile;

#if !WITH_EDITOR
	// Mark this as non-editor only to prevent mishaps from users
	ENGINE_API void OverrideBaseBufferSkinWeightData(USkeletalMesh* Mesh, int32 LODIndex);
#endif 
	ENGINE_API void SetDynamicDefaultSkinWeightProfile(USkeletalMesh* Mesh, int32 LODIndex, bool bSerialization = false);	
	ENGINE_API void ClearDynamicDefaultSkinWeightProfile(USkeletalMesh* Mesh, int32 LODIndex);
	ENGINE_API void SetupDynamicDefaultSkinWeightProfile();
	FSkinWeightVertexBuffer* GetDefaultOverrideBuffer() const { return DefaultOverrideSkinWeightBuffer; }

	// Buffer lookup for layered profiles.
	ENGINE_API FSkinWeightVertexBuffer* GetOverrideBuffer(const FSkinWeightProfileStack& InProfileStack) const;
	bool ContainsOverrideBuffer(const FSkinWeightProfileStack& InProfileStack) const;
	
	// Lookups for individual profiles.
	ENGINE_API bool ContainsProfile(const FName& ProfileName) const;
	ENGINE_API const FRuntimeSkinWeightProfileData* GetOverrideData(const FName& ProfileName) const;
	ENGINE_API FRuntimeSkinWeightProfileData& AddOverrideData(const FName& ProfileName);

	ENGINE_API void ReleaseResources();

	ENGINE_API SIZE_T GetResourcesSize() const;
	ENGINE_API SIZE_T GetCPUAccessMemoryOverhead() const;
 
	friend FArchive& operator<<(FArchive& Ar, FSkinWeightProfilesData& OverrideData);

	ENGINE_API void SerializeMetaData(FArchive& Ar);

	ENGINE_API void ReleaseCPUResources();

	ENGINE_API void CreateRHIBuffers(FRHICommandListBase& RHICmdList, TArray<TPair<FSkinWeightProfileStack, FSkinWeightRHIInfo>>& OutBuffers);

	ENGINE_API void InitRHIForStreaming(const TArray<TPair<FSkinWeightProfileStack, FSkinWeightRHIInfo>>& IntermediateBuffers, FRHIResourceReplaceBatcher& Batcher);
	ENGINE_API void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

	bool IsDefaultOverridden() const { return bDefaultOverridden; }
	bool IsStaticOverridden() const { return bStaticOverridden; }
	FSkinWeightProfileStack GetDefaultProfileStack() const { return DefaultProfileStack; }

	enum class EReadbackStatus : uint32
	{
		None = 0,
		PendingGPUReadback,
		GPUReadbackEnqueued,
		DataReadbackEnqueued,
		DataReadbackFinished,
		BufferInitialized
	};

	inline EReadbackStatus GetReadbackStatus() const { return (EReadbackStatus)ReadbackStatus.Load(); }
	inline const FSkinWeightProfileStack& GetPendingProfileStack() const { return PendingProfileStack; }
	inline void SetPendingProfileStack(FSkinWeightProfileStack InProfileStack) { PendingProfileStack = InProfileStack; }
	
	ENGINE_API void PumpGPUReadback();

	/**
	* @returns	True if a call to EnqueueGPUReadback would actually issue an RT command that
	*			produces a GPU readback (i.e. base buffer / weight data is available, and CPU
	*			data is not already a usable substitute). Use this to decide whether it is
	*			worth registering stream-out protection or otherwise paying the readback cost.
	*/
	ENGINE_API bool ShouldEnqueueGPUReadback() const;

	/**
	* @returns	Returns true if GPU Readback data is ready and buffer has been initialized. Otherwise,
	*			returns false and ticks the GPU Readback pump. If true, it is safe to update the skin
	*			weight override buffer for given Skeletal Mesh LOD this Skin Weight Data is associated
	*			with.
	*/
	ENGINE_API bool DoReadbackIfNeeded(const FSkinWeightProfileStack& InProfileStack);

	static ENGINE_API bool AllowCPU();
	static ENGINE_API bool HandleDelayedLoads();
	
private:
	friend class FSkinWeightProfileManager;
	friend class FSkinWeightProfileManagerAsyncTask;
	friend class FSkeletalMeshLODRenderData;

	inline void SetReadbackStatus(EReadbackStatus InStatus) { ReadbackStatus.Store((uint32)InStatus); }
	bool HasProfileStack(const FSkinWeightProfileStack& InProfileStack) const;
	void InitialiseProfileBuffer(const FSkinWeightProfileStack& InProfileStack);
	void ReleaseBuffer(const FSkinWeightProfileStack& InProfileStack, bool bForceRelease = false);

	void ApplyOverrideProfileStack(const FSkinWeightProfileStack& InProfileStack, FSkinWeightVertexBuffer* OverrideBuffer, const uint8* BaseBufferData = nullptr);
	bool IsPendingReadback() const;
	void EnqueueGPUReadback();
	bool IsGPUReadbackFinished() const;
	void EnqueueDataReadback();
	bool IsDataReadbackPending() const;
	bool IsDataReadbackFinished() const;
	void ResetGPUReadback();

	FSkinWeightVertexBuffer* BaseBuffer = nullptr;
	FSkinWeightVertexBuffer* DefaultOverrideSkinWeightBuffer = nullptr;

	TMap<FSkinWeightProfileStack, FSkinWeightVertexBuffer*> ProfileStackToBuffer;

	// Guards both the ProfileStackToBuffer map structure AND the contents of the
	// FSkinWeightVertexBuffer values stored in it (WeightData, Data, NumBoneWeights,
	// VertexBufferRHI). Any code path that mutates buffer contents — Apply, Init,
	// CreateRHIBuffer, InitRHI/ReleaseRHIForStreaming — must hold the WRITE lock.
	// The READ lock is only valid for "look up a pointer and stop" — once any
	// member of the buffer is touched (mutate or read of mutable state), promote
	// to write. This is required because Apply performs delete + new on WeightData
	// and concurrent readers must not observe a torn buffer.
	mutable UE::FRWSpinLock ProfileBufferMapLock;
	
	TMap<FName, FRuntimeSkinWeightProfileData> OverrideData;

	bool bDefaultOverridden = false;
	bool bStaticOverridden = false;
	
	FSkinWeightProfileStack DefaultProfileStack;

	FSkinWeightReadbackData ReadbackData;

	// Atomic GPU readback status as this can be modified from within
	// the render thread
	TAtomic<uint32> ReadbackStatus = 0;

	FSkinWeightProfileStack PendingProfileStack;
	
	mutable bool bEmittedOverrideWarning = false;
	
	mutable FCriticalSection OverrideDataLock;
};

