// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"
#include "StaticMeshVertexData.h"
#include "GPUSkinPublicDefs.h"
#include "SkeletalMeshTypes.h"
#include "GPUSkinVertexFactory.h"
#include "UObject/AnimObjectVersion.h"

class FStaticMeshVertexDataInterface;
struct FSoftSkinVertex;

typedef uint8 FBoneIndex8;
typedef uint16 FBoneIndex16;

/** Struct for storing skin weight info in vertex buffer */
template <bool bExtraBoneInfluences, typename BoneIndexType>
struct TLegacySkinWeightInfo
{
	enum
	{
		NumInfluences = bExtraBoneInfluences ? EXTRA_BONE_INFLUENCES : MAX_INFLUENCES_PER_STREAM,
	};
	BoneIndexType	InfluenceBones[NumInfluences];
	uint8			InfluenceWeights[NumInfluences];

	friend FArchive& operator<<(FArchive& Ar, TLegacySkinWeightInfo& I)
	{
		Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
		// serialize bone index and weight arrays in order
		// this is required when serializing as bulk data memory (see TArray::BulkSerialize notes)
		for (uint32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; InfluenceIndex++)
		{
			if (Ar.IsLoading() && Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::IncreaseBoneIndexLimitPerChunk)
			{
				// Older versions use uint8 bone indices
				uint8 BoneIndex = 0;
				Ar << BoneIndex;
				I.InfluenceBones[InfluenceIndex] = BoneIndex;
			}
			else
			{
				Ar << I.InfluenceBones[InfluenceIndex];
			}
		}

		for (uint32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; InfluenceIndex++)
		{
			Ar << I.InfluenceWeights[InfluenceIndex];
		}

		return Ar;
	}
};

/** An runtime structure for passing data to FSkinWeightVertexBuffer */
struct FSkinWeightInfo
{
	FBoneIndexType	InfluenceBones[MAX_TOTAL_INFLUENCES];
	uint16			InfluenceWeights[MAX_TOTAL_INFLUENCES];
};

struct FSkinWeightRHIInfo
{
	FBufferRHIRef DataVertexBufferRHI;
	FBufferRHIRef LookupVertexBufferRHI;
};

/** A lookup vertex buffer storing skin weight stream offset / influence count. Only used for unlimited bone influences. */
class FSkinWeightLookupVertexBuffer : public FVertexBuffer
{
public:
	/** Default constructor. */
	ENGINE_API FSkinWeightLookupVertexBuffer();

	/** Constructor (copy) */
	ENGINE_API FSkinWeightLookupVertexBuffer(const FSkinWeightLookupVertexBuffer& Other);

	/** Destructor. */
	ENGINE_API ~FSkinWeightLookupVertexBuffer();

	/** Assignment. Assumes that vertex buffer will be rebuilt */
	ENGINE_API FSkinWeightLookupVertexBuffer& operator=(const FSkinWeightLookupVertexBuffer& Other);

	/** Delete existing resources */
	ENGINE_API void CleanUp();

	/** @return true is LookupData is valid */
	bool IsLookupDataValid() const;

	const uint8* GetLookupData() const
	{
		return Data;
	}

	uint8* GetLookupData()
	{
		return Data;
	}

	void Init(uint32 InNumVertices);

	friend FArchive& operator<<(FArchive& Ar, FSkinWeightLookupVertexBuffer& VertexBuffer);

	void SerializeMetaData(FArchive& Ar);
	ENGINE_API void SetMetaData(uint32 InNumVertices);
	void CopyMetaData(const FSkinWeightLookupVertexBuffer& Other);

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	virtual FString GetFriendlyName() const override
	{ return TEXT("SkeletalMesh Vertex Weights Lookup"); }

	/** @return number of vertices in this vertex buffer */
	inline uint32 GetNumVertices() const
	{ return NumVertices; }

	/** @return cached stride for vertex data type for this vertex buffer */
	inline uint32 GetStride() const
	{ return sizeof(uint32); }

	/** @return total size of data in resource array */
	inline uint32 GetVertexDataSize() const
	{ return NumVertices * GetStride(); }

	// @param guaranteed only to be valid if the vertex buffer is valid
	FRHIShaderResourceView* GetSRV() const
	{ return SRVValue; }

	/** Set if the CPU needs access to this vertex buffer */
	void SetNeedsCPUAccess(bool bInNeedsCPUAccess)
	{ bNeedsCPUAccess = bInNeedsCPUAccess; }

	bool GetNeedsCPUAccess() const
	{ return bNeedsCPUAccess; }

	/** Create an RHI vertex buffer with CPU data. CPU data may be discarded after creation (see TResourceArray::Discard) */
	FBufferRHIRef CreateRHIBuffer(FRHICommandListBase& RHICmdList);

	/** Similar to Init/ReleaseRHI but only update existing SRV so references to the SRV stays valid */
	void InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher);
	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

	void GetWeightOffsetAndInfluenceCount(uint32 VertexIndex, uint32& OutWeightOffset, uint32& OutInfluenceCount) const;
	void SetWeightOffsetAndInfluenceCount(uint32 VertexIndex, uint32 WeightOffset, uint32 InfluenceCount);

protected:
	// guaranteed only to be valid if the vertex buffer is valid
	FShaderResourceViewRHIRef SRVValue;

private:
	void ResizeBuffer(uint32 InNumVertices);

	/** Allocates the vertex data storage type. */
	ENGINE_API void AllocateData();

	/** true if this vertex buffer will be used with CPU skinning. Resource arrays are set to cpu accessible if this is true */
	bool bNeedsCPUAccess;

	/** The vertex data storage type */
	FStaticMeshVertexDataInterface* LookupData;

	/** The cached vertex data pointer. */
	uint8* Data;

	/** The cached number of vertices. */
	uint32 NumVertices;

	static const EPixelFormat PixelFormat = PF_R32_UINT;
	static const uint32 PixelFormatStride = 4;
};

/** The implementation of the skin weight vertex data storage type. */
template<typename VertexDataType>
class FSkinWeightVertexData : public TStaticMeshVertexData<VertexDataType>
{
public:
	FSkinWeightVertexData(bool InNeedsCPUAccess = false)
		: TStaticMeshVertexData<VertexDataType>(InNeedsCPUAccess)
	{
	}
};

/** A vertex buffer storing bone index/weight data. */
class FSkinWeightDataVertexBuffer : public FVertexBuffer
{
public:
	/** Default constructor. */
	ENGINE_API FSkinWeightDataVertexBuffer();

	/** Constructor (copy) */
	ENGINE_API FSkinWeightDataVertexBuffer(const FSkinWeightDataVertexBuffer& Other);

	/** Destructor. */
	ENGINE_API ~FSkinWeightDataVertexBuffer();

	/** Assignment. Assumes that vertex buffer will be rebuilt */
	ENGINE_API FSkinWeightDataVertexBuffer& operator=(const FSkinWeightDataVertexBuffer& Other);

	/** Delete existing resources */
	ENGINE_API void CleanUp();

	ENGINE_API void Init(uint32 InNumWeights, uint32 InNumVertices);

	friend FArchive& operator<<(FArchive& Ar, FSkinWeightDataVertexBuffer& VertexBuffer);

	void SerializeMetaData(FArchive& Ar);
	ENGINE_API void SetMetaData(uint32 InNumVertices, uint32 InMaxBoneInfluences, bool bInUse16BitBoneIndex, bool bInUse16BitBoneWeight);
	void CopyMetaData(const FSkinWeightDataVertexBuffer& Other);

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;

	virtual FString GetFriendlyName() const override 
	{ return TEXT("SkeletalMesh Vertex Weights Data"); }

	/** @return number of vertices in this vertex buffer */
	inline uint32 GetNumVertices() const
	{ return NumVertices; }

	/** @return number of bone weights in this vertex buffer */
	inline uint32 GetNumBoneWeights() const
	{ return NumBoneWeights; }

	/** @return byte size of each bone index */
	inline uint32 GetBoneIndexByteSize() const
	{ return (Use16BitBoneIndex() ? sizeof(FBoneIndex16) : sizeof(FBoneIndex8)); }

	inline uint32 GetBoneWeightByteSize() const
	{
		return Use16BitBoneWeight() ? sizeof(uint16) : sizeof(uint8);
	}

	inline uint32 GetBoneIndexAndWeightByteSize() const
	{
		return GetBoneIndexByteSize() + GetBoneWeightByteSize(); 
	}

	/** @return vertex stride for when using constant number of bones per vertex buffer */
	inline uint32 GetConstantInfluencesVertexStride() const
	{ return GetBoneIndexAndWeightByteSize() * MaxBoneInfluences; }

	/** @return offset position for bone weights data for each vertex */
	inline uint32 GetConstantInfluencesBoneWeightsOffset() const
	{ return GetBoneIndexByteSize() * MaxBoneInfluences; }

	/** @return total size of data in resource array */
	inline uint32 GetVertexDataSize() const
	{ return NumBoneWeights * GetBoneIndexAndWeightByteSize(); }

	// @param guaranteed only to be valid if the vertex buffer is valid
	FRHIShaderResourceView* GetSRV() const
	{ return SRVValue; }

	/** Set if the CPU needs access to this vertex buffer */
	inline void SetNeedsCPUAccess(bool bInNeedsCPUAccess)
	{ bNeedsCPUAccess = bInNeedsCPUAccess; }

	inline bool GetNeedsCPUAccess() const
	{ return bNeedsCPUAccess; }

	inline bool GetVariableBonesPerVertex() const
	{ return bVariableBonesPerVertex; }

	/** Set if this will have extra streams for bone indices & weights. */
	ENGINE_API void SetMaxBoneInfluences(uint32 InMaxBoneInfluences);

	inline uint32 GetMaxBoneInfluences() const
	{ return MaxBoneInfluences; }

	inline void SetUse16BitBoneIndex(bool bInUse16BitBoneIndex)
	{ bUse16BitBoneIndex = bInUse16BitBoneIndex; }

	inline bool Use16BitBoneIndex() const
	{ return bUse16BitBoneIndex; }
	
	void SetUse16BitBoneWeight(bool bInUse16BitBoneWeight)
	{ bUse16BitBoneWeight = bInUse16BitBoneWeight; }

	bool Use16BitBoneWeight() const
	{ return bUse16BitBoneWeight; }
	
	

	ENGINE_API GPUSkinBoneInfluenceType GetBoneInfluenceType() const;
	ENGINE_API bool GetRigidWeightBone(uint32 VertexWeightOffset, uint32 VertexInfluenceCount, int32& OutBoneIndex) const;
	ENGINE_API uint32 GetBoneIndex(uint32 VertexWeightOffset, uint32 VertexInfluenceCount, uint32 InfluenceIndex) const;
	ENGINE_API void SetBoneIndex(uint32 VertexWeightOffset, uint32 VertexInfluenceCount, uint32 InfluenceIndex, uint32 BoneIndex);
	ENGINE_API uint16 GetBoneWeight(uint32 VertexWeightOffset, uint32 VertexInfluenceCount, uint32 InfluenceIndex) const;
	ENGINE_API void SetBoneWeight(uint32 VertexWeightOffset, uint32 VertexInfluenceCount, uint32 InfluenceIndex, uint16 BoneWeight);
	ENGINE_API void ResetVertexBoneWeights(uint32 VertexWeightOffset, uint32 VertexInfluenceCount);

	ENGINE_API void CopyDataFromBuffer(const uint8* InSkinWeightData, uint32 InNumVertices);

	/** Create an RHI vertex buffer with CPU data. CPU data may be discarded after creation (see TResourceArray::Discard) */
	FBufferRHIRef CreateRHIBuffer(FRHICommandListBase& RHICmdList);

	/** Similar to Init/ReleaseRHI but only update existing SRV so references to the SRV stays valid */
	void InitRHIForStreaming(FRHIBuffer* IntermediateBuffer, FRHIResourceReplaceBatcher& Batcher);
	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

	bool IsWeightDataValid() const;

	const uint8* GetWeightData() const
	{
		return Data;
	}

	uint8* GetWeightData()
	{
		return Data;
	}
	
protected:
	// guaranteed only to be valid if the vertex buffer is valid
	FShaderResourceViewRHIRef SRVValue;

private:
	void ResizeBuffer(uint32 InNumWeights, uint32 InNumVertices);

	EPixelFormat GetPixelFormat() const
	{ return bVariableBonesPerVertex ? PF_R8_UINT : PF_R32_UINT; }

	uint32 GetPixelFormatStride() const
	{ return bVariableBonesPerVertex ? 1 : 4; }

	/** true if this vertex buffer will be used with CPU skinning. Resource arrays are set to cpu accessible if this is true */
	bool bNeedsCPUAccess;

	bool bVariableBonesPerVertex;

	/** Has extra bone influences per Vertex, which means using a different TGPUSkinVertexBase */
	uint32 MaxBoneInfluences;

	/** Use 16 bit bone index instead of 8 bit */
	bool bUse16BitBoneIndex;

	/** Use 16 bit bone weight instead of 8 bit */
	bool bUse16BitBoneWeight;

	/** The vertex data storage type */
	FStaticMeshVertexDataInterface* WeightData;

	/** The cached vertex data pointer. */
	uint8* Data;

	/** The cached number of vertices. */
	uint32 NumVertices;

	/** Total number of bone weights across all vertices */
	uint32 NumBoneWeights;

	/** Allocates the vertex data storage type. */
	ENGINE_API void AllocateData();
};

/** A container for skin weights data vertex buffer and lookup vertex buffer. */
class FSkinWeightVertexBuffer
{
public:
	/** Default constructor. */
	ENGINE_API FSkinWeightVertexBuffer();

	/** Constructor (copy) */
	ENGINE_API FSkinWeightVertexBuffer(const FSkinWeightVertexBuffer& Other);

	/** Destructor. */
	ENGINE_API ~FSkinWeightVertexBuffer();

	/** Assignment. Assumes that vertex buffer will be rebuilt */
	ENGINE_API FSkinWeightVertexBuffer& operator=(const FSkinWeightVertexBuffer& Other);

	/** Delete existing resources */
	ENGINE_API void CleanUp();

#if WITH_EDITOR
	/** Init from another skin weight buffer */
	ENGINE_API void Init(const TArray<FSoftSkinVertex>& InVertices);
#endif // WITH_EDITOR

	/** Assignment operator for assigning array of weights to this buffer */
	ENGINE_API FSkinWeightVertexBuffer& operator=(const TArray<FSkinWeightInfo>& InWeights);
	ENGINE_API void GetSkinWeights(TArray<FSkinWeightInfo>& OutVertices) const;
	ENGINE_API FSkinWeightInfo GetVertexSkinWeights(uint32 VertexIndex) const;

	ENGINE_API void CopySkinWeightRawDataFromBuffer(const uint8* InSkinWeightData, uint32 InNumVertices);

	ENGINE_API friend FArchive& operator<<(FArchive& Ar, FSkinWeightVertexBuffer& VertexBuffer);

	void SerializeMetaData(FArchive& Ar);
	void CopyMetaData(const FSkinWeightVertexBuffer& Other);

	/** @return number of vertices in this vertex buffer */
	inline uint32 GetNumVertices() const
	{ return DataVertexBuffer.GetNumVertices(); }

	/** @return total size of data in resource array */
	inline uint32 GetVertexDataSize() const
	{ return LookupVertexBuffer.GetVertexDataSize() + DataVertexBuffer.GetVertexDataSize(); }

	void SetNeedsCPUAccess(bool bInNeedsCPUAccess)
	{
		DataVertexBuffer.SetNeedsCPUAccess(bInNeedsCPUAccess);
		LookupVertexBuffer.SetNeedsCPUAccess(bInNeedsCPUAccess);
	}

	bool GetNeedsCPUAccess() const
	{ return DataVertexBuffer.GetNeedsCPUAccess(); }

	void SetMaxBoneInfluences(uint32 InMaxBoneInfluences)
	{ DataVertexBuffer.SetMaxBoneInfluences(InMaxBoneInfluences); }

	uint32 GetMaxBoneInfluences() const
	{ return DataVertexBuffer.GetMaxBoneInfluences(); }

	void SetUse16BitBoneIndex(bool bInUse16BitBoneIndex)
	{ DataVertexBuffer.SetUse16BitBoneIndex(bInUse16BitBoneIndex); }

	bool Use16BitBoneIndex() const
	{ return DataVertexBuffer.Use16BitBoneIndex(); }

	void SetUse16BitBoneWeight(bool bInUse16BitBoneWeight)
	{ DataVertexBuffer.SetUse16BitBoneWeight(bInUse16BitBoneWeight); }
	
	bool Use16BitBoneWeight() const
	{ return DataVertexBuffer.Use16BitBoneWeight(); }
	
	uint32 GetBoneIndexByteSize() const
	{ return DataVertexBuffer.GetBoneIndexByteSize(); }

	uint32 GetBoneWeightByteSize() const
	{ return DataVertexBuffer.GetBoneWeightByteSize(); }
	
	uint32 GetBoneIndexAndWeightByteSize() const
	{ return DataVertexBuffer.GetBoneIndexAndWeightByteSize(); }
	
	bool GetVariableBonesPerVertex() const
	{ return DataVertexBuffer.GetVariableBonesPerVertex(); }

	uint32 GetConstantInfluencesVertexStride() const
	{ return DataVertexBuffer.GetConstantInfluencesVertexStride(); }

	uint32 GetConstantInfluencesBoneWeightsOffset() const
	{ return DataVertexBuffer.GetConstantInfluencesBoneWeightsOffset(); }

	FSkinWeightDataVertexBuffer* GetDataVertexBuffer()
	{ return &DataVertexBuffer; }
	
	const FSkinWeightDataVertexBuffer* GetDataVertexBuffer() const
	{ return &DataVertexBuffer; }

	const FSkinWeightLookupVertexBuffer* GetLookupVertexBuffer() const
	{ return &LookupVertexBuffer; }

	FSkinWeightRHIInfo CreateRHIBuffer(FRHICommandListBase& RHICmdList);

	ENGINE_API GPUSkinBoneInfluenceType GetBoneInfluenceType() const;
	ENGINE_API void GetVertexInfluenceOffsetCount(uint32 VertexIndex, uint32& VertexWeightOffset, uint32& VertexInfluenceCount) const;
	ENGINE_API bool GetRigidWeightBone(uint32 VertexIndex, int32& OutBoneIndex) const;
	ENGINE_API uint32 GetBoneIndex(uint32 VertexIndex, uint32 InfluenceIndex) const;
	ENGINE_API void SetBoneIndex(uint32 VertexIndex, uint32 InfluenceIndex, uint32 BoneIndex);
	ENGINE_API uint16 GetBoneWeight(uint32 VertexIndex, uint32 InfluenceIndex) const;
	ENGINE_API void SetBoneWeight(uint32 VertexIndex, uint32 InfluenceIndex, uint16 BoneWeight);
	ENGINE_API void ResetVertexBoneWeights(uint32 VertexIndex);
	
	/** Should only be called for legacy meshes or meshes that don't really have a LookupVertexBuffer, for example created in realtime.
		All variable influence meshes imported after the FAnimObjectVersion::UnlimitedBoneInfluences version already have a LookupVertexBuffer */
	ENGINE_API void RebuildLookupVertexBuffer();

	/** SetOwnerName should be called before BeginInitResources for the owner name to be successfully tracked. */
	ENGINE_API void SetOwnerName(const FName& OwnerName);
	ENGINE_API void BeginInitResources();
	ENGINE_API void BeginReleaseResources();
	ENGINE_API void ReleaseResources();

	/** Similar to Init/ReleaseRHI but only update existing SRV so references to the SRV stays valid */
	void InitRHIForStreaming(const FSkinWeightRHIInfo& RHIInfo, FRHIResourceReplaceBatcher& Batcher)
	{
		DataVertexBuffer.InitRHIForStreaming(RHIInfo.DataVertexBufferRHI, Batcher);
		LookupVertexBuffer.InitRHIForStreaming(RHIInfo.LookupVertexBufferRHI, Batcher);
	}

	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
	{
		DataVertexBuffer.ReleaseRHIForStreaming(Batcher);
		LookupVertexBuffer.ReleaseRHIForStreaming(Batcher);
	}

private:
	/** Skin weights for skinning */
	FSkinWeightDataVertexBuffer		DataVertexBuffer;

	/** Skin weights lookup buffer */
	FSkinWeightLookupVertexBuffer	LookupVertexBuffer;
};
