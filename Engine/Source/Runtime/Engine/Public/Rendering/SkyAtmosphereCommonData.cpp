// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyAtmosphereCommonData.cpp
=============================================================================*/

#include "SkyAtmosphereCommonData.h"

#include "Components/SkyAtmosphereComponent.h"
#include "ColorManagement/ColorSpace.h"
#include "StateStream/SkyAtmosphereStateStream.h"

//PRAGMA_DISABLE_OPTIMIZATION

const float FAtmosphereSetup::CmToSkyUnit = 0.00001f;			// Centimeters to Kilometers
const float FAtmosphereSetup::SkyUnitToCm = 1.0f / 0.00001f;	// Kilometers to Centimeters

FTentDistribution GetTentDistribution(const USkyAtmosphereComponent& SkyAtmosphereComponent) { return SkyAtmosphereComponent.OtherTentDistribution; }
FTentDistribution GetTentDistribution(const FSkyAtmosphereDynamicState& Ds)
{
	FTentDistribution D;
	D.TipAltitude = Ds.OtherTentDistributionTipAltitude;
	D.TipValue = Ds.OtherTentDistributionTipValue;
	D.Width = Ds.OtherTentDistributionWidth;
	return D;
}


void FAtmosphereSetup::ComputeAtmosphereVersion()
{
	uint32 Crc = 0;
	Crc = FCrc::MemCrc32((void*)&BottomRadiusKm,					sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&TopRadiusKm,						sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&MultiScatteringFactor,				sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&RayleighScattering.R,				4 * sizeof(float),	Crc);
	Crc = FCrc::MemCrc32((void*)&RayleighDensityExpScale,			sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&MieScattering.R,					4 * sizeof(float),	Crc);
	Crc = FCrc::MemCrc32((void*)&MieExtinction.R,					4 * sizeof(float),	Crc);
	Crc = FCrc::MemCrc32((void*)&MieAbsorption.R,					4 * sizeof(float),	Crc);
	Crc = FCrc::MemCrc32((void*)&MieDensityExpScale,				sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&MiePhaseG,							sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&AbsorptionExtinction.R,			4 * sizeof(float),	Crc);
	Crc = FCrc::MemCrc32((void*)&AbsorptionDensity0LayerWidth,		sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&AbsorptionDensity0ConstantTerm,	sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&AbsorptionDensity0LinearTerm,		sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&AbsorptionDensity1ConstantTerm,	sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&AbsorptionDensity1LinearTerm,		sizeof(float),		Crc);
	Crc = FCrc::MemCrc32((void*)&GroundAlbedo.R,					4 * sizeof(float),	Crc);
	TransmittanceAndMultiScatteringLUTsVersion = Crc;
}

template<typename T> void FAtmosphereSetup::InternalInit(const T& SkyAtmosphereComponent)
{
	// Convert Tent distribution to linear curve coefficients.
	auto TentToCoefficients = [](const FTentDistribution& Tent, float& LayerWidth, float& LinTerm0, float&  LinTerm1, float& ConstTerm0, float& ConstTerm1)
	{
		if (Tent.Width > 0.0f && Tent.TipValue > 0.0f)
		{
			const float px = Tent.TipAltitude;
			const float py = Tent.TipValue;
			const float slope = Tent.TipValue / Tent.Width;
			LayerWidth = px;
			LinTerm0 = slope;
			LinTerm1 = -slope;
			ConstTerm0 = py - px * LinTerm0;
			ConstTerm1 = py - px * LinTerm1;
		}
		else
		{
			LayerWidth = 0.0f;
			LinTerm0 = 0.0f;
			LinTerm1 = 0.0f;
			ConstTerm0 = 0.0f;
			ConstTerm1 = 0.0f;
		}
	};

	BottomRadiusKm = SkyAtmosphereComponent.BottomRadius;
	TopRadiusKm = SkyAtmosphereComponent.BottomRadius + FMath::Max(0.1f, SkyAtmosphereComponent.AtmosphereHeight);
	GroundAlbedo = FLinearColor(SkyAtmosphereComponent.GroundAlbedo);
	MultiScatteringFactor = FMath::Clamp(SkyAtmosphereComponent.MultiScatteringFactor, 0.0f, 100.0f);

	auto ConvertCoefficientsFromSRGBToWorkingColorSpace = [](FLinearColor CoeffSRGB)
	{
		using namespace UE::Color;

		const FColorSpace& WorkingColorSpace = FColorSpace::GetWorking();
		if (WorkingColorSpace.IsSRGB())
		{
			return CoeffSRGB;
		}
		else
		{
			// Compute the transmittance color from the coefficients.
			FLinearColor Transmittance = FLinearColor(
				FMath::Exp(-CoeffSRGB.R),
				FMath::Exp(-CoeffSRGB.G),
				FMath::Exp(-CoeffSRGB.B));

			// Convert transmittance color from sRGB to working color space.
			Transmittance = FColorSpaceTransform::GetSRGBToWorkingColorSpace().Apply(Transmittance);

			// New we have a transmittance in working color space, convert it back to coefficients for this working color space.
			return FLinearColor(
				-FMath::Loge(FMath::Max(0.00001, Transmittance.R)),
				-FMath::Loge(FMath::Max(0.00001, Transmittance.G)),
				-FMath::Loge(FMath::Max(0.00001, Transmittance.B)));
		}
	};

	// Rayleigh scattering
	{
		RayleighScattering = (SkyAtmosphereComponent.RayleighScattering * SkyAtmosphereComponent.RayleighScatteringScale).GetClamped(0.0f, 1e38f);
		RayleighScattering = ConvertCoefficientsFromSRGBToWorkingColorSpace(RayleighScattering);

		RayleighDensityExpScale = -1.0f / SkyAtmosphereComponent.RayleighExponentialDistribution;
	}

	// Mie scattering
	{

		MieScattering = (SkyAtmosphereComponent.MieScattering * SkyAtmosphereComponent.MieScatteringScale).GetClamped(0.0f, 1e38f);
		MieScattering = ConvertCoefficientsFromSRGBToWorkingColorSpace(MieScattering);

		MieAbsorption = (SkyAtmosphereComponent.MieAbsorption * SkyAtmosphereComponent.MieAbsorptionScale).GetClamped(0.0f, 1e38f);
		MieAbsorption = ConvertCoefficientsFromSRGBToWorkingColorSpace(MieAbsorption);

		MieExtinction = MieScattering + MieAbsorption;
		MiePhaseG = SkyAtmosphereComponent.MieAnisotropy;
		MieDensityExpScale = -1.0f / SkyAtmosphereComponent.MieExponentialDistribution;
	}

	// Ozone
	{
		AbsorptionExtinction = (SkyAtmosphereComponent.OtherAbsorption * SkyAtmosphereComponent.OtherAbsorptionScale).GetClamped(0.0f, 1e38f);
		AbsorptionExtinction = ConvertCoefficientsFromSRGBToWorkingColorSpace(AbsorptionExtinction);

		TentToCoefficients(GetTentDistribution(SkyAtmosphereComponent), AbsorptionDensity0LayerWidth, AbsorptionDensity0LinearTerm, AbsorptionDensity1LinearTerm, AbsorptionDensity0ConstantTerm, AbsorptionDensity1ConstantTerm);
	}

	TransmittanceMinLightElevationAngle = SkyAtmosphereComponent.TransmittanceMinLightElevationAngle;

	UpdateTransform(SkyAtmosphereComponent.GetComponentTransform(), uint8(SkyAtmosphereComponent.TransformMode));

	ComputeAtmosphereVersion();
}


FAtmosphereSetup::FAtmosphereSetup(const USkyAtmosphereComponent& SkyAtmosphereComponent)
{
	InternalInit(SkyAtmosphereComponent);
}

FAtmosphereSetup::FAtmosphereSetup(const FSkyAtmosphereDynamicState& Ds)
{
	InternalInit(Ds);
}

void FAtmosphereSetup::ApplyWorldOffset(const FVector& InOffset)
{
	PlanetCenterKm += InOffset * double(FAtmosphereSetup::CmToSkyUnit);
}

void FAtmosphereSetup::UpdateTransform(const FTransform& ComponentTransform, uint8 TranformMode)
{
	switch (ESkyAtmosphereTransformMode(TranformMode))
	{
	case ESkyAtmosphereTransformMode::PlanetTopAtAbsoluteWorldOrigin:
		PlanetCenterKm = FVector(0.0f, 0.0f, -BottomRadiusKm);
		break;
	case ESkyAtmosphereTransformMode::PlanetTopAtComponentTransform:
		PlanetCenterKm = FVector(0.0f, 0.0f, -BottomRadiusKm) + ComponentTransform.GetTranslation() * double(FAtmosphereSetup::CmToSkyUnit);
		break;
	case ESkyAtmosphereTransformMode::PlanetCenterAtComponentTransform:
		PlanetCenterKm = ComponentTransform.GetTranslation() * double(FAtmosphereSetup::CmToSkyUnit);
		break;
	default:
		check(false);
	}
}

FLinearColor FAtmosphereSetup::GetTransmittanceAtGroundLevel(const FVector& SunDirection) const
{
	// The following code is from SkyAtmosphere.usf and has been converted to lambda functions. 
	// It compute transmittance from the origin towards a sun direction. 

	auto RayIntersectSphere = [&](FVector3f RayOrigin, FVector3f RayDirection, FVector3f SphereOrigin, float SphereRadius)
	{
		FVector3f LocalPosition = RayOrigin - SphereOrigin;
		float LocalPositionSqr = FVector3f::DotProduct(LocalPosition, LocalPosition);

		FVector3f QuadraticCoef;
		QuadraticCoef.X = FVector3f::DotProduct(RayDirection, RayDirection);
		QuadraticCoef.Y = 2.0f * FVector3f::DotProduct(RayDirection, LocalPosition);
		QuadraticCoef.Z = LocalPositionSqr - SphereRadius * SphereRadius;

		float Discriminant = QuadraticCoef.Y * QuadraticCoef.Y - 4.0f * QuadraticCoef.X * QuadraticCoef.Z;

		// Only continue if the ray intersects the sphere
		FVector2D Intersections = { -1.0f, -1.0f };
		if (Discriminant >= 0)
		{
			float SqrtDiscriminant = sqrt(Discriminant);
			Intersections.X = (-QuadraticCoef.Y - 1.0f * SqrtDiscriminant) / (2 * QuadraticCoef.X);
			Intersections.Y = (-QuadraticCoef.Y + 1.0f * SqrtDiscriminant) / (2 * QuadraticCoef.X);
		}
		return Intersections;
	};

	// Nearest intersection of ray r,mu with sphere boundary
	auto raySphereIntersectNearest = [&](FVector3f RayOrigin, FVector3f RayDirection, FVector3f SphereOrigin, float SphereRadius)
	{
		FVector2D sol = RayIntersectSphere(RayOrigin, RayDirection, SphereOrigin, SphereRadius);
		const float sol0 = sol.X;
		const float sol1 = sol.Y;
		if (sol0 < 0.0f && sol1 < 0.0f)
		{
			return -1.0f;
		}
		if (sol0 < 0.0f)
		{
			return FMath::Max(0.0f, sol1);
		}
		else if (sol1 < 0.0f)
		{
			return FMath::Max(0.0f, sol0);
		}
		return FMath::Max(0.0f, FMath::Min(sol0, sol1));
	};

	auto OpticalDepth = [&](FVector3f RayOrigin, FVector3f RayDirection)
	{
		float TMax = raySphereIntersectNearest(RayOrigin, RayDirection, FVector3f(0.0f, 0.0f, 0.0f), TopRadiusKm);

		FLinearColor OpticalDepthRGB = FLinearColor(ForceInitToZero);
		FVector3f VectorZero = FVector3f(ForceInitToZero);
		if (TMax > 0.0f)
		{
			const float SampleCount = 15.0f;
			const float SampleStep = 1.0f / SampleCount;
			const float SampleLength = SampleStep * TMax;
			for (float SampleT = 0.0f; SampleT < 1.0f; SampleT += SampleStep)
			{
				FVector3f Pos = RayOrigin + RayDirection * (TMax * SampleT);
				const float viewHeight = (FVector3f::Distance(Pos, VectorZero) - BottomRadiusKm);

				const float densityMie = FMath::Max(0.0f, FMath::Exp(MieDensityExpScale * viewHeight));
				const float densityRay = FMath::Max(0.0f, FMath::Exp(RayleighDensityExpScale * viewHeight));
				const float densityOzo = FMath::Clamp(viewHeight < AbsorptionDensity0LayerWidth ?
					AbsorptionDensity0LinearTerm * viewHeight + AbsorptionDensity0ConstantTerm :
					AbsorptionDensity1LinearTerm * viewHeight + AbsorptionDensity1ConstantTerm,
					0.0f, 1.0f);

				FLinearColor SampleExtinction = densityMie * MieExtinction + densityRay * RayleighScattering + densityOzo * AbsorptionExtinction;
				OpticalDepthRGB += SampleLength * SampleExtinction;
			}
		}

		return OpticalDepthRGB;
	};

	// Assuming camera is along Z on (0,0,earthRadius + 500m)
	const FVector3f WorldPos = FVector3f(0.0f, 0.0f, BottomRadiusKm + 0.5);
	FVector2D AzimuthElevation = FMath::GetAzimuthAndElevation(SunDirection, FVector::ForwardVector, FVector::LeftVector, FVector::UpVector); // TODO: make it work over the entire virtual planet with a local basis
	AzimuthElevation.Y = FMath::Max(FMath::DegreesToRadians(TransmittanceMinLightElevationAngle), AzimuthElevation.Y);
	const FVector3f WorldDir = FVector3f(FMath::Cos(AzimuthElevation.Y), 0.0f, FMath::Sin(AzimuthElevation.Y)); // no need to take azimuth into account as transmittance is symmetrical around zenith axis.
	FLinearColor OpticalDepthRGB = OpticalDepth(WorldPos, WorldDir);
	return FLinearColor(FMath::Exp(-OpticalDepthRGB.R), FMath::Exp(-OpticalDepthRGB.G), FMath::Exp(-OpticalDepthRGB.B));
}

void FAtmosphereSetup::ComputeViewData(
	const FVector& WorldCameraOrigin, const FVector& PreViewTranslation, const FVector3f& ViewForward, const FVector3f& ViewRight,
	FVector3f& SkyCameraTranslatedWorldOriginTranslatedWorld, FVector4f& SkyPlanetTranslatedWorldCenterAndViewHeight, FMatrix44f& SkyViewLutReferential) const
{
	// The constants below should match the one in SkyAtmosphereCommon.ush
	// Always force to be 5 meters above the ground/sea level (to always see the sky and not be under the virtual planet occluding ray tracing) and lower for small planet radius
	const float PlanetRadiusOffset = 0.005f;		

	const float Offset = PlanetRadiusOffset * SkyUnitToCm;
	const float BottomRadiusWorld = BottomRadiusKm * SkyUnitToCm;
	const FVector PlanetCenterWorld = PlanetCenterKm * SkyUnitToCm;
	const FVector PlanetCenterTranslatedWorld = PlanetCenterWorld + PreViewTranslation;
	const FVector WorldCameraOriginTranslatedWorld = WorldCameraOrigin + PreViewTranslation;
	const FVector PlanetCenterToCameraTranslatedWorld = WorldCameraOriginTranslatedWorld - PlanetCenterTranslatedWorld;
	const float DistanceToPlanetCenterTranslatedWorld = PlanetCenterToCameraTranslatedWorld.Size();

	// If the camera is below the planet surface, we snap it back onto the surface.
	// This is to make sure the sky is always visible even if the camera is inside the virtual planet.
	SkyCameraTranslatedWorldOriginTranslatedWorld = FVector3f(
						DistanceToPlanetCenterTranslatedWorld < (BottomRadiusWorld + Offset) ?
						PlanetCenterTranslatedWorld + (BottomRadiusWorld + Offset) * (PlanetCenterToCameraTranslatedWorld / DistanceToPlanetCenterTranslatedWorld) :
						WorldCameraOriginTranslatedWorld);
	SkyPlanetTranslatedWorldCenterAndViewHeight = FVector4f((FVector3f)PlanetCenterTranslatedWorld, ((FVector)SkyCameraTranslatedWorldOriginTranslatedWorld - PlanetCenterTranslatedWorld).Size());

	// Now compute the referential for the SkyView LUT
	FVector PlanetCenterToWorldCameraPos = ((FVector)SkyCameraTranslatedWorldOriginTranslatedWorld - PlanetCenterTranslatedWorld) * CmToSkyUnit;
	FVector3f Up = (FVector3f)PlanetCenterToWorldCameraPos;
	Up.Normalize();
	FVector3f Forward = ViewForward;		// This can make texel visible when the camera is rotating. Use constant world direction instead?
	//FVector3f	Left = normalize(cross(Forward, Up)); 
	FVector3f	Left;
	Left = FVector3f::CrossProduct(Forward, Up);
	Left.Normalize();
	const float DotMainDir = FMath::Abs(FVector3f::DotProduct(Up, Forward));
	if (DotMainDir > 0.999f)
	{
		// When it becomes hard to generate a referential, generate it procedurally.
		// [ Duff et al. 2017, "Building an Orthonormal Basis, Revisited" ]
		const float Sign = Up.Z >= 0.0f ? 1.0f : -1.0f;
		const float a = -1.0f / (Sign + Up.Z);
		const float b = Up.X * Up.Y * a;
		Forward = FVector3f( 1 + Sign * a * FMath::Pow(Up.X, 2.0f), Sign * b, -Sign * Up.X );
		Left = FVector3f(b,  Sign + a * FMath::Pow(Up.Y, 2.0f), -Up.Y );

		SkyViewLutReferential.SetColumn(0, Forward);
		SkyViewLutReferential.SetColumn(1, Left);
		SkyViewLutReferential.SetColumn(2, Up);
		SkyViewLutReferential = SkyViewLutReferential.GetTransposed();
	}
	else
	{
		// This is better as it should be more stable with respect to camera forward.
		Forward = FVector3f::CrossProduct(Up, Left);
		Forward.Normalize();
		SkyViewLutReferential.SetColumn(0, Forward);
		SkyViewLutReferential.SetColumn(1, Left);
		SkyViewLutReferential.SetColumn(2, Up);
		SkyViewLutReferential = SkyViewLutReferential.GetTransposed();
	}
}
