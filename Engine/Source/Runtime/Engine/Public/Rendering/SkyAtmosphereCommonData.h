// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyAtmosphereCommonData.h
=============================================================================*/

#pragma once

#include "CoreMinimal.h"


struct FSkyAtmosphereDynamicState;
class USkyAtmosphereComponent;



struct FAtmosphereSetup
{
	ENGINE_API static const float CmToSkyUnit;
	ENGINE_API static const float SkyUnitToCm;

	FVector PlanetCenterKm;		// In sky unit (kilometers)
	float BottomRadiusKm;			// idem
	float TopRadiusKm;				// idem

	float MultiScatteringFactor;

	FLinearColor RayleighScattering;// Unit is 1/km
	float RayleighDensityExpScale;

	FLinearColor MieScattering;		// Unit is 1/km
	FLinearColor MieExtinction;		// idem
	FLinearColor MieAbsorption;		// idem
	float MieDensityExpScale;
	float MiePhaseG;

	FLinearColor AbsorptionExtinction;
	float AbsorptionDensity0LayerWidth;
	float AbsorptionDensity0ConstantTerm;
	float AbsorptionDensity0LinearTerm;
	float AbsorptionDensity1ConstantTerm;
	float AbsorptionDensity1LinearTerm;

	FLinearColor GroundAlbedo;

	float TransmittanceMinLightElevationAngle;

	ENGINE_API FAtmosphereSetup(const USkyAtmosphereComponent& SkyAtmosphereComponent);
	ENGINE_API FAtmosphereSetup(const FSkyAtmosphereDynamicState& Ds);

	template<typename T> void InternalInit(const T& SkyAtmosphereComponent);

	ENGINE_API FLinearColor GetTransmittanceAtGroundLevel(const FVector& SunDirection) const;

	ENGINE_API void UpdateTransform(const FTransform& ComponentTransform, uint8 TranformMode);
	ENGINE_API void ApplyWorldOffset(const FVector& InOffset);

	ENGINE_API void ComputeViewData(
		const FVector& WorldCameraOrigin, const FVector& PreViewTranslation, const FVector3f& ViewForward, const FVector3f& ViewRight,
		FVector3f& SkyCameraTranslatedWorldOrigin, FVector4f& SkyPlanetTranslatedWorldCenterAndViewHeight, FMatrix44f& SkyViewLutReferential) const;

	uint32 GetTransmittanceAndMultiScatteringLUTsVersion() const
	{
		return TransmittanceAndMultiScatteringLUTsVersion;
	}

private:
	uint32 TransmittanceAndMultiScatteringLUTsVersion; // A version to know if we have to recompute the transmittance and multiple scattering LUTs.

	void ComputeAtmosphereVersion();
};


