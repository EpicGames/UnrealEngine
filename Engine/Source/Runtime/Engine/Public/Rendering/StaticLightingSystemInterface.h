// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Components/PrimitiveComponent.h"
#include "Components/LightComponent.h"
#include "Components/SkyAtmosphereComponent.h"

class FMaterialRenderProxy;

class IStaticLightingSystem
{
public:
	virtual const class FMeshMapBuildData* GetPrimitiveMeshMapBuildData(const UPrimitiveComponent* Component, int32 LODIndex) { return nullptr; }
	virtual const class FLightComponentMapBuildData* GetLightComponentMapBuildData(const ULightComponent* Component) { return nullptr; }
	virtual const class FPrecomputedVolumetricLightmap* GetPrecomputedVolumetricLightmap() { return nullptr; }
	virtual ~IStaticLightingSystem() = default;
};

class IStaticLightingSystemImpl
{
public:	
	virtual IStaticLightingSystem* GetStaticLightingSystemForWorld(UWorld* InWorld) { return nullptr; }

	virtual void EditorTick() {}
	virtual bool IsStaticLightingSystemRunning() { return false; }
};

class FStaticLightingSystemInterface
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FPrimitiveComponentBasedSignature, UPrimitiveComponent* /*InComponent*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FLightComponentBasedSignature, ULightComponentBase* /*InComponent*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FStationaryLightChannelReassignmentSignature, ULightComponentBase* /*InComponent*/, int32 /*NewShadowMapChannel*/);
	DECLARE_MULTICAST_DELEGATE(FLightmassImportanceVolumeModifiedSignature);
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FMaterialInvalidationSignature, FMaterialRenderProxy* /*Material*/);

	static ENGINE_API FPrimitiveComponentBasedSignature OnPrimitiveComponentRegistered;
	static ENGINE_API FPrimitiveComponentBasedSignature OnPrimitiveComponentUnregistered;
	static ENGINE_API FLightComponentBasedSignature OnLightComponentRegistered;
	static ENGINE_API FLightComponentBasedSignature OnLightComponentUnregistered;
	static ENGINE_API FStationaryLightChannelReassignmentSignature OnStationaryLightChannelReassigned;
	static ENGINE_API FLightmassImportanceVolumeModifiedSignature OnLightmassImportanceVolumeModified;
	static ENGINE_API FMaterialInvalidationSignature OnMaterialInvalidated;
	static ENGINE_API FSimpleMulticastDelegate OnSkyAtmosphereModified;
	
	static ENGINE_API const class FMeshMapBuildData* GetPrimitiveMeshMapBuildData(const UPrimitiveComponent* Component, int32 LODIndex = 0);
	static ENGINE_API const class FLightComponentMapBuildData* GetLightComponentMapBuildData(const ULightComponent* Component);
	static ENGINE_API const class FPrecomputedVolumetricLightmap* GetPrecomputedVolumetricLightmap(UWorld* World);
	
	static ENGINE_API void EditorTick();
	static ENGINE_API void GameTick(float DeltaSeconds);
	static ENGINE_API bool IsStaticLightingSystemRunning();

	static ENGINE_API FStaticLightingSystemInterface* Get();

	ENGINE_API void RegisterImplementation(FName Name, IStaticLightingSystemImpl* Impl);
	ENGINE_API void UnregisterImplementation(FName Name);
	ENGINE_API IStaticLightingSystemImpl* GetPreferredImplementation();

private:
	TMap<FName, IStaticLightingSystemImpl*> Implementations;

	static FStaticLightingSystemInterface* Interface;
};
