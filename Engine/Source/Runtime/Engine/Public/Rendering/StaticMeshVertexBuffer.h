// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"
#include "RenderUtils.h"
#include "StaticMeshVertexData.h"
#include "PackedNormal.h"
#include "Components.h"
#include "Math/Vector2DHalf.h"
#include "RenderMath.h"

class FVertexFactory;
class FStaticMeshVertexDataInterface;
class FVertexFactory;
struct FStaticMeshDataType;

template<typename TangentTypeT>
struct TStaticMeshVertexTangentDatum
{
	TangentTypeT TangentX;
	TangentTypeT TangentZ;

	inline FVector3f GetTangentX() const
	{
		return FVector3f(TangentX.ToFVector());
	}

	inline FVector4f GetTangentZ() const
	{
		return TangentZ.ToFVector4f();
	}

	inline FVector3f GetTangentY() const
	{
		return FVector3f(GenerateYAxis(TangentX, TangentZ));
	}

	inline void SetTangents(FVector3f X, FVector3f Y, FVector3f Z)
	{
		TangentX = X;
		TangentZ = FVector4f(Z.X, Z.Y, Z.Z, GetBasisDeterminantSign(FVector3d(X), FVector3d(Y), FVector3d(Z)));
	}

	/**
	* Serializer
	*
	* @param Ar - archive to serialize with
	* @param V - vertex to serialize
	* @return archive that was used
	*/
	friend FArchive& operator<<(FArchive& Ar, TStaticMeshVertexTangentDatum& Vertex)
	{
		Ar << Vertex.TangentX;
		Ar << Vertex.TangentZ;
		return Ar;
	}
};

template<typename UVTypeT>
struct TStaticMeshVertexUVsDatum
{
	UVTypeT UVs;

	inline FVector2f GetUV() const
	{
		return UVs;
	}

	inline void SetUV(FVector2f UV)
	{
		UVs = UV;
	}

	/**
	* Serializer
	*
	* @param Ar - archive to serialize with
	* @param V - vertex to serialize
	* @return archive that was used
	*/
	friend FArchive& operator<<(FArchive& Ar, TStaticMeshVertexUVsDatum& Vertex)
	{
		Ar << Vertex.UVs;
		return Ar;
	}
};

enum class EStaticMeshVertexTangentBasisType
{
	Default,
	HighPrecision,
};

enum class EStaticMeshVertexUVType
{
	Default,
	HighPrecision,
};

template<EStaticMeshVertexTangentBasisType TangentBasisType>
struct TStaticMeshVertexTangentTypeSelector
{
};

template<>
struct TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::Default>
{
	typedef FPackedNormal TangentTypeT;
	static const EVertexElementType VertexElementType = VET_PackedNormal;
};

template<>
struct TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::HighPrecision>
{
	typedef FPackedRGBA16N TangentTypeT;
	static const EVertexElementType VertexElementType = VET_Short4N;
};

template<EStaticMeshVertexUVType UVType>
struct TStaticMeshVertexUVsTypeSelector
{
};

template<>
struct TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::Default>
{
	typedef FVector2DHalf UVsTypeT;
};

template<>
struct TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::HighPrecision>
{
	typedef FVector2f UVsTypeT;
};

/*
*  FStaticMeshVertexBufferFlags : options for FStaticMeshVertexBuffer::Init
*  bNeedsCPUAccess - Whether the vertex data needs to be accessed by the CPU after creation
*  bUseBackwardsCompatibleF16TruncUVs - Whether backwards compatible legacy truncation mode should be used for F16 UVs
*/
struct FStaticMeshVertexBufferFlags
{
	bool bNeedsCPUAccess = true;
	bool bUseBackwardsCompatibleF16TruncUVs = false;
};

/** Vertex buffer for a static mesh LOD */
class FStaticMeshVertexBuffer : public FRenderResource
{
	friend class FStaticMeshVertexBuffer;

public:

	/** Default constructor. */
	ENGINE_API FStaticMeshVertexBuffer();

	/** Destructor. */
	ENGINE_API ~FStaticMeshVertexBuffer();

	/** Delete existing resources */
	ENGINE_API void CleanUp();

	ENGINE_API void Init(uint32 InNumVertices, uint32 InNumTexCoords, bool bNeedsCPUAccess = true);
	
	/**
	* Initializes the buffer with the given vertices.
	* @param InVertices - The vertices to initialize the buffer with.
	* @param InNumTexCoords - The number of texture coordinate to store in the buffer.
	* @param Flags - Options for Init ; FStaticMeshVertexBufferFlags can be default constructed for default options
	*/
	ENGINE_API void Init(const TArray<FStaticMeshBuildVertex>& InVertices, uint32 InNumTexCoords, const FStaticMeshVertexBufferFlags & InInitFlags );
	ENGINE_API void Init(const FConstMeshBuildVertexView& InVertices, const FStaticMeshVertexBufferFlags& InInitFlags);

	/**
	* Initializes the buffer with the given vertices.
	* @param InVertices - The vertices to initialize the buffer with.
	* @param InNumTexCoords - The number of texture coordinate to store in the buffer.
	* @param bNeedsCPUAccess - Whether the vertex data needs to be accessed by the CPU after creation (default true)
	*/
	void Init(const FConstMeshBuildVertexView& InVertices, bool bNeedsCPUAccess = true)
	{
		FStaticMeshVertexBufferFlags Flags;
		Flags.bNeedsCPUAccess = bNeedsCPUAccess;
		Init(InVertices, Flags);
	}

	void Init(const TArray<FStaticMeshBuildVertex>& InVertices, uint32 InNumTexCoords, bool bNeedsCPUAccess = true)
	{
		FConstMeshBuildVertexView VertexView = MakeConstMeshBuildVertexView(InVertices);
		FStaticMeshVertexBufferFlags Flags;
		Flags.bNeedsCPUAccess = bNeedsCPUAccess;
		Init(VertexView, Flags);
	}

	/**
	* Initializes this vertex buffer with the contents of the given vertex buffer.
	* @param InVertexBuffer - The vertex buffer to initialize from.
	* @param bNeedsCPUAccess - Whether the vertex data needs to be accessed by the CPU after creation (default true)
	*/
	void Init(const FStaticMeshVertexBuffer& InVertexBuffer, bool bNeedsCPUAccess = true);

	/**
	 * Appends the specified vertices to the end of the buffer
	 *
	 * @param	Vertices	The vertex data to be appended.  Must not be nullptr.
	 * @param	NumVerticesToAppend		How many vertices should be added
	 * @param bUseBackwardsCompatibleF16TruncUVs - Whether backwards compatible legacy truncation mode should be used for F16 UVs (default false)
	 */
	ENGINE_API void AppendVertices( const FStaticMeshBuildVertex* Vertices, const uint32 NumVerticesToAppend, bool bUseBackwardsCompatibleF16TruncUVs = false);

	/**
	* Serializer
	*
	* @param	Ar				Archive to serialize with
	* @param	bNeedsCPUAccess	Whether the elements need to be accessed by the CPU
	*/
	ENGINE_API void Serialize(FArchive& Ar, bool bNeedsCPUAccess);

	void SerializeMetaData(FArchive& Ar);

	ENGINE_API void SetMetaData(uint32 NumTexCoords, uint32 NumVertices, bool bUseFullPrecisionUVs, bool bUseHighPrecisionTangentBasis);

	void ClearMetaData();

	/**
	* Specialized assignment operator, only used when importing LOD's.
	*/
	ENGINE_API void operator=(const FStaticMeshVertexBuffer &Other);

	template<EStaticMeshVertexTangentBasisType TangentBasisTypeT>
	inline FVector4f VertexTangentX_Typed(uint32 VertexIndex)const
	{
		typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<TangentBasisTypeT>::TangentTypeT> TangentType;
		TangentType* ElementData = reinterpret_cast<TangentType*>(TangentsDataPtr);
		check((void*)((&ElementData[VertexIndex]) + 1) <= (void*)(TangentsDataPtr + TangentsData->GetResourceSize()));
		check((void*)((&ElementData[VertexIndex]) + 0) >= (void*)(TangentsDataPtr));
		return FVector4f(ElementData[VertexIndex].GetTangentX());
	}

	inline FVector4f VertexTangentX(uint32 VertexIndex) const
	{
		checkSlow(VertexIndex < GetNumVertices());

		if (GetUseHighPrecisionTangentBasis())
		{
			return VertexTangentX_Typed<EStaticMeshVertexTangentBasisType::HighPrecision>(VertexIndex);
		}
		else
		{
			return VertexTangentX_Typed<EStaticMeshVertexTangentBasisType::Default>(VertexIndex);
		}
	}

	template<EStaticMeshVertexTangentBasisType TangentBasisTypeT>
	inline FVector4f VertexTangentZ_Typed(uint32 VertexIndex)const
	{
		typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<TangentBasisTypeT>::TangentTypeT> TangentType;
		TangentType* ElementData = reinterpret_cast<TangentType*>(TangentsDataPtr);
		check((void*)((&ElementData[VertexIndex]) + 1) <= (void*)(TangentsDataPtr + TangentsData->GetResourceSize()));
		check((void*)((&ElementData[VertexIndex]) + 0) >= (void*)(TangentsDataPtr));
		return  FVector4f(ElementData[VertexIndex].GetTangentZ());
	}

	inline FVector4f VertexTangentZ(uint32 VertexIndex) const
	{
		checkSlow(VertexIndex < GetNumVertices());

		if (GetUseHighPrecisionTangentBasis())
		{
			return VertexTangentZ_Typed<EStaticMeshVertexTangentBasisType::HighPrecision>(VertexIndex);
		}
		else
		{
			return VertexTangentZ_Typed<EStaticMeshVertexTangentBasisType::Default>(VertexIndex);
		}
	}

	template<EStaticMeshVertexTangentBasisType TangentBasisTypeT>
	inline FVector4f VertexTangentY_Typed(uint32 VertexIndex)const
	{
		typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<TangentBasisTypeT>::TangentTypeT> TangentType;
		TangentType* ElementData = reinterpret_cast<TangentType*>(TangentsDataPtr);
		check((void*)((&ElementData[VertexIndex]) + 1) <= (void*)(TangentsDataPtr + TangentsData->GetResourceSize()));
		check((void*)((&ElementData[VertexIndex]) + 0) >= (void*)(TangentsDataPtr));
		return FVector4f(ElementData[VertexIndex].GetTangentY());
	}

	/**
	* Calculate the binormal (TangentY) vector using the normal,tangent vectors
	*
	* @param VertexIndex - index into the vertex buffer
	* @return binormal (TangentY) vector
	*/
	inline FVector3f VertexTangentY(uint32 VertexIndex) const
	{
		checkSlow(VertexIndex < GetNumVertices());

		if (GetUseHighPrecisionTangentBasis())
		{
			return VertexTangentY_Typed<EStaticMeshVertexTangentBasisType::HighPrecision>(VertexIndex);
		}
		else
		{
			return VertexTangentY_Typed<EStaticMeshVertexTangentBasisType::Default>(VertexIndex);
		}
	}

	inline void SetVertexTangents(uint32 VertexIndex, FVector3f X, FVector3f Y, FVector3f Z)
	{
		checkSlow(VertexIndex < GetNumVertices());

		if (GetUseHighPrecisionTangentBasis())
		{
			typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::HighPrecision>::TangentTypeT> TangentType;
			TangentType* ElementData = reinterpret_cast<TangentType*>(TangentsDataPtr);
			check((void*)((&ElementData[VertexIndex]) + 1) <= (void*)(TangentsDataPtr + TangentsData->GetResourceSize()));
			check((void*)((&ElementData[VertexIndex]) + 0) >= (void*)(TangentsDataPtr));
			ElementData[VertexIndex].SetTangents(X, Y, Z);
		}
		else
		{
			typedef TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::Default>::TangentTypeT> TangentType;
			TangentType* ElementData = reinterpret_cast<TangentType*>(TangentsDataPtr);
			check((void*)((&ElementData[VertexIndex]) + 1) <= (void*)(TangentsDataPtr + TangentsData->GetResourceSize()));
			check((void*)((&ElementData[VertexIndex]) + 0) >= (void*)(TangentsDataPtr));
			ElementData[VertexIndex].SetTangents(X, Y, Z);
		}
	}

	/**
	* Set the vertex UV values at the given index in the vertex buffer
	*
	* @param VertexIndex - index into the vertex buffer
	* @param UVIndex - [0,MAX_STATIC_TEXCOORDS] value to index into UVs array
	* @param Vec2D - UV values to set
	* @param bUseBackwardsCompatibleF16TruncUVs - whether backwards compatible Truncate mode is used for F32 to F16 conversion
	*/
	inline void SetVertexUV(uint32 VertexIndex, uint32 UVIndex, const FVector2f& Vec2D, bool bUseBackwardsCompatibleF16TruncUVs = false)
	{
		checkSlow(VertexIndex < GetNumVertices());
		checkSlow(UVIndex < GetNumTexCoords());

		if (GetUseFullPrecisionUVs())
		{
			size_t UvStride = sizeof(FVector2f) * GetNumTexCoords();

			FVector2f* ElementData = reinterpret_cast<FVector2f*>(TexcoordDataPtr + (VertexIndex * UvStride));
			check((void*)((&ElementData[UVIndex]) + 1) <= (void*)(TexcoordDataPtr + TexcoordData->GetResourceSize()));
			check((void*)((&ElementData[UVIndex]) + 0) >= (void*)(TexcoordDataPtr));
			ElementData[UVIndex] = Vec2D;
		}
		else
		{
			size_t UvStride = sizeof(FVector2DHalf) * GetNumTexCoords();

			FVector2DHalf* ElementData = reinterpret_cast<FVector2DHalf*>(TexcoordDataPtr + (VertexIndex * UvStride));
			check((void*)((&ElementData[UVIndex]) + 1) <= (void*)(TexcoordDataPtr + TexcoordData->GetResourceSize()));
			check((void*)((&ElementData[UVIndex]) + 0) >= (void*)(TexcoordDataPtr));
		
			if ( bUseBackwardsCompatibleF16TruncUVs )
			{
				ElementData[UVIndex].SetTruncate( Vec2D );
			}
			else
			{
				ElementData[UVIndex] = Vec2D;
			}
		}
	}

	template<EStaticMeshVertexUVType UVTypeT>
	inline FVector2f GetVertexUV_Typed(uint32 VertexIndex, uint32 UVIndex)const
	{
		typedef TStaticMeshVertexUVsDatum<typename TStaticMeshVertexUVsTypeSelector<UVTypeT>::UVsTypeT> UVType;
		size_t UvStride = sizeof(UVType) * GetNumTexCoords();

		UVType* ElementData = reinterpret_cast<UVType*>(TexcoordDataPtr + (VertexIndex * UvStride));
		check((void*)((&ElementData[UVIndex]) + 1) <= (void*)(TexcoordDataPtr + TexcoordData->GetResourceSize()));
		check((void*)((&ElementData[UVIndex]) + 0) >= (void*)(TexcoordDataPtr));
		return ElementData[UVIndex].GetUV();
	}

	/**
	* Set the vertex UV values at the given index in the vertex buffer
	*
	* @param VertexIndex - index into the vertex buffer
	* @param UVIndex - [0,MAX_STATIC_TEXCOORDS] value to index into UVs array
	* @param 2D UV values
	*/
	inline FVector2f GetVertexUV(uint32 VertexIndex, uint32 UVIndex) const
	{
		checkSlow(VertexIndex < GetNumVertices());
		checkSlow(UVIndex < GetNumTexCoords());

		if (GetUseFullPrecisionUVs())
		{
			return GetVertexUV_Typed<EStaticMeshVertexUVType::HighPrecision>(VertexIndex, UVIndex);
		}
		else
		{
			return GetVertexUV_Typed<EStaticMeshVertexUVType::Default>(VertexIndex, UVIndex);
		}
	}

	inline uint32 GetNumVertices() const
	{
		return NumVertices;
	}

	inline uint32 GetNumTexCoords() const
	{
		return NumTexCoords;
	}

	inline bool GetUseFullPrecisionUVs() const
	{
		return bUseFullPrecisionUVs;
	}

	inline void SetUseFullPrecisionUVs(bool UseFull)
	{
		bUseFullPrecisionUVs = UseFull;
	}

	inline bool GetUseHighPrecisionTangentBasis() const
	{
		return bUseHighPrecisionTangentBasis;
	}

	inline void SetUseHighPrecisionTangentBasis(bool bUseHighPrecision)
	{
		bUseHighPrecisionTangentBasis = bUseHighPrecision;
	}

	inline uint32 GetResourceSize() const
	{
		return (TangentsStride + (TexcoordStride * GetNumTexCoords())) * NumVertices;
	}

	/** Create an RHI vertex buffer with CPU data. CPU data may be discarded after creation (see TResourceArray::Discard) */
	FBufferRHIRef CreateTangentsRHIBuffer(FRHICommandListBase& RHICmdList);
	FBufferRHIRef CreateTexCoordRHIBuffer(FRHICommandListBase& RHICmdList);

	/** Similar to Init/ReleaseRHI but only update existing SRV so references to the SRV stays valid */
	void InitRHIForStreaming(FRHIBuffer* IntermediateTangentsBuffer, FRHIBuffer* IntermediateTexCoordBuffer, FRHIResourceReplaceBatcher& Batcher);
	void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;
	ENGINE_API virtual void InitResource(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseResource() override;
	virtual FString GetFriendlyName() const override { return TEXT("Static-mesh vertices"); }

	ENGINE_API void BindTangentVertexBuffer(const FVertexFactory* VertexFactory, struct FStaticMeshDataType& Data) const;
	ENGINE_API void BindTexCoordVertexBuffer(const FVertexFactory* VertexFactory, struct FStaticMeshDataType& Data, int ClampedNumTexCoords = -1) const;
	ENGINE_API void BindPackedTexCoordVertexBuffer(const FVertexFactory* VertexFactory, struct FStaticMeshDataType& Data, int32 MaxNumTexCoords = -1) const;
	ENGINE_API void BindLightMapVertexBuffer(const FVertexFactory* VertexFactory, struct FStaticMeshDataType& Data, int LightMapCoordinateIndex) const;

	inline void* GetTangentData() { return TangentsDataPtr; }
	inline const void* GetTangentData() const { return TangentsDataPtr; }

	inline void* GetTexCoordData() { return TexcoordDataPtr; }
	inline const void* GetTexCoordData() const { return TexcoordDataPtr; }

	ENGINE_API int GetTangentSize() const;

	ENGINE_API int GetTexCoordSize() const;

	inline bool GetAllowCPUAccess() const
	{
		if (!TangentsData || !TexcoordData)
			return false;

		return TangentsData->GetAllowCPUAccess() && TexcoordData->GetAllowCPUAccess();
	}

	class FTangentsVertexBuffer : public FVertexBuffer
	{
		virtual FString GetFriendlyName() const override { return TEXT("FTangentsVertexBuffer"); }
	} TangentsVertexBuffer;

	class FTexcoordVertexBuffer : public FVertexBuffer
	{
		virtual FString GetFriendlyName() const override { return TEXT("FTexcoordVertexBuffer"); }
	} TexCoordVertexBuffer;

	inline bool IsValid()
	{
		return IsValidRef(TangentsVertexBuffer.VertexBufferRHI) && IsValidRef(TexCoordVertexBuffer.VertexBufferRHI);
	}

	FRHIShaderResourceView* GetTangentsSRV() const { return TangentsSRV; }
	FRHIShaderResourceView* GetTexCoordsSRV() const { return TextureCoordinatesSRV; }

	// Allows for creating SRV on demand
	// Note: consider cache the SRV locally if multiple compute tasks all need access to one in a single frame
	ENGINE_API FShaderResourceViewRHIRef GetOrCreateTangentsSRV(FRHICommandListBase& RHICmdList) const;
	ENGINE_API FShaderResourceViewRHIRef GetOrCreateTexCoordsSRV(FRHICommandListBase& RHICmdList) const;

private:

	/** The vertex data storage type */
	FStaticMeshVertexDataInterface* TangentsData;
	FShaderResourceViewRHIRef TangentsSRV;

	FStaticMeshVertexDataInterface* TexcoordData;
	FShaderResourceViewRHIRef TextureCoordinatesSRV;

	/** The cached vertex data pointer. */
	uint8* TangentsDataPtr;
	uint8* TexcoordDataPtr;

	/** The cached Tangent stride. */
	mutable uint32 TangentsStride; // Mutable to allow updating through const getter

	/** The cached Texcoord stride. */
	mutable uint32 TexcoordStride; // Mutable to allow updating through const getter

	/** The number of texcoords/vertex in the buffer. */
	uint32 NumTexCoords;

	/** The cached number of vertices. */
	uint32 NumVertices;

	/** Corresponds to UStaticMesh::UseFullPrecisionUVs. if true then 32 bit UVs are used */
	bool bUseFullPrecisionUVs;

	/** If true then RGB10A2 is used to store tangent else RGBA8 */
	bool bUseHighPrecisionTangentBasis;

	bool NeedsCPUAccess = true;

	/** Allocates the vertex data storage type. */
	void AllocateData(bool bNeedsCPUAccess = true);
	   
	void InitTangentAndTexCoordStrides();
};
