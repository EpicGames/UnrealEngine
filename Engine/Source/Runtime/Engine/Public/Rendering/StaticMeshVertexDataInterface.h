// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"

class FArchive;
class FResourceArrayInterface;

enum class EResizeBufferFlags
{
	None =										0, // No flags
	AllowSlackOnGrow =							1 << 0, // will allocate slack when growing the array.
	AllowSlackOnReduce=							1 << 1, // will leave the slack when reducing the array.
};
ENUM_CLASS_FLAGS(EResizeBufferFlags);

/** An interface to the static-mesh vertex data storage type. */

class FStaticMeshVertexDataInterface
{
public:

	/** Virtual destructor. */
	virtual ~FStaticMeshVertexDataInterface() = default;

	/**
	* Resizes the vertex data buffer, discarding any data which no longer fits.
	* @param NumVertices - The number of vertices to allocate the buffer for.
	* @param BufferFlags - Flags to define the expected behavior of the buffer	
	*/
	virtual void ResizeBuffer(uint32 NumVertices, EResizeBufferFlags BufferFlags = EResizeBufferFlags::None) = 0;


	virtual void Empty(uint32 NumVertices) = 0;

	virtual bool IsValidIndex(uint32 Index) = 0;

	/** @return The stride of the vertex data in the buffer. */
	virtual uint32 GetStride() const = 0;

	/** @return The number of elements. */
	virtual int32 Num() const = 0;

	/** @return A pointer to the data in the buffer. */
	virtual uint8* GetDataPointer() = 0;

	/** @return A pointer to the FResourceArrayInterface for the vertex data. */
	virtual FResourceArrayInterface* GetResourceArray() = 0;

	/** Serializer. */
	virtual void Serialize(FArchive& Ar, bool bForcePerElementSerialization = false) = 0;

	virtual void OverrideFreezeSizeAndAlignment(int64& Size, int32& Alignment) const = 0;

	virtual SIZE_T GetResourceSize() const = 0;

	virtual bool GetAllowCPUAccess() const = 0;
};
