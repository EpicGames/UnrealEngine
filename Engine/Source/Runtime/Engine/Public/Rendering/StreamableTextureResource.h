// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	StreamableTextureResource.h: Unreal texture related classes.
=============================================================================*/

#include "CoreMinimal.h"
#include "TextureResource.h"
#include "Streaming/StreamableRenderResourceState.h"
#include "Engine/Texture.h"

bool CanCreateWithPartiallyResidentMips(ETextureCreateFlags TexCreateFlags);

/** 
 * The rendering resource streamable texture.
 */
class FStreamableTextureResource : public FTextureResource
{
public:

	FStreamableTextureResource(UTexture* InOwner, const FTexturePlatformData* InPlatformData, const FStreamableRenderResourceState& InPostInitState, bool bAllowPartiallyResidentMips);

	// Dynamic cast methods.
	virtual FStreamableTextureResource* GetStreamableTextureResource() { return this; }
	// Dynamic cast methods (const).
	virtual const FStreamableTextureResource* GetStreamableTextureResource() const { return this; }

	virtual uint32 GetSizeX() const final override { return SizeX; }
	virtual uint32 GetSizeY() const final override { return SizeY; }
	// Depth for 3D texture or ArraySize for texture 2d arrays
	virtual uint32 GetSizeZ() const final override { return SizeZ; }
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() final override;

	// This is only coherent sync on the rendering thread. To get the gamethread coherent value, use UStreamableRenderResource.CacheStreamableResourceState
	inline FStreamableRenderResourceState GetState() const
	{ 
		checkSlow(IsInRenderingThread())
		return IsInitialized() ? State : FStreamableRenderResourceState(); 
	}

	// Get the state it will be in after InitRHI() gets called. Used to synchronize UStreamableRenderResource.CacheStreamableResourceState in CreateResource().
	inline FStreamableRenderResourceState GetPostInitState() const
	{ 
		checkSlow(IsInGameThread() && !IsInitialized());
		return State; 
	}

	void FinalizeStreaming(FRHICommandListBase& RHICmdList, FRHITexture* InTextureRHI);
	UE_DEPRECATED(5.7, "A RHICmdList argument is now required.")
	void FinalizeStreaming(FRHITexture* InTextureRHI);

	// Return the platform data mips, zero based on the resource mip count and not to the asset mip count.
	TArrayView<const FTexture2DMipMap*> GetPlatformMipsView() const;
	const FTexture2DMipMap* GetPlatformMip(int32 MipIdx) const;

	inline EPixelFormat GetPixelFormat() const { return PixelFormat; }

	// Return the texture creation flags, does not include TexCreate_Virtual as it could change per mip. Use IsTextureRHIPartiallyResident() instead.
	inline ETextureCreateFlags GetCreationFlags() const { return CreationFlags; }

	inline const FName& GetTextureName() const { return TextureName; }

	inline TextureGroup GetLODGroup() const { return LODGroup; }

	/** Returns the default mip map bias for this texture. */
	void RefreshSamplerStates();

	// Get the current first mip index, of this renderthread resource. Non streaming asset always return 0.
	inline int32 GetCurrentFirstMip() const
	{
		return State.IsValid() ? State.ResidentFirstLODIdx() : 0;
	}

	inline uint32 GetExtData() const { return PlatformData->GetExtData(); }

	/** Returns the platform mip size for the given mip count. */
	virtual uint64 GetPlatformMipsSize(uint32 NumMips) const = 0;

protected:

	virtual void CreateTexture(FRHICommandListBase& RHICmdList) = 0;
	virtual void CreatePartiallyResidentTexture(FRHICommandListBase& RHICmdList) = 0;

	/** Platform data should be ref counted eventually to remove any synchronization needed when the asset is rebuild. */
	const FTexturePlatformData* PlatformData;

	/** Different states for streaming synchronization. */
	FStreamableRenderResourceState State;

	/** Sample config */
	TEnumAsByte<ESamplerFilter> Filter = SF_Point;
	TEnumAsByte<ESamplerAddressMode> AddressU = AM_Wrap;
	TEnumAsByte<ESamplerAddressMode> AddressV = AM_Wrap;
	TEnumAsByte<ESamplerAddressMode> AddressW = AM_Wrap;
	float MipBias = 0;

	/** The width this resource, when all mips are streamed in. */
	uint32 SizeX = 0;
	/** The height when all mips are streamed in. */
	uint32 SizeY = 0;
	/** The 3d depth for volume texture or num  slices for 2d array when all mips are streamed in. This is NOT USED for cubemap arrays or cubemaps!*/
	uint32 SizeZ = 0;

	/** The FName of the texture asset */
	FName TextureName;
	/** The asset LOD group */
	TextureGroup LODGroup = TEXTUREGROUP_World;
	/** Format of the texture */
	EPixelFormat PixelFormat = PF_Unknown;
	/** Creation flags of the texture */
	ETextureCreateFlags  CreationFlags = TexCreate_None;
	/** Whether this texture should be updated using the virtual address mapping for each mip. */
	bool bUsePartiallyResidentMips = false;

	/** Max anisotropy. if 0, will use r.MaxAnisotropy */
	int8 MaxAniso = 0;

#if STATS
private:
	void CalcRequestedMipsSize();
	void IncrementTextureStats() const;
	void DecrementTextureStats() const;

	/** The FName of the LODGroup-specific stat */
	FName LODGroupStatName;
	/** Cached texture size for stats. */
	uint64 TextureSize = 0;
	/** Whether the owner is marked as neverstream */
	bool bIsNeverStream = false;
#endif
};

