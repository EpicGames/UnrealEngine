// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubstrateMaterialShared.h"
#include "Materials/MaterialExpressionSubstrate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubstrateMaterialShared)

IMPLEMENT_TYPE_LAYOUT(FSubstrateMaterialCompilationOutput);

FSubstrateMaterialCompilationOutput::FSubstrateMaterialCompilationOutput()
	: SubstrateMaterialType(0)
	, SubstrateClosureCount(0)
	, SubstrateUintPerPixel(0)
	, SubstrateMaterialBsdfFeatures(ESubstrateBsdfFeature::None)
#if WITH_EDITOR
	, SubstrateMaterialDescription()
	, SharedLocalBasesCount(0)
	, RequestedBytePerPixel(0)
	, PlatformBytePerPixel(0)
	, RequestedClosurePerPixel(0)
	, PlatformClosurePixel(0)
	, bIsThin(0)
	, MaterialType(0)
	, bMaterialOutOfBudgetHasBeenSimplified(0)
	, bMaterialUsesLegacyGBufferDataPassThrough(0)
	, RootOperatorIndex(0)
#endif
{
#if WITH_EDITOR
	for (uint32 i = 0; i < SUBSTRATE_COMPILATION_OUTPUT_MAX_OPERATOR; ++i)
	{
		Operators[i] = FSubstrateOperator();
	}
#endif
}


IMPLEMENT_TYPE_LAYOUT(FSubstrateOperator);

FSubstrateOperator::FSubstrateOperator()
{
#if WITH_EDITOR
	OperatorType = INDEX_NONE;
	bNodeRequestParameterBlending = false;
	Index = INDEX_NONE;
	ParentIndex = INDEX_NONE;
	LeftIndex = INDEX_NONE;
	RightIndex = INDEX_NONE;
	ThicknessIndex = INDEX_NONE;

	BSDFIndex = INDEX_NONE;
	BSDFType = 0;
	BSDFRegisteredSharedLocalBasis = FSubstrateRegisteredSharedLocalBasis();
	BSDFFeatures = ESubstrateBsdfFeature::None;
	SubUsage = SUBSTRATE_OPERATOR_SUBUSAGE_NONE;
	SubSurfaceType = uint8(EMaterialSubSurfaceType::MSS_None);

	bBSDFWritesEmissive = false;
	bBSDFWritesAmbientOcclusion = false;

	MaxDistanceFromLeaves = 0;
	LayerDepth = 0;
	bIsTop = false;
	bIsBottom = false;
	bUseParameterBlending = false;
	bRootOfParameterBlendingSubTree = false;
	MaterialExpressionGuid = FGuid();
#endif
}

void FSubstrateOperator::CombineFlagsForParameterBlending(FSubstrateOperator& A, FSubstrateOperator& B)
{
#if WITH_EDITOR
	BSDFFeatures = A.BSDFFeatures | B.BSDFFeatures;
	SubSurfaceType = uint32(SubstrateMergeSubSurfaceType(EMaterialSubSurfaceType(A.SubSurfaceType), EMaterialSubSurfaceType(B.SubSurfaceType)));
#endif
}

void FSubstrateOperator::CopyFlagsForParameterBlending(FSubstrateOperator& A)
{
#if WITH_EDITOR
	BSDFFeatures = A.BSDFFeatures;
	SubSurfaceType = A.SubSurfaceType;
#endif
}

bool FSubstrateOperator::IsDiscarded() const
{
#if WITH_EDITOR
	return bUseParameterBlending && !bRootOfParameterBlendingSubTree;
#else
	return true;
#endif
}

IMPLEMENT_TYPE_LAYOUT(FSubstrateRegisteredSharedLocalBasis);

FSubstrateRegisteredSharedLocalBasis::FSubstrateRegisteredSharedLocalBasis()
{
#if WITH_EDITOR
	NormalCodeChunk = INDEX_NONE;
	TangentCodeChunk = INDEX_NONE;
	NormalCodeChunkHash = 0;
	TangentCodeChunkHash = 0;
	GraphSharedLocalBasisIndex = 0;
#endif
}

const TCHAR* ToString(ESubstrateTileType Type)
{
	switch (Type)
	{
		case ESubstrateTileType::ESimple:							return TEXT("Simple");
		case ESubstrateTileType::ESingle:							return TEXT("Single");
		case ESubstrateTileType::EComplex:							return TEXT("Complex");
		case ESubstrateTileType::EComplexSpecial:					return TEXT("ComplexSpecial");
		case ESubstrateTileType::EOpaqueRoughRefraction:			return TEXT("Opaque/RoughRefraction");
		case ESubstrateTileType::EOpaqueRoughRefractionSSSWithout:	return TEXT("Opaque/RoughRefraction/SSSWithout");
		case ESubstrateTileType::EDecalSimple:						return TEXT("Decal/Simple");
		case ESubstrateTileType::EDecalSingle:						return TEXT("Decal/Single");
		case ESubstrateTileType::EDecalComplex:						return TEXT("Decal/Complex");

	}
	return TEXT("Unknown");
}