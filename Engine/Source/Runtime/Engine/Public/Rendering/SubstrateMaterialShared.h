// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SubstrateDefinitions.h"
#include "Serialization/MemoryImage.h"

#include "SubstrateMaterialShared.generated.h"

// Structures in this files are only used a compilation result return by the compiler.
// They are also used to present material information in the editor UI.


struct FSubstrateRegisteredSharedLocalBasis
{
	DECLARE_TYPE_LAYOUT(FSubstrateRegisteredSharedLocalBasis, NonVirtual);
public:
	FSubstrateRegisteredSharedLocalBasis();

	LAYOUT_FIELD_EDITORONLY(int32, NormalCodeChunk);
	LAYOUT_FIELD_EDITORONLY(int32, TangentCodeChunk);
	LAYOUT_FIELD_EDITORONLY(uint64, NormalCodeChunkHash);
	LAYOUT_FIELD_EDITORONLY(uint64, TangentCodeChunkHash);
	LAYOUT_FIELD_EDITORONLY(uint8, GraphSharedLocalBasisIndex);
};

UENUM()
enum class ESubstrateBsdfFeature : uint16
{
	None                             = 0u,
	SSS                              = 1u<<0u,
	MFPPluggedIn                     = 1u<<1u,
	EdgeColor                        = 1u<<2u,
	Fuzz                             = 1u<<3u,
	SecondRoughnessOrSimpleClearCoat = 1u<<4u,
	Anisotropy                       = 1u<<5u,
	Glint                            = 1u<<6u,
	SpecularProfile                  = 1u<<7u,
	Eye                              = 1u<<8u,
	EyeIrisNormalPluggedIn           = 1u<<9u,
	EyeIrisTangentPluggedIn          = 1u<<10u,
	Hair                             = 1u<<11u,
	Toon                             = 1u<<12u,

	// Complexity masks
	SingleMask =	
		  EdgeColor
		| Fuzz
		| SSS
		| SecondRoughnessOrSimpleClearCoat
		| MFPPluggedIn,

	ComplexMask = 
		  Anisotropy
		| SpecularProfile
		| Eye
		| Hair
		| Toon,

	ComplexSpecialMask = 
		  Glint
};
ENUM_CLASS_FLAGS(ESubstrateBsdfFeature)
DECLARE_INTRINSIC_TYPE_LAYOUT(ESubstrateBsdfFeature);

// This must map to the SUBSTRATE_TILE_TYPE defines.
enum ESubstrateTileType : uint32
{
	ESimple								= SUBSTRATE_TILE_TYPE_SIMPLE,
	ESingle								= SUBSTRATE_TILE_TYPE_SINGLE,
	EComplex							= SUBSTRATE_TILE_TYPE_COMPLEX,
	EComplexSpecial						= SUBSTRATE_TILE_TYPE_COMPLEX_SPECIAL,
	//ECount, 
	EOpaqueRoughRefraction				= SUBSTRATE_TILE_TYPE_ROUGH_REFRACT,
	EOpaqueRoughRefractionSSSWithout	= SUBSTRATE_TILE_TYPE_ROUGH_REFRACT_SSS_WITHOUT,
	EDecalSimple						= SUBSTRATE_TILE_TYPE_DECAL_SIMPLE,
	EDecalSingle						= SUBSTRATE_TILE_TYPE_DECAL_SINGLE,
	EDecalComplex						= SUBSTRATE_TILE_TYPE_DECAL_COMPLEX,
	ECount  //ETotalCount
};

ENGINE_API const TCHAR* ToString(ESubstrateTileType Type);

FORCEINLINE ESubstrateTileType GetSubstrateTileTypeFromMaterialType(uint32 InMaterialType)
{
	// InMaterialType should be one of these
	//  * SUBSTRATE_MATERIAL_TYPE_SIMPLE
	//  * SUBSTRATE_MATERIAL_TYPE_SINGLE
	//  * SUBSTRATE_MATERIAL_TYPE_COMPLEX
	//  * SUBSTRATE_MATERIAL_TYPE_COMPLEX_SPECIAL
	checkSlow(InMaterialType < SUBSTRATE_MATERIAL_TYPE_COUNT);
	return ESubstrateTileType(InMaterialType);
}

FORCEINLINE uint32 GetSubstrateTileTypeAsUint8(ESubstrateTileType In)
{
	return 1u << uint32(In);
}

FORCEINLINE bool UsesSubstrateTileType(uint8 InTypes, ESubstrateTileType In)
{
	return uint32(InTypes) & (1u << uint32(In));
}

struct FSubstrateOperator
{
	DECLARE_TYPE_LAYOUT(FSubstrateOperator, NonVirtual);
public:
	FSubstrateOperator();

	// !!!!!!!!!!
	// Not using LAYOUT_BITFIELD_EDITORONLY because it seems to cause issue with bit being shifted around when copy happens.
	// So in the meantime we find it out, LAYOUT_FIELD_EDITORONLY using uint8 is used.
	// !!!!!!!!!!

	LAYOUT_FIELD_EDITORONLY(int32, OperatorType);
	LAYOUT_FIELD_EDITORONLY(uint8, bNodeRequestParameterBlending);

	LAYOUT_FIELD_EDITORONLY(int32, Index);			// Index into the array of operators
	LAYOUT_FIELD_EDITORONLY(int32, ParentIndex);	// Parent operator index
	LAYOUT_FIELD_EDITORONLY(int32, LeftIndex);		// Left child operator index
	LAYOUT_FIELD_EDITORONLY(int32, RightIndex);		// Right child operator index
	LAYOUT_FIELD_EDITORONLY(int32, ThicknessIndex);	// Thickness expression index

	// Data used for BSDF type nodes only
	LAYOUT_FIELD_EDITORONLY(int32, BSDFIndex);		// Index in the array of BSDF if a BSDF operator
	LAYOUT_FIELD_EDITORONLY(int32, BSDFType);
	LAYOUT_FIELD_EDITORONLY(FSubstrateRegisteredSharedLocalBasis, BSDFRegisteredSharedLocalBasis);
	LAYOUT_FIELD_EDITORONLY(ESubstrateBsdfFeature, BSDFFeatures);
	LAYOUT_FIELD_EDITORONLY(uint8, SubUsage);		// Sometimes, Unlit or Weight operators are used to transport data for other meaning (e.g. Light Function or ConvertToDecal)
	LAYOUT_FIELD_EDITORONLY(uint8, SubSurfaceType);

	LAYOUT_FIELD_EDITORONLY(uint8, bBSDFWritesEmissive);
	LAYOUT_FIELD_EDITORONLY(uint8, bBSDFWritesAmbientOcclusion);

	// Data derived after the tree has been built.
	LAYOUT_FIELD_EDITORONLY(int32, MaxDistanceFromLeaves);
	LAYOUT_FIELD_EDITORONLY(int32, LayerDepth);
	LAYOUT_FIELD_EDITORONLY(uint8, bIsTop);
	LAYOUT_FIELD_EDITORONLY(uint8, bIsBottom);

	LAYOUT_FIELD_EDITORONLY(uint8, bUseParameterBlending);			// True when part of a sub tree where parameter blending is in use
	LAYOUT_FIELD_EDITORONLY(uint8, bRootOfParameterBlendingSubTree);// True when the root of a sub tree where parameter blending is in use. Only this node will register a BSDF
	LAYOUT_FIELD_EDITORONLY(FGuid, MaterialExpressionGuid);			// Material expression Guid for mapping between UMaterialExpression and FSubstrateOperator

	void CombineFlagsForParameterBlending(FSubstrateOperator& A, FSubstrateOperator& B);

	void CopyFlagsForParameterBlending(FSubstrateOperator& A);

	bool IsDiscarded() const;
	
	bool Has(ESubstrateBsdfFeature In) const 
	{ 
	#if WITH_EDITOR
		return EnumHasAnyFlags(BSDFFeatures, In); 
	#else
		return false;
	#endif
	}
};

#define SUBSTRATE_COMPILATION_OUTPUT_MAX_OPERATOR 24


#define SUBSTRATE_MATERIAL_TYPE_SINGLESLAB			0
#define SUBSTRATE_MATERIAL_TYPE_MULTIPLESLABS		1
#define SUBSTRATE_MATERIAL_TYPE_VOLUMETRICFOGCLOUD	2
#define SUBSTRATE_MATERIAL_TYPE_UNLIT				3
#define SUBSTRATE_MATERIAL_TYPE_HAIR				4
#define SUBSTRATE_MATERIAL_TYPE_SINGLELAYERWATER	5
#define SUBSTRATE_MATERIAL_TYPE_EYE					6
#define SUBSTRATE_MATERIAL_TYPE_LIGHTFUNCTION		7
#define SUBSTRATE_MATERIAL_TYPE_POSTPROCESS			8
#define SUBSTRATE_MATERIAL_TYPE_UI					9
#define SUBSTRATE_MATERIAL_TYPE_DECAL				10
#define SUBSTRATE_MATERIAL_TYPE_TOON				11
#define SUBSTRATE_MATERIAL_TYPE_MULTIPLEBSDFS		12

struct FSubstrateMaterialCompilationOutput
{
	DECLARE_TYPE_LAYOUT(FSubstrateMaterialCompilationOutput, NonVirtual);
public:

	FSubstrateMaterialCompilationOutput();

	////
	//// The following data is required at runtime
	////

	/** Substrate material type, at compile time (Possible values from SUBSTRATE_MATERIAL_TYPE_XXX: simple/single/complex/complex special) */
	LAYOUT_FIELD(uint8, SubstrateMaterialType);

	/** Substrate closure count, at compile time (0-7) */
	LAYOUT_FIELD(uint8, SubstrateClosureCount);

	/** Substrate uint per pixel, at compile time (0-255) */
	LAYOUT_FIELD(uint8, SubstrateUintPerPixel);

	/** Substrate feature used across the material (Aniso, SSS, Fuzz, ...)*/
	LAYOUT_FIELD(ESubstrateBsdfFeature, SubstrateMaterialBsdfFeatures);

	////
	//// The following data is only needed when compiling with the editor.
	////

	// Note we use LAYOUT_FIELD_EDITORONLY for bools because LAYOUT_BITFIELD_EDITORONLY was causing issues when serialising the structure.
	// SUBSTRATE_TODO pack that data.

	/** The Substrate verbose description */
	LAYOUT_FIELD_EDITORONLY(FMemoryImageString, SubstrateMaterialDescription);

	/** The number of local normal/tangent bases */
	LAYOUT_FIELD_EDITORONLY(uint8, SharedLocalBasesCount);
	/** Material requested byte count per pixel */
	LAYOUT_FIELD_EDITORONLY(uint8, RequestedBytePerPixel);
	/** The byte count per pixel supported by the platform the material has been compiled against */
	LAYOUT_FIELD_EDITORONLY(uint8, PlatformBytePerPixel);

	/** Material requested closure count per pixel */
	LAYOUT_FIELD_EDITORONLY(uint8, RequestedClosurePerPixel);
	/** The closure count per pixel supported by the platform the material has been compiled against */
	LAYOUT_FIELD_EDITORONLY(uint8, PlatformClosurePixel);

	/** Indicate that the material is considered a thin surface instead of a volume filled up with matter */
	LAYOUT_FIELD_EDITORONLY(uint8, bIsThin);
	/** Indicate the final material type */
	LAYOUT_FIELD_EDITORONLY(uint8, MaterialType);

	/** The byte per pixel count supported by the platform the material has been compiled against */
	LAYOUT_FIELD_EDITORONLY(uint8, bMaterialOutOfBudgetHasBeenSimplified, 1);
	/** True if the blendable GBuffer is used by Substrate and the Front material pin is not plugged in: in this case we can passthough substrate and use legacy base pass material shader.*/
	LAYOUT_FIELD_EDITORONLY(uint8, bMaterialUsesLegacyGBufferDataPassThrough, 1);

	LAYOUT_FIELD_EDITORONLY(int8, RootOperatorIndex);
	LAYOUT_ARRAY_EDITORONLY(FSubstrateOperator, Operators, SUBSTRATE_COMPILATION_OUTPUT_MAX_OPERATOR);
};

