// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	Texture2DResource.h: Implementation of FTexture2DResource used by streamable UTexture2D.
=============================================================================*/

#include "CoreMinimal.h"
#include "Rendering/StreamableTextureResource.h"

/**
 * FTextureResource implementation for streamable 2D textures.
 */
class FTexture2DResource : public FStreamableTextureResource
{
public:
	/**
	 * Minimal initialization constructor.
	 *
	 * @param InOwner			UTexture2D which this FTexture2DResource represents.
	 * @param InPostInitState	The renderthread coherent state the resource will have once InitRHI() will be called.		
 	 */
	FTexture2DResource(UTexture2D* InOwner, const FStreamableRenderResourceState& InPostInitState);

	/**
	 * Destructor, freeing MipData in the case of resource being destroyed without ever 
	 * having been initialized by the rendering thread via InitRHI.
	 */
	virtual ~FTexture2DResource();

	// Dynamic cast methods.
	virtual FTexture2DResource* GetTexture2DResource() { return this; }
	// Dynamic cast methods (const).
	virtual const FTexture2DResource* GetTexture2DResource() const { return this; }

	/** Set the value of Filter, AddressU, AddressV, AddressW and MipBias from FStreamableTextureResource on the gamethread. */
	void CacheSamplerStateInitializer(const UTexture2D* InOwner);

	/** Returns the platform mip size for the given mip count. */
	virtual uint64 GetPlatformMipsSize(uint32 NumMips) const override;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual bool IsProxy() const override { return ProxiedResource != nullptr; }
	
	// returns mip size in bytes; fills OutPitch
	static uint32 CalculateTightPackedMipSize(int32 SizeX,int32 SizeY,EPixelFormat PixelFormat,uint32 & OutPitch);

	// warn if "Pitch" is not compatible with tight-packed requirement
	static void WarnRequiresTightPackedMip(int32 SizeX,int32 SizeY,EPixelFormat PixelFormat,uint32 Pitch);

private:
	/**
	 * Make this Texture2DResource Proxy another one.
	 *
	 * @param InOwner             UTexture2D which this FTexture2DResource represents.
	 * @param InProxiedResource   The resource to proxy.
	 */
	FTexture2DResource(UTexture2D* InOwner, const FTexture2DResource* InProxiedResource);

	virtual void CreateTexture(FRHICommandListBase& RHICmdList) final override;
	virtual void CreatePartiallyResidentTexture(FRHICommandListBase& RHICmdList) final override;

	/** Texture streaming command classes that need to be friends in order to call Update/FinalizeMipCount.	*/
	friend class UTexture2D;
	friend class FTexture2DUpdate;
	friend class FTexture2DMipAllocator_Virtual;

	/** Resource memory allocated by the owner for serialize bulk mip data into								*/
	FTexture2DResourceMem* ResourceMem;

	/** Another resource being proxied by this one. */
	const FTexture2DResource* const ProxiedResource = nullptr;

	/** Local copy/ cache of mip data between creation and first call to InitRHI.							*/
	TArray<void*, TInlineAllocator<MAX_TEXTURE_MIP_COUNT> > MipData;
	TArray<int64, TInlineAllocator<MAX_TEXTURE_MIP_COUNT> > MipDataSize;

	/**
	 * Writes the data for a single mip-level into a destination buffer.
	 * @param MipIndex	The index of the mip-level to read.
	 * @param Dest		The address of the destination buffer to receive the mip-level's data.
	 * @param DestPitch	Number of bytes per row
	 */
	void GetData( uint32 MipIndex,void* Dest,uint32 DestPitch, uint64 DestSize );
};
