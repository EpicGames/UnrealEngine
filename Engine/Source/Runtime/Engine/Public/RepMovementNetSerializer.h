// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"
#include "RepMovementNetSerializer.generated.h"

USTRUCT()
struct FRepMovementNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};


namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FRepMovementNetSerializer, ENGINE_API);

}

