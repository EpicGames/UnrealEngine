// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/PackageMapClient.h"
#include "NetworkReplayStreaming.h"
#include "Net/Core/Connection/NetResult.h"
#include "Net/ReplayResult.h"
#include "ReplayTypes.h"
#include "Containers/ArrayView.h"

class APlayerController;
class UNetConnection;

class FReplayHelper;

class FReplayResultHandler final : public UE::Net::FNetResultHandler
{
	friend class FReplayHelper;

private:
	void InitResultHandler(FReplayHelper* InReplayHelper);

	virtual UE::Net::EHandleNetResult HandleNetResult(UE::Net::FNetResult&& InResult) override;

private:
	FReplayHelper* ReplayHelper = nullptr;
};

class FReplayHelper
{
	friend class UDemoNetDriver;
	friend class UDemoNetConnection;
	friend class UReplayNetConnection;
	friend class FReplayResultHandler;

public:
	FReplayHelper();
	~FReplayHelper();

	/* Do not copy or move this helper instance */
	FReplayHelper(FReplayHelper&&) = delete;
	FReplayHelper(const FReplayHelper&) = delete;
	FReplayHelper& operator=(const FReplayHelper&) = delete;
	FReplayHelper& operator=(FReplayHelper&&) = delete;

	/* For collection of GC references */
	void Serialize(FArchive& Ar);

private:

	TSharedPtr<INetworkReplayStreamer> Init(const FURL& URL);

	void SetPlaybackNetworkVersions(FArchive& Ar);
	void SetPlaybackNetworkVersions(UNetConnection* Connection);

	FReplayCustomVersion::Type GetPlaybackReplayVersion() const;

	void StartRecording(UNetConnection* Connection);
	void StopReplay();

	void OnStartRecordingComplete(const FStartStreamingResult& Result);

	void WriteNetworkDemoHeader(UNetConnection* Connection);
	bool ReadPlaybackDemoHeader(FString& Error);

	static void FlushNetChecked(UNetConnection& NetConnection);
	static void WritePacket(FArchive& Ar, uint8* Data, int32 Count);

	void OnSeamlessTravelStart(UWorld* InWorld, const FString& LevelName, UNetConnection* Connection);

	APlayerController* CreateSpectatorController(UNetConnection* Connection);

	bool HasLevelStreamingFixes() const
	{
		return bHasLevelStreamingFixes;
	}

	bool HasDeltaCheckpoints() const
	{
		return bHasDeltaCheckpoints;
	}

	bool HasGameSpecificFrameData() const
	{
		return bHasGameSpecificFrameData;
	}

	FGuid GetPlaybackGuid() const 
	{ 
		return PlaybackDemoHeader.Guid;	
	}

	void AddNewLevel(const FString& NewLevelName);

	void TickRecording(float DeltaSeconds, UNetConnection* Connection);

	void SaveCheckpoint(UNetConnection* Connection);
	void TickCheckpoint(UNetConnection* Connection);
	bool ShouldSaveCheckpoint() const;

	/** Returns either CheckpointSaveMaxMSPerFrame or the value of demo.CheckpointSaveMaxMSPerFrameOverride if it's >= 0. */
	float GetCheckpointSaveMaxMSPerFrame() const;

	DECLARE_DELEGATE_OneParam(FOnReplayRecordError, const UE::Net::TNetResult<EReplayResult>&);
	FOnReplayRecordError OnReplayRecordError;

	DECLARE_DELEGATE_OneParam(FOnReplayPlaybackError, const UE::Net::TNetResult<EReplayResult>&);
	FOnReplayPlaybackError OnReplayPlaybackError;

	static float GetClampedDeltaSeconds(UWorld* World, const float DeltaSeconds);

	uint32 GetDemoCurrentTimeInMS() const { return (uint32)((double)DemoCurrentTime * 1000); }
	uint32 GetLastCheckpointTimeInMS() const { return (uint32)((double)LastCheckpointTime * 1000); }

	void ResetState();

	void AddOrUpdateEvent(const FString& Name, const FString& Group, const FString& Meta, const TArray<uint8>& Data);

	void SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InProvider);

	void RequestCheckpoint();

	void RemoveActorFromCheckpoint(UNetConnection* Connection, AActor* Actor);

	void NotifyActorDestroyed(UNetConnection* Connection, AActor* Actor);

private:
	// Hooks used to determine when levels are streamed in, streamed out, or if there's a map change.
	void OnLevelAddedToWorld(ULevel* Level, UWorld* World);
	void OnLevelRemovedFromWorld(ULevel* Level, UWorld* World);

	void RecordFrame(float DeltaSeconds, UNetConnection* Connection);

	void WriteDemoFrame(UNetConnection* Connection, FArchive& Ar, TArray<FQueuedDemoPacket>& QueuedPackets, float FrameTime, EWriteDemoFrameFlags Flags);
	bool ReadDemoFrame(UNetConnection* Connection, FArchive& Ar, TArray<FPlaybackPacket>& InPlaybackPackets, const bool bForLevelFastForward, const FArchivePos MaxArchiveReadPos, float* OutTime);

	// Possible values returned by ReadPacket.
	enum class EReadPacketState
	{
		Success,	// A packet was read successfully and there may be more in the frame archive.
		End,		// No more data is present in the archive.
		Error,		// An error occurred while reading.
	};

	enum class EReadPacketMode
	{
		Default,	// Read the packet normally
		SkipData	// Skip packet data.
	};

	/**
	 * Reads a formatted Demo Packet from the given archive (which is expected to be in Demo Frame format).
	 *
	 * @param Archive			The archive from which to read.
	 * @param OutBuffer			Data used to store the read packet data.
	 *							Note, this will stomp on existing memory instead of appending.
	 * @param MaxBufferSize		Largest expected packet buffer size. Used to detect corruption.
	 * @param Mode				How to handle packet data
	 *
	 * @return EReadPacketState
	 */
	static const EReadPacketState ReadPacket(FArchive& Archive, TArray<uint8>& OutBuffer, const EReadPacketMode Mode);

	void CacheNetGuids(UNetConnection* Connection);

	bool SerializeGuidCache(UNetConnection* Connection, const FRepActorsCheckpointParams& Params, FArchive* CheckpointArchive);
	bool SerializeDeletedStartupActors(UNetConnection* Connection, const FRepActorsCheckpointParams& Params, FArchive* CheckpointArchive);
	bool SerializeDeltaDynamicDestroyed(UNetConnection* Connection, const FRepActorsCheckpointParams& Params, FArchive* CheckpointArchive);
	bool SerializeDeltaClosedChannels(UNetConnection* Connection, const FRepActorsCheckpointParams& Params, FArchive* CheckpointArchive);

	/**
	* Replicates the given prioritized actors, so their packets can be captured for recording.
	* This should be used for normal frame recording.
	* @see ReplicateCheckpointActor for recording during checkpoints.
	*
	* @param ToReplicate	The actors to replicate.
	* @param RepStartTime	The start time for replication.
	*
	* @return True if there is time remaining to replicate more Actors, False otherwise.
	*/
	bool ReplicateCheckpointActor(AActor* ToReplicate, UNetConnection* Connection, class FRepActorsCheckpointParams& Params);

	bool ReplicateActor(AActor* Actor, UNetConnection* Connection, bool bMustReplicate);

	TMap<TWeakObjectPtr<UObject>, FNetworkGUID, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UObject>, FNetworkGUID>> ObjectsWithExternalDataMap;

	struct FExternalDataWrapper
	{
		FNetworkGUID NetGUID;
		TArray<uint8> Data;
		int32 NumBits;
	};

	TMap<TWeakObjectPtr<UObject>, FExternalDataWrapper, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UObject>, FExternalDataWrapper>> ExternalDataMap;

	void SaveExternalData(UNetConnection* Connection, FArchive& Ar);
	void LoadExternalData(FArchive& Ar, const float TimeSeconds);

	bool SetExternalDataForObject(UNetConnection* Connection, UObject* OwningObject, const uint8* Src, const int32 NumBits);

	void ResetDeltaCheckpointTracking(UNetConnection* Connection);

	// Cached replay URL
	FURL DemoURL;

	FString ActiveReplayName;

	TSharedPtr<INetworkReplayStreamer> ReplayStreamer;

	TWeakObjectPtr<UWorld> World;

	// List of levels used in the current replay
	TArray<FLevelNameAndTime> LevelNamesAndTimes;

	/** Index of LevelNames that is currently loaded */
	int32 CurrentLevelIndex;

	/** Current record/playback frame number */
	int32 DemoFrameNum;

	/** Current record/playback position in seconds */
	float DemoCurrentTime;

	/** Total time of demo in seconds */
	float DemoTotalTime;

	// Last time a checkpoint was saved
	double LastCheckpointTime;

	// Time of the last frame we've read (in seconds).
	float LatestReadFrameTime;

	bool bWasStartRecordingSuccessful;
	bool bIsWaitingForStream;
	bool bIsLoadingCheckpoint;

	// Whether or not the Streaming Level Fixes are enabled for capture or playback.
	bool bHasLevelStreamingFixes;

	// Checkpoints are delta compressed
	bool bHasDeltaCheckpoints;

	// Allow appending per frame game specific data
	bool bHasGameSpecificFrameData;

	// If true, will skip recording, but leaves the replay open so that recording can be resumed again.
	bool bPauseRecording;

	bool bRecordMapChanges;

	float CheckpointSaveMaxMSPerFrame;

	// This header is valid during playback (so we know what version to pass into serializers, etc)
	FNetworkDemoHeader PlaybackDemoHeader;

	enum class ECheckpointSaveState
	{
		Idle,
		ProcessCheckpointActors,
		SerializeDeletedStartupActors,
		SerializeDeltaDynamicDestroyed,
		SerializeDeltaClosedChannels,
		CacheNetGuids,
		SerializeGuidCache,
		SerializeNetFieldExportGroupMap,
		SerializeDemoFrameFromQueuedDemoPackets,
		Finalize,
	};

	struct FCheckpointStepHelper
	{
		FCheckpointStepHelper() = delete;
		FCheckpointStepHelper(ECheckpointSaveState InCheckpointState, const double InCheckpointStartTime, int32* InCurrentIndex, int32 InTotalCount)
			: CheckpointState(InCheckpointState)
			, CheckpointStartTime(InCheckpointStartTime)
			, CurrentIndex(InCurrentIndex)
			, TotalCount(InTotalCount)
		{
			check(InCurrentIndex);
			StartTime = FPlatformTime::Seconds();
		}

		~FCheckpointStepHelper()
		{
			const double EndTime = FPlatformTime::Seconds();
			const double TotalTimeInMS = (EndTime - CheckpointStartTime) * 1000.0;
			const double StepTimeInMS = (EndTime - StartTime) * 1000.0;

			const TCHAR* StateStr = TEXT("Unknown");

			switch (CheckpointState)
			{
			case ECheckpointSaveState::ProcessCheckpointActors:
				StateStr = TEXT("ProcessCheckpointActors");
				break;
			case ECheckpointSaveState::SerializeDeletedStartupActors:
				StateStr = TEXT("SerializeDeletedStartupActors");
				break;
			case ECheckpointSaveState::SerializeDeltaDynamicDestroyed:
				StateStr = TEXT("SerializeDeltaDynamicDestroyed");
				break;
			case ECheckpointSaveState::SerializeDeltaClosedChannels:
				StateStr = TEXT("SerializeDeltaClosedChannels");
				break;
			case ECheckpointSaveState::SerializeGuidCache:
				StateStr = TEXT("SerializeGuidCache");
				break;
			default:
				ensureMsgf(false, TEXT("FCheckpointStepHelper: Unsupported checkpoint state: %d"), EnumToUnderlyingType(CheckpointState));
				break;
			}

			UE_LOGF(LogDemo, Verbose, "Checkpoint. %ls: %i/%i, took %.2fms (Total this frame: %.2fms)", StateStr, *CurrentIndex, TotalCount, StepTimeInMS, TotalTimeInMS);
		}

	private:
		ECheckpointSaveState CheckpointState;
		double StartTime = 0.0;
		double CheckpointStartTime = 0.0;
		int32* CurrentIndex = nullptr;
		int32 TotalCount = 0;
	};

	/** When we save a checkpoint, we remember all of the actors that need a checkpoint saved out by adding them to this list */
	struct FPendingCheckPointActor
	{
		TWeakObjectPtr< AActor > Actor;
		int32 LevelIndex;
	};

	struct FNetGuidCacheItem
	{
		FNetworkGUID NetGuid;
		FNetGuidCacheObject NetGuidCacheObject;
	};

	/** Checkpoint state */
	struct FCheckpointSaveStateContext
	{
		FCheckpointSaveStateContext() 
			: CheckpointSaveState(ECheckpointSaveState::Idle)
			, TotalCheckpointSaveTimeSeconds(0.0)
			, TotalCheckpointReplicationTimeSeconds(0.0)
			, bWriteCheckpointOffset(false)
			, TotalCheckpointSaveFrames(0)
			, TotalCheckpointActors(0)
			, CheckpointOffset(0)
			, GuidCacheSize(0)
			, NextAmortizedItem(0)
			, NumNetGuidsForRecording(0)
			, NetGuidsCountPos(0)
		{}

		ECheckpointSaveState CheckpointSaveState;						// Current state of checkpoint SaveState
		FPackageMapAckState CheckpointAckState;							// Current ack state of packagemap for the current checkpoint being saved
		TArray<FPendingCheckPointActor> PendingCheckpointActors;		// Actors to be serialized by pending checkpoint
		TMap<TWeakObjectPtr<AActor>, int32, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<AActor>, int32>> PendingActorToIndex;
		double				TotalCheckpointSaveTimeSeconds;				// Total time it took to save checkpoint including the finaling part across all frames
		double				TotalCheckpointReplicationTimeSeconds;		// Total time it took to write all replicated objects across all frames
		bool				bWriteCheckpointOffset;
		int32				TotalCheckpointSaveFrames;					// Total number of frames used to save a checkpoint
		int32				TotalCheckpointActors;
		FArchivePos			CheckpointOffset;
		uint32				GuidCacheSize;

		FDeltaCheckpointData DeltaCheckpointData;
		TArray<FNetworkGUID> DeltaChannelCloseKeys;

		TArray<FNetGuidCacheItem> NetGuidCacheSnapshot;
		int32 NextAmortizedItem;
		int32 NumNetGuidsForRecording;
		FArchivePos NetGuidsCountPos;

		TArray<FString> CheckpointDeletedNetStartupActors;

		TMap<FName, uint32> NameTableMap;

		void CountBytes(FArchive& Ar) const;
	};

	FCheckpointSaveStateContext CheckpointSaveContext;

	FDeltaCheckpointData RecordingDeltaCheckpointData;

	TArray<TUniquePtr<FDeltaCheckpointData>> PlaybackDeltaCheckpointData;

	/**
	 * During recording, all unique streaming levels since recording started.
	 * During playback, all streaming level instances we've created.
	 */
	TSet<TWeakObjectPtr<UObject>> UniqueStreamingLevels;

	/**
	 * During recording, streaming levels waiting to be saved next frame.
	 * During playback, streaming levels that have recently become visible.
	 */
	TSet<TWeakObjectPtr<UObject>> NewStreamingLevelsThisFrame;

	TArray<FQueuedDemoPacket> QueuedDemoPackets;
	TArray<FQueuedDemoPacket> QueuedCheckpointPackets;

	/**
	 * Helps keeps tabs on what levels are Ready, Have Seen data, Level Name, and Index into the main status list.
	 *
	 * A Level is not considered ready until the following criteria are met:
	 *	- UWorld::AddToWorld has been called, signifying the level is both Loaded and Visible (in the streaming sense).
	 *	- Either:
	 *		No packets of data have been processed for the level (yet),
	 *		OR The level has been fully fast-forwarded.
	 *
	 * A level is marked as Seen once the replay has seen a packet marked for the level.
	 */
	struct FLevelStatus
	{
		FLevelStatus(const FString& LevelPackageName) :
			LevelName(LevelPackageName),
			LevelIndex(INDEX_NONE),
			bIsReady(false),
			bHasBeenSeen(false)
		{
		}

		// Level name.
		FString LevelName;

		// Level index (in AllLevelStatuses).
		int32 LevelIndex;

		// Whether or not the level is ready to receive streaming data.
		bool bIsReady;

		// Whether or not we've seen replicated data for the level. Only set during playback.
		bool bHasBeenSeen;

		void CountBytes(FArchive& Ar) const
		{
			LevelName.CountBytes(Ar);
		}
	};

	// Tracks all available level statuses.
	// When Recording, this will be in order of replication, and all statuses will be assumed Seen and Visible (even if unmarked).
	// During Playback, there's no guaranteed order. Levels will be added either when they are added to the world, or when we handle the first
	// frame containing replicated data.
	// Use SeenLevelStatuses and LevelStatusesByName for querying.
	TArray<FLevelStatus> AllLevelStatuses;

	// Since Arrays are dynamically allocated, we can't just hold onto pointers.
	// If we tried, the underlying memory could be moved without us knowing.
	// Therefore, we track the Index into the array which should be independent of allocation.

	// Index of level status (in AllLevelStatuses list).
	TMap<FString, int32> LevelStatusesByName;

	// Maintain a quick lookup for loaded levels directly to LevelStatus
	TMap<const ULevel*, int32> LevelStatusIndexByLevel;

	// Map of ULevel GetFName to weak object pointer to the level
	// Populated during playback, not using NetworkRemapPath because it is never serialized
	TMap<FName, TWeakObjectPtr<ULevel>> WeakLevelsByName;

	void ResetLevelMap();
	void ClearLevelMap();

	// List of seen level statuses indices (in AllLevelStatuses).
	TArray<int32> SeenLevelStatuses;

	// Only used during recording.
	uint32 NumLevelsAddedThisFrame;

	// Levels that are currently pending for fast forward.
	// Using raw pointers, because we manually keep when levels are added and removed.
	TSet<class ULevel*> LevelsPendingFastForward;

	bool bPendingCheckpointRequest;

	UE::Net::FNetResultManager ResultManager;

	static FString GetLevelPackageName(const ULevel& InLevel);

	void ResetLevelStatuses();
	void ClearLevelStreamingState()
	{
		AllLevelStatuses.Empty();
		LevelStatusesByName.Empty();
		SeenLevelStatuses.Empty();
		LevelsPendingFastForward.Empty();
		NumLevelsAddedThisFrame = 0;
		LevelStatusIndexByLevel.Reset();
	}

	FLevelStatus& FindOrAddLevelStatus(const ULevel& Level)
	{
		// see if we can find it in the cache
		int32* LevelStatusIndex = LevelStatusIndexByLevel.Find(&Level);

		if (LevelStatusIndex)
		{
			return AllLevelStatuses[*LevelStatusIndex];
		}

		FLevelStatus& LevelStatus = FindOrAddLevelStatus(GetLevelPackageName(Level));
		LevelStatusIndexByLevel.Add(&Level, LevelStatus.LevelIndex);

		return LevelStatus;
	}

	FLevelStatus& FindOrAddLevelStatus(const FString& LevelPackageName)
	{
		return FindOrAddLevelStatus(FString(LevelPackageName));
	}

	FLevelStatus& FindOrAddLevelStatus(FString&& LevelPackageName)
	{
		if (int32* LevelStatusIndex = LevelStatusesByName.Find(LevelPackageName))
		{
			return AllLevelStatuses[*LevelStatusIndex];
		}

		const int32 Index = AllLevelStatuses.Emplace(LevelPackageName);
		AllLevelStatuses[Index].LevelIndex = Index;

		LevelStatusesByName.Add(MoveTemp(LevelPackageName), Index);
		NumLevelsAddedThisFrame++;

		return AllLevelStatuses[Index];
	}

	FLevelStatus& GetLevelStatus(const int32 SeenLevelIndex)
	{
		return AllLevelStatuses[SeenLevelStatuses[SeenLevelIndex - 1]];
	}

	FLevelStatus& GetLevelStatus(const FString& LevelPackageName)
	{
		return AllLevelStatuses[LevelStatusesByName[LevelPackageName]];
	}

	/** ExternalDataToObjectMap is used to map a FNetworkGUID to the proper FReplayExternalDataArray */
	TMap<FNetworkGUID, FReplayExternalDataArray> ExternalDataToObjectMap;

	/** PlaybackFrames are used to buffer per frame data up when we read a demo frame, which we can then process when the time is right */
	TMap<float, TMap<FString, TArray<uint8>>> PlaybackFrames;

	/** Net startup actors that need to be destroyed after checkpoints are loaded */
	TArray<FString> RecordingDeletedNetStartupActors;

	/** Net startup actors that need to be destroyed after checkpoints are loaded */
	TSet<FString> PlaybackDeletedNetStartupActors;

	TSharedPtr<IAnalyticsProvider> AnalyticsProvider;

	void ReadDeletedStartupActors(UNetConnection* Connection, FArchive& Ar, TSet<FString>& DeletedStartupActors);

	ECheckpointSaveState GetCheckpointSaveState() const { return CheckpointSaveContext.CheckpointSaveState; }

	void ProcessCheckpointActors(UNetConnection* Connection, TArrayView<FPendingCheckPointActor> PendingActors, int32& NextIndex, FRepActorsCheckpointParams& Params);

	void NotifyReplayError(UE::Net::TNetResult<EReplayResult>&& Result);

	bool bRecording;
public:
	static constexpr int32 MAX_DEMO_READ_WRITE_BUFFER = 1024 * 2;
	static constexpr int32 MAX_DEMO_STRING_SERIALIZATION_SIZE = 16 * 1024 * 1024;
};
