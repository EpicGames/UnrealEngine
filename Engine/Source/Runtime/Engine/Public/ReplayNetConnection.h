// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/NetConnection.h"
#include "ReplayHelper.h"
#include "ReplayNetConnection.generated.h"

UCLASS(transient, config=Engine, MinimalAPI)
class UReplayNetConnection : public UNetConnection
{
	GENERATED_BODY()

public:
	ENGINE_API UReplayNetConnection(const FObjectInitializer& ObjectInitializer);

	ENGINE_API virtual void Tick(float DeltaSEconds) override;

	// UNetConnection interface.
	ENGINE_API virtual void InitConnection(UNetDriver* InDriver, EConnectionState InState, const FURL& InURL, int32 InConnectionSpeed = 0, int32 InMaxPacket=0) override;
	ENGINE_API virtual FString LowLevelGetRemoteAddress(bool bAppendPort = false) override;
	ENGINE_API virtual FString LowLevelDescribe() override;
	ENGINE_API virtual void LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits) override;
	
	UE_DEPRECATED(5.6, "Replaced with a version that doesn't have a saturate option.")
	ENGINE_API virtual int32 IsNetReady(bool bSaturate) override;

	virtual bool IsNetReady() const override
	{
		return true;
	}
	
	ENGINE_API virtual void HandleClientPlayer(APlayerController* PC, UNetConnection* NetConnection) override;
	ENGINE_API virtual TSharedPtr<const FInternetAddr> GetRemoteAddr() override;
	ENGINE_API virtual bool ClientHasInitializedLevel(const ULevel* TestLevel) const override;
	virtual FString RemoteAddressToString() override { return TEXT("Replay"); }
	ENGINE_API virtual void CleanUp() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void NotifyActorDestroyed(AActor* Actor, bool IsSeamlessTravel = false) override;
	ENGINE_API virtual bool IsReplayReady() const override;

	ENGINE_API virtual void NotifyActorChannelCleanedUp(UActorChannel* Channel, EChannelCloseReason CloseReason) override;

	virtual void InitRemoteConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override {}
	virtual void InitLocalConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override {}

	ENGINE_API void StartRecording();

	ENGINE_API void AddEvent(const FString& Group, const FString& Meta, const TArray<uint8>& Data);
	ENGINE_API void AddOrUpdateEvent(const FString& EventName, const FString& Group, const FString& Meta, const TArray<uint8>& Data);

	ENGINE_API void AddUserToReplay(const FString& UserString);

	FString GetActiveReplayName() const { return ReplayHelper.ActiveReplayName; }
	float GetReplayCurrentTime() const { return ReplayHelper.DemoCurrentTime; }
	float GetReplayTotalTime() const { return ReplayHelper.DemoTotalTime; }

	ENGINE_API bool IsSavingCheckpoint() const;

	ENGINE_API void OnSeamlessTravelStart(UWorld* CurrentWorld, const FString& LevelName);

	ENGINE_API void SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InProvider);

	ENGINE_API void SetCheckpointSaveMaxMSPerFrame(const float InCheckpointSaveMaxMSPerFrame);

	ENGINE_API void RequestCheckpoint();

	ENGINE_API bool SetExternalDataForObject(UObject* OwningObject, const uint8* Src, const int32 NumBits);

private:
	ENGINE_API void TrackSendForProfiler(const void* Data, int32 NumBytes);

	FReplayHelper ReplayHelper;

	int32 DemoFrameNum;

	FDelegateHandle OnLevelRemovedFromWorldHandle;
	FDelegateHandle OnLevelAddedToWorldHandle;

	ENGINE_API void OnLevelRemovedFromWorld(class ULevel* Level, class UWorld* World);
	ENGINE_API void OnLevelAddedToWorld(class ULevel* Level, class UWorld* World);

	ENGINE_API FName NetworkRemapPath(FName InPackageName, bool bReading);
};
