// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/GameInstanceSubsystem.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "ReplaySubsystem.generated.h"

class UReplayNetConnection;

namespace UE::ReplaySubsystem
{
	enum class EStopReplayFlags : uint32
	{
		None = 0x0,
		Flush = 0x1,
	};

	ENUM_CLASS_FLAGS(EStopReplayFlags);
};

UCLASS(DisplayName = "Replay Subsystem", MinimalAPI)
class UReplaySubsystem : public UGameInstanceSubsystem
{
	GENERATED_BODY()

public:
	/* UGameInstanceSubsystem */
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;

	/**
	 * Begin replay recording
	 *
	 * @param Name Replay name (session name, file name, etc)
	 * @param FriendlyName Description of replay, preferably human readable 
	 * @param AdditionalOptions Additional options values, if any, such as a replay streamer override
	 * @param AnalyticsProvider Any analytics provider interface in case the replay subsystem/streamer has events to report
	 */
	ENGINE_API void RecordReplay(const FString& Name, const FString& FriendlyName, const TArray<FString>& AdditionalOptions, TSharedPtr<IAnalyticsProvider> AnalyticsProvider);

	/**
	 * Begin replay playback
	 *
	 * @param Name Replay name (session name, file name, etc)
	 * @param WorldOverride world overridef for playing back on specific UWorld
	 * @param AdditionalOptions addition options values, if any, such as a replay streamer override
	 */
	ENGINE_API bool PlayReplay(const FString& Name, UWorld* WorldOverride, const TArray<FString>& AdditionalOptions);

	/**
	 * Stop replay recording/playback
	 */
	ENGINE_API void StopReplay();

	/**
	 * Get current recording/playing replay name
	 *
	 * @return FString Name of relpay (session id, file name, etc)
	 */
	UFUNCTION(BlueprintCallable, Category=Replay)
	ENGINE_API FString GetActiveReplayName() const;

	/**
	 * Get current recording/playing total replay time
	 *
	* @return float total recording/playing time in seconds
	*/
	UFUNCTION(BlueprintCallable, Category = Replay)
	ENGINE_API float GetReplayTotalTime() const;

	/**
	 * Get current recording/playing replay time
	 *
	 * @return float Current recording/playback time in seconds
	 */
	UFUNCTION(BlueprintCallable, Category=Replay)
	ENGINE_API float GetReplayCurrentTime() const;

	/**
	 * Add a user to be associated with the replay (legacy)
	 *
	 * @param UserString String representing user (platform specific id, user name, etc)
	 */
	ENGINE_API void AddUserToReplay(const FString& UserString);

	UFUNCTION(BlueprintCallable, Category=Replay)
	ENGINE_API bool IsRecording() const;
	
	UFUNCTION(BlueprintCallable, Category=Replay)
	ENGINE_API bool IsPlaying() const;
	
	ENGINE_API bool IsSavingCheckpoint() const;

	/**
	 * If given replay playback was deferred and has not started playback yet, this will return true.
	 *
	 * @param ReplayName Name of the replay file.
	 * @return True if replay playback was deferred
	 */
	ENGINE_API virtual bool IsReplayDeferred(const FString& ReplayName) const;

	/**
	 * Add an event to the currently recording replay, associated with the current time
	 *
	 * @param Group Event group identifier
	 * @param Meta Metadata associated with the event
	 * @param Data Buffer of bytes representing the event payload
	 */
	ENGINE_API void AddEvent(const FString& Group, const FString& Meta, const TArray<uint8>& Data);

	/**
	 * Add or update an existing event in the recording replay, see AddEvent as well
	 *
	 * @param EventName Unique event name identifier
	 * @param Group Event group identifier
	 * @param Meta Metadata associated with the event
	 * @param Data Buffer of bytes representing the event payload
	 */
	ENGINE_API void AddOrUpdateEvent(const FString& EventName, const FString& Group, const FString& Meta, const TArray<uint8>& Data);

	/**
	 * Set per frame limit spent recording checkpoint data
	 *
	 * @param InCheckpointSaveMaxMSPerFrame Time in milliseconds
	 */
	ENGINE_API void SetCheckpointSaveMaxMSPerFrame(const float InCheckpointSaveMaxMSPerFrame);

	/**
	 * Request a checkpoint write, if currently recording.
	 *
	*/
	UFUNCTION(BlueprintCallable, Category=Replay)
	ENGINE_API void RequestCheckpoint();

	/**
	 * Add external data associated with an object to the recording replay
	 *
	 * @param OwningObject Recorded UObject to associate the data with
	 * @param Src Pointer to the external data buffer
	 * @param NumBits Number of bits to store from Src
	 */
	ENGINE_API void SetExternalDataForObject(UObject* OwningObject, const uint8* Src, const int32 NumBits);

	/**
	 * Whether to reload the default map when StopReplay is called.
	 */
	UPROPERTY(EditAnywhere, Category=Replay)
	bool bLoadDefaultMapOnStop = true;

private:
	void StopExistingReplays(UWorld* InWorld, UE::ReplaySubsystem::EStopReplayFlags Flags = UE::ReplaySubsystem::EStopReplayFlags::None);

	void OnSeamlessTravelStart(UWorld* CurrentWorld, const FString& LevelName);
	void OnSeamlessLevelTransition(UWorld* CurrentWorld);
	void OnCopyWorldData(UWorld* CurrentWorld, UWorld* LoadedWorld);

	TWeakObjectPtr<UReplayNetConnection> ReplayConnection;
};
