// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectVersion.h"
#include "Misc/EngineVersion.h"
#include "Misc/NetworkGuid.h"
#include "Misc/NetworkVersion.h"
#include "Net/Common/Packets/PacketTraits.h"
#include "Net/ReplayResult.h"
#include "IPAddress.h"
#include "Serialization/BitReader.h"
#include "Serialization/CustomVersion.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "ReplayTypes.generated.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogDemo, Log, All);

class UNetConnection;
enum class EChannelCloseReason : uint8;
enum ELifetimeCondition : int;

enum class EReplayHeaderFlags : uint32
{
	None = 0,
	ClientRecorded = (1 << 0),
	HasStreamingFixes = (1 << 1),
	DeltaCheckpoints = (1 << 2),
	GameSpecificFrameData = (1 << 3),
	ReplayConnection = (1 << 4),
	ActorPrioritizationEnabled = (1 << 5),
	NetRelevancyEnabled = (1 << 6),
	AsyncRecorded = (1 << 7),
};

ENUM_CLASS_FLAGS(EReplayHeaderFlags);

const TCHAR* LexToString(EReplayHeaderFlags Flag);

enum class EWriteDemoFrameFlags : uint32
{
	None = 0,
	SkipGameSpecific = (1 << 0),
};

ENUM_CLASS_FLAGS(EWriteDemoFrameFlags);

struct FPlaybackPacket
{
	TArray<uint8>		Data;
	float				TimeSeconds;
	int32				LevelIndex;
	uint32				SeenLevelIndex;

	void CountBytes(FArchive& Ar) const
	{
		Data.CountBytes(Ar);
	}
};

USTRUCT()
struct FLevelNameAndTime
{
	GENERATED_BODY()

	FLevelNameAndTime()
		: LevelChangeTimeInMS(0)
	{}

	FLevelNameAndTime(const FString& InLevelName, uint32 InLevelChangeTimeInMS)
		: LevelName(InLevelName)
		, LevelChangeTimeInMS(InLevelChangeTimeInMS)
	{}

	UPROPERTY()
	FString LevelName;

	UPROPERTY()
	uint32 LevelChangeTimeInMS;

	friend FArchive& operator<<(FArchive& Ar, FLevelNameAndTime& LevelNameAndTime)
	{
		Ar << LevelNameAndTime.LevelName;
		Ar << LevelNameAndTime.LevelChangeTimeInMS;
		return Ar;
	}

	void CountBytes(FArchive& Ar) const
	{
		LevelName.CountBytes(Ar);
	}
};

enum UE_DEPRECATED(5.2, "Using custom versions instead going forward, see FReplayCustomVersion") ENetworkVersionHistory
{
	HISTORY_REPLAY_INITIAL = 1,
	HISTORY_SAVE_ABS_TIME_MS = 2,				// We now save the abs demo time in ms for each frame (solves accumulation errors)
	HISTORY_INCREASE_BUFFER = 3,				// Increased buffer size of packets, which invalidates old replays
	HISTORY_SAVE_ENGINE_VERSION = 4,			// Now saving engine net version + InternalProtocolVersion
	HISTORY_EXTRA_VERSION = 5,					// We now save engine/game protocol version, checksum, and changelist
	HISTORY_MULTIPLE_LEVELS = 6,				// Replays support seamless travel between levels
	HISTORY_MULTIPLE_LEVELS_TIME_CHANGES = 7,	// Save out the time that level changes happen
	HISTORY_DELETED_STARTUP_ACTORS = 8,			// Save DeletedNetStartupActors inside checkpoints
	HISTORY_HEADER_FLAGS = 9,					// Save out enum flags with demo header
	HISTORY_LEVEL_STREAMING_FIXES = 10,			// Optional level streaming fixes.
	HISTORY_SAVE_FULL_ENGINE_VERSION = 11,		// Now saving the entire FEngineVersion including branch name
	HISTORY_HEADER_GUID = 12,					// Save guid to demo header
	HISTORY_CHARACTER_MOVEMENT = 13,			// Change to using replicated movement and not interpolation
	HISTORY_CHARACTER_MOVEMENT_NOINTERP = 14,	// No longer recording interpolated movement samples
	HISTORY_GUID_NAMETABLE = 15,				// Added a string table for exported guids
	HISTORY_GUIDCACHE_CHECKSUMS = 16,			// Removing guid export checksums from saved data, they are ignored during playback
	HISTORY_SAVE_PACKAGE_VERSION_UE = 17,		// Save engine and licensee package version as well, in case serialization functions need them for compatibility
	HISTORY_RECORDING_METADATA = 18,			// Adding additional record-time information to the header
	HISTORY_USE_CUSTOM_VERSION = 19,			// Serializing replay and network versions as custom verions going forward

	// -----<new versions can be added before this line>-------------------------------------------------
	HISTORY_PLUS_ONE,
	HISTORY_LATEST = HISTORY_PLUS_ONE - 1
};

struct FReplayCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Original replay versions from ENetworkVersionHistory
		ReplayInitial = 1,
		SaveAbsTimeMs = 2,					// We now save the abs demo time in ms for each frame (solves accumulation errors)
		IncreaseBuffer = 3,					// Increased buffer size of packets, which invalidates old replays
		SaveEngineVersion = 4,				// Now saving engine net version + InternalProtocolVersion
		ExtraVersion = 5,					// We now save engine/game protocol version, checksum, and changelist
		MultipleLevels = 6,					// Replays support seamless travel between levels
		MultipleLvelsTimeChanges,			// Save out the time that level changes happen
		DeletedStartupActors = 8,			// Save DeletedNetStartupActors inside checkpoints
		HeaderFlags = 9,					// Save out enum flags with demo header
		LevelStreamingFixes = 10,			// Optional level streaming fixes.
		SaveFullEngineVersion = 11,			// Now saving the entire FEngineVersion including branch name
		HeaderGuid = 12,					// Save guid to demo header
		CharacterMovement = 13,				// Change to using replicated movement and not interpolation
		CharacterMovementNoInterp = 14,		// No longer recording interpolated movement samples
		GuidNameTable = 15,					// Added a string table for exported guids
		GuidCacheChecksums = 16,			// Removing guid export checksums from saved data, they are ignored during playback
		SavePackageVersionUE = 17,			// Save engine and licensee package version as well, in case serialization functions need them for compatibility
		RecordingMetadata = 18,				// Adding additional record-time information to the header
		CustomVersions = 19,				// Serializing replay and network versions as custom verions going forward

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1,

		MinSupportedVersion = CharacterMovement	// Minimum supported playback version
	};

	// The GUID for this custom version number
	ENGINE_API const static FGuid Guid;

	FReplayCustomVersion() = delete;
};

inline constexpr uint32 NETWORK_DEMO_MAGIC = 0x2CF5A13D;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_DEPRECATED(5.2, "Now using custom versions, see FReplayCustomVersion::Latest")
inline constexpr uint32 NETWORK_DEMO_VERSION = ENetworkVersionHistory::HISTORY_LATEST;
UE_DEPRECATED(5.2, "Using custom versions instead going forward.")
inline constexpr uint32 MIN_NETWORK_DEMO_VERSION = ENetworkVersionHistory::HISTORY_CHARACTER_MOVEMENT;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UE_DEPRECATED(5.2, "No longer used.")
inline constexpr uint32 NETWORK_DEMO_METADATA_MAGIC = 0x3D06B24E;
UE_DEPRECATED(5.2, "No longer used.")
inline constexpr uint32 NETWORK_DEMO_METADATA_VERSION = 0;

struct FNetworkDemoHeader
{
	uint32	Magic;									// Magic to ensure we're opening the right file.
	
	UE_DEPRECATED(5.2, "No longer used in favor of custom versions, kept for backwards compatibility.")
	uint32	Version;								// Version number to detect version mismatches.

	UE_DEPRECATED(5.2, "No longer used.")
	uint32	NetworkChecksum;						// Network checksum

	UE_DEPRECATED(5.2, "No longer used in favor of custom versions, kept for backwards compatibility.")
	uint32	EngineNetworkProtocolVersion;			// Version of the engine internal network format

	UE_DEPRECATED(5.2, "No longer used in favor of custom versions, kept for backwards compatibility.")
	uint32	GameNetworkProtocolVersion;				// Version of the game internal network format

	FCustomVersionContainer CustomVersions;

	FGuid	Guid;									// Unique identifier

	float MinRecordHz;
	float MaxRecordHz;
	float FrameLimitInMS;
	float CheckpointLimitInMS;

	FString Platform;
	EBuildConfiguration BuildConfig;
	EBuildTargetType BuildTarget;

	FEngineVersion EngineVersion;					// Full engine version on which the replay was recorded
	EReplayHeaderFlags HeaderFlags;					// Replay flags
	TArray<FLevelNameAndTime> LevelNamesAndTimes;	// Name and time changes of levels loaded for demo
	TArray<FString> GameSpecificData;				// Area for subclasses to write stuff
	FPackageFileVersion PackageVersionUE;			// Engine package version on which the replay was recorded
	int32 PackageVersionLicenseeUE;					// Licensee package version on which the replay was recorded

	ENGINE_API FNetworkDemoHeader();

	ENGINE_API void SetDefaultNetworkVersions();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FNetworkDemoHeader(const FNetworkDemoHeader&) = default;
	FNetworkDemoHeader& operator=(const FNetworkDemoHeader&) = default;
	FNetworkDemoHeader(FNetworkDemoHeader&&) = default;
	FNetworkDemoHeader& operator=(FNetworkDemoHeader&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ENGINE_API friend FArchive& operator << (FArchive& Ar, FNetworkDemoHeader& Header);

	ENGINE_API void CountBytes(FArchive& Ar) const;

	ENGINE_API uint32 GetCustomVersion(const FGuid& VersionGuid) const;
};

// The type we use to store offsets in the archive
typedef int64 FArchivePos;

struct FDeltaCheckpointData
{
	/** Net startup actors that were destroyed */
	TArray<FString> RecordingDeletedNetStartupActors;
	/** Net startup actors that were destroyed */
	TSet<FString> DestroyedNetStartupActors;

	/** Destroyed dynamic actors that were active in the previous checkpoint */
	TSet<FNetworkGUID> DestroyedDynamicActors;
	/** Channels closed that were open in the previous checkpoint, and the reason why */
	TMap<FNetworkGUID, EChannelCloseReason> ChannelsToClose;

	void CountBytes(FArchive& Ar) const;
};

class FRepActorsCheckpointParams
{
public:
	const double StartCheckpointTime;
	const double CheckpointMaxUploadTimePerFrame;
};

struct FQueuedDemoPacket
{
	/** The packet data to send */
	TArray<uint8> Data;

	/** The size of the packet in bits */
	int32 SizeBits;

	/** The traits applied to the packet, if applicable */
	FOutPacketTraits Traits;

	/** Index of the level this packet is associated with. 0 indicates no association. */
	uint32 SeenLevelIndex;

public:
	FQueuedDemoPacket(uint8* InData, int32 InSizeBytes, int32 InSizeBits)
		: Data()
		, SizeBits(InSizeBits)
		, Traits()
		, SeenLevelIndex(0)
	{
		Data.AddUninitialized(InSizeBytes);
		FMemory::Memcpy(Data.GetData(), InData, InSizeBytes);
	}

	FQueuedDemoPacket(uint8* InData, int32 InSizeBits, FOutPacketTraits& InTraits)
		: Data()
		, SizeBits(InSizeBits)
		, Traits(InTraits)
		, SeenLevelIndex(0)
	{
		int32 SizeBytes = FMath::DivideAndRoundUp(InSizeBits, 8);

		Data.AddUninitialized(SizeBytes);
		FMemory::Memcpy(Data.GetData(), InData, SizeBytes);
	}

	void CountBytes(FArchive& Ar) const
	{
		Data.CountBytes(Ar);
	}
};

/*------------------------------------------------------------------------------------------
	FInternetAddrDemo - dummy internet addr that can be used for anything that requests it.
--------------------------------------------------------------------------------------------*/
class FInternetAddrDemo : public FInternetAddr
{
public:

	FInternetAddrDemo()
	{
	}

	virtual TArray<uint8> GetRawIp() const override
	{
		return TArray<uint8>();
	}

	virtual void SetRawIp(const TArray<uint8>& RawAddr) override
	{
	}

	void SetIp(uint32 InAddr) override
	{
	}


	void SetIp(const TCHAR* InAddr, bool& bIsValid) override
	{
	}

	void GetIp(uint32& OutAddr) const override
	{
		OutAddr = 0;
	}

	void SetPort(int32 InPort) override
	{
	}


	void GetPort(int32& OutPort) const override
	{
		OutPort = 0;
	}


	int32 GetPort() const override
	{
		return 0;
	}

	void SetAnyAddress() override
	{
	}

	void SetBroadcastAddress() override
	{
	}

	void SetLoopbackAddress() override
	{
	}

	FString ToString(bool bAppendPort) const override
	{
		return FString("Demo Internet Address");
	}

	virtual bool operator==(const FInternetAddr& Other) const override
	{
		return Other.ToString(true) == ToString(true);
	}

	bool operator!=(const FInternetAddrDemo& Other) const
	{
		return !(FInternetAddrDemo::operator==(Other));
	}

	virtual uint32 GetTypeHash() const override
	{
		return GetConstTypeHash();
	}

	uint32 GetConstTypeHash() const
	{
		return GetTypeHashHelper(ToString(true));
	}

	friend uint32 GetTypeHash(const FInternetAddrDemo& A)
	{
		return A.GetConstTypeHash();
	}

	virtual bool IsValid() const override
	{
		return true;
	}

	virtual TSharedRef<FInternetAddr> Clone() const override
	{
		return DemoInternetAddr.ToSharedRef();
	}

	static TSharedPtr<FInternetAddr> DemoInternetAddr;
};

class FScopedForceUnicodeInArchive
{
public:
	FScopedForceUnicodeInArchive() = delete;
	FScopedForceUnicodeInArchive(FScopedForceUnicodeInArchive&&) = delete;
	FScopedForceUnicodeInArchive(const FScopedForceUnicodeInArchive&) = delete;
	FScopedForceUnicodeInArchive& operator=(const FScopedForceUnicodeInArchive&) = delete;
	FScopedForceUnicodeInArchive& operator=(FScopedForceUnicodeInArchive&&) = delete;

	FScopedForceUnicodeInArchive(FArchive& InArchive)
		: Archive(InArchive)
		, bWasUnicode(InArchive.IsForcingUnicode())
	{
		EnableFastStringSerialization();
	}

	~FScopedForceUnicodeInArchive()
	{
		RestoreStringSerialization();
	}

private:
	void EnableFastStringSerialization()
	{
		if constexpr (TIsCharEncodingCompatibleWith<WIDECHAR, TCHAR>::Value)
		{
			Archive.SetForceUnicode(true);
		}
	}

	void RestoreStringSerialization()
	{
		if constexpr (TIsCharEncodingCompatibleWith<WIDECHAR, TCHAR>::Value)
		{
			Archive.SetForceUnicode(bWasUnicode);
		}
	}

	FArchive& Archive;
	bool bWasUnicode;
};

/**
 * Helps track Offsets in an Archive before the actual size of the offset is known.
 * This relies on serialization always used a fixed number of bytes for primitive types,
 * and Sane implementations of Seek and Tell.
 */
class FScopedStoreArchiveOffset
{
public:
	FScopedStoreArchiveOffset() = delete;
	FScopedStoreArchiveOffset(FScopedStoreArchiveOffset&&) = delete;
	FScopedStoreArchiveOffset(const FScopedStoreArchiveOffset&) = delete;
	FScopedStoreArchiveOffset& operator=(const FScopedStoreArchiveOffset&) = delete;
	FScopedStoreArchiveOffset& operator=(FScopedStoreArchiveOffset&&) = delete;

	FScopedStoreArchiveOffset(FArchive& InAr) :
		Ar(InAr),
		StartPosition(Ar.Tell())
	{
		// Save room for the offset here.
		FArchivePos TempOffset = 0;
		Ar << TempOffset;
	}

	~FScopedStoreArchiveOffset()
	{
		const FArchivePos CurrentPosition = Ar.Tell();
		FArchivePos Offset = CurrentPosition - (StartPosition + sizeof(FArchivePos));
		Ar.Seek(StartPosition);
		Ar << Offset;
		Ar.Seek(CurrentPosition);
	}

private:

	FArchive& Ar;
	const FArchivePos StartPosition;
};

class FReplayExternalData
{
public:
	FReplayExternalData() : TimeSeconds(0.0f)
	{
	}

	FReplayExternalData(FBitReader&& InReader, const float InTimeSeconds) 
		: Reader(MoveTemp(InReader))
		, TimeSeconds(InTimeSeconds)
	{
	}

	FBitReader	Reader;
	float		TimeSeconds;

	void CountBytes(FArchive& Ar) const
	{
		Reader.CountMemory(Ar);
	}
};

// Using an indirect array here since FReplayExternalData stores an FBitReader, and it's not safe to store an FArchive directly in a TArray.
typedef TIndirectArray<FReplayExternalData> FReplayExternalDataArray;

// Can be used to override Version Data in a Replay's Header either Right Before Writing a Replay Header or Right After Reading a Replay Header.
struct FOverridableReplayVersionData
{
public:
	UE_DEPRECATED(5.2, "No longer used in favor of CustomVersions")
	uint32 Version;                       // Version number to detect version mismatches.
	UE_DEPRECATED(5.2, "No longer used in favor of CustomVersions")
	uint32 EngineNetworkProtocolVersion;  // Version of the engine internal network format
	UE_DEPRECATED(5.2, "No longer used in favor of CustomVersions")
	uint32 GameNetworkProtocolVersion;    // Version of the game internal network format

	FCustomVersionContainer CustomVersions;
	FEngineVersion EngineVersion;         // Full engine version on which the replay was recorded
	FPackageFileVersion PackageVersionUE; // Engine package version on which the replay was recorded
	int32 PackageVersionLicenseeUE;       // Licensee package version on which the replay was recorded

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FOverridableReplayVersionData(FOverridableReplayVersionData&&) = default;
	FOverridableReplayVersionData(const FOverridableReplayVersionData&) = default;
	FOverridableReplayVersionData& operator=(FOverridableReplayVersionData&&) = default;
	FOverridableReplayVersionData& operator=(const FOverridableReplayVersionData&) = default;

	// Init with Demo Header Version Data
	FOverridableReplayVersionData(const FNetworkDemoHeader& DemoHeader)
		: Version                     (DemoHeader.Version)
		, EngineNetworkProtocolVersion(DemoHeader.EngineNetworkProtocolVersion)
		, GameNetworkProtocolVersion  (DemoHeader.GameNetworkProtocolVersion)
		, CustomVersions              (DemoHeader.CustomVersions)
		, EngineVersion               (DemoHeader.EngineVersion)
		, PackageVersionUE            (DemoHeader.PackageVersionUE)
		, PackageVersionLicenseeUE    (DemoHeader.PackageVersionLicenseeUE)
	{
	}

	// Apply Version Data to Demo Header Passed In
	void ApplyVersionDataToDemoHeader(FNetworkDemoHeader& DemoHeader)
	{
		DemoHeader.Version                      = Version;
		DemoHeader.EngineNetworkProtocolVersion = EngineNetworkProtocolVersion;
		DemoHeader.GameNetworkProtocolVersion   = GameNetworkProtocolVersion;
		DemoHeader.CustomVersions               = CustomVersions;
		DemoHeader.EngineVersion                = EngineVersion;
		DemoHeader.PackageVersionUE             = PackageVersionUE;
		DemoHeader.PackageVersionLicenseeUE     = PackageVersionLicenseeUE;
	}
	ENGINE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	uint32 GetCustomVersion(const FGuid& VersionGuid) const;
};
