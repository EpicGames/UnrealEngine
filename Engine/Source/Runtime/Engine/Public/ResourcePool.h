// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 Resource.h: Template for pooling resources using buckets.
 =============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "TickableObjectRenderThread.h"
#include "RHICommandList.h"
#include "Async/Mutex.h"

/** A templated pool for resources that can only be freed at a 'safe' point in the frame. */
template<typename ResourceType, class ResourcePoolPolicy, class ResourceCreationArguments>
class TResourcePool
{
public:
	TResourcePool() = default;

	/** Constructor with policy argument
	 * @param InPolicy An initialised policy object
	 */
	TResourcePool(ResourcePoolPolicy InPolicy)
	: Policy(InPolicy)
	{}

	/** Destructor */
	virtual ~TResourcePool()
	{
		DrainPool(true);
	}
	
	/** Gets the size a pooled object will use when constructed from the pool.
	 * @param Args the argument object for construction.
	 * @returns The size for a pooled object created with Args
	 */
	uint32 PooledSizeForCreationArguments(ResourceCreationArguments Args)
	{
		const uint32 BucketIndex = Policy.GetPoolBucketIndex(Args);
		return Policy.GetPoolBucketSize(BucketIndex);
	}
	
	/** Creates a pooled resource.
	 * @param Args the argument object for construction.
	 * @returns An initialised resource.
	 */
	ResourceType CreatePooledResource(FRHICommandListBase& RHICmdList, ResourceCreationArguments Args)
	{
		// Find the appropriate bucket based on size
		const uint32 BucketIndex = Policy.GetPoolBucketIndex(Args);
		TArray<FPooledResource>& PoolBucket = ResourceBuckets[BucketIndex];

		{
			UE::TConditionalScopeLock Lock(CS, !bLocked);

			if (PoolBucket.Num() > 0)
			{
				// Reuse the last entry in this size bucket
				return PoolBucket.Pop(EAllowShrinking::No).Resource;
			}
		}

		// Nothing usable was found in the free pool, create a new resource
		return Policy.CreateResource(RHICmdList, Args);
	}
	
	/** Release a resource back into the pool.
	 * @param Resource The resource to return to the pool
	 */
	void ReleasePooledResource(ResourceType&& Resource)
	{
		FPooledResource NewEntry;
		NewEntry.CreationArguments = Policy.GetCreationArguments(Resource);
		NewEntry.Resource = Forward<ResourceType&&>(Resource);
		NewEntry.FrameFreed = GFrameNumberRenderThread;
		NewEntry.BucketIndex = Policy.GetPoolBucketIndex(NewEntry.CreationArguments);
		
		// Add to this frame's array of free resources
		const int32 SafeFrameIndex = GFrameNumberRenderThread % ResourcePoolPolicy::NumSafeFrames;

		UE::TConditionalScopeLock Lock(CS, !bLocked);
		SafeResourceBuckets[SafeFrameIndex].Emplace(MoveTemp(NewEntry));
	}
	
	/** Drain the pool of freed resources that need to be culled or prepared for reuse.
	 * @param bForceDrainAll Clear the pool of all free resources, rather than obeying the policy
	 */
	void DrainPool(bool bForceDrainAll)
	{
		uint32 NumToCleanThisFrame = ResourcePoolPolicy::NumToDrainPerFrame;
		uint32 CullAfterFramesNum = ResourcePoolPolicy::CullAfterFramesNum;

		check(!bLocked);
		UE::TScopeLock Lock(CS);

		if(!bForceDrainAll)
		{
			// DrainPool won't necessarily be called at the same frequency as the render thread number increments.
			// Therefore, we track the frame number from the previous drain that occurred and drain all frames between
			// the last and the current frame number (capping out at draining all frames).

			uint32 SafeFrameNumber = GFrameNumberRenderThread + 1;

			if (LastSafeFrameNumber == INDEX_NONE)
			{
				LastSafeFrameNumber = SafeFrameNumber;
			}

			uint32 NumSafeFramesToDrain = FMath::Clamp<uint32>(SafeFrameNumber - LastSafeFrameNumber, 1, ResourcePoolPolicy::NumSafeFrames);

			for (uint32 Index = 0; Index < NumSafeFramesToDrain; ++Index)
			{
				// Index of the bucket that is now old enough to be reused
				const int32 SafeFrameIndex = (LastSafeFrameNumber + Index) % ResourcePoolPolicy::NumSafeFrames;

				for (FPooledResource& PoolEntry : SafeResourceBuckets[SafeFrameIndex])
				{
					ResourceBuckets[PoolEntry.BucketIndex].Emplace(MoveTemp(PoolEntry));
				}
				SafeResourceBuckets[SafeFrameIndex].Reset();
			}

			LastSafeFrameNumber = SafeFrameNumber;
		}
		else
		{
			for (int32 FrameIndex = 0; FrameIndex < ResourcePoolPolicy::NumSafeFrames; FrameIndex++)
			{
				for (FPooledResource& PoolEntry : SafeResourceBuckets[FrameIndex])
				{
					ResourceBuckets[PoolEntry.BucketIndex].Emplace(MoveTemp(PoolEntry));
				}
				SafeResourceBuckets[FrameIndex].Reset();
			}
		}
		
		// Clean a limited number of old entries to reduce hitching when leaving a large level
		for (int32 BucketIndex = 0; BucketIndex < ResourcePoolPolicy::NumPoolBuckets; BucketIndex++)
		{
			for (int32 EntryIndex = ResourceBuckets[BucketIndex].Num() - 1; EntryIndex >= 0; EntryIndex--)
			{
				FPooledResource& PoolEntry = ResourceBuckets[BucketIndex][EntryIndex];
				
				// Clean entries that are unlikely to be reused
				if ((GFrameNumberRenderThread - PoolEntry.FrameFreed) > CullAfterFramesNum || bForceDrainAll)
				{
					Policy.FreeResource(ResourceBuckets[BucketIndex][EntryIndex].Resource);
					
					ResourceBuckets[BucketIndex].RemoveAtSwap(EntryIndex, EAllowShrinking::No);
					
					--NumToCleanThisFrame;
					if (!bForceDrainAll && NumToCleanThisFrame == 0)
					{
						break;
					}
				}
			}
			
			if (!bForceDrainAll && NumToCleanThisFrame == 0)
			{
				break;
			}
		}
	}

	/** A scope that takes a single lock so that individual allocations / deallocations can skip it. */
	class FLockScope
	{
	public:
		FLockScope(TResourcePool& InPool)
			: Pool(InPool)
			, Lock(Pool.CS)
		{
			check(!Pool.bLocked);
			Pool.bLocked = true;
		}

		~FLockScope()
		{
			Pool.bLocked = false;
		}

	private:
		TResourcePool& Pool;
		UE::TScopeLock<UE::FMutex> Lock;
	};
	
private:
	/** Pooling policy for this resource */
	ResourcePoolPolicy Policy;
	
	// Describes a Resource in the free pool.
	struct FPooledResource
	{
		/** The actual resource */
		ResourceType Resource;
		/** The arguments used to create the resource */
		ResourceCreationArguments CreationArguments;
		/** The frame the resource was freed */
		uint32 FrameFreed;
		uint32 BucketIndex;
	};

	uint32 LastSafeFrameNumber = INDEX_NONE;
	UE::FMutex CS;
	bool bLocked = false;

	// Pool of free Resources, indexed by bucket for constant size search time.
	TArray<FPooledResource> ResourceBuckets[ResourcePoolPolicy::NumPoolBuckets];
	
	// Resources that have been freed more recently than NumSafeFrames ago.
	TArray<FPooledResource> SafeResourceBuckets[ResourcePoolPolicy::NumSafeFrames];
};

/** A resource pool that automatically handles render-thread resources */
template<typename ResourceType, class ResourcePoolPolicy, class ResourceCreationArguments>
class TRenderResourcePool : public TResourcePool<ResourceType, ResourcePoolPolicy, ResourceCreationArguments>, public FTickableObjectRenderThread, public FRenderResource
{
public:
	/** Constructor */
	TRenderResourcePool() :
		FTickableObjectRenderThread(false)
	{
	}
	
	/** Constructor with policy argument
	 * @param InPolicy An initialised policy object
	 */
	TRenderResourcePool(ResourcePoolPolicy InPolicy) :
		TResourcePool<ResourceType, ResourcePoolPolicy, ResourceCreationArguments>(InPolicy),
		FTickableObjectRenderThread(false)
	{
	}
	
	/** Destructor */
	virtual ~TRenderResourcePool() = default;
	
	/** Creates a pooled resource.
	 * @param Args the argument object for construction.
	 * @returns An initialised resource or the policy's NullResource if not initialised.
	 */
	ResourceType CreatePooledResource(FRHICommandListBase& RHICmdList, ResourceCreationArguments Args)
	{
		if (IsInitialized())
		{
			return TResourcePool<ResourceType, ResourcePoolPolicy, ResourceCreationArguments>::CreatePooledResource(RHICmdList, Args);
		}
		else
		{
			return ResourceType();
		}
	}

	UE_DEPRECATED(5.4, "CreatePooledResource requires an RHI command list.")
	ResourceType CreatePooledResource(ResourceCreationArguments Args)
	{
		if (IsInitialized())
		{
			return TResourcePool<ResourceType, ResourcePoolPolicy, ResourceCreationArguments>::CreatePooledResource(FRHICommandListImmediate::Get(), Args);
		}
		else
		{
			return ResourceType();
		}
	}
	
	/** Release a resource back into the pool.
	 * @param Resource The resource to return to the pool
	 */
	void ReleasePooledResource(ResourceType&& Resource)
	{
		if (IsInitialized())
		{
			TResourcePool<ResourceType, ResourcePoolPolicy, ResourceCreationArguments>::ReleasePooledResource(Forward<ResourceType&&>(Resource));
		}
	}
	
public: // From FTickableObjectRenderThread
	virtual void Tick(FRHICommandListImmediate& RHICmdList,  float DeltaTime ) override
	{
		ensure(IsInRenderingThread());

		TResourcePool<ResourceType, ResourcePoolPolicy, ResourceCreationArguments>::DrainPool(false);
	}
	
	virtual bool IsTickable() const override
	{
		return true;
	}
	
	virtual bool NeedsRenderingResumedForRenderingThreadTick() const override
	{
		return true;
	}
	
public: // From FRenderResource
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FTickableObjectRenderThread::Register();
	}

	virtual void ReleaseRHI() override
	{
		FTickableObjectRenderThread::Unregister();
		TResourcePool<ResourceType, ResourcePoolPolicy, ResourceCreationArguments>::DrainPool(true);
	}
};
