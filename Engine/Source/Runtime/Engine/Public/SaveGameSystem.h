// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Tasks/Pipe.h"
#include "Modules/ModuleInterface.h"

/**
 * Interface for platform feature modules
 */

/** Defines the interface to platform's save game system (or a generic file based one) */
class ISaveGameSystem
{
public:

	// Possible result codes when using DoesSaveGameExist.
	// Not all codes are guaranteed to be returned on all platforms.
	enum class ESaveExistsResult
	{
		OK,						// Operation on the file completely successfully.
		DoesNotExist,			// Operation on the file failed, because the file was not found / does not exist.
		Corrupt,				// Operation on the file failed, because the file was corrupt.
		UnspecifiedError		// Operation on the file failed due to an unspecified error.
	};

	/** Returns true if the platform has a native UI (like many consoles) */
	virtual bool PlatformHasNativeUI() = 0;

	/** Some save systems don't support more than one user, in those cases the game needs to avoid saving settings for anyone but the primary user. */
	virtual bool DoesSaveSystemSupportMultipleUsers() = 0;

	/** Return true if the named savegame exists (probably not useful with NativeUI */
	virtual bool DoesSaveGameExist(const TCHAR* Name, const int32 UserIndex) = 0;

	/** Similar to DoesSaveGameExist, except returns a result code with more information. */
	virtual ESaveExistsResult DoesSaveGameExistWithResult(const TCHAR* Name, const int32 UserIndex) = 0;

	/** Gets a list of all known saves.  This isn't possible on all platforms. */
	virtual bool GetSaveGameNames(TArray<FString>& FoundSaves, const int32 UserIndex)
	{
		return false;
	}

	/** Saves the game, blocking until complete. Platform may use FGameDelegates to get more information from the game */
	virtual bool SaveGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, const TArray<uint8>& Data) = 0;

	/** Loads the game, blocking until complete */
	virtual bool LoadGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, TArray<uint8>& Data) = 0;

	/** Delete an existing save game, blocking until complete */
	virtual bool DeleteGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex) = 0;


	// indicates that an async savegame operation has completed. SlotName, PlatformUserId, bool=success. Should always be called from the gamethread.
	typedef TFunction<void(const FString&, FPlatformUserId, bool)> FSaveGameAsyncOpCompleteCallback;

	// indicates that an async savegame load has completed. SlotName, PlatformUserId, bool=success, SavedData. Should always be called from the gamethread.
	typedef TFunction<void(const FString&, FPlatformUserId, bool, const TArray<uint8>&)> FSaveGameAsyncLoadCompleteCallback;

	// indicates that an async savegame exists check has completed. SlotName, PlatformUserId, save exists result. Should always be called from the gamethread.
	typedef TFunction<void(const FString&, FPlatformUserId, ISaveGameSystem::ESaveExistsResult)> FSaveGameAsyncExistsCallback;

	// indicates that an async savegame initialize has completed. PlatformUserId, bool=success. Should always be called from the gamethread.
	typedef TFunction<void(FPlatformUserId, bool)> FSaveGameAsyncInitCompleteCallback;

	// indicates that an async savegame list names has completed. PlatformUserId, bool=success, FoundSaves. Should always be called from the game thread
	typedef TFunction<void(FPlatformUserId, bool, const TArray<FString>&)> FSaveGameAsyncGetNamesCallback;


	/* Asyncronously checks if the named savegame exists. Default implementation runs blocking version on a background thread */
	ENGINE_API virtual void DoesSaveGameExistAsync(const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncExistsCallback Callback);

	/* Saves the named savegame asyncronously. Default implementation runs blocking version on a background thread */
	ENGINE_API virtual void SaveGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, TSharedRef<const TArray<uint8>> Data, FSaveGameAsyncOpCompleteCallback Callback);

	/* Load the named savegame asyncronously. Default implementation runs blocking version on a background thread */
	ENGINE_API virtual void LoadGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncLoadCompleteCallback Callback);

	/* Combines DoesSaveGameExistAsync and LoadGameAsync into one operation for simpler usage, treats all errors as the save game not existing */
	ENGINE_API virtual void LoadGameIfExistsAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncLoadCompleteCallback Callback);

	/* Delete the named savegame asyncronously. Default implementation runs blocking version on a background thread */
	ENGINE_API virtual void DeleteGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncOpCompleteCallback Callback);

	/* Gets a list of all known saves if the platform supports it. Default implementatino runs blocking version on a background thread */
	ENGINE_API virtual void GetSaveGameNamesAsync(FPlatformUserId PlatformUserId, FSaveGameAsyncGetNamesCallback Callback);

	/* (Optional) initialise the save system for the given user. Useful if the platform may display UI on first use - this can be called as part of the user login flow, for example. 
	   If this is not used, any UI may be displayed on the first use of the other save api functions */
	ENGINE_API virtual void InitAsync(bool bAttemptToUseUI, FPlatformUserId PlatformUserId, FSaveGameAsyncInitCompleteCallback Callback);

protected:

	// save task pipe prevents multiple async save operations happening in parallel. note that the order is not guaranteed
	static ENGINE_API UE::Tasks::FPipe AsyncTaskPipe;

	// helper function for calling back to the game thread when an async save operation has completed
	ENGINE_API void OnAsyncComplete(TFunction<void()> Callback);
};


/** A generic save game system that just uses IFileManager to save/load with normal files */
class FGenericSaveGameSystem : public ISaveGameSystem
{
public:
	virtual bool PlatformHasNativeUI() override
	{
		return false;
	}

	virtual bool DoesSaveSystemSupportMultipleUsers() override
	{
		return false;
	}

	virtual ESaveExistsResult DoesSaveGameExistWithResult(const TCHAR* Name, const int32 UserIndex) override
	{
		if (IFileManager::Get().FileSize(*GetSaveGamePath(Name)) >= 0)
		{
			return ESaveExistsResult::OK;
		}
		return ESaveExistsResult::DoesNotExist;
	}

	virtual bool DoesSaveGameExist(const TCHAR* Name, const int32 UserIndex) override
	{
		return ESaveExistsResult::OK == DoesSaveGameExistWithResult(Name, UserIndex);
	}

	virtual bool GetSaveGameNames(TArray<FString>& FoundSaves, const int32 UserIndex) override
	{
		TArray<FString> FoundFiles;
		const FString SaveGameDirectory = FPaths::ProjectSavedDir() / TEXT("SaveGames/");
		IFileManager::Get().FindFiles(FoundFiles, *SaveGameDirectory, TEXT("*.sav"));

		for (FString& File : FoundFiles)
		{
			FoundSaves.Add(FPaths::GetBaseFilename(File));
		}

		return true;
	}

	virtual bool SaveGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, const TArray<uint8>& Data) override
	{
		return FFileHelper::SaveArrayToFile(Data, *GetSaveGamePath(Name));
	}

	virtual bool LoadGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, TArray<uint8>& Data) override
	{
		return FFileHelper::LoadFileToArray(Data, *GetSaveGamePath(Name));
	}

	virtual bool DeleteGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex) override
	{
		return IFileManager::Get().Delete(*GetSaveGamePath(Name), true, false, !bAttemptToUseUI);
	}

protected:

	/** Get the path to save game file for the given name, a platform _may_ be able to simply override this and no other functions above */
	virtual FString GetSaveGamePath(const TCHAR* Name)
	{
		return FString::Printf(TEXT("%sSaveGames/%s.sav"), *FPaths::ProjectSavedDir(), Name);
	}
};



/** helper base class for async-compatible save games */
class FBaseAsyncSaveGameSystem : public ISaveGameSystem
{
public:

	virtual bool DoesSaveGameExist(const TCHAR* Name, const int32 UserIndex) override
	{
		return ESaveExistsResult::OK == DoesSaveGameExistWithResult(Name, UserIndex);
	}

	// syncronous save functions
	ENGINE_API virtual ESaveExistsResult DoesSaveGameExistWithResult(const TCHAR* Name, const int32 UserIndex) override;
	ENGINE_API virtual bool GetSaveGameNames(TArray<FString>& FoundSaves, const int32 UserIndex) override;
	ENGINE_API virtual bool SaveGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, const TArray<uint8>& Data) override;
	ENGINE_API virtual bool LoadGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, TArray<uint8>& Data) override;
	ENGINE_API virtual bool DeleteGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex) override;

	// asyncronous save functions
	ENGINE_API virtual void DoesSaveGameExistAsync(const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncExistsCallback Callback) override;
	ENGINE_API virtual void SaveGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, TSharedRef<const TArray<uint8>> Data, FSaveGameAsyncOpCompleteCallback Callback) override;
	ENGINE_API virtual void LoadGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncLoadCompleteCallback Callback) override;
	ENGINE_API virtual void LoadGameIfExistsAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncLoadCompleteCallback Callback) override;
	ENGINE_API virtual void DeleteGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncOpCompleteCallback Callback) override;
	ENGINE_API virtual void GetSaveGameNamesAsync(FPlatformUserId PlatformUserId, FSaveGameAsyncGetNamesCallback Callback) override;

protected:
	// internal async helpers that require implementation. NB. OutResult will be null for async calls
	virtual UE::Tasks::FTask InternalDoesSaveGameExistAsync(const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncExistsCallback Callback, TSharedPtr<ESaveExistsResult> OutResult = nullptr) = 0;
	virtual UE::Tasks::FTask InternalSaveGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, TSharedRef<const TArray<uint8>> Data, FSaveGameAsyncOpCompleteCallback Callback, TSharedPtr<bool> OutResult = nullptr) = 0;
	virtual UE::Tasks::FTask InternalLoadGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, TSharedRef<TArray<uint8>> Data, FSaveGameAsyncLoadCompleteCallback Callback, TSharedPtr<bool> OutResult = nullptr) = 0;
	virtual UE::Tasks::FTask InternalDeleteGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncOpCompleteCallback Callback, TSharedPtr<bool> OutResult = nullptr) = 0;
	virtual UE::Tasks::FTask InternalGetSaveGameNamesAsync(FPlatformUserId PlatformUserId, TSharedRef<TArray<FString>> FoundSaves, FSaveGameAsyncGetNamesCallback Callback, TSharedPtr<bool> OutResult = nullptr) = 0;

	// specializations can override this
	ENGINE_API virtual void WaitForAsyncTask(UE::Tasks::FTask AsyncSaveTask);

};

/**
 * Save game system Module Interface
 */
class ISaveGameSystemModule : public IModuleInterface
{
public:

	virtual ISaveGameSystem* GetSaveGameSystem() = 0;
};


