// Copyright Epic Games, Inc. All Rights Reserved.

/*===================================================================================
	Scalability.h: Manager class for handling scalability settings
=====================================================================================*/

#pragma once

#include "CoreMinimal.h"

enum EShaderPlatform : uint16;

namespace Scalability
{ 
	enum class EQualityLevelBehavior
	{
		EAbsolute,
		ERelativeToMax,
	};

	inline const int32 DefaultQualityLevel = 3;

	/**
	 * Structure for holding the state of the engine scalability groups
	 * Actual engine state you can get though GetQualityLevels().
	**/
	struct FQualityLevels
	{
		float ResolutionQuality;
		int32 ViewDistanceQuality;
		int32 AntiAliasingQuality;
		int32 ShadowQuality;
		int32 GlobalIlluminationQuality;
		int32 ReflectionQuality;
		int32 PostProcessQuality;
		int32 TextureQuality;
		int32 EffectsQuality;
		int32 FoliageQuality;
		int32 ShadingQuality;
		int32 LandscapeQuality;

		float CPUBenchmarkResults;
		float GPUBenchmarkResults;
		TArray<float> CPUBenchmarkSteps;
		TArray<float> GPUBenchmarkSteps;

		// Allows us to avoid SetDefaults for static init variables, as SetDefaults is not defined to call during static int
		FQualityLevels(bool bSetDefaults = true)
			: CPUBenchmarkResults(-1.0f)
			, GPUBenchmarkResults(-1.0f)
		{
			if (bSetDefaults)
			{
				SetDefaults();
			}
		}
		
		bool operator==(const FQualityLevels& Other ) const
		{
			return ResolutionQuality == Other.ResolutionQuality &&
				ViewDistanceQuality == Other.ViewDistanceQuality &&
				AntiAliasingQuality == Other.AntiAliasingQuality &&
				ShadowQuality == Other.ShadowQuality &&
				GlobalIlluminationQuality == Other.GlobalIlluminationQuality &&
				ReflectionQuality == Other.ReflectionQuality &&
				PostProcessQuality == Other.PostProcessQuality &&
				TextureQuality == Other.TextureQuality &&
				EffectsQuality == Other.EffectsQuality &&
				FoliageQuality == Other.FoliageQuality &&
				ShadingQuality == Other.ShadingQuality &&
				LandscapeQuality == Other.LandscapeQuality;
		}

		bool operator!=(const FQualityLevels& Other ) const
		{
			return !(*this == Other);
		}

		/** used for DisplayInternals to quickly identify why a screenshot looks different */
		uint32 GetHash() const
		{
			return FCrc::TypeCrc32<float>(ResolutionQuality) ^
				FCrc::TypeCrc32<int32>(ViewDistanceQuality) ^
				FCrc::TypeCrc32<int32>(AntiAliasingQuality) ^
				FCrc::TypeCrc32<int32>(ShadowQuality) ^
				FCrc::TypeCrc32<int32>(GlobalIlluminationQuality) ^
				FCrc::TypeCrc32<int32>(ReflectionQuality) ^
				FCrc::TypeCrc32<int32>(PostProcessQuality) ^
				FCrc::TypeCrc32<int32>(TextureQuality) ^
				FCrc::TypeCrc32<int32>(EffectsQuality) ^
				FCrc::TypeCrc32<int32>(FoliageQuality) ^
				FCrc::TypeCrc32<int32>(ShadingQuality) ^
				FCrc::TypeCrc32<int32>(LandscapeQuality);
		}

		// Sets all other settings based on an overall value
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetFromSingleQualityLevel(int32 Value);

		// Sets all other settings based on an overall value, but relative to the maximum.
		// @param Value 0: maximum level, 1: maximumlevel -1, etc
		ENGINE_API void SetFromSingleQualityLevelRelativeToMax(int32 Value);

		// Returns the overall value if all settings are set to the same thing
		// @param Value -1:custom, 0:low, 1:medium, 2:high, 3:epic, 4:cinematic
		ENGINE_API int32 GetSingleQualityLevel() const;

		// Returns the minimum set quality level from all settings
		// @param Value -1:custom, 0:low, 1:medium, 2:high, 3:epic, 4:cinematic
		ENGINE_API int32 GetMinQualityLevel() const;

		// Sets view distance quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetViewDistanceQuality(int32 Value);

		// Sets anti-aliasing quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetAntiAliasingQuality(int32 Value);

		// Sets shadow quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetShadowQuality(int32 Value);

		// Sets shadow quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetGlobalIlluminationQuality(int32 Value);

		// Sets shadow quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetReflectionQuality(int32 Value);

		// Sets the post-processing quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetPostProcessQuality(int32 Value);

		// Sets the texture quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetTextureQuality(int32 Value);

		// Sets the visual effects quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetEffectsQuality(int32 Value);

		// Sets the foliage quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetFoliageQuality(int32 Value);

		// Sets the shading quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetShadingQuality(int32 Value);

		// Sets the landscape quality
		// @param Value 0:low, 1:medium, 2:high, 3:epic, 4:cinematic (gets clamped if needed)
		ENGINE_API void SetLandscapeQuality(int32 Value);

		ENGINE_API void SetBenchmarkFallback();

		ENGINE_API void SetDefaults();
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnScalabilitySettingsChanged, const Scalability::FQualityLevels&);
	ENGINE_API extern FOnScalabilitySettingsChanged OnScalabilitySettingsChanged;

	/** Structure holding the details of a preset. */
	struct FResolutionPreset
	{
		FString Name;
		int32 Id = 0;
		float ResolutionQuality = 100.0f;
	};

	ENGINE_API TArray<FResolutionPreset> GetResolutionPresets();

	/** This is the only suggested way to set the current state - don't set CVars directly **/
	ENGINE_API void SetQualityLevels(const FQualityLevels& QualityLevels, bool bForce = false);

#if WITH_EDITOR
	ENGINE_API void ApplyCachedQualityLevels();
	/** Gets the default quality level from the Device Profile or system defaults */
	ENGINE_API FQualityLevels GetDefaultQualityLevels();
	/** Resets the Quality Levels to the Device Profile or system defaults */
	ENGINE_API void ResetQualityLevelsToDefault();
#endif

	/** This is the only suggested way to get the current state - don't get CVars directly */
	ENGINE_API FQualityLevels GetQualityLevels();

	/** Applies quality levels for temporary status which will NOT be saved to user settings e.g. mobile device low-power mode.
		Originally active settings are backed-up or restored on toggle. */
	ENGINE_API void ToggleTemporaryQualityLevels(bool bEnable, const FQualityLevels& QualityLevelsOverride = FQualityLevels());

	/** Are active scalability settings a temporary override. */
	ENGINE_API bool IsTemporaryQualityLevelActive();

	/** Gets the effects quality directly for the passed thread.
	*
	* @param bGameThread	If true, the game thread value for the CVar is returned, otherwise the render thread value is returned. Useful when accessing the CVar from a game task.
	*/
	ENGINE_API int32 GetEffectsQualityDirect(bool bGameThread);

	/**  */
	ENGINE_API void InitScalabilitySystem();

	/** @param IniName e.g. GEditorPerProjectIni or GGameUserSettingsIni */
	ENGINE_API void LoadState(const FString& IniName);

	/**
	 * @param IniName e.g. GEditorPerProjectIni or GGameUserSettingsIni
	 * @param Section the section to load from
	 */
	ENGINE_API void LoadState(const FString& IniName, const TCHAR* Section);
	
	/** @param IniName e.g. GEditorPerProjectIni or GGameUserSettingsIni */
	ENGINE_API void SaveState(const FString& IniName);
	
	/** 
	 * @param IniName e.g. GEditorPerProjectIni or GGameUserSettingsIni 
	 * @param Section the section to save into
	 */
	ENGINE_API void SaveState(const FString& IniName, const TCHAR* Section);

	/**
	 * Sends an analytic event with all quality level data
	 *
	 * @param bAutoApplied	Whether or not the quality levels were auto-applied (true) or applied by the user (false).
	 */
	ENGINE_API void RecordQualityLevelsAnalytics(bool bAutoApplied);

	/** Compute scalability quality levels from existing CPU and GPU perf indices **/
	ENGINE_API FQualityLevels ComputeQualityLevelsFromPerfIndex(float CPUPerfIndex, float GPUPerfIndex);
	
	/** Run synthbenchmark and configure scalability based on results **/
	ENGINE_API FQualityLevels BenchmarkQualityLevels(uint32 WorkScale=10, float CPUMultiplier = 1.0f, float GPUMultiplier = 1.0f);

	/** Process a console command line **/
	ENGINE_API void ProcessCommand(const TCHAR* Cmd, FOutputDevice& Ar);

	/** Returns the number of steps for each quality level */
	ENGINE_API FQualityLevels GetQualityLevelCounts();

	/** Minimum single axis scale for render resolution */
	inline constexpr float MinResolutionScale = 0.0f;

	/** Maximum single axis scale for render resolution */
	inline constexpr float MaxResolutionScale = 100.0f;

	UE_DEPRECATED(5.3, "Uses FLegacyScreenPercentageDriver::GetCVarResolutionFraction() instead")
	ENGINE_API float GetResolutionScreenPercentage();

	/** Returns a human readable name for a scalability quality level */
	ENGINE_API FText GetScalabilityNameFromQualityLevel(int32 QualityLevel);

#if WITH_EDITOR
	/** Set an Editor preview scalability platform */
	void ENGINE_API ChangeScalabilityPreviewPlatform(FName NewPlatformScalabilityName, const EShaderPlatform& ShaderPlatform);
#endif

	ENGINE_API FText GetQualityLevelText(int32 Value, int32 NumLevels);

	ENGINE_API FString GetScalabilitySectionString(const TCHAR* InGroupName, int32 InQualityLevel, int32 InNumLevels);
}
