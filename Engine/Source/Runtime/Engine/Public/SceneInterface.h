// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Math/MathFwd.h"
#include "Misc/EnumClassFlags.h"
#include "RenderGraphFwd.h"
#include "Engine/EngineTypes.h"
#include "PrimitiveComponentId.h"
#include "SceneViewOwner.h"
#include "ShowFlags.h"
#include "PathTracingOutputInvalidateReason.h"

class AWorldSettings;
class IAnimBankTransformProvider;
class IAnimSequenceTransformProvider;
class IAnimRuntimeTransformProvider;
class FArchive;
class FFloat16Color;
class FInstanceCullingManager;
class FLightSceneProxy;
class FMaterial;
class FMaterialShaderMap;
class FName;
class FOutputDevice;
class FPrimitiveSceneInfo;
class FRDGBuilder;
class FRDGExternalAccessQueue;
class FRectLightSceneProxy;
class FRenderResource;
class FRenderTarget;
class FRHICommandListImmediate;
class FRHIUniformBuffer;
class FScene;
class FSceneRenderer;
class FSceneViewStateInterface;
class FSkyAtmosphereRenderSceneInfo;
class FSkyAtmosphereSceneProxy;
class FSkyLightSceneProxy;
class FSparseVolumeTextureViewerSceneProxy;
class FTexture;
class FVertexFactory;
class FViewInfo;
class FVolumetricCloudRenderSceneInfo;
class FVolumetricCloudSceneProxy;
class UDecalComponent;
struct FDeferredDecalUpdateParams;
class UInstancedStaticMeshComponent;
class ULightComponent;
class UPlanarReflectionComponent;
class UPrimitiveComponent;
class UReflectionCaptureComponent;
class URuntimeVirtualTextureComponent;
class USkyLightComponent;
class UStaticMeshComponent;
class UTextureCube;
enum class EPrimitiveDirtyState : uint8;
enum class EVTInvalidatePriority : uint8;
enum class EShadingPath;
enum EShaderPlatform : uint16;
namespace ERHIFeatureLevel { enum Type : int; }
struct FHairStrandsInstance;
struct FLightRenderParameters;
struct FPersistentPrimitiveIndex;
template<int32 MaxSHOrder> class TSHVectorRGB;
using FSHVectorRGB3 = TSHVectorRGB<3>;
struct FCustomPrimitiveData;
class FSceneViewFamily;
class FGPUSceneWriteDelegate;

struct FExponentialHeightFogDynamicState;
struct FPrimitiveSceneDesc;
struct FInstancedStaticMeshSceneDesc;
struct FLightSceneDesc;
struct FPrimitiveUniformShaderParametersBuilder;

enum EBasePassDrawListType
{
	EBasePass_Default=0,
	EBasePass_Masked,
	EBasePass_MAX
};

enum class EUpdateAllPrimitiveSceneInfosAsyncOps
{
	None = 0,
	
	// Cached mesh draw commands are cached asynchronously.
	CacheMeshDrawCommands = 1 << 0,
	
	// Light primitive interactions are created asynchronously.
	CreateLightPrimitiveInteractions = 1 << 1,

	// Material uniform expressions are cached asynchronously.
	CacheMaterialUniformExpressions = 1 << 2,

	All = CacheMeshDrawCommands | CreateLightPrimitiveInteractions | CacheMaterialUniformExpressions
};
ENUM_CLASS_FLAGS(EUpdateAllPrimitiveSceneInfosAsyncOps);

enum class EPrimitiveDirtyRayTracingState
{
	None = 0,
	MeshCommandsAndInstances = 1 << 0,
	DrawInGame = 1 << 1,
	LightingChannelMask = 1 << 2,

	Default = MeshCommandsAndInstances,
	All = MeshCommandsAndInstances | DrawInGame | LightingChannelMask
};
ENUM_CLASS_FLAGS(EPrimitiveDirtyRayTracingState);

/**
 * An interface to the private scene manager implementation of a scene.  Use GetRendererModule().AllocateScene to create.
 * The scene
 */
class FSceneInterface
{
public:
	ENGINE_API FSceneInterface(ERHIFeatureLevel::Type InFeatureLevel);

	// FSceneInterface interface

	/** 
	 * Adds a new primitive component to the scene
	 * 
	 * @param Primitive - primitive component to add
	 */
	virtual void AddPrimitive(UPrimitiveComponent* Primitive) = 0;
	/** 
	 * Removes a primitive component from the scene
	 * 
	 * @param Primitive - primitive component to remove
	 */
	virtual void RemovePrimitive(UPrimitiveComponent* Primitive) = 0;
	/** Called when a primitive is being unregistered and will not be immediately re-registered. */
	virtual void ReleasePrimitive(UPrimitiveComponent* Primitive) = 0;

	/** Batched versions of Add / Remove / ReleasePrimitive, for improved CPU performance */
	virtual void BatchAddPrimitives(TArrayView<UPrimitiveComponent*> InPrimitives) = 0;
	virtual void BatchRemovePrimitives(TArrayView<UPrimitiveComponent*> InPrimitives) = 0;
	virtual void BatchReleasePrimitives(TArrayView<UPrimitiveComponent*> InPrimitives) = 0;

	/**
	* Updates all primitive scene info additions, remobals and translation changes
	*/
	virtual void UpdateAllPrimitiveSceneInfos(FRDGBuilder& GraphBuilder, EUpdateAllPrimitiveSceneInfosAsyncOps AsyncOps = EUpdateAllPrimitiveSceneInfosAsyncOps::None) = 0;
	ENGINE_API void UpdateAllPrimitiveSceneInfos(FRHICommandListImmediate& RHICmdList);

	/** 
	 * Updates the transform of a primitive which has already been added to the scene. 
	 * 
	 * @param Primitive - primitive component to update
	 */
	virtual void UpdatePrimitiveTransform(UPrimitiveComponent* Primitive) = 0;
	virtual void UpdatePrimitiveOcclusionBoundsSlack(UPrimitiveComponent* Primitive, float NewSlack) = 0;
	virtual void UpdatePrimitiveDrawDistance(UPrimitiveComponent* Primitive, float MinDrawDistance, float MaxDrawDistance, float VirtualTextureMaxDrawDistance) = 0;
	virtual void UpdateInstanceCullDistance(UPrimitiveComponent* Primitive, float StartCullDistance, float EndCullDistance) = 0;
	/** Updates primitive attachment state. */
	virtual void UpdatePrimitiveAttachment(UPrimitiveComponent* Primitive) = 0;

	/**
	 * Call to notify that an instace proxy update has been queued for the primitive.
	 *
	 * @param Primitive - primitive component to update
	 */
	virtual void UpdatePrimitiveInstances(UPrimitiveComponent* Primitive) = 0;	

	/** 
	 * Updates the custom primitive data of a primitive component which has already been added to the scene. 
	 * 
	 * @param Primitive - Primitive component to update
	 */
	virtual void UpdateCustomPrimitiveData(UPrimitiveComponent* Primitive) = 0;
	/**
	 * Updates distance field scene data (transforms, uv scale, self-shadow bias, etc.) but doesn't change allocation in the atlas
	 */
	virtual void UpdatePrimitiveDistanceFieldSceneData_GameThread(UPrimitiveComponent* Primitive) {}

	/** Finds the  primitive with the associated index into the primitive array. */
	virtual FPrimitiveSceneInfo* GetPrimitiveSceneInfo(int32 PrimitiveIndex) const = 0;

	/** Finds the  primitive with the associated component id. */
	virtual FPrimitiveSceneInfo* GetPrimitiveSceneInfo(FPrimitiveComponentId PrimitiveId) const = 0;
	virtual FPrimitiveSceneInfo* GetPrimitiveSceneInfo(const FPersistentPrimitiveIndex& PersistentPrimitiveIndex) const = 0;

	/** Get the primitive previous local to world (used for motion blur). Returns true if the matrix was set. */
	virtual bool GetPreviousLocalToWorld(const FPrimitiveSceneInfo* PrimitiveSceneInfo, FMatrix& OutPreviousLocalToWorld) const { return false; }
	/** 
	 * Adds a new light component to the scene
	 * 
	 * @param Light - light component to add
	 */
	virtual void AddLight(ULightComponent* Light) = 0;

	/** 
	 * Removes a light component from the scene
	 * 
	 * @param Light - light component to remove
	 */
	virtual void RemoveLight(ULightComponent* Light) = 0;

	/** 
	 * Adds a new light component to the scene which is currently invisible, but needed for editor previewing
	 * 
	 * @param Light - light component to add
	 */
	virtual void AddInvisibleLight(ULightComponent* Light) = 0;

	/**
	 * Adds new lights to the scene
	 *
	 * @param InLights - lights to add
	 */
	virtual void BatchAddLights(TArrayView<FLightSceneDesc> InLights) = 0;

	/**
	 * Removes lights from the scene
	 *
	 * @param InLights - lights to remove
	 */
	virtual void BatchRemoveLights(TArrayView<FLightSceneDesc> InLights) = 0;

	/**
	 * Adds new invisible lights to the scene
	 *
	 * @param InLights - lights to add
	 */
	virtual void BatchAddInvisibleLights(TArrayView<FLightSceneDesc> InLights) = 0;

	virtual void SetSkyLight(FSkyLightSceneProxy* Light) = 0;
	virtual void DisableSkyLight(FSkyLightSceneProxy* Light) = 0;

	virtual bool HasSkyLightRequiringLightingBuild() const = 0;
	virtual bool HasAtmosphereLightRequiringLightingBuild() const = 0;

	/** 
	 * Adds a new decal component to the scene
	 * 
	 * @param Component - component to add
	 */
	virtual void AddDecal(UDecalComponent* Component) = 0;
	/** 
	 * Removes a decal component from the scene
	 * 
	 * @param Component - component to remove
	 */
	virtual void RemoveDecal(UDecalComponent* Component) = 0;
	/** 
	 * Updates the transform of a decal which has already been added to the scene. 
	 *
	 * @param Decal - Decal component to update
	 */
	virtual void UpdateDecalTransform(UDecalComponent* Component) = 0;
	virtual void UpdateDecalFadeOutTime(UDecalComponent* Component) = 0;
	virtual void UpdateDecalFadeInTime(UDecalComponent* Component) = 0;
	virtual void BatchUpdateDecals(TArray<FDeferredDecalUpdateParams>&& UpdateParams) = 0;

	/** Adds a reflection capture to the scene. */
	virtual void AddReflectionCapture(class UReflectionCaptureComponent* Component) {}

	/** Removes a reflection capture from the scene. */
	virtual void RemoveReflectionCapture(class UReflectionCaptureComponent* Component) {}

	/** Reads back reflection capture data from the GPU.  Very slow operation that blocks the GPU and rendering thread many times. */
	virtual void GetReflectionCaptureData(UReflectionCaptureComponent* Component, class FReflectionCaptureData& OutCaptureData) {}

	/** Updates a reflection capture's transform, and then re-captures the scene. */
	virtual void UpdateReflectionCaptureTransform(class UReflectionCaptureComponent* Component) {}

	/** 
	 * Allocates reflection captures in the scene's reflection cubemap array and updates them by recapturing the scene.
	 * Existing captures will only be updated.  Must be called from the game thread.
	 */
	virtual void AllocateReflectionCaptures(const TArray<UReflectionCaptureComponent*>& NewCaptures, const TCHAR* CaptureReason, bool bVerifyOnlyCapturing, bool bCapturingForMobile, bool bInsideTick) {}
	virtual void ReleaseReflectionCubemap(UReflectionCaptureComponent* CaptureComponent) {}

	/**
	  * Resets reflection capture data to allow it to be rebuilt from scratch.  Allows shrinking of the number of items in
	  * the capture array, which otherwise only grows.  This can allow the user to solve cases where an out of memory (OOM)
	  * condition prevented captures from being generated.  The user can remove some items or reduce the capture resolution
	  * to fix the issue, and run "Build Reflection Captures" (which calls this function), instead of needing to restart
	  * the editor.  The flag "bOnlyIfOOM" only does the reset if an OOM had occurred.
	  */
	virtual void ResetReflectionCaptures(bool bOnlyIfOOM) {}

	/** 
	 * Updates the contents of the given sky capture by rendering the scene. 
	 * This must be called on the game thread.
	 */
	virtual void UpdateSkyCaptureContents(const USkyLightComponent* CaptureComponent, bool bCaptureEmissiveOnly, UTextureCube* SourceCubemap, FTexture* OutProcessedTexture, float& OutAverageBrightness, FSHVectorRGB3& OutIrradianceEnvironmentMap, TArray<FFloat16Color>* OutRadianceMap, FLinearColor* SpecifiedCubemapColorScale) {}

	virtual void AllocateAndCaptureFrameSkyEnvMap(FRDGBuilder& GraphBuilder, FSceneRenderer& SceneRenderer, FViewInfo& MainView, bool bShouldRenderSkyAtmosphere, bool bShouldRenderVolumetricCloud, FInstanceCullingManager& InstanceCullingManager, FRDGExternalAccessQueue& ExternalAccessQueue) {}
	virtual void ValidateSkyLightRealTimeCapture(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SceneColorTexture) {}

	virtual void AddPlanarReflection(class UPlanarReflectionComponent* Component) {}
	virtual void RemovePlanarReflection(class UPlanarReflectionComponent* Component) {}
	virtual void UpdatePlanarReflectionTransform(UPlanarReflectionComponent* Component) {}

	/** 
	* Updates the contents of the given scene capture by rendering the scene. 
	* This must be called on the game thread.
	*/
	virtual void UpdateSceneCaptureContents(class USceneCaptureComponent2D* CaptureComponent, class ISceneRenderBuilder& SceneRenderBuilder) {}
	virtual void UpdateSceneCaptureContents(class USceneCaptureComponentCube* CaptureComponent, class ISceneRenderBuilder& SceneRenderBuilder) {}
	virtual void UpdatePlanarReflectionContents(class UPlanarReflectionComponent* CaptureComponent, class FSceneRenderer& MainSceneRenderer, class ISceneRenderBuilder& SceneRenderBuilder) {}

	virtual void AddPrecomputedLightVolume(const class FPrecomputedLightVolume* Volume) {}
	virtual void RemovePrecomputedLightVolume(const class FPrecomputedLightVolume* Volume) {}

	virtual bool HasPrecomputedVolumetricLightmap_RenderThread() const { return false; }
	virtual void AddPrecomputedVolumetricLightmap(const class FPrecomputedVolumetricLightmap* Volume, bool bIsPersistentLevel) {}
	virtual void RemovePrecomputedVolumetricLightmap(const class FPrecomputedVolumetricLightmap* Volume) {}

	/** Add a runtime virtual texture object to the scene. */
	virtual void AddRuntimeVirtualTexture(URuntimeVirtualTextureComponent* Component) {}
	/** Removes a runtime virtual texture object from the scene. */
	virtual void RemoveRuntimeVirtualTexture(URuntimeVirtualTextureComponent* Component) {}
	/* Get the bitmasks describing which virtual texture objects will hide the associated primitives. */
	virtual void GetRuntimeVirtualTextureHidePrimitiveMask(uint8& bHideMaskEditor, uint8& bHideMaskGame) const {}
	
	/** 
	 * Invalidates pages in a runtime virtual texture object. 
	 * @param Component RVT component containing the runtime virtual texture to invalidate
	 * @param WorldBounds World-space bounds of the area of the runtime virtual texture to invalidate
	 * @param InvalidatePriority Ensures the corresponding pages get processed in priority. This allows increased responsiveness when 
	 *  there are more pages being updated than can be handled in a given frame (when throttling) 
	 */
	virtual void InvalidateRuntimeVirtualTexture(URuntimeVirtualTextureComponent* Component, FBoxSphereBounds const& WorldBounds, EVTInvalidatePriority InvalidatePriority) {}

	/** Request preload of an area of a runtime virtual texture at a given mip level. */
	virtual void RequestPreloadRuntimeVirtualTexture(class URuntimeVirtualTextureComponent* Component, FBoxSphereBounds const& WorldBounds, int32 Level) {}

	/** Mark scene as needing to restart path tracer accumulation. */
	virtual void InvalidatePathTracedOutput(PathTracing::EInvalidateReason InvalidateReason = PathTracing::EInvalidateReason::Uncategorized) {}

	/**  Invalidates Lumen surface cache and forces it to be refreshed. Useful to make material updates more responsive. */
	virtual void InvalidateLumenSurfaceCache_GameThread(UPrimitiveComponent* Component) {}

	/** 
	 * Retrieves primitive uniform shader parameters that are internal to the renderer. 
	 * NOTE: prefer BuildPrimitiveUniformShaderParameters_RenderThread since it is more efficient and general.
	 */
	virtual void GetPrimitiveUniformShaderParameters_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool& bHasPrecomputedVolumetricLightmap, FMatrix& PreviousLocalToWorld, int32& SingleCaptureIndex, bool& OutputVelocity) const = 0;

	/**
	 * Sets up primitive uniform shader parameters that are internal to the renderer.
	 */
	virtual void BuildPrimitiveUniformShaderParameters_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, FPrimitiveUniformShaderParametersBuilder& Builder) const = 0;
	 
	/** 
	 * Updates the transform of a light which has already been added to the scene. 
	 *
	 * @param Light - light component to update
	 */
	virtual void UpdateLightTransform(ULightComponent* Light) = 0;
	/** 
	 * Updates the color and brightness of a light which has already been added to the scene. 
	 *
	 * @param Light - light component to update
	 */
	virtual void UpdateLightColorAndBrightness(ULightComponent* Light) = 0;

	virtual void UpdateLightProxy(ULightComponent* Light, TFunction<void(FLightSceneProxy*)>&& Func) = 0;

	/** Sets the precomputed visibility handler for the scene, or NULL to clear the current one. */
	virtual void SetPrecomputedVisibility(const class FPrecomputedVisibilityHandler* PrecomputedVisibilityHandler) {}

	/** Sets the precomputed volume distance field for the scene, or NULL to clear the current one. */
	virtual void SetPrecomputedVolumeDistanceField(const class FPrecomputedVolumeDistanceField* PrecomputedVolumeDistanceField) {}

	UE_DEPRECATED(5.8, "No longer implemented. For similar use-cases refer to IRendererModule::UpdateStaticDrawListsForMaterials.")
	virtual void UpdateStaticDrawLists() {}

	/** Update render states that possibly cached inside renderer, like mesh draw commands. More lightweight than re-registering the scene proxy. */
	virtual void UpdateCachedRenderStates(class FPrimitiveSceneProxy* SceneProxy) {}
	
	/** Updates the selected state values that might be cached inside the renderer */
	virtual void UpdatePrimitiveSelectedState_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bIsSelected) {}

	/** Updates the velocity state values that might be cached inside the renderer */
	virtual void UpdatePrimitiveVelocityState_RenderThread(FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bIsBeingMoved) {}

	/** 
	 * Adds a new exponential height fog component to the scene
	 * 
	 * @param FogComponent - fog component to add
	 */	
	virtual void AddExponentialHeightFog(uint64 Id, const FExponentialHeightFogDynamicState& State) = 0;
	/** 
	 * Removes a exponential height fog component from the scene
	 * 
	 * @param FogComponent - fog component to remove
	 */	
	virtual void RemoveExponentialHeightFog(uint64 Id) = 0;
	/**
	 * @return True if there are any exponential height fog potentially enabled in the scene
	 */
	virtual bool HasAnyExponentialHeightFog() const = 0;

	/**
	 * Adds a new local height fog component to the scene
	 *
	 * @param FogProxy - fog proxy to add
	 */
	virtual void AddLocalFogVolume(class FLocalFogVolumeSceneProxy* FogProxy) = 0;
	/**
	 * Removes a local height fog component from the scene
	 *
	 * @param FogProxy - fog proxy to remove
	 */
	virtual void RemoveLocalFogVolume(class FLocalFogVolumeSceneProxy* FogProxy) = 0;
	/**
	 * @return True if there are any local height fog potentially enabled in the scene
	 */
	virtual bool HasAnyLocalFogVolume() const = 0;

	/**
	 * Adds the unique volumetric cloud component to the scene
	 *
	 * @param SkyAtmosphereSceneProxy - the sky atmosphere proxy
	 */
	virtual void AddSkyAtmosphere(FSkyAtmosphereSceneProxy* SkyAtmosphereSceneProxy, bool bStaticLightingBuilt) = 0;
	/**
	 * Removes the unique volumetric cloud component to the scene
	 *
	 * @param SkyAtmosphereSceneProxy - the sky atmosphere proxy
	 */
	virtual void RemoveSkyAtmosphere(FSkyAtmosphereSceneProxy* SkyAtmosphereSceneProxy) = 0;
	/**
	 * Returns the scene's unique info if it exists
	 */
	virtual FSkyAtmosphereRenderSceneInfo* GetSkyAtmosphereSceneInfo() = 0;
	virtual const FSkyAtmosphereRenderSceneInfo* GetSkyAtmosphereSceneInfo() const = 0;

	virtual void AddSparseVolumeTextureViewer(FSparseVolumeTextureViewerSceneProxy* SVTV) = 0;
	virtual void RemoveSparseVolumeTextureViewer(FSparseVolumeTextureViewerSceneProxy* SVTV) = 0;

	virtual IAnimBankTransformProvider* GetAnimBankTransformProvider() = 0;
	virtual IAnimRuntimeTransformProvider* GetAnimRuntimeTransformProvider() = 0;
	virtual IAnimSequenceTransformProvider* GetAnimSequenceTransformProvider() = 0;

	/**
	 * Adds the unique volumetric cloud component to the scene
	 *
	 * @param VolumetricCloudSceneProxy - the sky atmosphere proxy
	 */
	virtual void AddVolumetricCloud(FVolumetricCloudSceneProxy* VolumetricCloudSceneProxy) = 0;
	/**
	 * Removes the unique volumetric cloud component to the scene
	 *
	 * @param VolumetricCloudSceneProxy - the sky atmosphere proxy
	 */
	virtual void RemoveVolumetricCloud(FVolumetricCloudSceneProxy* VolumetricCloudSceneProxy) = 0;

	/**
	 * Adds a hair strands proxy to the scene
	 *
	 * @param Proxy - the hair strands proxy
	 */
	virtual void AddHairStrands(FHairStrandsInstance* Proxy) = 0;

	/**
	 * Removes a hair strands proxy to the scene
	 *
	 * @param Proxy - the hair strands proxy
	 */
	virtual void RemoveHairStrands(FHairStrandsInstance* Proxy) = 0;

	/**
	 * Return the IES profile index corresponding to the local light proxy
	 *
	 * @param Proxy - the local light proxy
	 * @param Out - the light parameters which will be filled with the IES profile  index information
	 */
	virtual void GetLightIESAtlasSlot(const FLightSceneProxy* Proxy, FLightRenderParameters* Out) = 0;

	/**
	 * Return the rect. light atlas slot information corresponding to the rect light proxy
	 *
	 * @param Proxy - the rect light proxy
	 * @param Out - the light parameters which will be filled with the rect light atlas information
	 */
	virtual void GetRectLightAtlasSlot(const FRectLightSceneProxy* Proxy, FLightRenderParameters* Out) = 0;

	/**
	 * Set the physics field scene proxy to the scene
	 *
	 * @param PhysicsFieldSceneProxy - the physics field scene proxy
	 */
	virtual void SetPhysicsField(class FPhysicsFieldSceneProxy* PhysicsFieldSceneProxy) = 0;

	/**
	 * Reset the physics field scene proxy
	 */
	virtual void ResetPhysicsField() = 0;

	/**
	 * Set the shader print/debug cvars to be able to show the fields
	 */
	virtual void ShowPhysicsField() = 0;

	/**
	 * Reset the physics field scene proxy
	 */
	virtual void UpdatePhysicsField(FRDGBuilder& GraphBuilder, FViewInfo& View) = 0;

	/**
	 * Returns the scene's unique info if it exists
	 */
	virtual FVolumetricCloudRenderSceneInfo* GetVolumetricCloudSceneInfo() = 0;
	virtual const FVolumetricCloudRenderSceneInfo* GetVolumetricCloudSceneInfo() const = 0;

	/**
	 * Adds a wind source component to the scene.
	 * @param WindComponent - The component to add.
	 */
	virtual void AddWindSource(class UWindDirectionalSourceComponent* WindComponent) = 0;
	/**
	 * Removes a wind source component from the scene.
	 * @param WindComponent - The component to remove.
	 */
	virtual void RemoveWindSource(class UWindDirectionalSourceComponent* WindComponent) = 0;
	/**
	 * Update a wind source component from the scene.
	 * @param WindComponent - The component to update.
	 */
	virtual void UpdateWindSource(class UWindDirectionalSourceComponent* WindComponent) = 0;
	/**
	 * Accesses the wind source list.  Must be called in the rendering thread.
	 * @return The list of wind sources in the scene.
	 */
	virtual const TArray<class FWindSourceSceneProxy*>& GetWindSources_RenderThread() const = 0;

	/** Accesses wind parameters.  XYZ will contain wind direction * Strength, W contains wind speed. */
	virtual void GetWindParameters(const FVector& Position, FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const = 0;

	/** Accesses wind parameters safely for game thread applications */
	virtual void GetWindParameters_GameThread(const FVector& Position, FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const = 0;

	/** Same as GetWindParameters, but ignores point wind sources. */
	virtual void GetDirectionalWindParameters(FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const = 0;

	/** 
	 * Adds a SpeedTree wind computation object to the scene.
	 * @param StaticMesh - The SpeedTree static mesh whose wind to add.
	 */
	virtual void AddSpeedTreeWind(class FVertexFactory* VertexFactory, const class UStaticMesh* StaticMesh) = 0;

	/** 
	 * Removes a SpeedTree wind computation object to the scene.
	 * @param StaticMesh - The SpeedTree static mesh whose wind to remove.
	 */
	virtual void RemoveSpeedTreeWind_RenderThread(class FVertexFactory* VertexFactory, const class UStaticMesh* StaticMesh) = 0;

	/** Ticks the SpeedTree wind object and updates the uniform buffer. */
	virtual void UpdateSpeedTreeWind(double CurrentTime) = 0;

	/** 
	 * Looks up the SpeedTree uniform buffer for the passed in vertex factory.
	 * @param VertexFactory - The vertex factory registered for SpeedTree.
	 */
	virtual FRHIUniformBuffer* GetSpeedTreeUniformBuffer(const FVertexFactory* VertexFactory) const = 0;

	// FPrimtiveDesc version for primitive/light scene interactions
	virtual void AddPrimitive(FPrimitiveSceneDesc* Primitive) = 0;
	virtual void RemovePrimitive(FPrimitiveSceneDesc* Primitive) = 0;
	virtual void ReleasePrimitive(FPrimitiveSceneDesc* Primitive) = 0;
	virtual void UpdatePrimitiveTransform(FPrimitiveSceneDesc* Primitive) = 0;

	virtual void BatchAddPrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives) = 0;
	virtual void BatchRemovePrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives) = 0;
	virtual void BatchReleasePrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives) = 0;

	virtual void BatchRemovePrimitives(TArray<FPrimitiveSceneProxy*>&& InPrimitives) = 0;
	
	virtual void UpdateCustomPrimitiveData(FPrimitiveSceneDesc* Primitive, const FCustomPrimitiveData& CustomPrimitiveData) = 0;

	virtual void UpdatePrimitiveInstances(FPrimitiveSceneDesc* Primitive) = 0;	

	UE_EXPERIMENTAL(5.6, "Experimental API, scene writing delegate may be internalized in the future.")
	virtual void UpdatePrimitiveInstancesFromCompute(FPrimitiveSceneDesc* Primitive, FGPUSceneWriteDelegate&& DataWriterGPU) = 0;

	UE_DEPRECATED(5.8, "Use UpdatePrimitivesIsDrawn_RenderThread instead.")
	virtual void UpdatePrimitivesDrawnInGame_RenderThread(TArrayView<FPrimitiveSceneProxy*> PrimitiveSceneProxies, bool bDrawnInGame) final {}

	virtual void UpdatePrimitivesIsDrawn_RenderThread(TArrayView<FPrimitiveSceneProxy*> PrimitiveSceneProxies, bool bIsDrawn) = 0;

	/**
	 * Release this scene and remove it from the rendering thread
	 */
	virtual void Release() = 0;
	/**
	 * Retrieves the lights interacting with the passed in primitive and adds them to the out array.
	 *
	 * @param	Primitive				Primitive to retrieve interacting lights for
	 * @param	RelevantLights	[out]	Array of lights interacting with primitive
	 */
	virtual void GetRelevantLights( UPrimitiveComponent* Primitive, TArray<const ULightComponent*>* RelevantLights ) const = 0;

	/**
	 * Retrieves the number of lights interacting with the passed in primitive.
	 *
	 * @param	Primitive				Primitive to retrieve interacting lights for
	 */
	virtual int32 GetRelevantLightCount(UPrimitiveComponent* Primitive) const = 0;
	/**
	 * Indicates if hit proxies should be processed by this scene
	 *
	 * @return true if hit proxies should be rendered in this scene.
	 */
	UE_DEPRECATED(5.8, "No longer exposed, only used internally.")
	virtual bool RequiresHitProxies() const = 0;
	/**
	 * Get the optional UWorld that is associated with this scene
	 * 
	 * @return UWorld instance used by this scene
	 */
	virtual class UWorld* GetWorld() const = 0;
	
	/**
	 * Return the scene to be used for rendering. Note that this can return null if rendering has
	 * been disabled!
	 */
	virtual FScene* GetRenderScene() = 0;
	virtual const FScene* GetRenderScene() const = 0;

	virtual void OnWorldCleanup()
	{
	}
	virtual void UpdateSceneSettings(AWorldSettings* WorldSettings) {}

	class IPrimitiveTransformUpdater
	{
	public:
		virtual ~IPrimitiveTransformUpdater() = default;

		ENGINE_API static IPrimitiveTransformUpdater* SetInstanceTLS(IPrimitiveTransformUpdater* Instance);
		ENGINE_API static IPrimitiveTransformUpdater* GetInstanceTLS();
	};

	class FPrimitiveTransformUpdateScope
	{
	public:
		explicit FPrimitiveTransformUpdateScope(IPrimitiveTransformUpdater* Updater)
		{
			PreviousUpdater = IPrimitiveTransformUpdater::SetInstanceTLS(Updater);
		}

		~FPrimitiveTransformUpdateScope()
		{
			IPrimitiveTransformUpdater::SetInstanceTLS(PreviousUpdater);
		}

	private:
		IPrimitiveTransformUpdater* PreviousUpdater;
	};

	virtual IPrimitiveTransformUpdater* CreatePrimitiveTransformUpdater(int32 MaxPrimitives) { return nullptr; }
	virtual void UpdatePrimitiveTransforms(IPrimitiveTransformUpdater* Updater) {}

	/**
	* Gets the GPU Skin Cache system associated with the scene.
	*/
	virtual class FGPUSkinCache* GetGPUSkinCache()
	{
		return nullptr;
	}

	virtual class FSkeletalMeshUpdater* GetSkeletalMeshUpdater()
	{
		return nullptr;
	}

	/**
	 * Gets the compute work scheduler objects associated with the scene.
	 */
	virtual void GetComputeTaskWorkers(TArray<class IComputeTaskWorker*>& OutWorkers) const {}

	/**
	 * Sets the FX system associated with the scene.
	 */
	virtual void SetFXSystem( class FFXSystemInterface* InFXSystem ) = 0;

	/**
	 * Get the FX system associated with the scene.
	 */
	virtual class FFXSystemInterface* GetFXSystem() = 0;

	virtual void DumpUnbuiltLightInteractions( FOutputDevice& Ar ) const { }

	/** Updates the scene's list of parameter collection id's and their uniform buffers. */
	virtual void UpdateParameterCollections(const TArray<class FMaterialParameterCollectionInstanceResource*>& InParameterCollections) {}

	/**
	 * Exports the scene.
	 *
	 * @param	Ar		The Archive used for exporting.
	 **/
	virtual void Export( FArchive& Ar ) const
	{}

	/**
	 * Shifts scene data by provided delta
	 * Called on world origin changes
	 * 
	 * @param	InOffset	Delta to shift scene by
	 */
	virtual void ApplyWorldOffset(const FVector& InOffset) {}

	/**
	 * Notification that level was added to a world
	 * 
	 * @param	InLevelName		Level name
	 */
	virtual void OnLevelAddedToWorld(const FName& InLevelName, UWorld* InWorld, bool bIsLightingScenario) {}
	virtual void OnLevelRemovedFromWorld(const FName& InLevelName, UWorld* InWorld, bool bIsLightingScenario) {}

	/**
	 * @return True if there are any lights in the scene
	 */
	virtual bool HasAnyLights() const = 0;

	virtual bool IsEditorScene() const { return false; }

	virtual void UpdateEarlyZPassMode() {}

	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }

	EShaderPlatform GetShaderPlatform() const { return ShaderPlatform; }

	static ENGINE_API EShadingPath GetShadingPath(ERHIFeatureLevel::Type InFeatureLevel);

	EShadingPath GetShadingPath() const
	{
		return GetShadingPath(GetFeatureLevel());
	}

#if WITH_EDITOR
	/**
	 * Initialize the pixel inspector buffers.
	 * @return True if implemented false otherwise.
	 */
	virtual bool InitializePixelInspector(FRenderTarget* BufferFinalColor, FRenderTarget* BufferSceneColor, FRenderTarget* BufferDepth, FRenderTarget* BufferHDR, FRenderTarget* BufferA, FRenderTarget* BufferBCDEF, int32 BufferIndex)
	{
		return false;
	}

	/**
	 * Add a pixel inspector request.
	 * @return True if implemented false otherwise.
	 */
	virtual bool AddPixelInspectorRequest(class FPixelInspectorRequest *PixelInspectorRequest)
	{
		return false;
	}
#endif //WITH_EDITOR

	virtual TConstArrayView<FPrimitiveSceneProxy*> GetPrimitiveSceneProxies() const = 0;

	/**
	 * Returns the FPrimitiveComponentId for all primitives in the scene
	 */
	virtual TConstArrayView<FPrimitiveComponentId> GetScenePrimitiveComponentIds() const = 0;

	virtual void StartFrame() {}
	virtual void EndFrame(FRHICommandListImmediate& RHICmdList) {}
	virtual uint32 GetFrameNumber() const { return 0; }
	virtual void IncrementFrameNumber() {}

	virtual void UpdateCachedRayTracingState(class FPrimitiveSceneProxy* SceneProxy, EPrimitiveDirtyRayTracingState PrimitiveDirtyState) {}

	void UpdateCachedRayTracingState(class FPrimitiveSceneProxy* SceneProxy)
	{
		UpdateCachedRayTracingState(SceneProxy, EPrimitiveDirtyRayTracingState::Default);
	}

	virtual bool RequestGPUSceneUpdate(FPrimitiveSceneInfo& PrimitiveSceneInfo, EPrimitiveDirtyState PrimitiveDirtyState) { return false; }
	virtual bool RequestUniformBufferUpdate(FPrimitiveSceneInfo& PrimitiveSceneInfo) { return false; }

	virtual void RefreshNaniteRasterBins(FPrimitiveSceneInfo& PrimitiveSceneInfo) {}
	virtual void ReloadNaniteFixedFunctionBins() {}

	/** Contains settings used to construct scene view for custom render pass during the renderer construction. */
	struct FCustomRenderPassRendererInput
	{
		FCustomRenderPassRendererInput()
			: EngineShowFlags(ESFIM_Game)
		{
			EngineShowFlags.DisableFeaturesForUnlit();
		}

		/** Data used to construct scene view for the custom render pass. */
		FVector ViewLocation;
		FMatrix ViewRotationMatrix;
		FMatrix ProjectionMatrix;
		TSet<FPrimitiveComponentId> HiddenPrimitives;
		TOptional<TSet<FPrimitiveComponentId>> ShowOnlyPrimitives;
		FEngineShowFlags EngineShowFlags;
		FSceneViewOwner ViewActor;
		class FSceneViewStateInterface* ViewStateInterface = nullptr;
		int32 PostVolumeUserFlags = 0;						// UserFlags from Post Process Volume, used if bOverridesPostVolumeUserFlags set, allows per-view material overrides
		bool bIsSceneCapture = false;
		bool bUseMainViewFamilyShowFlags = false;			// Custom render pass should use flags from main view family, rather than ones in this structure
		bool bOverridesPostVolumeUserFlags = false;			// Use PostVolumeUserFlags, instead of flags from main view family

		class FCustomRenderPassBase* CustomRenderPass = nullptr;
	};

	/** Enqueues a new custom render pass to execute the next time this scene is rendered by ANY scene renderer. It will be immediately removed from the scene afterwards. */
	virtual bool AddCustomRenderPass(const FSceneViewFamily* ViewFamily, const FCustomRenderPassRendererInput& CustomRenderPassInput) { return false; }

protected:
	virtual ~FSceneInterface() = default;

	/** This scene's feature level */
	ERHIFeatureLevel::Type FeatureLevel;

	/** The shader platform this scene was created with */
	EShaderPlatform ShaderPlatform;

	friend class FSceneViewStateReference;
};
