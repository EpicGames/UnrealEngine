// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneManagement.h: Scene manager definitions.
=============================================================================*/

#pragma once

// Includes the draw mesh macros

#include "CoreMinimal.h"
#include "Containers/ChunkedArray.h"
#include "Stats/Stats.h"
#include "Misc/Guid.h"
#include "Misc/MemStack.h"
#include "Misc/IQueuedWork.h"
#include "RHI.h"
#include "RenderResource.h"
#include "EngineDefines.h"
#include "HitProxies.h"
#include "SceneTypes.h"
#include "ConvexVolume.h"
#include "PrimitiveDrawingUtils.h"
#include "PrimitiveUniformShaderParameters.h"
#include "RendererInterface.h"
#include "BatchedElements.h"
#include "MeshBatch.h"
#include "MeshElementCollector.h"
#include "SceneUtils.h"
#include "LightmapUniformShaderParameters.h"
#include "DynamicBufferAllocator.h"
#include "Math/SHMath.h"
#include "GlobalRenderResources.h"

#if RHI_RAYTRACING
#include "RayTracingInstance.h"
#include "RayTracingGeometryManagerTypes.h"
#endif // RHI_RAYTRACING

class FCanvas;
class FGlobalDynamicIndexBuffer;
class FGlobalDynamicReadBuffer;
class FGlobalDynamicVertexBuffer;
class FLightMap;
class FLightmapResourceCluster;
class FLightSceneInfo;
class FLightSceneProxy;
class FPrimitiveSceneProxy;
class FScene;
class FSceneViewState;
class FShadowMap;
class FStaticLightingBuildContext;
class FStaticMeshRenderData;
class FTexture;
class UDecalComponent;
class ULightMapTexture2D;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UShadowMapTexture2D;
class UTexture2D;
class UTexture;
struct FDynamicMeshVertex;
class ULightMapVirtualTexture2D;
class FGPUScenePrimitiveCollector;
class FRayTracingGeometry;
struct FViewMatrices;
struct FEngineShowFlags;
class FViewport;
class FLandscapeRayTracingStateList;
struct FPrimitiveUniformShaderParametersBuilder;
struct FSceneViewStateSystemMemoryMirror;

namespace UE { namespace Color { class FColorSpace; } }

DECLARE_LOG_CATEGORY_EXTERN(LogBufferVisualization, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogNaniteVisualization, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogLumenVisualization, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogVirtualShadowMapVisualization, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogMultiView, Log, All);

// -----------------------------------------------------------------------------


/**
 * struct to hold the temporal LOD state within a view state
 */
struct FTemporalLODState
{
	/** The last two camera origin samples collected for stateless temporal LOD transitions */
	FVector	TemporalLODViewOrigin[2];
	/** The last two time samples collected for stateless temporal LOD transitions */
	float	TemporalLODTime[2];
	/** If non-zero, then we are doing temporal LOD smoothing, this is the time interval. */
	float	TemporalLODLag;

	FTemporalLODState()
		: TemporalLODLag(0.0f) // nothing else is used if this is zero
	{

	}

	/* Returns the blend factor between the last two LOD samples */
	ENGINE_API float GetTemporalLODTransition(float LastRenderTime) const;
	
	ENGINE_API void UpdateTemporalLODTransition(const FSceneView& View, float LastRenderTime);
};

enum ESequencerState
{
	ESS_None,
	ESS_Paused,
	ESS_Playing,
};

/**
 * The scene manager's persistent view state.
 */
class FSceneViewStateInterface
{
public:
	FSceneViewStateInterface()
		:	bValidEyeAdaptationBuffer(0)
	{}
	
	/** Called in the game thread to destroy the view state. */
	virtual void Destroy() = 0;

public:
	/** @return	the derived view state object */
	virtual FSceneViewState* GetConcreteViewState () = 0;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) = 0;

	virtual SIZE_T GetSizeBytes() const { return 0; }

	/** Resets pool for GetReusableMID() */
	virtual void OnStartPostProcessing(FSceneView& CurrentView) = 0;

	/**
	 * Allows MIDs being created and released during view rendering without the overhead of creating and releasing objects
	 * As MID are not allowed to be parent of MID this gets fixed up by parenting it to the next Material or MIC
	 * @param InSource can be Material, MIC or MID, must not be 0
	 */
	virtual UMaterialInstanceDynamic* GetReusableMID(class UMaterialInterface* InSource) = 0;

	/**
	 * Clears the pool of mids being referenced by this view state 
	 */
	virtual void ClearMIDPool(FStringView MidParentRootPath = {}) = 0;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** If frozen view matrices are available, return a pointer to them */
	virtual const FViewMatrices* GetFrozenViewMatrices() const = 0;

	/** If frozen view matrices are available, set those as active on the SceneView */
	virtual void ActivateFrozenViewMatrices(FSceneView& SceneView) = 0;

	/** If frozen view matrices were set, restore the previous view matrices */
	virtual void RestoreUnfrozenViewMatrices(FSceneView& SceneView) = 0;
#endif
	// rest some state (e.g. FrameIndexMod8, TemporalAASampleIndex) to make the rendering [more] deterministic
	virtual void ResetViewState() = 0;

	/** Returns the temporal LOD struct from the viewstate */
	virtual FTemporalLODState& GetTemporalLODState() = 0;
	virtual const FTemporalLODState& GetTemporalLODState() const = 0;

	/** 
	 * Returns the blend factor between the last two LOD samples
	 */
	virtual float GetTemporalLODTransition() const = 0;

	/** 
	 * returns a unique key for the view state, non-zero
	 */
	virtual uint32 GetViewKey() const = 0;

	/* Return the active volumetric cloud texture, can be null. */
	virtual FRDGTextureRef GetVolumetricCloudTexture(FRDGBuilder& GraphBuilder) = 0;
	virtual FVector2f GetVolumetricCloudTextureUVScale() const = 0;
	virtual FVector2f GetVolumetricCloudTextureUVMax() const = 0;

	//
	virtual uint32 GetCurrentTemporalAASampleIndex() const = 0;

	/**
	 * returns the distance field temporal sample index
	 */
	virtual uint32 GetDistanceFieldTemporalSampleIndex() const = 0;

	/** Tells if the eye adaptation buffer exists without attempting to allocate it. */
	bool HasValidEyeAdaptationBuffer() const { return bValidEyeAdaptationBuffer; }

	/** Returns the eye adaptation buffer. */
	virtual FRDGPooledBuffer* GetCurrentEyeAdaptationBuffer() const = 0;

	/** Returns the eye adaptation exposure. */
	virtual float GetLastEyeAdaptationExposure() const = 0;

	/** Returns the average scene luminance. */
	virtual float GetLastAverageSceneLuminance() const = 0;

	virtual void SetExposureCompensationCurve(class UCurveFloat* NewCurve) {}

	virtual void SetSequencerState(ESequencerState InSequencerState) = 0;

	virtual ESequencerState GetSequencerState() = 0;

	/** Returns the current PreExposure value. PreExposure is a custom scale applied to the scene color to prevent buffer overflow. */
	virtual float GetPreExposure() const = 0;

	/** 
	 * returns the occlusion frame counter 
	 */
	virtual uint32 GetOcclusionFrameCounter() const = 0;

#if RHI_RAYTRACING
	/**
	* returns the path tracer sample index
	*/
	virtual uint32 GetPathTracingSampleIndex() const = 0;

	/**
	* returns the path tracer sample count
	*/
	virtual uint32 GetPathTracingSampleCount() const = 0;

	virtual void SetLandscapeRayTracingStates(TPimplPtr<FLandscapeRayTracingStateList>&& InLandscapeRayTracingStates) = 0;
	virtual FLandscapeRayTracingStateList* GetLandscapeRayTracingStates() const = 0;
#endif

	/** Similar to above, but adds Lumen Scene Data */
	virtual void AddLumenSceneData(FSceneInterface* InScene, float SurfaceCacheResolution = 1.0f) {}
	virtual void RemoveLumenSceneData(FSceneInterface* InScene) {}
	virtual bool HasLumenSceneData() const = 0;
	
	UE_DEPRECATED(5.3, "SetViewParent is deprecated")
	void SetViewParent(FSceneViewStateInterface*) {}

	UE_DEPRECATED(5.3, "GetViewParent is deprecated")
	FSceneViewStateInterface* GetViewParent() { return nullptr; }

	UE_DEPRECATED(5.3, "GetViewParent is deprecated")
	const FSceneViewStateInterface* GetViewParent() const { return nullptr; }

	UE_DEPRECATED(5.3, "HasViewParent is deprecated")
	bool HasViewParent() const { return false; }

	UE_DEPRECATED(5.3, "IsViewParent is deprecated")
	bool IsViewParent() const { return false; }
	
	/**
	* Scene view state mirroring to system memory, useful for offline high resolution tiled rendering (such as via Movie Render Queue).
	* Very slow and memory intensive!  A single instance of FSceneViewStateSystemMemoryMirror should be shared for all tile scene view
	* states, as it can internally share some of the memory cost.
	*/
	ENGINE_API static TPimplPtr<FSceneViewStateSystemMemoryMirror> SystemMemoryMirrorAllocate();

	virtual void SystemMemoryMirrorBackup(FSceneViewStateSystemMemoryMirror* SystemMemoryMirror) = 0;
	virtual void SystemMemoryMirrorRestore(FSceneViewStateSystemMemoryMirror* SystemMemoryMirror) = 0;

protected:
	// Don't allow direct deletion of the view state, Destroy should be called instead.
	virtual ~FSceneViewStateInterface() = default;

	uint8 bValidEyeAdaptationBuffer : 1;

private:
	friend class FScene;
};

class UE_DEPRECATED(5.7, "FFrozenSceneViewMatricesGuard is no longer used.") FFrozenSceneViewMatricesGuard
{
public:
	FFrozenSceneViewMatricesGuard(FSceneView& SV) {}
	~FFrozenSceneViewMatricesGuard() {}
};

/**
 * Global working color space shader parameters (color space conversion matrices).
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FWorkingColorSpaceShaderParameters, ENGINE_API)
	SHADER_PARAMETER(FMatrix44f, ToXYZ)
	SHADER_PARAMETER(FMatrix44f, FromXYZ)
	SHADER_PARAMETER(FMatrix44f, ToAP1)
	SHADER_PARAMETER(FMatrix44f, FromAP1)
	SHADER_PARAMETER(FMatrix44f, ToAP0)
	SHADER_PARAMETER(FMatrix44f, FromAP0)
	SHADER_PARAMETER(uint32, bIsSRGB)
END_SHADER_PARAMETER_STRUCT()

class FDefaultWorkingColorSpaceUniformBuffer : public TUniformBuffer<FWorkingColorSpaceShaderParameters>
{
	typedef TUniformBuffer<FWorkingColorSpaceShaderParameters> Super;
public:

	void Update(FRHICommandListBase& RHICmdList, const UE::Color::FColorSpace& InColorSpace);
};

ENGINE_API extern TGlobalResource<FDefaultWorkingColorSpaceUniformBuffer> GDefaultWorkingColorSpaceUniformBuffer;


/**
 * The types of interactions between a light and a primitive.
 */
enum ELightInteractionType
{
	LIT_CachedIrrelevant,
	LIT_CachedLightMap,
	LIT_Dynamic,
	LIT_CachedSignedDistanceFieldShadowMap2D,

	LIT_MAX
};

/**
 * Information about an interaction between a light and a mesh.
 */
class FLightInteraction
{
public:

	// Factory functions.
	static FLightInteraction Dynamic() { return FLightInteraction(LIT_Dynamic); }
	static FLightInteraction LightMap() { return FLightInteraction(LIT_CachedLightMap); }
	static FLightInteraction Irrelevant() { return FLightInteraction(LIT_CachedIrrelevant); }
	static FLightInteraction ShadowMap2D() { return FLightInteraction(LIT_CachedSignedDistanceFieldShadowMap2D); }

	// Accessors.
	ELightInteractionType GetType() const { return Type; }

	/**
	 * Minimal initialization constructor.
	 */
	FLightInteraction(ELightInteractionType InType)
		: Type(InType)
	{}

private:
	ELightInteractionType Type;
};





/** The number of coefficients that are stored for each light sample. */ 
inline constexpr int32 NUM_STORED_LIGHTMAP_COEF = 4;

/** The number of directional coefficients which the lightmap stores for each light sample. */ 
inline constexpr int32 NUM_HQ_LIGHTMAP_COEF = 2;

/** The number of simple coefficients which the lightmap stores for each light sample. */ 
inline constexpr int32 NUM_LQ_LIGHTMAP_COEF = 2;

/** The index at which simple coefficients are stored in any array containing all NUM_STORED_LIGHTMAP_COEF coefficients. */ 
inline constexpr int32 LQ_LIGHTMAP_COEF_INDEX = 2;

/** Compile out low quality lightmaps to save memory */
// @todo-mobile: Need to fix this!
#ifndef ALLOW_LQ_LIGHTMAPS
#define ALLOW_LQ_LIGHTMAPS (PLATFORM_DESKTOP || PLATFORM_IOS || PLATFORM_ANDROID)
#endif

/** Compile out high quality lightmaps to save memory */
#define ALLOW_HQ_LIGHTMAPS 1

/** Make sure at least one is defined */
#if !ALLOW_LQ_LIGHTMAPS && !ALLOW_HQ_LIGHTMAPS
#error At least one of ALLOW_LQ_LIGHTMAPS and ALLOW_HQ_LIGHTMAPS needs to be defined!
#endif

/**
 * Information about an interaction between a light and a mesh.
 */
class FLightMapInteraction
{
public:

	// Factory functions.
	static FLightMapInteraction None()
	{
		FLightMapInteraction Result;
		Result.Type = LMIT_None;
		return Result;
	}

	static FLightMapInteraction GlobalVolume()
	{
		FLightMapInteraction Result;
		Result.Type = LMIT_GlobalVolume;
		return Result;
	}

	static FLightMapInteraction Texture(
		const class ULightMapTexture2D* const* InTextures,
		const ULightMapTexture2D* InSkyOcclusionTexture,
		const ULightMapTexture2D* InAOMaterialMaskTexture,
		const FVector4f* InCoefficientScales,
		const FVector4f* InCoefficientAdds,
		const FVector2D& InCoordinateScale,
		const FVector2D& InCoordinateBias,
		bool bAllowHighQualityLightMaps);

	static FLightMapInteraction InitVirtualTexture(
		const ULightMapVirtualTexture2D* VirtualTexture,
		const FVector4f* InCoefficientScales,
		const FVector4f* InCoefficientAdds,
		const FVector2D& InCoordinateScale,
		const FVector2D& InCoordinateBias,
		bool bAllowHighQualityLightMaps);

	/** Default constructor. */
	FLightMapInteraction():
#if ALLOW_HQ_LIGHTMAPS
		HighQualityTexture(NULL),
		SkyOcclusionTexture(NULL),
		AOMaterialMaskTexture(NULL),
#endif
#if ALLOW_LQ_LIGHTMAPS
		LowQualityTexture(NULL),
#endif
#if ALLOW_HQ_LIGHTMAPS || ALLOW_LQ_LIGHTMAPS
		VirtualTexture(NULL),
#endif
		Type(LMIT_None)
	{}

	// Accessors.
	ELightMapInteractionType GetType() const { return Type; }
	
	const ULightMapTexture2D* GetTexture(bool bHighQuality) const
	{
		check(Type == LMIT_Texture);
#if ALLOW_LQ_LIGHTMAPS && ALLOW_HQ_LIGHTMAPS
		return bHighQuality ? HighQualityTexture : LowQualityTexture;
#elif ALLOW_HQ_LIGHTMAPS
		return HighQualityTexture;
#else
		return LowQualityTexture;
#endif
	}

	const ULightMapTexture2D* GetSkyOcclusionTexture() const
	{
		check(Type == LMIT_Texture);
#if ALLOW_HQ_LIGHTMAPS
		return SkyOcclusionTexture;
#else
		return NULL;
#endif
	}

	const ULightMapTexture2D* GetAOMaterialMaskTexture() const
	{
		check(Type == LMIT_Texture);
#if ALLOW_HQ_LIGHTMAPS
		return AOMaterialMaskTexture;
#else
		return NULL;
#endif
	}

	const ULightMapVirtualTexture2D* GetVirtualTexture() const
	{
		check(Type == LMIT_Texture);
#if ALLOW_HQ_LIGHTMAPS || ALLOW_LQ_LIGHTMAPS
		return VirtualTexture;
#else
		return NULL;
#endif
	}

	const FVector4f* GetScaleArray() const
	{
#if ALLOW_LQ_LIGHTMAPS && ALLOW_HQ_LIGHTMAPS
		return AllowsHighQualityLightmaps() ? HighQualityCoefficientScales : LowQualityCoefficientScales;
#elif ALLOW_HQ_LIGHTMAPS
		return HighQualityCoefficientScales;
#else
		return LowQualityCoefficientScales;
#endif
	}

	const FVector4f* GetAddArray() const
	{
#if ALLOW_LQ_LIGHTMAPS && ALLOW_HQ_LIGHTMAPS
		return AllowsHighQualityLightmaps() ? HighQualityCoefficientAdds : LowQualityCoefficientAdds;
#elif ALLOW_HQ_LIGHTMAPS
		return HighQualityCoefficientAdds;
#else
		return LowQualityCoefficientAdds;
#endif
	}
	
	const FVector2D& GetCoordinateScale() const
	{
		check(Type == LMIT_Texture);
		return CoordinateScale;
	}
	const FVector2D& GetCoordinateBias() const
	{
		check(Type == LMIT_Texture);
		return CoordinateBias;
	}

	uint32 GetNumLightmapCoefficients() const
	{
#if ALLOW_LQ_LIGHTMAPS && ALLOW_HQ_LIGHTMAPS
#if PLATFORM_DESKTOP && (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR)		// This is to allow for dynamic switching between simple and directional light maps in the PC editor
		if( !AllowsHighQualityLightmaps() )
		{
			return NUM_LQ_LIGHTMAP_COEF;
		}
#endif
		return NumLightmapCoefficients;
#elif ALLOW_HQ_LIGHTMAPS
		return NUM_HQ_LIGHTMAP_COEF;
#else
		return NUM_LQ_LIGHTMAP_COEF;
#endif
	}

	/**
	* @return true if high quality lightmaps are allowed
	*/
	inline bool AllowsHighQualityLightmaps() const
	{
#if ALLOW_LQ_LIGHTMAPS && ALLOW_HQ_LIGHTMAPS
		return bAllowHighQualityLightMaps;
#elif ALLOW_HQ_LIGHTMAPS
		return true;
#else
		return false;
#endif
	}

	/** These functions are used for the Dummy lightmap policy used in LightMap density view mode. */
	/** 
	 *	Set the type.
	 *
	 *	@param	InType				The type to set it to.
	 */
	void SetLightMapInteractionType(ELightMapInteractionType InType)
	{
		Type = InType;
	}
	/** 
	 *	Set the coordinate scale.
	 *
	 *	@param	InCoordinateScale	The scale to set it to.
	 */
	void SetCoordinateScale(const FVector2D& InCoordinateScale)
	{
		CoordinateScale = InCoordinateScale;
	}
	/** 
	 *	Set the coordinate bias.
	 *
	 *	@param	InCoordinateBias	The bias to set it to.
	 */
	void SetCoordinateBias(const FVector2D& InCoordinateBias)
	{
		CoordinateBias = InCoordinateBias;
	}

private:

#if ALLOW_HQ_LIGHTMAPS
	FVector4f HighQualityCoefficientScales[NUM_HQ_LIGHTMAP_COEF];
	FVector4f HighQualityCoefficientAdds[NUM_HQ_LIGHTMAP_COEF];
	const class ULightMapTexture2D* HighQualityTexture;
	const ULightMapTexture2D* SkyOcclusionTexture;
	const ULightMapTexture2D* AOMaterialMaskTexture;
#endif

#if ALLOW_LQ_LIGHTMAPS
	FVector4f LowQualityCoefficientScales[NUM_LQ_LIGHTMAP_COEF];
	FVector4f LowQualityCoefficientAdds[NUM_LQ_LIGHTMAP_COEF];
	const class ULightMapTexture2D* LowQualityTexture;
#endif

#if ALLOW_LQ_LIGHTMAPS && ALLOW_HQ_LIGHTMAPS
	bool bAllowHighQualityLightMaps;
	uint32 NumLightmapCoefficients;
#endif

#if ALLOW_HQ_LIGHTMAPS || ALLOW_LQ_LIGHTMAPS
	const ULightMapVirtualTexture2D* VirtualTexture;
#endif

	ELightMapInteractionType Type;

	FVector2D CoordinateScale;
	FVector2D CoordinateBias;
};

/** Information about the static shadowing information for a primitive. */
class FShadowMapInteraction
{
public:

	// Factory functions.
	static FShadowMapInteraction None()
	{
		FShadowMapInteraction Result;
		Result.Type = SMIT_None;
		return Result;
	}

	static FShadowMapInteraction GlobalVolume()
	{
		FShadowMapInteraction Result;
		Result.Type = SMIT_GlobalVolume;
		return Result;
	}

	static FShadowMapInteraction Texture(
		class UShadowMapTexture2D* InTexture,
		const FVector2D& InCoordinateScale,
		const FVector2D& InCoordinateBias,
		const bool* InChannelValid,
		const FVector4f& InInvUniformPenumbraSize)
	{
		FShadowMapInteraction Result;
		Result.Type = SMIT_Texture;
		Result.ShadowTexture = InTexture;
		Result.CoordinateScale = InCoordinateScale;
		Result.CoordinateBias = InCoordinateBias;
		Result.InvUniformPenumbraSize = InInvUniformPenumbraSize;

		for (int Channel = 0; Channel < 4; Channel++)
		{
			Result.bChannelValid[Channel] = InChannelValid[Channel];
		}

		return Result;
	}

	static FShadowMapInteraction InitVirtualTexture(
		class ULightMapVirtualTexture2D* InTexture,
		const FVector2D& InCoordinateScale,
		const FVector2D& InCoordinateBias,
		const bool* InChannelValid,
		const FVector4f& InInvUniformPenumbraSize)
	{
		FShadowMapInteraction Result;
		Result.Type = SMIT_Texture;
		Result.VirtualTexture = InTexture;
		Result.CoordinateScale = InCoordinateScale;
		Result.CoordinateBias = InCoordinateBias;
		Result.InvUniformPenumbraSize = InInvUniformPenumbraSize;
		for (int Channel = 0; Channel < 4; Channel++)
		{
			Result.bChannelValid[Channel] = InChannelValid[Channel];
		}

		return Result;
	}

	/** Default constructor. */
	FShadowMapInteraction() :
		ShadowTexture(nullptr),
		VirtualTexture(nullptr),
		InvUniformPenumbraSize(FVector4f(0, 0, 0, 0)),
		Type(SMIT_None)
	{
		for (int Channel = 0; Channel < UE_ARRAY_COUNT(bChannelValid); Channel++)
		{
			bChannelValid[Channel] = false;
		}
	}

	// Accessors.
	EShadowMapInteractionType GetType() const { return Type; }

	UShadowMapTexture2D* GetTexture() const
	{
		checkSlow(Type == SMIT_Texture);
		return ShadowTexture;
	}

	const ULightMapVirtualTexture2D* GetVirtualTexture() const
	{
		checkSlow(Type == SMIT_Texture);
		return VirtualTexture;
	}

	const FVector2D& GetCoordinateScale() const
	{
		checkSlow(Type == SMIT_Texture);
		return CoordinateScale;
	}

	const FVector2D& GetCoordinateBias() const
	{
		checkSlow(Type == SMIT_Texture);
		return CoordinateBias;
	}

	bool GetChannelValid(int32 ChannelIndex) const
	{
		checkSlow(Type == SMIT_Texture);
		return bChannelValid[ChannelIndex];
	}

	inline FVector4f GetInvUniformPenumbraSize() const
	{
		return InvUniformPenumbraSize;
	}

private:
	UShadowMapTexture2D* ShadowTexture;
	const ULightMapVirtualTexture2D* VirtualTexture;
	FVector2D CoordinateScale;
	FVector2D CoordinateBias;
	bool bChannelValid[4];
	FVector4f InvUniformPenumbraSize;
	EShadowMapInteractionType Type;
};

class FLightMap;
class FShadowMap;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLightmapResourceClusterShaderParameters,ENGINE_API)
	SHADER_PARAMETER_TEXTURE(Texture2D, LightMapTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, SkyOcclusionTexture) 
	SHADER_PARAMETER_TEXTURE(Texture2D, AOMaterialMaskTexture) 
	SHADER_PARAMETER_TEXTURE(Texture2D, StaticShadowTexture)
	SHADER_PARAMETER_SRV(Texture2D<float4>, VTLightMapTexture) // VT
	SHADER_PARAMETER_SRV(Texture2D<float4>, VTLightMapTexture_1) // VT
	SHADER_PARAMETER_SRV(Texture2D<float4>, VTSkyOcclusionTexture) // VT
	SHADER_PARAMETER_SRV(Texture2D<float4>, VTAOMaterialMaskTexture) // VT
	SHADER_PARAMETER_SRV(Texture2D<float4>, VTStaticShadowTexture) // VT
	SHADER_PARAMETER_SAMPLER(SamplerState, LightMapSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, LightMapSampler_1)
	SHADER_PARAMETER_SAMPLER(SamplerState, SkyOcclusionSampler) 
	SHADER_PARAMETER_SAMPLER(SamplerState, AOMaterialMaskSampler) 
	SHADER_PARAMETER_SAMPLER(SamplerState, StaticShadowTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D<uint4>, LightmapVirtualTexturePageTable0) // VT
	SHADER_PARAMETER_TEXTURE(Texture2D<uint4>, LightmapVirtualTexturePageTable1) // VT
END_GLOBAL_SHADER_PARAMETER_STRUCT()

class FLightmapClusterResourceInput
{
public:

	FLightmapClusterResourceInput()
	{
		LightMapTextures[0] = nullptr;
		LightMapTextures[1] = nullptr;
		SkyOcclusionTexture = nullptr;
		AOMaterialMaskTexture = nullptr;
		LightMapVirtualTextures[0] = nullptr;
		LightMapVirtualTextures[1] = nullptr;
		ShadowMapTexture = nullptr;
	}

	const UTexture2D* LightMapTextures[2];
	const UTexture2D* SkyOcclusionTexture;
	const UTexture2D* AOMaterialMaskTexture;
	const ULightMapVirtualTexture2D* LightMapVirtualTextures[2];
	const UTexture2D* ShadowMapTexture;

	friend uint32 GetTypeHash(const FLightmapClusterResourceInput& Cluster)
	{
		// TODO - LightMapVirtualTexture needed here? What about Sky/AO textures?  Or is it enough to just check LightMapTexture[n]?
		return
			PointerHash(Cluster.LightMapTextures[0],
			PointerHash(Cluster.LightMapTextures[1],
			PointerHash(Cluster.LightMapVirtualTextures[0],
			PointerHash(Cluster.LightMapVirtualTextures[1],
			PointerHash(Cluster.ShadowMapTexture)))));
	}

	bool operator==(const FLightmapClusterResourceInput& Rhs) const
	{
		return LightMapTextures[0] == Rhs.LightMapTextures[0]
			&& LightMapTextures[1] == Rhs.LightMapTextures[1]
			&& SkyOcclusionTexture == Rhs.SkyOcclusionTexture
			&& AOMaterialMaskTexture == Rhs.AOMaterialMaskTexture
			&& LightMapVirtualTextures[0] == Rhs.LightMapVirtualTextures[0]
			&& LightMapVirtualTextures[1] == Rhs.LightMapVirtualTextures[1]
			&& ShadowMapTexture == Rhs.ShadowMapTexture;
	}
};

ENGINE_API void GetLightmapClusterResourceParameters(
	ERHIFeatureLevel::Type FeatureLevel, 
	const FLightmapClusterResourceInput& Input,
	const IAllocatedVirtualTexture* AllocatedVT,
	FLightmapResourceClusterShaderParameters& Parameters);

class FDefaultLightmapResourceClusterUniformBuffer : public TUniformBuffer< FLightmapResourceClusterShaderParameters >
{
	typedef TUniformBuffer< FLightmapResourceClusterShaderParameters > Super;
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

ENGINE_API extern TGlobalResource< FDefaultLightmapResourceClusterUniformBuffer > GDefaultLightmapResourceClusterUniformBuffer;

/**
 * An interface to cached lighting for a specific mesh.
 */
class FLightCacheInterface
{
public:
	virtual ~FLightCacheInterface() = default;

	// @param LightSceneProxy must not be 0
	virtual FLightInteraction GetInteraction(const class FLightSceneProxy* LightSceneProxy) const = 0;

	// helper function to implement GetInteraction(), call after checking for this: if(LightSceneProxy->HasStaticShadowing())
	// @param LightSceneProxy same as in GetInteraction(), must not be 0
	ENGINE_API ELightInteractionType GetStaticInteraction(const FLightSceneProxy* LightSceneProxy, const TArray<FGuid>& IrrelevantLights) const;
	
	ENGINE_API void CreatePrecomputedLightingUniformBuffer_RenderingThread(ERHIFeatureLevel::Type FeatureLevel);

	ENGINE_API bool GetVirtualTextureLightmapProducer(ERHIFeatureLevel::Type FeatureLevel, FVirtualTextureProducerHandle& OutProducerHandle);

	// @param InLightMap may be 0
	void SetLightMap(const FLightMap* InLightMap)
	{
		LightMap = InLightMap;
	}

	void SetResourceCluster(const FLightmapResourceCluster* InResourceCluster)
	{
		checkSlow(InResourceCluster);
		ResourceCluster = InResourceCluster;
	}

	// @return may be 0
	const FLightMap* GetLightMap() const
	{
		return LightMap;
	}

	// @param InShadowMap may be 0
	void SetShadowMap(const FShadowMap* InShadowMap)
	{
		ShadowMap = InShadowMap;
	}

	// @return may be 0
	const FShadowMap* GetShadowMap() const
	{
		return ShadowMap;
	}

	const FLightmapResourceCluster* GetResourceCluster() const
	{
		return ResourceCluster;
	}

	void SetGlobalVolumeLightmap(bool bInGlobalVolumeLightmap)
	{
		bGlobalVolumeLightmap = bInGlobalVolumeLightmap;
	}

	FRHIUniformBuffer* GetPrecomputedLightingBuffer() const
	{
		return PrecomputedLightingUniformBuffer;
	}

	void SetPrecomputedLightingBuffer(FRHIUniformBuffer* InPrecomputedLightingUniformBuffer)
	{
		PrecomputedLightingUniformBuffer = InPrecomputedLightingUniformBuffer;
	}

	ENGINE_API FLightMapInteraction GetLightMapInteraction(ERHIFeatureLevel::Type InFeatureLevel) const;

	ENGINE_API FShadowMapInteraction GetShadowMapInteraction(ERHIFeatureLevel::Type InFeatureLevel) const;

public:
	// Load parameters from GPUScene when possible
	// Basically this is the same as VF_SUPPORTS_PRIMITIVE_SCENE_DATA on the vertex factory, but we can't deduce automatically
	// because we don't know about VF type until we see the actual mesh batch
	bool bCanUsePrecomputedLightingParametersFromGPUScene = false;
	
private:

	bool bGlobalVolumeLightmap = false;

	// The light-map used by the element. may be 0
	const FLightMap* LightMap = nullptr;

	// The shadowmap used by the element, may be 0
	const FShadowMap* ShadowMap = nullptr;

	const FLightmapResourceCluster* ResourceCluster = nullptr;

	/** The uniform buffer holding mapping the lightmap policy resources. */
	FUniformBufferRHIRef PrecomputedLightingUniformBuffer = nullptr;
};


template<typename TPendingTextureType>
class FAsyncEncode : public IQueuedWork
{
private:
	TPendingTextureType* PendingTexture;
	FThreadSafeCounter& Counter;
	const FStaticLightingBuildContext* LightingContext;
	class ITextureCompressorModule* Compressor;

public:

	FAsyncEncode(TPendingTextureType* InPendingTexture, const FStaticLightingBuildContext* InLightingContext, FThreadSafeCounter& InCounter, ITextureCompressorModule* InCompressor) : PendingTexture(nullptr), Counter(InCounter), Compressor(InCompressor)
	{
		LightingContext = InLightingContext;
		PendingTexture = InPendingTexture;
	}

	void Abandon()
	{
		PendingTexture->StartEncoding(LightingContext, Compressor);
		Counter.Decrement();
	}

	void DoThreadedWork()
	{
		PendingTexture->StartEncoding(LightingContext, Compressor);
		Counter.Decrement();
	}
};



// Information about a single shadow cascade.
class FShadowCascadeSettings
{
public:
	// The following 3 floats represent the view space depth of the split planes for this cascade.
	// SplitNear <= FadePlane <= SplitFar

	// The distance from the camera to the near split plane, in world units (linear).
	float SplitNear;

	// The distance from the camera to the far split plane, in world units (linear).
	float SplitFar;

	// in world units (linear).
	float SplitNearFadeRegion;

	// in world units (linear).
	float SplitFarFadeRegion;

	// ??
	// The distance from the camera to the start of the fade region, in world units (linear).
	// The area between the fade plane and the far split plane is blended to smooth between cascades.
	float FadePlaneOffset;

	// The length of the fade region (SplitFar - FadePlaneOffset), in world units (linear).
	float FadePlaneLength;

	// The accurate bounds of the cascade used for primitive culling.
	FConvexVolume ShadowBoundsAccurate;

	FPlane NearFrustumPlane;
	FPlane FarFrustumPlane;

	/** When enabled, the cascade only renders objects marked with bCastFarShadows enabled (e.g. Landscape). */
	bool bFarShadowCascade;

	/** 
	 * Index of the split if this is a whole scene shadow from a directional light, 
	 * Or index of the direction if this is a whole scene shadow from a point light, otherwise INDEX_NONE. 
	 */
	int32 ShadowSplitIndex;

	/** Strength of depth bias across cascades. */
	float CascadeBiasDistribution;

	FShadowCascadeSettings()
		: SplitNear(0.0f)
		, SplitFar(WORLD_MAX)
		, SplitNearFadeRegion(0.0f)
		, SplitFarFadeRegion(0.0f)
		, FadePlaneOffset(SplitFar)
		, FadePlaneLength(SplitFar - FadePlaneOffset)
		, bFarShadowCascade(false)
		, ShadowSplitIndex(INDEX_NONE)
		, CascadeBiasDistribution(1)
	{
	}
};

/** A projected shadow transform. */
class FProjectedShadowInitializer
{
public:

	/** A translation that is applied to world-space before transforming by one of the shadow matrices. */
	FVector PreShadowTranslation;

	FMatrix WorldToLight;
	/** Non-uniform scale to be applied after WorldToLight. */
	FVector2D Scales;

	FBoxSphereBounds SubjectBounds;
	FVector4 WAxis;
	float MinLightW;
	float MaxDistanceToCastInLightW;

	/** Default constructor. */
	FProjectedShadowInitializer()
	{}

	bool IsCachedShadowValid(const FProjectedShadowInitializer& CachedShadow) const
	{
		return PreShadowTranslation == CachedShadow.PreShadowTranslation
			&& WorldToLight == CachedShadow.WorldToLight
			&& Scales == CachedShadow.Scales
			&& SubjectBounds.Origin == CachedShadow.SubjectBounds.Origin
			&& SubjectBounds.BoxExtent == CachedShadow.SubjectBounds.BoxExtent
			&& SubjectBounds.SphereRadius == CachedShadow.SubjectBounds.SphereRadius
			&& WAxis == CachedShadow.WAxis
			&& MinLightW == CachedShadow.MinLightW
			&& MaxDistanceToCastInLightW == CachedShadow.MaxDistanceToCastInLightW;
	}
};

/** Information needed to create a per-object projected shadow. */
class FPerObjectProjectedShadowInitializer : public FProjectedShadowInitializer
{
public:

};

/** Information needed to create a whole scene projected shadow. */
class FWholeSceneProjectedShadowInitializer : public FProjectedShadowInitializer
{
public:
	FShadowCascadeSettings CascadeSettings;
	bool bOnePassPointLightShadow;
	bool bRayTracedDistanceField;

	FWholeSceneProjectedShadowInitializer() :
		bOnePassPointLightShadow(false),
		bRayTracedDistanceField(false)
	{}

	bool IsCachedShadowValid(const FWholeSceneProjectedShadowInitializer& CachedShadow) const
	{
		return FProjectedShadowInitializer::IsCachedShadowValid((const FProjectedShadowInitializer&)CachedShadow)
			&& bOnePassPointLightShadow == CachedShadow.bOnePassPointLightShadow
			&& bRayTracedDistanceField == CachedShadow.bRayTracedDistanceField;
	}
};

ENGINE_API bool DoesPlatformSupportDistanceFields(const FStaticShaderPlatform Platform);

ENGINE_API bool DoesPlatformSupportDistanceFieldShadowing(EShaderPlatform Platform);

ENGINE_API bool DoesPlatformSupportDistanceFieldAO(EShaderPlatform Platform);

ENGINE_API bool DoesProjectSupportDistanceFields();

ENGINE_API bool ShouldAllPrimitivesHaveDistanceField(EShaderPlatform ShaderPlatform);

ENGINE_API bool ShouldCompileDistanceFieldShaders(EShaderPlatform ShaderPlatform);
ENGINE_API bool ShouldCompileDistanceFieldAOShaders(EShaderPlatform ShaderPlatform);

/**
 * Centralized decision function to avoid diverging logic.
 */
inline bool PrimitiveNeedsDistanceFieldSceneData(bool bTrackAllPrimitives,
	bool bCastsDynamicIndirectShadow,
	bool bAffectsDistanceFieldLighting,
	bool bIsDrawnInGame,
	bool bCastsHiddenShadow,
	bool bCastsDynamicShadow,
	bool bAffectsDynamicIndirectLighting,
	bool bAffectIndirectLightingWhileHidden)
{
	return (bTrackAllPrimitives || bCastsDynamicIndirectShadow)
		&& bAffectsDistanceFieldLighting
		&& (bIsDrawnInGame || bCastsHiddenShadow || bAffectIndirectLightingWhileHidden)
		&& (bCastsDynamicShadow || bAffectsDynamicIndirectLighting);
}

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FMobileReflectionCaptureShaderParameters,ENGINE_API)
	SHADER_PARAMETER(FVector4f, Params) // x - inv average brightness, y - sky cubemap max mip, z - unused, w - brightness of reflection capture
	SHADER_PARAMETER_TEXTURE(TextureCube, Texture)
	SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
	SHADER_PARAMETER_TEXTURE(TextureCube, TextureBlend)			// Only used when this refelction is a sky light
	SHADER_PARAMETER_SAMPLER(SamplerState, TextureBlendSampler)	// Idem
END_GLOBAL_SHADER_PARAMETER_STRUCT()

class FDefaultMobileReflectionCaptureUniformBuffer : public TUniformBuffer<FMobileReflectionCaptureShaderParameters>
{
	typedef TUniformBuffer<FMobileReflectionCaptureShaderParameters> Super;
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

ENGINE_API extern TGlobalResource<FDefaultMobileReflectionCaptureUniformBuffer> GDefaultMobileReflectionCaptureUniformBuffer;

/** Shader paraneter structure for rendering lights. */
BEGIN_SHADER_PARAMETER_STRUCT(FLightShaderParameters, ENGINE_API)
	// Position of the light in the translated world space.
	SHADER_PARAMETER(FVector3f, TranslatedWorldPosition)

	// 1 / light's falloff radius from Position.
	SHADER_PARAMETER(float, InvRadius)

	// Color of the light.
	SHADER_PARAMETER(FVector3f, Color)

	// The exponent for the falloff of the light intensity from the distance.
	SHADER_PARAMETER(float, FalloffExponent)

	// Direction of the light if applies.
	SHADER_PARAMETER(FVector3f, Direction)

	// Factor to applies on the specular.
	SHADER_PARAMETER(float, SpecularScale)

	// Factor to applies on the diffuse.
	SHADER_PARAMETER(float, DiffuseScale)

	// One tangent of the light if applies.
	// Note: BiTangent is on purpose not stored for memory optimisation purposes.
	SHADER_PARAMETER(FVector3f, Tangent)

	// Radius of the point light.
	SHADER_PARAMETER(float, SourceRadius)

	// Dimensions of the light, for spot light, but also
	SHADER_PARAMETER(FVector2f, SpotAngles)

	// Radius of the soft source.
	SHADER_PARAMETER(float, SoftSourceRadius)

	// Other dimensions of the light source for rect light specifically.
	SHADER_PARAMETER(float, SourceLength)

	// Barn door angle for rect light
	SHADER_PARAMETER(float, RectLightBarnCosAngle)

	// Barn door length for rect light
	SHADER_PARAMETER(float, RectLightBarnLength)

	// Rect. light atlas transformation
	SHADER_PARAMETER(FVector2f, RectLightAtlasUVOffset)
	SHADER_PARAMETER(FVector2f, RectLightAtlasUVScale)
	SHADER_PARAMETER(float, RectLightAtlasMaxLevel)

	// IES texture slice index
	SHADER_PARAMETER(float, IESAtlasIndex)

	// Index of the light function in the atlas
	SHADER_PARAMETER(uint32, LightFunctionAtlasLightIndex)

	// Wether the light affect translucent material or not
	SHADER_PARAMETER(uint32, bAffectsTranslucentLighting)

END_SHADER_PARAMETER_STRUCT()


// Movable local light shadow parameters for mobile deferred
BEGIN_SHADER_PARAMETER_STRUCT(FMobileMovableLocalLightShadowParameters,ENGINE_API)
	SHADER_PARAMETER(FVector4f, SpotLightShadowSharpenAndFadeFractionAndReceiverDepthBiasAndSoftTransitionScale) // x ShadowSharpen, y ShadowFadFraction, z ReceiverDepthBias, w SoftTransitionScale
	SHADER_PARAMETER(FVector4f, SpotLightShadowmapMinMax)
	SHADER_PARAMETER(FMatrix44f, SpotLightShadowWorldToShadowMatrix)
	SHADER_PARAMETER(FVector4f, LocalLightShadowBufferSize)
	SHADER_PARAMETER_TEXTURE(Texture2D, LocalLightShadowTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, LocalLightShadowSampler)
	SHADER_PARAMETER(uint32, ShadowedBits)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

/**
 * Generic parameters used to render a light
 * Has a 1:1 mapping with FLightShaderParameters, but can also be used in other contexts
 * Primary difference is position is stored as FVector3d in absolute world space, which is not appropriate for sending directly to GPU
 */
struct FLightRenderParameters
{
	ENGINE_API void MakeShaderParameters(const FViewMatrices& ViewMatrices, float Exposure, FLightShaderParameters& OutShaderParameters) const;
	ENGINE_API float GetLightExposureScale(float Exposure) const;
	static ENGINE_API float GetLightExposureScale(float Exposure, float InverseExposureBlend);

	// Position of the light in world space.
	FVector WorldPosition;

	// 1 / light's falloff radius from Position.
	float InvRadius;

	// Color of the light.
	FLinearColor Color;

	// The exponent for the falloff of the light intensity from the distance.
	float FalloffExponent;

	// Direction of the light if applies.
	FVector3f Direction;

	// Factor to applies on the specular.
	float SpecularScale;

	// Factor to applies on the diffuse.
	float DiffuseScale;

	// One tangent of the light if applies.
	// Note: BiTangent is on purpose not stored for memory optimisation purposes.
	FVector3f Tangent;

	// Radius of the point light.
	float SourceRadius;

	// Dimensions of the light, for spot light, but also
	FVector2f SpotAngles;

	// Radius of the soft source.
	float SoftSourceRadius;

	// Other dimensions of the light source for rect light specifically.
	float SourceLength;

	// Barn door angle for rect light
	float RectLightBarnCosAngle;

	// Barn door length for rect light
	float RectLightBarnLength;

	// Rect. light atlas transformation
	FVector2f RectLightAtlasUVOffset;
	FVector2f RectLightAtlasUVScale;
	float RectLightAtlasMaxLevel;

	// IES atlas slice index
	float IESAtlasIndex;

	// Index of the light in the Light function atlas data
	uint32 LightFunctionAtlasLightIndex;

	// Wether this lights affect translucent materials or not
	uint32 bAffectsTranslucentLighting;

	float InverseExposureBlend;

	// Return Invalid rect light atlas MIP level
	static float GetRectLightAtlasInvalidMIPLevel() { return 32.f;  }
};


extern ENGINE_API void ComputeShadowCullingVolume(bool bReverseCulling, const FVector* CascadeFrustumVerts, const FVector& LightDirection, FConvexVolume& ConvexVolumeOut, FPlane& NearPlaneOut, FPlane& FarPlaneOut);

/**
 * An interface implemented by dynamic resources which need to be initialized and cleaned up by the rendering thread.
 */
class FDynamicPrimitiveResource
{
public:
	UE_DEPRECATED(5.3, "InitPrimitiveResource now requires a command list.")
	ENGINE_API void InitPrimitiveResource();

	virtual void InitPrimitiveResource(FRHICommandListBase& RHICmdList) = 0;
	virtual void ReleasePrimitiveResource() = 0;
};

/**
 * An interface to a scene interaction.
 */
class FViewElementDrawer
{
public:

	/**
	 * Draws the interaction using the given draw interface.
	 */
	virtual void Draw(const FSceneView* View,FPrimitiveDrawInterface* PDI) {}
};

/**
 * An interface used to query a primitive for its static elements.
 */
class FStaticPrimitiveDrawInterface
{
public:
	virtual ~FStaticPrimitiveDrawInterface() = default;

	virtual void SetHitProxy(HHitProxy* HitProxy) = 0;

	/**
	  * Reserve memory for specified number of meshes in order to minimize number of allocations inside DrawMesh.
	  */
	virtual void ReserveMemoryForMeshes(int32 MeshNum) = 0;

	virtual void DrawMesh(
		const FMeshBatch& Mesh,
		float ScreenSize
		) = 0;
};


/** Primitive draw interface implementation used to store primitives requested to be drawn when gathering dynamic mesh elements. */
class FSimpleElementCollector : public FPrimitiveDrawInterface
{
public:

	ENGINE_API FSimpleElementCollector();
	ENGINE_API ~FSimpleElementCollector();

	ENGINE_API virtual void SetHitProxy(HHitProxy* HitProxy) override;
	virtual void AddReserveLines(uint8 DepthPriorityGroup, int32 NumLines, bool bDepthBiased = false, bool bThickLines = false) override {}

	ENGINE_API virtual void DrawSprite(
		const FVector& Position,
		float SizeX,
		float SizeY,
		const FTexture* Sprite,
		const FLinearColor& Color,
		uint8 DepthPriorityGroup,
		float U,
		float UL,
		float V,
		float VL,
		uint8 BlendMode = SE_BLEND_Masked,
		float OpacityMaskRevVal = .5f
	) override;

	ENGINE_API virtual void DrawLine(
		const FVector& Start,
		const FVector& End,
		const FLinearColor& Color,
		uint8 DepthPriorityGroup,
		float Thickness = 0.0f,
		float DepthBias = 0.0f,
		bool bScreenSpace = false
		) override;

	ENGINE_API virtual void DrawTranslucentLine(
		const FVector& Start,
		const FVector& End,
		const FLinearColor& Color,
		uint8 DepthPriorityGroup,
		float Thickness = 0.0f,
		float DepthBias = 0.0f,
		bool bScreenSpace = false
	) override;

	ENGINE_API virtual void DrawPoint(
		const FVector& Position,
		const FLinearColor& Color,
		float PointSize,
		uint8 DepthPriorityGroup
		) override;

	ENGINE_API virtual void RegisterDynamicResource(FDynamicPrimitiveResource* DynamicResource) override;

	// Not supported
	virtual bool IsHitTesting() override
	{ 
		static bool bTriggered = false;

		if (!bTriggered)
		{
			bTriggered = true;
			ensureMsgf(false, TEXT("FSimpleElementCollector::DrawMesh called"));
		}

		return false; 
	}

	// Not supported
	virtual int32 DrawMesh(const FMeshBatch& Mesh) override
	{
		static bool bTriggered = false;

		if (!bTriggered)
		{
			bTriggered = true;
			ensureMsgf(false, TEXT("FSimpleElementCollector::DrawMesh called"));
		}

		return 0;
	}

	ENGINE_API void DrawBatchedElements(FRHICommandList& RHICmdList, const FMeshPassProcessorRenderState& DrawRenderState, const FSceneView& InView, EBlendModeFilter::Type Filter, ESceneDepthPriorityGroup DPG, uint32 NumInstances = 1) const;

	class FAllocationInfo
	{
	public:
		FAllocationInfo() = default;

	private:
		FBatchedElements::FAllocationInfo BatchedElements;
		FBatchedElements::FAllocationInfo TopBatchedElements;
		uint32 NumDynamicResources = 0;

		friend FSimpleElementCollector;
	};

	/** Accumulates allocation info for use calling Reserve. */
	ENGINE_API void AddAllocationInfo(FAllocationInfo& AllocationInfo) const;

	/** Reserves memory for all containers. */
	ENGINE_API void Reserve(const FAllocationInfo& AllocationInfo);

	/** Appends contents of another batched elements into this one and clears the other one. */
	ENGINE_API void Append(FSimpleElementCollector& Other);

	bool HasAnyPrimitives() const
	{
		return BatchedElements.HasPrimsToDraw() || TopBatchedElements.HasPrimsToDraw();
	}

	bool HasPrimitives(ESceneDepthPriorityGroup DPG) const
	{
		if (DPG == SDPG_World)
		{
			return BatchedElements.HasPrimsToDraw();
		}

		return TopBatchedElements.HasPrimsToDraw();
	}

	/** The batched simple elements. */
	FBatchedElements BatchedElements;
	FBatchedElements TopBatchedElements;

private:
	FHitProxyId HitProxyId;

	/** The dynamic resources which have been registered with this drawer. */
	TArray<FDynamicPrimitiveResource*,SceneRenderingAllocator> DynamicResources;

	friend class FMeshElementCollector;
};


/** 
 * Base class for a resource allocated from a FMeshElementCollector with AllocateOneFrameResource, which the collector releases.
 * This is useful for per-frame structures which are referenced by a mesh batch given to the FMeshElementCollector.
 */
class FOneFrameResource
{
public:

	virtual ~FOneFrameResource() = default;
};

/** One-frame resource that owns a TArray<FMeshBatch> for use as shared backing storage for mesh batch views. */
struct FOneFrameMeshBatchArray : public FOneFrameResource
{
	TArray<FMeshBatch> MeshBatches;
};

/** 
 * A reference to a mesh batch that is added to the collector, together with some cached relevance flags. 
 */
struct FMeshBatchAndRelevance
{
	const FMeshBatch* Mesh;

	/** The render info for the primitive which created this mesh, required. */
	const FPrimitiveSceneProxy* PrimitiveSceneProxy;

private:
	/** 
	 * Cached usage information to speed up traversal in the most costly passes (depth-only, base pass, shadow depth), 
	 * This is done so the Mesh does not have to be dereferenced to determine pass relevance. 
	 */
	uint32 bHasOpaqueMaterial : 1;
	uint32 bHasMaskedMaterial : 1;
	uint32 bRenderInMainPass : 1;

public:
	ENGINE_API FMeshBatchAndRelevance(const FMeshBatch& InMesh, const FPrimitiveSceneProxy* InPrimitiveSceneProxy, ERHIFeatureLevel::Type FeatureLevel);

	bool GetHasOpaqueMaterial() const { return bHasOpaqueMaterial; }
	bool GetHasMaskedMaterial() const { return bHasMaskedMaterial; }
	bool GetHasOpaqueOrMaskedMaterial() const { return bHasOpaqueMaterial || bHasMaskedMaterial; }
	bool GetRenderInMainPass() const { return bRenderInMainPass; }
};

#if RHI_RAYTRACING

struct FRayTracingDynamicGeometryUpdateParams
{
	UE_DEPRECATED(5.8, "Use MeshBatchesView instead. Allocate mesh batches from FRayTracingInstanceCollector::AllocateMeshBatchArray() for shared ownership.")
	TArray<FMeshBatch> MeshBatches;

	/** Non-owning view of mesh batches. Preferred over MeshBatches TArray to avoid redundant copies.
	 *  When set, GetMeshBatches() returns this view instead of the deprecated MeshBatches array. */
	TConstArrayView<FMeshBatch> MeshBatchesView;

	/** Returns the mesh batches, preferring the view if set, falling back to the deprecated owned array. */
	TConstArrayView<FMeshBatch> GetMeshBatches() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (MeshBatchesView.Num() > 0) return MeshBatchesView;
		return TConstArrayView<FMeshBatch>(MeshBatches);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool bUsingIndirectDraw = false;
	// When bUsingIndirectDraw == false, NumVertices == the actual number of vertices to process
	// When bUsingIndirectDraw == true, it is the maximum possible vertices that GPU can emit
	uint32 NumVertices = 0;
	uint32 VertexBufferSize = 0;
	uint32 NumTriangles = 0;

	FRayTracingGeometry* Geometry = nullptr;
	FRWBuffer* Buffer = nullptr;

	bool bApplyWorldPositionOffset = true;

	uint32 InstanceId = 0;
	FMatrix44f WorldToInstance = FMatrix44f::Identity;

	bool bAlphaMasked = false;
	FBufferRHIRef IndexBuffer = nullptr;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRayTracingDynamicGeometryUpdateParams() = default;
	FRayTracingDynamicGeometryUpdateParams(const FRayTracingDynamicGeometryUpdateParams&) = default;
	FRayTracingDynamicGeometryUpdateParams& operator=(const FRayTracingDynamicGeometryUpdateParams&) = default;
	FRayTracingDynamicGeometryUpdateParams(FRayTracingDynamicGeometryUpdateParams&&) = default;
	FRayTracingDynamicGeometryUpdateParams& operator=(FRayTracingDynamicGeometryUpdateParams&&) = default;
	~FRayTracingDynamicGeometryUpdateParams() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

namespace RayTracing
{
	class FDynamicRayTracingInstancesContext;
}

/**
 * Collector used to gather ray tracing instances and related resources.
 * It is also the actual owner of the temporary, per-frame resources created for each ray tracing instance.
 */
class FRayTracingInstanceCollector : public FMeshElementCollector
{
public:
	// No MeshBatch should be allocated from an FRayTracingInstanceCollector.
	inline FMeshBatch& AllocateMesh() = delete;
	void RegisterOneFrameMaterialProxy(FMaterialRenderProxy* Proxy) = delete;

	TConstArrayView<const FSceneView*> GetViews() const { return Views; }
	uint32 GetVisibilityMap() const { return VisibilityMap; }

	UE_DEPRECATED(5.7, "Use Views array passed directly to GetDynamicRayTracingInstances.")
	const FSceneView* GetReferenceView() const { return Views[0]; }

	UE_DEPRECATED(5.7, "Please specify ViewIndex.")
	ENGINE_API void AddRayTracingInstance(FRayTracingInstance Instance);

	ENGINE_API void AddRayTracingInstance(int32 ViewIndex, FRayTracingInstance Instance);

	ENGINE_API void AddReferencedGeometry(RayTracing::FGeometryHandle Geometry);
	ENGINE_API void AddReferencedGeometryGroup(RayTracing::FGeometryGroupHandle GeometryGroup);
	ENGINE_API void AddReferencedGeometryGroupForDynamicUpdate(RayTracing::FGeometryGroupHandle GeometryGroup);

	UE_DEPRECATED(5.7, "Please specify ViewIndex.")
	ENGINE_API void AddRayTracingGeometryUpdate(FRayTracingDynamicGeometryUpdateParams Params);

	ENGINE_API void AddRayTracingGeometryUpdate(int32 ViewIndex, FRayTracingDynamicGeometryUpdateParams Params);

	UE_DEPRECATED(5.7, "Please specify ViewIndex.")
	void AddRDGPooledBuffer(FRDGPooledBuffer* PooledBuffer)
	{
		AddRDGPooledBuffer(0, PooledBuffer);
	}

	void AddRDGPooledBuffer(int32 ViewIndex, FRDGPooledBuffer* PooledBuffer)
	{
		RDGPooledBuffers[ViewIndex].Add(PooledBuffer);
	}

	/** Allocate a frame-lifetime mesh batch array. Both FRayTracingInstance::MaterialsView and
	 *  FRayTracingDynamicGeometryUpdateParams::MeshBatchesView can reference the returned array
	 *  to avoid redundant copies and ensure both paths see the same mesh batch data. */
	TArray<FMeshBatch>& AllocateMeshBatchArray()
	{
		return AllocateOneFrameResource<FOneFrameMeshBatchArray>().MeshBatches;
	}

private:

	ENGINE_API FRayTracingInstanceCollector(
		ERHIFeatureLevel::Type InFeatureLevel,
		FSceneRenderingBulkObjectAllocator& InBulkAllocator,
		bool bInTrackReferencedGeometry);

	ENGINE_API void AddViewMeshArrays(
		const FSceneView* InView,
		FGPUScenePrimitiveCollector* DynamicPrimitiveCollector
	);

	ENGINE_API void SetVisibilityMap(uint32 VisibilityMap);

	struct FRayTracingInstanceAndViewIndex
	{
		FRayTracingInstance Instance;
		int32 ViewIndex;
	};

	struct FRayTracingDynamicGeometryUpdateRequest
	{
		FRayTracingDynamicGeometryUpdateParams Params;
		int32 ViewIndex;
	};

	TArray<const FSceneView*> Views;
	TArray<TSet<FRDGPooledBuffer*>, TInlineAllocator<2, SceneRenderingAllocator> > RDGPooledBuffers;
	TChunkedArray<FRayTracingInstanceAndViewIndex, 16384, SceneRenderingAllocator> RayTracingInstances;
	TChunkedArray<FRayTracingDynamicGeometryUpdateRequest, 16384, SceneRenderingAllocator> RayTracingGeometriesToUpdate;

	TSet<RayTracing::FGeometryHandle> ReferencedGeometries;
	TSet<RayTracing::FGeometryGroupHandle> ReferencedGeometryGroups;
	TSet<RayTracing::FGeometryGroupHandle> ReferencedGeometryGroupsForDynamicUpdate;

	const bool bTrackReferencedGeometry;

	uint32 VisibilityMap;

	friend RayTracing::FDynamicRayTracingInstancesContext;
};

#endif

class FDynamicPrimitiveUniformBuffer : public FOneFrameResource
{
public:
	ENGINE_API FDynamicPrimitiveUniformBuffer();
	// FDynamicPrimitiveUniformBuffer is non-copyable
	FDynamicPrimitiveUniformBuffer(const FDynamicPrimitiveUniformBuffer&) = delete;
	ENGINE_API virtual ~FDynamicPrimitiveUniformBuffer();

	TUniformBuffer<FPrimitiveUniformShaderParameters> UniformBuffer;

	ENGINE_API void Set(FRHICommandListBase& RHICmdList, FPrimitiveUniformShaderParametersBuilder& Builder);

	ENGINE_API void Set(
		FRHICommandListBase& RHICmdList,
		const FMatrix& LocalToWorld,
		const FMatrix& PreviousLocalToWorld,
		const FVector& ActorPositionWS, 
		const FBoxSphereBounds& WorldBounds,
		const FBoxSphereBounds& LocalBounds,
		const FBoxSphereBounds& PreSkinnedLocalBounds,
		bool bReceivesDecals,
		bool bHasPrecomputedVolumetricLightmap,
		bool bOutputVelocity,
		const FCustomPrimitiveData* CustomPrimitiveData);

	ENGINE_API void Set(
		FRHICommandListBase& RHICmdList,
		const FMatrix& LocalToWorld,
		const FMatrix& PreviousLocalToWorld,
		const FBoxSphereBounds& WorldBounds,
		const FBoxSphereBounds& LocalBounds,
		const FBoxSphereBounds& PreSkinnedLocalBounds,
		bool bReceivesDecals,
		bool bHasPrecomputedVolumetricLightmap,
		bool bOutputVelocity,
		const FCustomPrimitiveData* CustomPrimitiveData);

	ENGINE_API void Set(
		FRHICommandListBase& RHICmdList,
		const FMatrix& LocalToWorld,
		const FMatrix& PreviousLocalToWorld,
		const FBoxSphereBounds& WorldBounds,
		const FBoxSphereBounds& LocalBounds,
		const FBoxSphereBounds& PreSkinnedLocalBounds,
		bool bReceivesDecals,
		bool bHasPrecomputedVolumetricLightmap,
		bool bOutputVelocity);

	/** Pass-through implementation which calls the overloaded Set function with LocalBounds for PreSkinnedLocalBounds. */
	ENGINE_API void Set(
		FRHICommandListBase& RHICmdList,
		const FMatrix& LocalToWorld,
		const FMatrix& PreviousLocalToWorld,
		const FBoxSphereBounds& WorldBounds,
		const FBoxSphereBounds& LocalBounds,
		bool bReceivesDecals,
		bool bHasPrecomputedVolumetricLightmap,
		bool bOutputVelocity);

	UE_DEPRECATED(5.4, "Set requires a command list")
	ENGINE_API void Set(
		const FMatrix& LocalToWorld,
		const FMatrix& PreviousLocalToWorld,
		const FVector& ActorPositionWS, 
		const FBoxSphereBounds& WorldBounds,
		const FBoxSphereBounds& LocalBounds,
		const FBoxSphereBounds& PreSkinnedLocalBounds,
		bool bReceivesDecals,
		bool bHasPrecomputedVolumetricLightmap,
		bool bOutputVelocity,
		const FCustomPrimitiveData* CustomPrimitiveData);

	UE_DEPRECATED(5.4, "Set requires a command list")
	ENGINE_API void Set(
		const FMatrix& LocalToWorld,
		const FMatrix& PreviousLocalToWorld,
		const FBoxSphereBounds& WorldBounds,
		const FBoxSphereBounds& LocalBounds,
		const FBoxSphereBounds& PreSkinnedLocalBounds,
		bool bReceivesDecals,
		bool bHasPrecomputedVolumetricLightmap,
		bool bOutputVelocity,
		const FCustomPrimitiveData* CustomPrimitiveData);

	UE_DEPRECATED(5.4, "Set requires a command list")
	ENGINE_API void Set(
		const FMatrix& LocalToWorld,
		const FMatrix& PreviousLocalToWorld,
		const FBoxSphereBounds& WorldBounds,
		const FBoxSphereBounds& LocalBounds,
		const FBoxSphereBounds& PreSkinnedLocalBounds,
		bool bReceivesDecals,
		bool bHasPrecomputedVolumetricLightmap,
		bool bOutputVelocity);

	/** Pass-through implementation which calls the overloaded Set function with LocalBounds for PreSkinnedLocalBounds. */
	UE_DEPRECATED(5.4, "Set requires a command list")
	ENGINE_API void Set(
		const FMatrix& LocalToWorld,
		const FMatrix& PreviousLocalToWorld,
		const FBoxSphereBounds& WorldBounds,
		const FBoxSphereBounds& LocalBounds,
		bool bReceivesDecals,
		bool bHasPrecomputedVolumetricLightmap,
		bool bOutputVelocity);
};

extern ENGINE_API FVector CalcConeVert(float Angle1, float Angle2, float AzimuthAngle);
extern ENGINE_API void BuildConeVerts(float Angle1, float Angle2, float Scale, float XOffset, uint32 NumSides, TArray<FDynamicMeshVertex>& OutVerts, TArray<uint32>& OutIndices);

extern ENGINE_API void BuildCylinderVerts(const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis, double Radius, double HalfHeight, uint32 Sides, TArray<FDynamicMeshVertex>& OutVerts, TArray<uint32>& OutIndices);

/** Build tapered cylinder vertices with a separate base and top radius. */
extern ENGINE_API void BuildTaperedCylinderVerts(const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis, double BaseRadius, double TopRadius, double HalfHeight, uint32 Sides, TArray<FDynamicMeshVertex>& OutVerts, TArray<uint32>& OutIndices);


/**
 * Given a base color and a selection state, returns a color which accounts for the selection state.
 * @param BaseColor - The base color of the object.
 * @param bSelected - The selection state of the object.
 * @param bHovered - True if the object has hover focus
 * @param bUseOverlayIntensity - True if the selection color should be modified by the selection intensity
 * @return The color to draw the object with, accounting for the selection state
 */
extern ENGINE_API FLinearColor GetSelectionColor(const FLinearColor& BaseColor,bool bSelected,bool bHovered, bool bUseOverlayIntensity = true);
extern ENGINE_API FLinearColor GetViewSelectionColor(const FLinearColor& BaseColor, const FSceneView& View, bool bSelected, bool bHovered, bool bUseOverlayIntensity, bool bIndividuallySelected);

/**
 * Returns true if the given view is "rich", and all primitives should be forced down the dynamic drawing path so that ApplyViewModeOverrides can implement the rich view feature.
 * A view is rich if is missing the EngineShowFlags.Materials showflag, or has any of the render mode affecting showflags.
 */
extern ENGINE_API bool IsRichView(const FSceneViewFamily& ViewFamily);

extern ENGINE_API void ApplyViewModeOverrides(
	int32 ViewIndex,
	const FEngineShowFlags& EngineShowFlags,
	ERHIFeatureLevel::Type FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	bool bSelected,
	struct FMeshBatch& Mesh,
	FMeshElementCollector& Collector
	);

/** Draws the UV layout of the supplied asset (either StaticMeshRenderData OR SkeletalMeshRenderData, not both!) */
extern ENGINE_API void DrawUVs(FViewport* InViewport, FCanvas* InCanvas, int32 InTextYPos, const int32 LODLevel, int32 UVChannel, TArray<FVector2D> SelectedEdgeTexCoords, class FStaticMeshRenderData* StaticMeshRenderData, class FSkeletalMeshLODRenderData* SkeletalMeshRenderData);

/** Will return the view to use taking into account VR which has 2 views */
ENGINE_API const FSceneView& GetLODView(const FSceneView& InView);

/**
 * Computes the screen size of a given sphere bounds in the given view.
 * The screen size is the projected diameter of the bounding sphere of the model.
 * i.e. 0.5 means half the screen's maximum dimension.
 * @param Origin - Origin of the bounds in world space
 * @param SphereRadius - Radius of the sphere to use to calculate screen coverage
 * @param View - The view to calculate the display factor for
 * @return float - The screen size calculated
 */
float ENGINE_API ComputeBoundsScreenSize(const FVector4& Origin, const float SphereRadius, const FSceneView& View);

/**
 * Computes the screen size of a given sphere bounds in the given view.
 * The screen size is the projected diameter of the bounding sphere of the model. 
 * i.e. 0.5 means half the screen's maximum dimension.
 * @param BoundsOrigin - Origin of the bounds in world space
 * @param SphereRadius - Radius of the sphere to use to calculate screen coverage
 * @param ViewOrigin - The origin of the view to calculate the display factor for
 * @param ProjMatrix - The projection matrix used to scale screen size bounds
 * @return float - The screen size calculated
 */
float ENGINE_API ComputeBoundsScreenSize(const FVector4& BoundsOrigin, const float SphereRadius, const FVector4& ViewOrigin, const FMatrix& ProjMatrix);

/**
 * Computes the screen radius squared of a given sphere bounds in the given view. This is used at
 * runtime instead of ComputeBoundsScreenSize to avoid a square root.
 * It is a wrapper for the version below that does not take a FSceneView reference but parameters directly
 * @param Origin - Origin of the bounds in world space
 * @param SphereRadius - Radius of the sphere to use to calculate screen coverage
 * @param View - The view to calculate the display factor for
 * @return float - The screen size calculated
 */
float ENGINE_API ComputeBoundsScreenRadiusSquared(const FVector4& Origin, const float SphereRadius, const FSceneView& View);

/**
 * Computes the screen radius squared of a given sphere bounds in the given view. This is used at
 * runtime instead of ComputeBoundsScreenSize to avoid a square root.
 * @param Origin - Origin of the bounds in world space
 * @param SphereRadius - Radius of the sphere to use to calculate screen coverage
 * @param ViewOrigin - The view origin involved in the calculation
 * @param ProjMatrix - The projection matrix of the view involved in the calculation
 * @return float - The screen size calculated
 */
float ENGINE_API ComputeBoundsScreenRadiusSquared(const FVector4& BoundsOrigin, const float SphereRadius, const FVector4& ViewOrigin, const FMatrix& ProjMatrix);

/**
 * Computes the draw distance of a given sphere bounds in the given view with the specified screen size.
 * @param ScreenSize - The screen size (as computed by ComputeBoundsScreenSize)
 * @param SphereRadius - Radius of the sphere to use to calculate screen coverage
 * @param ProjMatrix - The projection matrix used to scale screen size bounds
 * @return float - The draw distance calculated
 */
float ENGINE_API ComputeBoundsDrawDistance(const float ScreenSize, const float SphereRadius, const FMatrix& ProjMatrix);

/**
 * Computes the LOD level for the given static meshes render data in the given view.
 * @param RenderData - Render data for the mesh
 * @param Origin - Origin of the bounds of the mesh in world space
 * @param SphereRadius - Radius of the sphere to use to calculate screen coverage
 * @param View - The view to calculate the LOD level for
 * @param FactorScale - multiplied by the computed screen size before computing LOD
 */
int8 ENGINE_API ComputeStaticMeshLOD(const FStaticMeshRenderData* RenderData, const FVector4& Origin, const float SphereRadius, const FSceneView& View, int32 MinLOD, float FactorScale = 1.0f);

/**
 * Computes the LOD level for the given static meshes render data in the given view, for one of the two temporal LOD samples
 * @param RenderData - Render data for the mesh
 * @param Origin - Origin of the bounds of the mesh in world space
 * @param SphereRadius - Radius of the sphere to use to calculate screen coverage
 * @param View - The view to calculate the LOD level for
 * @param FactorScale - multiplied by the computed screen size before computing LOD
 * @param SampleIndex - index (0 or 1) of the temporal sample to use
 */
int8 ENGINE_API ComputeTemporalStaticMeshLOD( const FStaticMeshRenderData* RenderData, const FVector4& Origin, const float SphereRadius, const FSceneView& View, int32 MinLOD, float FactorScale, int32 SampleIndex );

/** 
 * Contains LODs to render. 
 * Interpretation of LODIndex0 and LODIndex1 depends on flags.
 * By default the two LODs are the ones used in a dithered LOD transition.
 * But they also be interpreted as the start and end of a range where we submit multiple LODs and select/cull on GPU.
 */
struct FLODMask
{
	// Assumes a max lod index of 127.
	// In fact MAX_STATIC_MESH_LODS is 8 so we could use 3 bits per LODIndex and fit in a uint8 here.
	uint16 LODIndex0 : 7;
	uint16 LODIndex1 : 7;
	uint16 bIsValid : 1;
	uint16 bIsRange : 1;

	FLODMask()
		: LODIndex0(0)
		, LODIndex1(0)
		, bIsValid(0)
		, bIsRange(0)
	{
	}

	bool IsValid() const
	{
		return bIsValid;
	}
	void SetLOD(uint32 LODIndex)
	{
		LODIndex0 = LODIndex1 = (uint8)LODIndex;
		bIsValid = 1;
		bIsRange = 0;
	}
	void SetLODSample(uint32 LODIndex, uint32 SampleIndex)
	{
		if (SampleIndex == 0)
		{
			LODIndex0 = (uint8)LODIndex;
		}
		else if (SampleIndex == 1)
		{
			LODIndex1 = (uint8)LODIndex;
		}
		bIsValid = 1;
		bIsRange = 0;
	}
	void SetLODRange(uint32 MinLODIndex, uint32 MaxLODIndex)
	{
		LODIndex0 = (uint8)MinLODIndex;
		LODIndex1 = (uint8)MaxLODIndex;
		bIsValid = 1;
		bIsRange = 1;
	}
	void ClampToFirstLOD(uint32 FirstLODIdx)
	{
		LODIndex0 = LODIndex0 > (uint8)FirstLODIdx ? LODIndex0 : (uint8)FirstLODIdx;
		LODIndex1 = LODIndex1 > (uint8)FirstLODIdx ? LODIndex1 : (uint8)FirstLODIdx;
	}
	bool IsDithered() const
	{
		return IsValid() && !bIsRange && LODIndex0 != LODIndex1;
	}
	bool IsLODRange() const
	{
		return IsValid() && bIsRange && LODIndex0 != LODIndex1;
	}
	bool ContainsLOD(int32 LODIndex) const
	{
		if (!IsValid())
		{
			return false;
		}
		if (bIsRange)
		{
			return (int32)LODIndex0 <= LODIndex && (int32)LODIndex1 >= LODIndex;
		}
		return (int32)LODIndex0 == LODIndex || (int32)LODIndex1 == LODIndex;
	}
	bool IsMinLODInRange(int32 LODIndex) const
	{
		return IsLODRange() && LODIndex == LODIndex0;
	}
	bool IsMaxLODInRange(int32 LODIndex) const
	{
		return IsLODRange() && LODIndex == LODIndex1;
	}

	//#dxr_todo UE-72106: We should probably add both LoDs but mask them based on their 
	//LodFade value within the BVH based on the LodFadeMask in the GBuffer
	int8 GetRayTracedLOD() const
	{
		return LODIndex1;
	}
};

/**
 * Computes the LOD to render for the list of static meshes in the given view.
 * @param StaticMeshes - List of static meshes.
 * @param View - The view to render the LOD level for
 * @param Origin - Origin of the bounds of the primitive in world space
 * @param SphereRadius - Radius of the sphere bounds of the primitive in world space
 */
FLODMask ENGINE_API ComputeLODForMeshes(const TArray<class FStaticMeshBatchRelevance>& StaticMeshRelevances, const FSceneView& View, const FVector4& Origin, float SphereRadius, int32 ForcedLODLevel, float& OutScreenRadiusSquared, int8 CurFirstLODIdx, float ScreenSizeScale = 1.0f, bool bDitheredLODTransition = true);

/**
 * Computes the LOD to render for the list of static meshes in the given view.
 * @param StaticMeshes - List of static meshes.
 * @param View - The view to render the LOD level for
 * @param Origin - Origin of the bounds of the primitive in world space
 * @param SphereRadius - Radius of the sphere bounds of the primitive in world space
 * @param InstanceSphereRadius - Radius of the sphere bounds for a single mesh instance in the primitive. If not 0.f then the return FLODMask will contain a range of LODs ready for LOD selection on the GPU
 */
FLODMask ENGINE_API ComputeLODForMeshes(const TArray<class FStaticMeshBatchRelevance>& StaticMeshRelevances, const FSceneView& View, const FVector4& Origin, float SphereRadius, float InstanceSphereRadius, int32 ForcedLODLevel, float& OutScreenRadiusSquared, int8 CurFirstLODIdx, float ScreenSizeScale = 1.0f);

class FSharedSamplerState : public FRenderResource
{
public:
	FSamplerStateRHIRef SamplerStateRHI;
	bool bWrap;

	FSharedSamplerState(bool bInWrap) :
		bWrap(bInWrap)
	{}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	virtual void ReleaseRHI() override
	{
		SamplerStateRHI.SafeRelease();
	}
};

/** Sampler state using Wrap addressing and taking filter mode from the world texture group. */
extern ENGINE_API FSharedSamplerState* Wrap_WorldGroupSettings;

/** Sampler state using Clamp addressing and taking filter mode from the world texture group. */
extern ENGINE_API FSharedSamplerState* Clamp_WorldGroupSettings;

/** Initializes the shared sampler states. */
extern ENGINE_API void InitializeSharedSamplerStates();
