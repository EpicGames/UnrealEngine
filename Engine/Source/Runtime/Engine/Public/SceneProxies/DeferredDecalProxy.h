// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/BoxSphereBounds.h"

class FSceneView;
class UDecalComponent;
class UMaterialInterface;
class USceneComponent;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Encapsulates the data which is used to render a decal parallel to the game thread. */
class FDeferredDecalProxy
{
public:
	/** constructor */
	ENGINE_API FDeferredDecalProxy(const UDecalComponent* InComponent);
	ENGINE_API FDeferredDecalProxy(const USceneComponent* InComponent, UMaterialInterface* InMaterial);
	ENGINE_API FDeferredDecalProxy(const struct FDeferredDecalSceneProxyDesc& Desc);

	/**
	 * Updates the decal proxy's cached transform and bounds.
	 * @param InComponentToWorldIncludingDecalSize - The new component-to-world transform including the DecalSize
	 * @param InBounds - The new world-space bounds including the DecalSize
	 */
	ENGINE_API void SetTransformIncludingDecalSize(const FTransform& InComponentToWorldIncludingDecalSize, const FBoxSphereBounds& InBounds);

	ENGINE_API void InitializeFadingParameters(float AbsSpawnTime, float FadeDuration, float FadeStartDelay, float FadeInDuration, float FadeInStartDelay);

	/** @return True if the decal is visible in the given view. */
	ENGINE_API bool IsShown( const FSceneView* View ) const;

	inline const FBoxSphereBounds& GetBounds() const { return Bounds; }

	/** Pointer back to the game thread owner component. */
	const USceneComponent* Component;

	UMaterialInterface* DecalMaterial;

	/** Used to compute the projection matrix on the render thread side, includes the DecalSize  */
	FTransform ComponentTrans;

private:
	/** Whether or not the decal should be drawn in the game, or when the editor is in 'game mode'. */
	bool DrawInGame;

	/** Whether or not the decal should be drawn in the editor. */
	bool DrawInEditor;

	FBoxSphereBounds Bounds;

public:

	/** Larger values draw later (on top). */
	int32 SortOrder;

	float InvFadeDuration;

	float InvFadeInDuration;

	/**
	* FadeT = saturate(1 - (AbsTime - FadeStartDelay - AbsSpawnTime) / FadeDuration)
	*
	*		refactored as muladd:
	*		FadeT = saturate((AbsTime * -InvFadeDuration) + ((FadeStartDelay + AbsSpawnTime + FadeDuration) * InvFadeDuration))
	*/
	float FadeStartDelayNormalized;

	float FadeInStartDelayNormalized;

	float FadeScreenSize;

	FLinearColor DecalColor = FLinearColor::White;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FDeferredDecalUpdateParams
{
	enum class EOperationType : int8
	{
		AddToSceneAndUpdate,				// Adds the decal to the scene an updates the parameters
		Update,								// Updates the decals parameters
		RemoveFromSceneAndDelete,			// Remove the decal from the scene and deletes the proxy
	};

	FTransform				Transform;
	FDeferredDecalProxy*	DecalProxy = nullptr;
	FBoxSphereBounds		Bounds;
	float					AbsSpawnTime = 0.0f;
	float					FadeDuration = 0.0f;
	float					FadeStartDelay = 1.0f;
	float					FadeInDuration = 0.0f;
	float					FadeInStartDelay = 0.0f;
	float					FadeScreenSize = 0.01f;
	int32					SortOrder = 0;
	FLinearColor			DecalColor = FLinearColor::White;
	EOperationType			OperationType = EOperationType::Update;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
