// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Hash/xxhash.h"
#include "Math/DoubleFloat.h"
#include "ShaderParameterMacros.h"

class FMobileReflectionCaptureShaderParameters;
class FTexture;
class UReflectionCaptureComponent;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Reflection capture shapes. */
namespace EReflectionCaptureShape
{
	enum Type
	{
		Sphere,
		Box,
		Plane,
		Num
	};
}

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Represents a reflection capture to the renderer. */

class FReflectionCaptureProxy
{
public:
	const class UReflectionCaptureComponent* Component;

	int32 PackedIndex;

	/** Used with mobile renderer */
	TUniformBufferRef<FMobileReflectionCaptureShaderParameters> MobileUniformBuffer;
	FTexture* EncodedHDRCubemap;
	float EncodedHDRAverageBrightness;

	EReflectionCaptureShape::Type Shape;

	// Properties shared among all shapes
	FDFVector3 Position;
	float InfluenceRadius;
	float Brightness;
	FXxHash64 Guid;
	FVector3f CaptureOffset;
	int32 SortedCaptureIndex; // Index into ReflectionSceneData.SortedCaptures (and ReflectionCaptures uniform buffer).

	// Box properties
	FMatrix44f BoxTransform;
	FVector3f BoxScales;
	float BoxTransitionDistance;

	// Plane properties
	FPlane4f LocalReflectionPlane;
	FVector4 ReflectionXAxisAndYScale;

	bool bUsingPreviewCaptureData;

	ENGINE_API FReflectionCaptureProxy(const class UReflectionCaptureComponent* InComponent);

	ENGINE_API void SetTransform(const FMatrix& InTransform);
	ENGINE_API void UpdateMobileUniformBuffer(FRHICommandListBase& RHICmdList);
	
	UE_DEPRECATED(5.3, "UpdateMobileUniformBuffer now takes a command list.")
	ENGINE_API void UpdateMobileUniformBuffer();
};

////////////////////////////////////////////////////////////////////////////////////////////////////
