// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"
#include "Rendering/SkyAtmosphereCommonData.h"

struct FSkyAtmosphereDynamicState;
class FSkyAtmosphereRenderSceneInfo;
class USkyAtmosphereComponent;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Represents a USkyAtmosphereComponent to the rendering thread. */
class FSkyAtmosphereSceneProxy
{
public:

	// Initialization constructor.
	ENGINE_API FSkyAtmosphereSceneProxy(const USkyAtmosphereComponent* InComponent);
	ENGINE_API FSkyAtmosphereSceneProxy(const FSkyAtmosphereDynamicState& Ds);
	ENGINE_API ~FSkyAtmosphereSceneProxy();

	FLinearColor GetSkyLuminanceFactor() const { return SkyLuminanceFactor; }
	FLinearColor GetSkyAndAerialPerspectiveLuminanceFactor() const { return SkyAndAerialPerspectiveLuminanceFactor; }
	float GetAerialPespectiveViewDistanceScale() const { return AerialPespectiveViewDistanceScale; }
	float GetHeightFogContribution() const { return HeightFogContribution; }
	float GetAerialPerspectiveStartDepthKm() const { return AerialPerspectiveStartDepthKm; }
	float GetTraceSampleCountScale() const { return TraceSampleCountScale; }

	const FAtmosphereSetup& GetAtmosphereSetup() const { return AtmosphereSetup; }

	bool IsHoldout() const { return bHoldout; }
	bool IsRenderedInMainPass() const { return bRenderInMainPass; }

	void UpdateTransform(const FTransform& ComponentTransform, uint8 TranformMode) { AtmosphereSetup.UpdateTransform(ComponentTransform, TranformMode); }
	void ApplyWorldOffset(const FVector3f& InOffset) { AtmosphereSetup.ApplyWorldOffset((FVector)InOffset); }

	ENGINE_API FVector GetAtmosphereLightDirection(int32 AtmosphereLightIndex, const FVector& DefaultDirection) const;

	bool bStaticLightingBuilt;
	FSkyAtmosphereRenderSceneInfo* RenderSceneInfo;
private:

	FAtmosphereSetup AtmosphereSetup;

	FLinearColor SkyLuminanceFactor;
	FLinearColor SkyAndAerialPerspectiveLuminanceFactor;
	float AerialPespectiveViewDistanceScale;
	float HeightFogContribution;
	float AerialPerspectiveStartDepthKm;
	float TraceSampleCountScale;
	bool bHoldout;
	bool bRenderInMainPass;

	bool OverrideAtmosphericLight[NUM_ATMOSPHERE_LIGHTS];
	FVector OverrideAtmosphericLightDirection[NUM_ATMOSPHERE_LIGHTS];

	friend class FSkyAtmosphereStateStreamImpl;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
