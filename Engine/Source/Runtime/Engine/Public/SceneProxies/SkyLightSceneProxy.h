// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "Math/Color.h"
#include "Math/SHMath.h"

class FTexture;
class USkyLightComponent;
enum EOcclusionCombineMode : int;
namespace ECastRayTracedShadow { enum Type : int; };

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSkyLightSceneProxy
{
public:

	/** Initialization constructor. */
	ENGINE_API FSkyLightSceneProxy(const class USkyLightComponent* InLightComponent);

	ENGINE_API void Initialize(
		float InBlendFraction, 
		const FSHVectorRGB3* InIrradianceEnvironmentMap, 
		const FSHVectorRGB3* BlendDestinationIrradianceEnvironmentMap,
		const float* InAverageBrightness,
		const float* BlendDestinationAverageBrightness,
		const FLinearColor* InSpecifiedCubemapColorScale);

	const USkyLightComponent* LightComponent;
	FTexture* ProcessedTexture;
	float BlendFraction;
	float SkyDistanceThreshold;
	FTexture* BlendDestinationProcessedTexture;
	uint8 bCastShadows:1;
	uint8 bWantsStaticShadowing:1;
	uint8 bHasStaticLighting:1;
	uint8 bCastVolumetricShadow:1;
	TEnumAsByte<ECastRayTracedShadow::Type> CastRayTracedShadow;
	uint8 bAffectReflection:1;
	uint8 bAffectGlobalIllumination:1;
	uint8 bTransmission:1;
	TEnumAsByte<EOcclusionCombineMode> OcclusionCombineMode;
	float AverageBrightness;
	float IndirectLightingIntensity;
	float VolumetricScatteringIntensity;
	FSHVectorRGB3 IrradianceEnvironmentMap;
	float OcclusionMaxDistance;
	float Contrast;
	float OcclusionExponent;
	float MinOcclusion;
	FLinearColor OcclusionTint;
	bool bCloudAmbientOcclusion;
	float CloudAmbientOcclusionExtent;
	float CloudAmbientOcclusionStrength;
	float CloudAmbientOcclusionMapResolutionScale;
	float CloudAmbientOcclusionApertureScale;
	int32 SamplesPerPixel;
	bool bRealTimeCaptureEnabled;
	FVector CapturePosition;
	uint32 CaptureCubeMapResolution;
	FLinearColor LowerHemisphereColor;
	bool bLowerHemisphereIsSolidColor;
	FLinearColor SpecifiedCubemapColorScale;

	bool IsMovable() { return bMovable; }

	void SetLightColor(const FLinearColor& InColor)
	{
		LightColor = InColor;
	}
	ENGINE_API FLinearColor GetEffectiveLightColor() const;

#if WITH_EDITOR
	float SecondsToNextIncompleteCapture;
	bool bCubemapSkyLightWaitingForCubeMapTexture;
	bool bCaptureSkyLightWaitingForShaders;
	bool bCaptureSkyLightWaitingForMeshesOrTextures;
#endif

private:
	FLinearColor LightColor;
	const uint8 bMovable : 1;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
