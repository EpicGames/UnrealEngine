// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Calculated wind data with support for accumulating other weighted wind data */
class FWindData
{
public:
	FWindData()
		: Speed(0.0f)
		, MinGustAmt(0.0f)
		, MaxGustAmt(0.0f)
		, Direction(1.0f, 0.0f, 0.0f)
	{
	}

	ENGINE_API void PrepareForAccumulate();
	ENGINE_API void AddWeighted(const FWindData& InWindData, float Weight);
	ENGINE_API void NormalizeByTotalWeight(float TotalWeight);

	float Speed;
	float MinGustAmt;
	float MaxGustAmt;
	FVector Direction;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Represents a wind source component to the scene manager in the rendering thread. */

class FWindSourceSceneProxy
{
public:	

	/** Initialization constructor. */
	FWindSourceSceneProxy(const FVector& InDirection, float InStrength, float InSpeed, float InMinGustAmt, float InMaxGustAmt) :
	  Position(FVector::ZeroVector),
		  Direction(InDirection),
		  Strength(InStrength),
		  Speed(InSpeed),
		  MinGustAmt(InMinGustAmt),
		  MaxGustAmt(InMaxGustAmt),
		  Radius(0),
		  bIsPointSource(false)
	  {}

	  /** Initialization constructor. */
	FWindSourceSceneProxy(const FVector& InPosition, float InStrength, float InSpeed, float InMinGustAmt, float InMaxGustAmt, float InRadius) :
	  Position(InPosition),
		  Direction(FVector::ZeroVector),
		  Strength(InStrength),
		  Speed(InSpeed),
		  MinGustAmt(InMinGustAmt),
		  MaxGustAmt(InMaxGustAmt),
		  Radius(InRadius),
		  bIsPointSource(true)
	  {}

	  ENGINE_API bool GetWindParameters(const FVector& EvaluatePosition, FWindData& WindData, float& Weight) const;
	  ENGINE_API bool GetDirectionalWindParameters(FWindData& WindData, float& Weight) const;
	  ENGINE_API void ApplyWorldOffset(FVector InOffset);

private:

	FVector Position;
	FVector	Direction;
	float Strength;
	float Speed;
	float MinGustAmt;
	float MaxGustAmt;
	float Radius;
	bool bIsPointSource;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
