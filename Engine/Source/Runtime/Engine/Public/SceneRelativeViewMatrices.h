// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FViewMatrices;

/** Various view matrices stored as floats, along with LWC tile position, suitable for sending to GPU */
struct FRelativeViewMatrices
{
	struct FInitializer
	{
		FMatrix ViewToWorld;
		FMatrix WorldToView;
		FMatrix ViewToClip;
		FMatrix ClipToView;
		FMatrix PrevViewToWorld;
		FMatrix PrevClipToView;
	};

	ENGINE_API static FRelativeViewMatrices Create(const FViewMatrices& Matrices, const FViewMatrices& PrevMatrices);
	ENGINE_API static FRelativeViewMatrices Create(const FInitializer& Initializer);

	/** Use the same matrices for current and previous */
	inline static FRelativeViewMatrices Create(const FViewMatrices& Matrices)
	{
		return Create(Matrices, Matrices);
	}

	FMatrix44f RelativeWorldToView;
	FMatrix44f RelativeWorldToClip;
	FMatrix44f ViewToRelativeWorld;
	FMatrix44f ClipToRelativeWorld;
	FMatrix44f ViewToClip;
	FMatrix44f ClipToView;
	FMatrix44f PrevViewToRelativeWorld;
	FMatrix44f PrevClipToRelativeWorld;
	FMatrix44f PrevClipToView;
	FVector3f TilePosition;
};

struct FDFRelativeViewMatrices
{
	struct FInitializer
	{
		FVector ViewOrigin;
		FMatrix ViewToWorld;
		FMatrix WorldToView;
		FMatrix ViewToClip;
		FMatrix ClipToView;
		FMatrix PrevViewToWorld;
		FMatrix PrevClipToView;
	};

	ENGINE_API static FDFRelativeViewMatrices Create(const FViewMatrices& Matrices, const FViewMatrices& PrevMatrices);
	ENGINE_API static FDFRelativeViewMatrices Create(const FInitializer& Initializer);

	/** Use the same matrices for current and previous */
	inline static FDFRelativeViewMatrices Create(const FViewMatrices& Matrices)
	{
		return Create(Matrices, Matrices);
	}

	FMatrix44f RelativeWorldToView;
	FMatrix44f RelativeWorldToClip;
	FMatrix44f ViewToRelativeWorld;
	FMatrix44f ClipToRelativeWorld;
	FMatrix44f ViewToClip;
	FMatrix44f ClipToView;
	FMatrix44f PrevViewToRelativeWorld;
	FMatrix44f PrevClipToRelativeWorld;
	FMatrix44f PrevClipToView;
	FVector3f PositionHigh;
};