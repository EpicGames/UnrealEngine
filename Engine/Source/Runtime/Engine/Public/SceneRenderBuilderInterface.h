// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "RenderingThread.h"

class FRHICommandListImmediate;
class FRDGBuilder;
class FSceneRenderer;
class FSceneInterface;
class FSceneViewFamily;
class FHitProxyConsumer;

struct FEngineShowFlags;
struct FSceneRenderUpdateInputs;

enum class ESceneRenderGroupFlags : uint8
{
	None       = 0,

	// This group of renderers will perform a GPU capture.
	GpuCapture = 1 << 1,

	// This group of renderers will perform a GPU dump.
	GpuDump    = 1 << 2
};
ENUM_CLASS_FLAGS(ESceneRenderGroupFlags);

// A setup of inputs passed into a scene render callback function.
struct FSceneRenderFunctionInputs
{
	FSceneRenderFunctionInputs(FSceneRenderer* InRenderer, FSceneRenderUpdateInputs* InSceneUpdateInputs, const TCHAR* InName, const TCHAR* InFullPath)
		: Renderer(InRenderer)
		, SceneUpdateInputs(InSceneUpdateInputs)
		, Name(InName)
		, FullPath(InFullPath)
	{}

	FSceneRenderer* Renderer;
	FSceneRenderUpdateInputs* SceneUpdateInputs;
	const TCHAR* Name;
	const TCHAR* FullPath;
};

// The user must return whether FSceneRenderer::Render was called inside of the function.
using FSceneRenderFunction = TUniqueFunction<bool(FRDGBuilder&, const FSceneRenderFunctionInputs&)>;

// A game side builder interface that collects scene renderers to process as one workload.
class ISceneRenderBuilder
{
public:
	virtual ~ISceneRenderBuilder() = default;

	// Creates an instance of a scene render builder using a specified scene. The scene must be
	// valid and all scene renderers must reference the same scene instance.
	static ENGINE_API TUniquePtr<ISceneRenderBuilder> Create(FSceneInterface* SceneInterface);

	/** Creates a scene renderer based on the current feature level. */
	virtual FSceneRenderer* CreateSceneRenderer(FSceneViewFamily* ViewFamily) = 0;

	/** Creates a linked set of scene renderers with related view families based on the current feature level. */
	virtual TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> CreateLinkedSceneRenderers(
		TConstArrayView<FSceneViewFamily*> ViewFamilies,
		FHitProxyConsumer* HitProxyConsumer) = 0;

	/** Adds a command that executes on the game thread and is pipelined with scene render commands on the render thread.
	 *  This is useful to process game thread side logic in between scene renders which may insert render commands. Though
	 *  be mindful of how commands may be deferred until RDG execution in certain configurations.
	 */
	virtual void AddCommand(TUniqueFunction<void()>&& Function) = 0;

	// Utility method that enqueues a render command that is interleaved with scene render commands.
	void AddRenderCommand(TUniqueFunction<void(FRHICommandListImmediate&)>&& Function)
	{
		AddCommand([Function = MoveTemp(Function)]() mutable
		{
			ENQUEUE_RENDER_COMMAND(SceneRenderBuilder_AddRenderCommand)(MoveTemp(Function));
		});
	}

	// Adds a command to render a scene renderer on the render thread using RDG.
	virtual void AddRenderer(FSceneRenderer* Renderer, FString&& Name, FSceneRenderFunction&& Function) = 0;

	void AddRenderer(FSceneRenderer* Renderer, FSceneRenderFunction&& Function)
	{
		AddRenderer(Renderer, {}, MoveTemp(Function));
	}

	/** Whether a renderer with the provided show flags can be added to the builder. If not compatible, you must call Execute()
	 *  first to flush existing renderers prior to adding.
	 */
	virtual bool IsCompatible(const FEngineShowFlags& EngineShowFlags) const = 0;

	//////////////////////////////////////////////////////////////////////////

	// Call to execute all enqueued commands on the render thread. Resets the builder back to empty.
	virtual void Execute() = 0;

	// Returns a linear allocator for allocating objects that can be used safely in command functions.
	virtual FConcurrentLinearBulkObjectAllocator& GetAllocator() = 0;

private:
	friend class FSceneRenderGroupScope;
	virtual void BeginGroup(FString&& Name, ESceneRenderGroupFlags Flags) = 0;
	virtual void EndGroup() = 0;
};

// A scope class used to construct a scene render group.
class FSceneRenderGroupScope
{
	ISceneRenderBuilder& SceneRenderBuilder;

public:
	FSceneRenderGroupScope(ISceneRenderBuilder& InSceneRenderBuilder, FString&& Name, ESceneRenderGroupFlags Flags)
		: SceneRenderBuilder(InSceneRenderBuilder)
	{
		SceneRenderBuilder.BeginGroup(Forward<FString&&>(Name), Flags);
	}

	~FSceneRenderGroupScope()
	{
		SceneRenderBuilder.EndGroup();
	}
};

// A scope macro used to construct a scene render group scope.
#define SCENE_RENDER_GROUP_SCOPE(SceneRenderBuilder, Name, Flags) \
	FSceneRenderGroupScope UE_JOIN(__SceneRenderGroup_ScopeRef_,__LINE__)( \
		  (SceneRenderBuilder)                                             \
		, Forward<FString&&>(Name)                                         \
		, Flags                                                            \
	)