// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphDefinitions.h"
#include "ShaderParameterMacros.h"
#include "GBufferInfo.h"
#include "SceneUtils.h"

class FSceneViewFamily;
struct FMinimalSceneTextures;

/** A uniform buffer containing common scene textures used by materials or global shaders. */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, ENGINE_API)
	// Scene Color / Depth / Partial Depth
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ScenePartialDepthTexture)

	// GBuffer
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferATexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferBTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferCTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferDTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferETexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferFTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferVelocityTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferSGGXTexture)

	// SSAO
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ScreenSpaceAOTexture)

	// Custom Depth / Stencil
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CustomDepthTexture)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint2>, CustomStencilTexture)

	// Misc
	SHADER_PARAMETER_SAMPLER(SamplerState, PointClampSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FMobileSceneTextureUniformParameters, ENGINE_API)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorTextureSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, SceneDepthTextureArray)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthTextureSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ScenePartialDepthTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ScenePartialDepthTextureSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CustomDepthTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, CustomDepthTextureArray)
	SHADER_PARAMETER_SAMPLER(SamplerState, CustomDepthTextureSampler)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint2>, CustomStencilTexture)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<uint2>, CustomStencilTextureArray)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneVelocityTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneVelocityTextureSampler)
	// GBuffer
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferATexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferBTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferCTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferDTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthAuxTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, SceneDepthAuxTextureArray)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, LocalLightTextureA)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, LocalLightTextureB)
	SHADER_PARAMETER_SAMPLER(SamplerState, GBufferATextureSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, GBufferBTextureSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, GBufferCTextureSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, GBufferDTextureSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthAuxTextureSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSceneTextureShaderParameters, ENGINE_API)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileSceneTextureUniformParameters, MobileSceneTextures)
END_SHADER_PARAMETER_STRUCT()

extern ENGINE_API FSceneTextureShaderParameters GetSceneTextureShaderParameters(TRDGUniformBufferRef<FSceneTextureUniformParameters> UniformBuffer);

extern ENGINE_API FSceneTextureShaderParameters GetSceneTextureShaderParameters(TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> UniformBuffer);

extern ENGINE_API void GetSceneColorFormatAndCreateFlags(ERHIFeatureLevel::Type FeatureLevel, bool bRequiresAlphaChannel, ETextureCreateFlags ExtraSceneColorCreateFlags, uint32 NumSamples, bool bMemorylessMSAA, EPixelFormat& SceneColorFormat, ETextureCreateFlags& SceneColorCreateFlags);

enum class ESceneTextureExtracts : uint32
{
	/** No textures are extracted from the render graph after execution. */
	None = 0,

	/** Extracts scene depth after execution */
	Depth = 1 << 0,

	/** Extracts custom depth after execution. */
	CustomDepth = 1 << 1,

	/** Extracts all available textures after execution. */
	All = Depth | CustomDepth
};

struct FSceneTexturesConfigInitSettings
{
	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::Num;
	FIntPoint Extent = FIntPoint::ZeroValue;
	bool bRequireMultiView = false;
	bool bRequiresAlphaChannel = false;
	bool bSupportsXRTargetManagerDepthAlloc = false;
	ETextureCreateFlags ExtraSceneColorCreateFlags = ETextureCreateFlags::None;
	ETextureCreateFlags ExtraSceneDepthCreateFlags = ETextureCreateFlags::None;
};

/** Struct containing the scene texture configuration used to create scene textures.  Use InitializeViewFamily to initialize the
 *  SceneTexturesConfig structure in the FViewFamilyInfo.  A global singleton instance is maintained manually with static Set / Get
 *  functions, but will soon be deprecated, in preference of using the structure from the FViewFamilyInfo.
 */
struct FSceneTexturesConfig
{
	// Sets the persistent global config instance.
	static void Set(const FSceneTexturesConfig& Config)
	{
		GlobalInstance = Config;
	}

	// Gets the persistent global config instance. If unset, will return a default constructed instance.
	static const FSceneTexturesConfig& Get()
	{
		return GlobalInstance;
	}

	FSceneTexturesConfig()
		: bRequireMultiView{}
		, bIsUsingGBuffers{}
		, bKeepDepthContent{ 1 }
		, bRequiresDepthAux{}
		, bPreciseDepthAux{}
		, bCustomResolveSubpass{}
		, bSamplesCustomStencil{}
		, bMemorylessMSAA{}
		, bSupportsXRTargetManagerDepthAlloc{}
	{}

	ENGINE_API void Init(const FSceneTexturesConfigInitSettings& InitSettings);
    ENGINE_API void BuildSceneColorAndDepthFlags();
	ENGINE_API uint32 GetGBufferRenderTargetsInfo(FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo, EGBufferLayout Layout = GBL_Default) const;
	ENGINE_API void SetupMobileGBufferFlags(bool bRequiresMultiPass);

	// Number of MSAA samples in the Editor.Primitive<Color/Depth> textures. See also r.MSAA.CompositingSampleCount
	ENGINE_API static uint32 GetEditorPrimitiveNumSamples(ERHIFeatureLevel::Type FeatureLevel);

	inline bool IsValid() const
	{
		return ShadingPath != EShadingPath::Num;
	}

	// Extractions to queue for after execution of the render graph.
	ESceneTextureExtracts Extracts = ESceneTextureExtracts::All;

	// Enums describing the shading / feature / platform configurations used to construct the config.
	EShadingPath ShadingPath = EShadingPath::Num;
	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::SM5;
	EShaderPlatform ShaderPlatform = SP_PCD3D_SM5;

	// Extent of all full-resolution textures.
	FIntPoint Extent = FIntPoint::ZeroValue;

	// Downsample factors to divide against the full resolution texture extent.
	uint32 SmallDepthDownsampleFactor = 2;

	// Number of MSAA samples used by color and depth targets.
	uint32 NumSamples = 1;

	// Number of MSAA sampled used by the editor primitive composition targets.
	uint32 EditorPrimitiveNumSamples = 1;

	// Pixel format to use when creating scene color.
	EPixelFormat ColorFormat = PF_Unknown;

	// Create flags when creating scene color / depth textures
	ETextureCreateFlags ColorCreateFlags = ETextureCreateFlags::None;
	ETextureCreateFlags DepthCreateFlags = ETextureCreateFlags::None;

    // Flags passed in from initializer
    ETextureCreateFlags ExtraSceneColorCreateFlags = ETextureCreateFlags::None;
    ETextureCreateFlags ExtraSceneDepthCreateFlags = ETextureCreateFlags::None;
    
	// Optimized clear values to use for color / depth textures.
	FClearValueBinding ColorClearValue = FClearValueBinding::Black;
	FClearValueBinding DepthClearValue = FClearValueBinding::DepthFar;

	// (Deferred Shading) Dynamic GBuffer configuration used to control allocation and slotting of base pass textures.
	FGBufferParams GBufferParams[GBL_Num];
	FGBufferBindings GBufferBindings[GBL_Num];

	// (VR) True if scene color and depth should be multi-view allocated.
	uint32 bRequireMultiView : 1;

	// True if platform is using GBuffers.
	uint32 bIsUsingGBuffers : 1;

	// (Mobile) True if the platform should write depth content back to memory.
	uint32 bKeepDepthContent : 1;

	// (Mobile) True if platform requires SceneDepthAux target
	uint32 bRequiresDepthAux : 1;
	
	// (Mobile) True if SceneDepthAux should use a precise pixel format
	uint32 bPreciseDepthAux : 1;
	
	// (Mobile) True if Custom MSAA resolve subpass is enabled
	uint32 bCustomResolveSubpass : 1;

	// (Mobile) True if CustomStencil are sampled in a shader
	uint32 bSamplesCustomStencil : 1;
	
	// (Mobile) True if MSAA targets can be memoryless
	uint32 bMemorylessMSAA : 1;

	// (XR) True if we can request an XR depth swapchain
	uint32 bSupportsXRTargetManagerDepthAlloc : 1;
    
    // True if we require an alpha channel for scene color
    bool bRequiresAlphaChannel = false;

	// (Mobile XR) Number of render target layers in mobile multi view.
	uint32 MobileMultiViewRenderTargetNumLayers = 2;

private:
	static ENGINE_API FSceneTexturesConfig GlobalInstance;
};
