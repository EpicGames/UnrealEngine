// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/RefCounting.h"
#include "Containers/List.h"

#include "PrimitiveDirtyState.h"
#include "PrimitiveComponentId.h"
#include "LightDefinitions.h"

#include "SceneTypes.generated.h"

class FLightMap;
class FSceneViewStateInterface;
class FShadowMap;

namespace ERHIFeatureLevel { enum Type : int; }

/** A reference to a light-map. */
typedef TRefCountPtr<FLightMap> FLightMapRef;

/** A reference to a shadow-map. */
typedef TRefCountPtr<FShadowMap> FShadowMapRef;

/** Custom primitive data payload. */
USTRUCT()
struct FCustomPrimitiveData
{
	GENERATED_USTRUCT_BODY()

	inline bool operator==(const FCustomPrimitiveData& Other) const
	{
		return Data == Other.Data;
	}

	static constexpr int32 NumCustomPrimitiveDataFloat4s = 9; // Must match NUM_CUSTOM_PRIMITIVE_DATA in SceneData.ush
	static constexpr int32 NumCustomPrimitiveDataFloats = NumCustomPrimitiveDataFloat4s * 4;

	UPROPERTY(EditAnywhere, Category=Rendering)
	TArray<float> Data;
};

/** 
 * Class used to reference an FSceneViewStateInterface that allows destruction and recreation of all FSceneViewStateInterface's when needed. 
 * This is used to support reloading the renderer module on the fly.
 */
class FSceneViewStateReference
{
public:
	FSceneViewStateReference()
	: Reference(nullptr), ShareOriginTarget(nullptr), ShareOriginRefCount(0)
	{
	}

	ENGINE_API virtual ~FSceneViewStateReference();

	/**
	 * Allocates the Scene view state.
	 */
	ENGINE_API void Allocate(ERHIFeatureLevel::Type FeatureLevel);

	/**
	  * Mark that a view state shares an origin with another view state, allowing sharing of some internal state, saving memory and performance.
	  * Typically used for cube map faces.  Must be called before "Allocate" is called on the source view state (best practice is to call
	  * immediately after creating the view state).  Multiple view states can point to the same shared origin (for example, the first face of a
	  * cube map), but sharing can't be nested.  Sharing view states must be destroyed before the Target is destroyed.
	  */
	ENGINE_API void ShareOrigin(FSceneViewStateReference* Target);

	/** Destorys the Scene view state. */
	ENGINE_API void Destroy();

	FSceneViewStateInterface* GetReference()
	{
		return Reference;
	}

private:
	FSceneViewStateInterface* Reference;
	TLinkedList<FSceneViewStateReference*> GlobalListLink;

	FSceneViewStateReference* ShareOriginTarget;

	/** Number of other view states that share this view state's origin. */
	int32 ShareOriginRefCount;

	static TLinkedList<FSceneViewStateReference*>*& GetSceneViewStateList();

	void AllocateInternal(ERHIFeatureLevel::Type FeatureLevel);
};

/** Different light component types. The enum uses values defined in a shared header with shader code so that the two sides are always consistent. */
enum ELightComponentType
{
	LightType_Directional = LIGHT_TYPE_DIRECTIONAL,
	LightType_Point		  = LIGHT_TYPE_POINT,
	LightType_Spot		  = LIGHT_TYPE_SPOT,
	LightType_Rect 		  = LIGHT_TYPE_RECT,
	LightType_MAX         = LIGHT_TYPE_MAX,
	LightType_NumBits = 2
};

static_assert(LightType_MAX <= (1 << LightType_NumBits), "LightType_NumBits is not large enough to hold all possible light types");

/**
 * The types of interactions between a light and a primitive.
 */
enum ELightMapInteractionType
{
	LMIT_None	= 0,
	LMIT_GlobalVolume = 1,
	LMIT_Texture = 2,

	LMIT_NumBits = 3
};

enum EShadowMapInteractionType
{
	SMIT_None = 0,
	SMIT_GlobalVolume = 1,
	SMIT_Texture = 2,

	SMIT_NumBits = 3
};

/** Quality levels that a material can be compiled for. */
namespace EMaterialQualityLevel
{
	enum Type : uint8
	{
		Low,
		High,
		Medium,
		Epic,
		Num
	};
}

ENGINE_API FString LexToString(EMaterialQualityLevel::Type QualityLevel);

//
//	EMaterialProperty
//
UENUM(BlueprintType)
enum EMaterialProperty : int
{
	MP_EmissiveColor = 0 UMETA(DisplayName = "Emissive"),
	MP_Opacity UMETA(DisplayName = "Opacity"),
	MP_OpacityMask UMETA(DisplayName = "Opacity Mask"),
	MP_DiffuseColor UMETA(Hidden),			// used in Lightmass, not exposed to user, computed from: BaseColor, Metallic				Also used in Substrate which uses Albedo/F0 parameterization
	MP_SpecularColor UMETA(Hidden),			// used in Lightmass, not exposed to user, derived from: SpecularColor, Metallic, Specular	Also used in Substrate which uses Albedo/F0 parameterization
	MP_BaseColor UMETA(DisplayName = "Diffuse"),
	MP_Metallic UMETA(DisplayName = "Metallic"),
	MP_Specular UMETA(DisplayName = "Specular"),
	MP_Roughness UMETA(DisplayName = "Roughness "),
	MP_Anisotropy UMETA(DisplayName = "Anisotropy"),
	MP_Normal UMETA(DisplayName = "Normal"),
	MP_Tangent UMETA(DisplayName = "Tangent"),
	MP_WorldPositionOffset UMETA(DisplayName = "World Position Offset"),
	MP_WorldDisplacement_DEPRECATED UMETA(Hidden),
	MP_TessellationMultiplier_DEPRECATED UMETA(Hidden),
	MP_SubsurfaceColor UMETA(DisplayName = "Subsurface"),
	MP_CustomData0 UMETA(Hidden),
	MP_CustomData1 UMETA(Hidden),
	MP_AmbientOcclusion UMETA(DisplayName = "Ambient Occlusion"),
	MP_Refraction UMETA(DisplayName = "Refraction"),
	MP_CustomizedUVs0 UMETA(Hidden),
	MP_CustomizedUVs1 UMETA(Hidden),
	MP_CustomizedUVs2 UMETA(Hidden),
	MP_CustomizedUVs3 UMETA(Hidden),
	MP_CustomizedUVs4 UMETA(Hidden),
	MP_CustomizedUVs5 UMETA(Hidden),
	MP_CustomizedUVs6 UMETA(Hidden),
	MP_CustomizedUVs7 UMETA(Hidden),
	MP_PixelDepthOffset UMETA(Hidden),
	MP_ShadingModel UMETA(Hidden),
	MP_FrontMaterial UMETA(DisplayName = "Front Material"),
	MP_SurfaceThickness UMETA(Hidden),
	MP_Displacement UMETA(Hidden),

	//^^^ New material properties go above here ^^^^
	MP_MaterialAttributes,
	MP_CustomOutput UMETA(Hidden),
	MP_MAX UMETA(DisplayName = "None"),

	// Aliases
	MP_LastCustomizedUVs = MP_CustomizedUVs7,
	MP_NumCustomizedUVs = MP_LastCustomizedUVs - MP_CustomizedUVs0 + 1,
};

/** Blend modes supported for simple element rendering */
enum ESimpleElementBlendMode : int
{
	SE_BLEND_Opaque = 0,
	SE_BLEND_Masked,
	SE_BLEND_Translucent,
	SE_BLEND_Additive,
	SE_BLEND_Modulate,
	SE_BLEND_MaskedDistanceField,
	SE_BLEND_MaskedDistanceFieldShadowed,
	SE_BLEND_TranslucentDistanceField,
	SE_BLEND_TranslucentDistanceFieldShadowed,
	SE_BLEND_AlphaComposite,
	SE_BLEND_AlphaHoldout,
	// Like SE_BLEND_Translucent, but modifies destination alpha
	SE_BLEND_AlphaBlend,
	// Like SE_BLEND_Translucent, but reads from an alpha-only texture
	SE_BLEND_TranslucentAlphaOnly,
	SE_BLEND_TranslucentAlphaOnlyWriteAlpha,

	SE_BLEND_RGBA_MASK_START,
	SE_BLEND_RGBA_MASK_END = SE_BLEND_RGBA_MASK_START + 31, //Using 5bit bit-field for red, green, blue, alpha and desaturation

	SE_BLEND_MAX
};


/** Setting to control shadow invalidation behavior (in particular with respect to Virtual Shadow Maps and future methods). */
UENUM()
enum class EShadowCacheInvalidationBehavior : uint8
{
	/** Default. Invalidates based on World Position Offset material, and transform changes, attempting to do the right thing given the information available to the renderer. */
	Auto,
	/** Always invalidate shadows, can be used to flag a primitive that is using some method of animating that is not known to the system. */
	Always,
	/** Suppresses invalidations that would otherwise be generated by e.g., World Position Offset (WPO). Can be used for example when they use WPO statically used, or the artifacts are subtle enough to not warrant the performace overhead (use with care). */
	Rigid,
	/** 
	 * In addition to "Rigid" behavior, also suppress invalidations due to transform changes. Add/Remove will still trigger invalidations. 
	 * If the primitive is actually moved or animated somehow the visual result is undefined.
	 */
	Static,
};

/**
 * This struct captures summary information about material features in the primitive
 */
struct FPrimitiveMaterialPropertyDescriptor
{
	FVector2f MinMaxMaterialDisplacement = FVector2f::ZeroVector;
	float MaxWorldPositionOffsetDisplacement = 0.0f;
	bool bAnyMaterialHasWorldPositionOffset = false;
	bool bAnyMaterialHasPixelAnimation = false;
	bool bAnyMaterialHasPerInstanceCustomData = false;
	bool bAnyMaterialHasPerInstanceRandom = false;
};
