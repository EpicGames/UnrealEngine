// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This file contains the various draw mesh macros that display draw calls
 * inside of PIX.
 */

// Colors that are defined for a particular mesh type
// Each event type will be displayed using the defined color
#pragma once

#include "UObject/ObjectMacros.h"
#include "RHIFeatureLevel.h"

#include "SceneUtils.generated.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogSceneUtils, Log, All);

class FRHICommandListImmediate;
enum EShaderPlatform : uint16;

enum class EShadingPath
{
	Mobile,
	Deferred,
	Num,
};

inline EShadingPath GetFeatureLevelShadingPath(FStaticFeatureLevel InFeatureLevel)
{
	return (InFeatureLevel >= ERHIFeatureLevel::SM5) ? EShadingPath::Deferred : EShadingPath::Mobile;
}

enum class EMobileHDRMode
{
	Unset,
	Disabled,
	EnabledFloat16,
};

/** Used by rendering project settings. */
UENUM()
enum EAntiAliasingMethod : int
{
	AAM_None UMETA(DisplayName = "None"),
	AAM_FXAA UMETA(DisplayName = "Fast Approximate Anti-Aliasing (FXAA)"),
	AAM_TemporalAA UMETA(DisplayName = "Temporal Anti-Aliasing (TAA)"),
	/** Only supported with forward shading.  MSAA sample count is controlled by r.MSAACount. */
	AAM_MSAA UMETA(DisplayName = "Multisample Anti-Aliasing (MSAA)"),
	AAM_TSR UMETA(DisplayName = "Temporal Super-Resolution (TSR)"),
	AAM_SMAA UMETA(DisplayName = "Subpixel Morphological Anti-Aliasing (SMAA)"),
	AAM_MAX,
};

/** Returns whether the anti-aliasing method use a temporal accumulation */
inline bool IsTemporalAccumulationBasedMethod(EAntiAliasingMethod AntiAliasingMethod)
{
	return AntiAliasingMethod == AAM_TemporalAA || AntiAliasingMethod == AAM_TSR;
}

/** True if Alpha Propagate is enabled for the mobile renderer. */
ENGINE_API bool IsMobilePropagateAlphaEnabled(EShaderPlatform Platform);
ENGINE_API bool IsMobileTonemapSubpassEnabled(EShaderPlatform Platform, bool bMultiViewRendering);
ENGINE_API bool IsMobileTonemapSubpassEnabledInline(EShaderPlatform Platform, bool bMultiViewRendering, uint32 NumMSAASamples);
ENGINE_API bool IsMobileColorsRGB();

ENGINE_API EAntiAliasingMethod GetDefaultAntiAliasingMethod(const FStaticFeatureLevel InFeatureLevel);

ENGINE_API const TCHAR* GetShortAntiAliasingName(EAntiAliasingMethod AntiAliasingMethod);

ENGINE_API uint32 GetDefaultMSAACount(const FStaticFeatureLevel InFeatureLevel, uint32 PlatformMaxSampleCount = 8);

enum class ECustomDepthMode : uint8
{
	// Custom depth is disabled.
	Disabled,

	// Custom depth is enabled.
	Enabled,

	// Custom depth is enabled and uses stencil.
	EnabledWithStencil,
};

ENGINE_API extern ECustomDepthMode GetCustomDepthMode();

inline bool IsCustomDepthPassEnabled()
{
	return GetCustomDepthMode() != ECustomDepthMode::Disabled;
}

// Callback for calling one action (typical use case: delay a clear until it's actually needed)
class FDelayedRendererAction
{
public:
	typedef void (TDelayedFunction)(FRHICommandListImmediate& RHICommandList, void* UserData);

	FDelayedRendererAction()
		: Function(nullptr)
		, UserData(nullptr)
		, bFunctionCalled(false)
	{
	}

	FDelayedRendererAction(TDelayedFunction* InFunction, void* InUserData)
		: Function(InFunction)
		, UserData(InUserData)
		, bFunctionCalled(false)
	{
	}

	inline void SetDelayedFunction(TDelayedFunction* InFunction, void* InUserData)
	{
		check(!bFunctionCalled);
		check(!Function);
		Function = InFunction;
		UserData = InUserData;
	}

	inline bool HasDelayedFunction() const
	{
		return Function != nullptr;
	}

	inline void RunFunctionOnce(FRHICommandListImmediate& RHICommandList)
	{
		if (!bFunctionCalled)
		{
			if (Function)
			{
				Function(RHICommandList, UserData);
			}
			bFunctionCalled = true;
		}
	}

	inline bool HasBeenCalled() const
	{
		return bFunctionCalled;
	}

protected:
	TDelayedFunction* Function;
	void* UserData;
	bool bFunctionCalled;
};
