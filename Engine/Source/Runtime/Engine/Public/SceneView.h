// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "ConvexVolume.h"
#include "DebugViewModeHelpers.h"
#include "DynamicRenderScaling.h"
#include "EngineDefines.h"
#include "FinalPostProcessSettings.h"
#include "GameTime.h"
#include "GlobalDistanceFieldConstants.h"
#include "Interfaces/Interface_PostProcessVolume.h"
#include "Math/MirrorMatrix.h"
#include "PrimitiveComponentId.h"
#include "RenderResource.h"
#include "SceneViewOwner.h"
#include "ShowFlags.h"
#include "StereoRendering.h"
#include "StereoRenderUtils.h"
#include "ShaderParameterMacros.h"

#define MAX_PHYSICS_FIELD_TARGETS 32

class FSceneView;
class FSceneInterface;
class FSceneViewFamily;
class FSceneViewStateInterface;
class FViewElementDrawer;
class ISceneViewExtension;
class FSceneViewFamily;
class FVolumetricFogViewResources;
class ISceneRenderer;
class ISpatialUpscaler;
struct FMinimalViewInfo;
namespace UE::RenderViewportFeedback
{
	class FCollector;
}

namespace UE::Renderer::Private
{

class ITemporalUpscaler;
class IMegaFilters;
}

class FRenderTarget;

// Projection data for a FSceneView
struct FSceneViewProjectionData
{
	/** The view origin. */
	FVector ViewOrigin = FVector::ZeroVector;

	/** Rotation matrix transforming from world space to view space. */
	FMatrix ViewRotationMatrix = FMatrix::Identity;

	/** UE projection matrix projects such that clip space Z=1 is the near plane, and Z=0 is the infinite far plane. */
	FMatrix ProjectionMatrix = FMatrix::Identity;

	//The unconstrained (no aspect ratio bars applied) view rectangle (also unscaled)
	FIntRect ViewRect = FIntRect(0,0,0,0);

	//The vector (including distance) from the camera to it's viewtarget, if set. Primarily only used for Ortho views.
	FVector CameraToViewTarget = FVector::ZeroVector;

protected:
	// The constrained view rectangle (identical to UnconstrainedUnscaledViewRect if aspect ratio is not constrained)
	FIntRect ConstrainedViewRect = FIntRect(0,0,0,0);

public:
	void SetViewRectangle(const FIntRect& InViewRect)
	{
		ViewRect = InViewRect;
		ConstrainedViewRect = InViewRect;
	}

	void SetConstrainedViewRectangle(const FIntRect& InViewRect)
	{
		ConstrainedViewRect = InViewRect;
	}

	bool IsValidViewRectangle() const
	{
		return (ConstrainedViewRect.Min.X >= 0) &&
			(ConstrainedViewRect.Min.Y >= 0) &&
			(ConstrainedViewRect.Width() > 0) &&
			(ConstrainedViewRect.Height() > 0);
	}

	bool IsPerspectiveProjection() const
	{
		return ProjectionMatrix.M[3][3] < 1.0f;
	}

	const FIntRect& GetViewRect() const { return ViewRect; }
	const FIntRect& GetConstrainedViewRect() const { return ConstrainedViewRect; }

	FMatrix ComputeViewProjectionMatrix() const
	{
		return FTranslationMatrix(-ViewOrigin) * ViewRotationMatrix * ProjectionMatrix;
	}

	//Function for retrieving the NearPlane from the existing projection matrix
	static float GetNearPlaneFromProjectionMatrix(const FMatrix& ProjectionMatrix)
	{
		if (ProjectionMatrix.M[3][3] < 1.0f)
		{
			// Infinite projection with reversed Z.
			return static_cast<float>(ProjectionMatrix.M[3][2]);
		}
		else
		{
			// Ortho projection with reversed Z.
			return static_cast<float>((1.0f - ProjectionMatrix.M[3][2]) / (ProjectionMatrix.M[2][2] == 0.0f ? UE_DELTA : ProjectionMatrix.M[2][2]));
		}
	}

	float GetNearPlaneFromProjectionMatrix() const
	{
		return GetNearPlaneFromProjectionMatrix(ProjectionMatrix);
	}

	// Function for correcting Ortho camera near plane locations to avoid artifacts behind camera view origin
	static ENGINE_API bool UpdateOrthoPlanes(FSceneViewProjectionData* InOutProjectionData, float& NearPlane, float& FarPlane, float HalfOrthoWidth, bool bUseCameraHeightAsViewTarget);
	inline bool UpdateOrthoPlanes(float& NearPlane, float& FarPlane, float HalfOrthoWidth, bool bUseCameraHeightAsViewTarget)
	{
		return UpdateOrthoPlanes(this, NearPlane, FarPlane, HalfOrthoWidth, bUseCameraHeightAsViewTarget);
	}
	ENGINE_API bool UpdateOrthoPlanes(FMinimalViewInfo& MinimalViewInfo);
	ENGINE_API bool UpdateOrthoPlanes(bool bUseCameraHeightAsViewTarget);
};

/** Method used for primary screen percentage method. */
enum class EPrimaryScreenPercentageMethod
{
	// Add spatial upscale pass at the end of post processing chain, before the secondary upscale.
	SpatialUpscale,

	// Let temporal AA's do the upscale.
	TemporalUpscale,

	// No upscaling or up sampling, just output the view rect smaller.
	// This is useful for VR's render thread dynamic resolution with MSAA.
	RawOutput,
};

/**
 * Method used for second screen percentage method, that is a second spatial upscale pass at the
 * very end, independent of screen percentage show flag.
 */
enum class ESecondaryScreenPercentageMethod
{
	// Helpful to work on aliasing issue on HighDPI monitors.
	NearestSpatialUpscale,

	// Upscale to simulate smaller pixel density on HighDPI monitors.
	LowerPixelDensitySimulation,

	// TODO: Same config as primary upscale?
};

struct FFirstPersonParameters
{
	/** FOV correction factor applied to the first person transform used on primitives tagged as "IsFirstPerson". This should be computed as tan(SceneFOVRadians * 0.5) / tan(FirstPersonFOVRadians * 0.5). */
	float FOVCorrectionFactor = 1.0f;

	/** The scale to apply to primitives tagged as "IsFirstPerson". This is used to scale down primitives towards the camera such that they are small enough not to intersect with the scene. */
	float Scale = 1.0f;

	/** If bUseParameters is true, FOV and Scale should be applied to primitives tagged as "IsFirstPerson". */
	bool bUseParameters = false;

	FFirstPersonParameters() = default;
	FFirstPersonParameters(float InFOVCorrectionFactor, float InScale, bool bInUseParameters) : FOVCorrectionFactor(InFOVCorrectionFactor), Scale(InScale), bUseParameters(bInUseParameters) {}
};

// Construction parameters for a FSceneView
struct FSceneViewInitOptions : public FSceneViewProjectionData
{
	const FSceneViewFamily* ViewFamily;
	FSceneViewStateInterface* SceneViewStateInterface;
	FSceneViewStateInterface* ExposureSceneViewStateInterface;
	FSceneViewOwner ViewActor; // TODO: Rename to ViewOwner
	int32 PlayerIndex;
	FViewElementDrawer* ViewElementDrawer;

	FLinearColor BackgroundColor;
	FLinearColor OverlayColor;
	FLinearColor ColorScale;
	FLinearColor SkylightScale;

	/** For stereoscopic rendering, whether or not this is a full pass, or a primary / secondary pass */
	EStereoscopicPass StereoPass;

	/** For stereoscopic rendering, a unique index to identify the view across view families */
	int32 StereoViewIndex;

	/** Conversion from world units (uu) to meters, so we can scale motion to the world appropriately */
	float WorldToMetersScale;

	/** The view origin and rotation without any stereo offsets applied to it */
	FVector ViewLocation;
	FRotator ViewRotation;

	TSet<FPrimitiveComponentId> HiddenPrimitives;

	/** The primitives which are visible for this view. If the array is not empty, all other primitives will be hidden. */
	TOptional<TSet<FPrimitiveComponentId>> ShowOnlyPrimitives;

	// -1,-1 if not setup
	FIntPoint CursorPos;

	float LODDistanceFactor;

	/** If > 0, overrides the view's far clipping plane with a plane at the specified distance. */
	float OverrideFarClippingPlaneDistance;

	/** World origin offset value. Non-zero only for a single frame when origin is rebased */
	FVector OriginOffsetThisFrame;

	/** Was there a camera cut this frame? */
	bool bInCameraCut;

	/** Whether to use FOV when computing mesh LOD. */
	bool bUseFieldOfViewForLOD;

	/** Actual field of view and that desired by the camera originally */
	float FOV;
	float DesiredFOV;

	/** Parameters controlling the rendering (FOV and scale) of first person primitives. */
	FFirstPersonParameters FirstPersonParams;

	/** Whether this view is being used to render a scene capture. */
	bool bIsSceneCapture;

	/** Whether the scene capture is a cube map (bIsSceneCapture will also be set). */
	bool bIsSceneCaptureCube;

	/** Whether this view uses ray tracing, for views that are used to render a scene capture. */
	bool bSceneCaptureUsesRayTracing;

	/** Whether this view is being used to render a reflection capture. */
	bool bIsReflectionCapture;

	/** Whether this view is being used to render a planar reflection. */
	bool bIsPlanarReflection;

	/** Whether this view should be excluded from extent history tracking. */
	bool bExcludeFromSceneTextureExtents = false;

	/** If > 0, overrides the view's resolution fraction. */
	float OverridePrimaryResolutionFraction;

	/** Resolution fraction that scales with the amount of overscan in the view */
	float OverscanResolutionFraction = 1.0f;

	/** Fraction of the view to crop to during the secondary upscale pass, with 1.0 meaning no crop */
	float CropFraction = 1.0f;

	/**
	 * The fraction for each edge between 0.0 and 1.0 of the view to crop to during the final post process upscale, with 1.0 meaning no crop.
	 * By convention, X is the left edge, Y is the right edge, Z is the top edge, and W is the bottom edge. Stacks with uniform CropFraction.
	 */
	FVector4f AsymmetricCropFraction = FVector4f::One();
	
#if WITH_EDITOR
	/** default to 0'th view index, which is a bitfield of 1 */
	uint64 EditorViewBitflag;

	/** Whether game screen percentage should be disabled. */
	bool bDisableGameScreenPercentage;
#endif

	FSceneViewInitOptions()
		: ViewFamily(NULL)
		, SceneViewStateInterface(NULL)
		, ExposureSceneViewStateInterface(NULL)
		, PlayerIndex(INDEX_NONE)
		, ViewElementDrawer(NULL)
		, BackgroundColor(FLinearColor::Transparent)
		, OverlayColor(FLinearColor::Transparent)
		, ColorScale(FLinearColor::White)
		, SkylightScale(FLinearColor::White)
		, StereoPass(EStereoscopicPass::eSSP_FULL)
		, StereoViewIndex(INDEX_NONE)
		, WorldToMetersScale(100.f)
		, ViewLocation(ForceInitToZero)
		, ViewRotation(ForceInitToZero)
		, CursorPos(-1, -1)
		, LODDistanceFactor(1.0f)
		, OverrideFarClippingPlaneDistance(-1.0f)
		, OriginOffsetThisFrame(ForceInitToZero)
		, bInCameraCut(false)
		, bUseFieldOfViewForLOD(true)
		, FOV(90.f)
		, DesiredFOV(90.f)
		, FirstPersonParams()
		, bIsSceneCapture(false)
		, bIsSceneCaptureCube(false)
		, bSceneCaptureUsesRayTracing(false)
		, bIsReflectionCapture(false)
		, bIsPlanarReflection(false)
		, OverridePrimaryResolutionFraction(-1.0)
#if WITH_EDITOR
		, EditorViewBitflag(1)
		, bDisableGameScreenPercentage(false)
		//@TODO: , const TBitArray<>& InSpriteCategoryVisibility=TBitArray<>()
#endif
	{
	}
};


//////////////////////////////////////////////////////////////////////////

struct FViewMatrices
{
	struct FMinimalInitializer
	{
		FMatrix ViewRotationMatrix = FMatrix::Identity;
		FMatrix ProjectionMatrix = FMatrix::Identity; // ViewToClip
		FVector ViewOrigin = FVector::ZeroVector;
		FIntRect ConstrainedViewRect = FIntRect(0, 0, 0, 0);
		FVector CameraToViewTarget = FVector::ZeroVector;
		EStereoscopicPass StereoPass = EStereoscopicPass::eSSP_FULL;
		FFirstPersonParameters FirstPersonParams = {};
	};

	FViewMatrices()
	{
		ViewToClip.SetIdentity();
		ViewToClipNoAA.SetIdentity();
		ClipToView.SetIdentity();
		WorldToView.SetIdentity();
		ViewToWorld.SetIdentity();
		WorldToClip.SetIdentity();
		ClipToWorld.SetIdentity();
		HMDViewMatrixNoRoll.SetIdentity();
		TranslatedViewMatrix.SetIdentity();
		InvTranslatedViewMatrix.SetIdentity();
		OverriddenTranslatedViewMatrix.SetIdentity();
		OverriddenInvTranslatedViewMatrix.SetIdentity();
		TranslatedWorldToClip.SetIdentity();
		ClipToTranslatedWorld.SetIdentity();
		ScreenToClipMatrix.SetIdentity();
		WorldToFirstPerson.SetIdentity();
		FirstPersonToWorld.SetIdentity();
		PreViewTranslation = FVector::ZeroVector;
		ViewOrigin = FVector::ZeroVector;
		CameraToViewTarget = FVector::ZeroVector;
		ProjectionScale = FVector2D::ZeroVector;
		TemporalAAProjectionJitter = FVector2D::ZeroVector;
		ScreenScale = 1.f;
	}

	ENGINE_API FViewMatrices(const FMinimalInitializer& Initializer);

	ENGINE_API FViewMatrices(const FSceneViewInitOptions& InitOptions);

private:
	void Init(const FMinimalInitializer& Initializer);

	/** ProjectionMatrix : UE projection matrix projects such that clip space Z=1 is the near plane, and Z=0 is the infinite far plane. */
	FMatrix		ViewToClip;
	/** ProjectionNoAAMatrix : UE projection matrix projects such that clip space Z=1 is the near plane, and Z=0 is the infinite far plane. Don't apply any AA jitter */
	FMatrix		ViewToClipNoAA;
	/** InvProjectionMatrix : UE projection matrix projects such that clip space Z=1 is the near plane, and Z=0 is the infinite far plane. */
	FMatrix		ClipToView;
	// ViewMatrix..
	FMatrix		WorldToView;
	// InvViewMatrix..
	FMatrix		ViewToWorld;
	// ViewProjectionMatrix : UE projection matrix projects such that clip space Z=1 is the near plane, and Z=0 is the infinite far plane. */
	FMatrix		WorldToClip;
	// InvViewProjectionMatrix : UE projection matrix projects such that clip space Z=1 is the near plane, and Z=0 is the infinite far plane. */
	FMatrix		ClipToWorld;
	// HMD WorldToView with roll removed
	FMatrix		HMDViewMatrixNoRoll;
	/** WorldToView with PreViewTranslation. */
	FMatrix		TranslatedViewMatrix;
	/** ViewToWorld with PreViewTranslation. */
	FMatrix		InvTranslatedViewMatrix;
	/** WorldToView with PreViewTranslation. */
	FMatrix		OverriddenTranslatedViewMatrix;
	/** ViewToWorld with PreViewTranslation. */
	FMatrix		OverriddenInvTranslatedViewMatrix;
	/** TranslatedViewProjectionMatrix: The view-projection transform, starting from world-space points translated by -ViewOrigin. */
	FMatrix		TranslatedWorldToClip;
	/** InvTranslatedViewProjectionMatrix: The inverse view-projection transform, ending with world-space points translated by -ViewOrigin. */
	FMatrix		ClipToTranslatedWorld;
	/** The screen to clip matrix (defined depending on whether this is a perspective or ortho projection view)*/
	FMatrix		ScreenToClipMatrix;
	/** Scale and shear to avoid first person primitives to clip with the scene and to achieve a first person specific FOV. Transforms from world space to first person "space". */
	FMatrix		WorldToFirstPerson;
	/** Inverse FirstPersonTransform. Transforms from first person "space" to world space. */
	FMatrix		FirstPersonToWorld;
	/** The translation to apply to the world before TranslatedViewProjectionMatrix. Usually it is -ViewOrigin but with rereflections this can differ */
	FVector		PreViewTranslation;
	/** The camera/viewport location in world space */
	FVector		ViewOrigin;
	/** The current view target's location proportional to the camera location */
	FVector		CameraToViewTarget;
	/** Scale applied by the projection matrix in X and Y. */
	FVector2D	ProjectionScale;
	/** TemporalAA jitter offset currently stored in the projection matrix */
	FVector2D	TemporalAAProjectionJitter;

	/**
	 * Scale factor to use when computing the size of a sphere in pixels.
	 * 
	 * A common calculation is to determine the size of a sphere in pixels when projected on the screen:
	 *		ScreenRadius = max(0.5 * ViewSizeX * ProjMatrix[0][0], 0.5 * ViewSizeY * ProjMatrix[1][1]) * SphereRadius / ProjectedSpherePosition.W
	 * Instead you can now simply use:
	 *		ScreenRadius = ScreenScale * SphereRadius / ProjectedSpherePosition.W
	 */
	float ScreenScale;

	/** Depth test scaling; differs between perspective and orthographic  */
	float PerProjectionDepthThicknessScale;

	/** Parameters controlling the rendering (FOV and scale) of first person primitives. */
	FFirstPersonParameters FirstPersonParams;

	//
	// World = TranslatedWorld - PreViewTranslation
	// TranslatedWorld = World + PreViewTranslation
	// 

	// ----------------

public:
	ENGINE_API void UpdateViewMatrix(const FVector& ViewLocation, const FRotator& ViewRotation);

	void UpdatePlanarReflectionViewMatrix(const FSceneView& SourceView, const FMirrorMatrix& MirrorMatrix);

	const FMatrix& GetViewToClip() const
	{
		return ViewToClip;
	}

	const FMatrix& GetViewToClipNoAA() const
	{
		return ViewToClipNoAA;
	}

	const FMatrix& GetClipToView() const
	{
		return ClipToView;
	}

	const FMatrix& GetWorldToView() const
	{
		return WorldToView;
	}

	const FMatrix& GetViewToWorld() const
	{
		return ViewToWorld;
	}

	const FMatrix& GetWorldToClip() const
	{
		return WorldToClip;
	}

	const FMatrix& GetClipToWorld() const
	{
		return ClipToWorld;
	}
	
	inline const FMatrix& GetHMDViewMatrixNoRoll() const
	{
		return HMDViewMatrixNoRoll;
	}
	
	inline const FMatrix& GetTranslatedViewMatrix() const
	{
		return TranslatedViewMatrix;
	}

	inline const FMatrix& GetInvTranslatedViewMatrix() const
	{
		return InvTranslatedViewMatrix;
	}

	inline const FMatrix& GetOverriddenTranslatedViewMatrix() const
	{
		return OverriddenTranslatedViewMatrix;
	}

	inline const FMatrix& GetOverriddenInvTranslatedViewMatrix() const
	{
		return OverriddenInvTranslatedViewMatrix;
	}

	const FMatrix& GetTranslatedWorldToClip() const
	{
		return TranslatedWorldToClip;
	}

	const FMatrix& GetClipToTranslatedWorld() const
	{
		return ClipToTranslatedWorld;
	}

	inline const FMatrix& GetScreenToClipMatrix() const
	{
		return ScreenToClipMatrix;
	}

	const FMatrix& GetWorldToFirstPerson() const
	{
		return WorldToFirstPerson;
	}

	const FMatrix& GetFirstPersonToWorld() const
	{
		return FirstPersonToWorld;
	}

	UE_DEPRECATED(5.8, "Use GetViewToClip() instead")
	inline const FMatrix& GetProjectionMatrix()						const { return GetViewToClip(); }	
	UE_DEPRECATED(5.8, "Use GetViewToClipNoAA() instead")
	inline const FMatrix& GetProjectionNoAAMatrix() 				const { return GetViewToClipNoAA(); }	
	UE_DEPRECATED(5.8, "Use GetClipToView() instead")
	inline const FMatrix& GetInvProjectionMatrix() 					const { return GetClipToView(); }	
	UE_DEPRECATED(5.8, "Use GetWorldToView() instead")
	inline const FMatrix& GetViewMatrix() 							const { return GetWorldToView(); }	
	UE_DEPRECATED(5.8, "Use GetViewToWorld() instead")
	inline const FMatrix& GetInvViewMatrix() 						const { return GetViewToWorld(); }
	UE_DEPRECATED(5.8, "Use GetWorldToClip() instead")
	inline const FMatrix& GetViewProjectionMatrix() 				const { return GetWorldToClip(); }
	UE_DEPRECATED(5.8, "Use GetClipToWorld() instead")
	inline const FMatrix& GetInvViewProjectionMatrix() 				const { return GetClipToWorld(); }
	UE_DEPRECATED(5.8, "Use GetTranslatedWorldToClip() instead")
	inline const FMatrix& GetTranslatedViewProjectionMatrix() 		const { return GetTranslatedWorldToClip(); }
	UE_DEPRECATED(5.8, "Use GetClipToTranslatedWorld() instead")
	inline const FMatrix& GetInvTranslatedViewProjectionMatrix()	const { return GetClipToTranslatedWorld(); }
	UE_DEPRECATED(5.8, "Use GetWorldToFirstPerson() instead")
	inline const FMatrix& GetFirstPersonTransform() 				const { return GetWorldToFirstPerson(); }
	UE_DEPRECATED(5.8, "Use GetFirstPersonToWorld() instead")
	inline const FMatrix& GetInvFirstPersonTransform() 				const { return GetFirstPersonToWorld(); }

	inline float GetFirstPersonScale() const
	{
		return FirstPersonParams.bUseParameters ? FirstPersonParams.Scale : 1.0f;
	}

	inline const FVector& GetPreViewTranslation() const
	{
		return PreViewTranslation;
	}
	
	inline const FVector& GetViewOrigin() const
	{
		return ViewOrigin;
	}

	inline const FVector& GetCameraToViewTarget() const
	{
		return CameraToViewTarget;
	}

	inline float GetScreenScale() const
	{
		return ScreenScale;
	}

	inline const FVector2D& GetProjectionScale() const
	{
		return ProjectionScale;
	} 

	/** @return true:perspective, false:orthographic */
	inline bool IsPerspectiveProjection() const
	{
		return ViewToClip.M[3][3] < 1.0f;
	}

	inline FVector2f GetInvTanHalfFov() const
	{
		//No concept of FOV for orthographic projection so only return perspective related values or 1.0f
		if (IsPerspectiveProjection())
		{
			return FVector2f(static_cast<float>(ViewToClip.M[0][0]), static_cast<float>(ViewToClip.M[1][1]));
		}
		return  FVector2f(1.0f, 1.0f);
	}

	inline FVector2f GetTanHalfFov() const
	{
		//No concept of FOV for orthographic projection so only return perspective related values or 1.0f
		if (IsPerspectiveProjection())
		{
			return FVector2f(static_cast<float>(ClipToView.M[0][0]), static_cast<float>(ClipToView.M[1][1]));
		}
		return  FVector2f(1.0f,1.0f);
	}

	//Used for initializing the View Uniform Buffer TanAndInvTanHalfFOV variable without repeated calls to IsPerspectiveProjection
	inline FVector4f GetTanAndInvTanHalfFOV() const
	{
		//No concept of FOV for orthographic projection so only return perspective related values or 1.0f
		if (IsPerspectiveProjection())
		{
			return FVector4f(static_cast<float>(ClipToView.M[0][0]), //ClipToView[0][0] - X axis
							 static_cast<float>(ClipToView.M[1][1]), //ClipToView[1][1] - Y axis
							 static_cast<float>(ViewToClip.M[0][0]), //ViewToClip[0][0] - 1/X axis
							 static_cast<float>(ViewToClip.M[1][1])); //ViewToClip[1][1] - 1/Y axis
		}
		return FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	}

	inline float GetPerProjectionDepthThicknessScale() const
	{
		return PerProjectionDepthThicknessScale;
	}

	FMatrix ScreenToClipProjectionMatrix() const;

	ENGINE_API void HackOverrideViewMatrixForShadows(const FMatrix& InViewMatrix);

	void SaveProjectionNoAAMatrix()
	{
		ViewToClipNoAA = ViewToClip;
	}

	void HackAddTemporalAAProjectionJitter(const FVector2D& InTemporalAAProjectionJitter)
	{
		ensure(TemporalAAProjectionJitter.X == 0.0f && TemporalAAProjectionJitter.Y == 0.0f);

		TemporalAAProjectionJitter = InTemporalAAProjectionJitter;

		if (IsPerspectiveProjection())
		{
			ViewToClip.M[2][0] += TemporalAAProjectionJitter.X;
			ViewToClip.M[2][1] += TemporalAAProjectionJitter.Y;
		}
		else
		{
			ViewToClip.M[3][0] += TemporalAAProjectionJitter.X;
			ViewToClip.M[3][1] += TemporalAAProjectionJitter.Y;
		}
		ClipToView = InvertProjectionMatrix(ViewToClip);

		RecomputeDerivedMatrices();
	}

	void HackRemoveTemporalAAProjectionJitter()
	{
		if (IsPerspectiveProjection())
		{
			ViewToClip.M[2][0] -= TemporalAAProjectionJitter.X;
			ViewToClip.M[2][1] -= TemporalAAProjectionJitter.Y;
		}
		else
		{
			ViewToClip.M[3][0] -= TemporalAAProjectionJitter.X;
			ViewToClip.M[3][1] -= TemporalAAProjectionJitter.Y;
		}
		ClipToView = InvertProjectionMatrix(ViewToClip);

		TemporalAAProjectionJitter = FVector2D::ZeroVector;
		RecomputeDerivedMatrices();
	}

	const FMatrix ComputeProjectionNoAAMatrix() const
	{
		FMatrix ProjNoAAMatrix = ViewToClip;

		if (IsPerspectiveProjection())
		{
			ProjNoAAMatrix.M[2][0] -= TemporalAAProjectionJitter.X;
			ProjNoAAMatrix.M[2][1] -= TemporalAAProjectionJitter.Y;
		}
		else
		{
			ProjNoAAMatrix.M[3][0] -= TemporalAAProjectionJitter.X;
			ProjNoAAMatrix.M[3][1] -= TemporalAAProjectionJitter.Y;
		}

		return ProjNoAAMatrix;
	}

	inline const FVector2D GetTemporalAAJitter() const
	{
		return TemporalAAProjectionJitter;
	}

	const FMatrix ComputeViewRotationProjectionMatrix() const
	{
		return WorldToView.RemoveTranslation() * ViewToClip;
	}
	
	const FMatrix ComputeInvProjectionNoAAMatrix() const
	{
		return InvertProjectionMatrix( ComputeProjectionNoAAMatrix() );
	}

	// @return in radians (horizontal,vertical)
	const FVector2D ComputeHalfFieldOfViewPerAxis() const
	{
		if(IsPerspectiveProjection())
		{
			const FMatrix ClipToViewNoAA = ComputeInvProjectionNoAAMatrix();

			FVector VCenter = FVector(ClipToViewNoAA.TransformPosition(FVector(0.0, 0.0, 0.0)));
			FVector VUp = FVector(ClipToViewNoAA.TransformPosition(FVector(0.0, 1.0, 0.0)));
			FVector VRight = FVector(ClipToViewNoAA.TransformPosition(FVector(1.0, 0.0, 0.0)));

			VCenter.Normalize();
			VUp.Normalize();
			VRight.Normalize();

			using ResultType = decltype(FVector2D::X);
			return FVector2D((ResultType)FMath::Acos(VCenter | VRight), (ResultType)FMath::Acos(VCenter | VUp));
		}
		return FVector2D::Zero();
	}

	FMatrix::FReal ComputeNearPlane() const
	{
		return ( ViewToClip.M[3][3] - ViewToClip.M[3][2] ) / ( ViewToClip.M[2][2] - ViewToClip.M[2][3] );
	}

	FMatrix::FReal ComputeOrthoFarPlane() const
	{
		if(!IsPerspectiveProjection())
		{
			return ComputeNearPlane() - ClipToView.M[2][2];
		}
		return 0.0;
	}

	void ApplyWorldOffset(const FVector& InOffset)
	{
		ViewOrigin+= InOffset;
		PreViewTranslation-= InOffset;
	
		WorldToView.SetOrigin(WorldToView.GetOrigin() + WorldToView.TransformVector(-InOffset));
		ViewToWorld.SetOrigin(ViewOrigin);
		RecomputeDerivedMatrices();
	}

	FVector2f GetOrthoDimensions() const
	{
		if (!IsPerspectiveProjection())
		{
			return  FVector2f(static_cast<float>(ClipToView.M[0][0]) * 2.0f, static_cast<float>(ClipToView.M[1][1]) * 2.0f);
		}
		return FVector2f::ZeroVector;
	}

	// @return in radians (horizontal,vertical)
	FVector2f GetFirstPersonHalfFieldOfViewPerAxis() const
	{
		if (IsPerspectiveProjection())
		{
			const FVector4f FirstPersonTandAndInvTanHalfFOV = GetFirstPersonTanAndInvTanHalfFOV();
			const FVector2f FirstPersonHalfFOV = FVector2f(FMath::Atan(FirstPersonTandAndInvTanHalfFOV.X), FMath::Atan(FirstPersonTandAndInvTanHalfFOV.Y));
			return FirstPersonHalfFOV;
		}
		return FVector2f::Zero();
	}

	FVector4f GetFirstPersonTanAndInvTanHalfFOV() const
	{
		// No concept of FOV for orthographic projection so only return perspective related values or 1.0f
		if (IsPerspectiveProjection())
		{
			const FVector4f SceneTanAndInvTanHalfFOV = GetTanAndInvTanHalfFOV();
			const float FOVCorrectionFactor = FirstPersonParams.bUseParameters ? FirstPersonParams.FOVCorrectionFactor : 1.0f;
			// FOVCorrectionFactor = SceneTanHalfFOV / FirstPersonTanHalfFov -> FirstPersonTanHalfFov = SceneTanHalfFOV / FOVCorrectionFactor
			const FVector4f FirstPersonTanAndInvTanHalfFOV = FVector4f(
				SceneTanAndInvTanHalfFOV.X / FOVCorrectionFactor,
				SceneTanAndInvTanHalfFOV.Y / FOVCorrectionFactor,
				SceneTanAndInvTanHalfFOV.Z * FOVCorrectionFactor,
				SceneTanAndInvTanHalfFOV.W * FOVCorrectionFactor);
			return FirstPersonTanAndInvTanHalfFOV;
		}
		return FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	}

private:
	inline void RecomputeDerivedMatrices()
	{
		// Compute the view projection matrix and its inverse.
		WorldToClip = GetWorldToView() * GetViewToClip();
		ClipToWorld = GetClipToView() * GetViewToWorld();

		// Compute a transform from view origin centered world-space to clip space.
		TranslatedWorldToClip = GetTranslatedViewMatrix() * GetViewToClip();
		ClipToTranslatedWorld = GetClipToView() * GetInvTranslatedViewMatrix();
	}

	static const FMatrix InvertProjectionMatrix( const FMatrix& M )
	{
		if( M.M[1][0] == 0.0f &&
			M.M[3][0] == 0.0f &&
			M.M[0][1] == 0.0f &&
			M.M[3][1] == 0.0f &&
			M.M[0][2] == 0.0f &&
			M.M[1][2] == 0.0f &&
			M.M[0][3] == 0.0f &&
			M.M[1][3] == 0.0f &&
			M.M[2][3] == 1.0f &&
			M.M[3][3] == 0.0f )
		{
			// Solve the common case directly with very high precision.
			/*
			M = 
			| a | 0 | 0 | 0 |
			| 0 | b | 0 | 0 |
			| s | t | c | 1 |
			| 0 | 0 | d | 0 |
			*/

			double a = M.M[0][0];
			double b = M.M[1][1];
			double c = M.M[2][2];
			double d = M.M[3][2];
			double s = M.M[2][0];
			double t = M.M[2][1];

			return FMatrix(
				FPlane( 1.0 / a, 0.0f, 0.0f, 0.0f ),
				FPlane( 0.0f, 1.0 / b, 0.0f, 0.0f ),
				FPlane( 0.0f, 0.0f, 0.0f, 1.0 / d ),
				FPlane( -s/a, -t/b, 1.0f, -c/d )
			);
		}
		else
		{
			return M.Inverse();
		}
	}
};

//////////////////////////////////////////////////////////////////////////

inline constexpr int MAX_MOBILE_SHADOWCASCADES = 4;

/** The uniform shader parameters for a mobile directional light and its shadow.
  * One uniform buffer will be created for the first directional light in each lighting channel.
  */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT_WITH_CONSTRUCTOR(FMobileDirectionalLightShaderParameters, ENGINE_API)
	SHADER_PARAMETER_EX(FLinearColor, DirectionalLightColor, EShaderPrecisionModifier::Half)
	SHADER_PARAMETER_EX(FVector4f, DirectionalLightDirectionAndShadowTransition, EShaderPrecisionModifier::Half)
	SHADER_PARAMETER_EX(FVector4f, DirectionalLightShadowSize, EShaderPrecisionModifier::Half)
	SHADER_PARAMETER_EX(FVector4f, DirectionalLightDistanceFadeMADAndSpecularScale, EShaderPrecisionModifier::Half) // .z is used for SpecularScale, .w is for DiffuseScale
	SHADER_PARAMETER_EX(FVector4f, DirectionalLightShadowDistances, EShaderPrecisionModifier::Half)
	SHADER_PARAMETER_EX(FVector4f, DirectionalLightShadowDepthRange, EShaderPrecisionModifier::Half)
	SHADER_PARAMETER_ARRAY(FMatrix44f, DirectionalLightScreenToShadow, [MAX_MOBILE_SHADOWCASCADES])
	SHADER_PARAMETER(uint32, DirectionalLightNumCascades)
	SHADER_PARAMETER(uint32, DirectionalLightShadowMapChannelMask)
	SHADER_PARAMETER_TEXTURE(Texture2D, DirectionalLightShadowTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, DirectionalLightShadowSampler)
	SHADER_PARAMETER(uint32, ShadowedBits)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

//////////////////////////////////////////////////////////////////////////

/** 
 * Enumeration for currently used translucent lighting volume cascades 
 */
enum ETranslucencyVolumeCascade
{
	TVC_Inner,
	TVC_Outer,

	TVC_MAX,
};

#define SKY_IRRADIANCE_ENVIRONMENT_MAP_VEC4_COUNT 8

//VIEW_UNIFORM_BUFFER_MEMBER(FMatrix44f, PrevProjection)
//VIEW_UNIFORM_BUFFER_MEMBER(FMatrix44f, PrevViewProj)
//VIEW_UNIFORM_BUFFER_MEMBER(FMatrix44f, PrevViewRotationProj)

// View uniform buffer member declarations
#define VIEW_UNIFORM_BUFFER_MEMBER_TABLE \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, TranslatedWorldToClip) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, RelativeWorldToClip) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, ClipToRelativeWorld)  \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, TranslatedWorldToView) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, ViewToTranslatedWorld) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, TranslatedWorldToCameraView) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, CameraViewToTranslatedWorld) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, ViewToClip) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, ViewToClipNoAA) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, ClipToView) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, ClipToTranslatedWorld) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, SVPositionToTranslatedWorld) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, ScreenToRelativeWorld) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, ScreenToTranslatedWorld) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, MobileMultiviewShadowTransform) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, MobileMultiviewDecalTransform) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, FirstPersonTransform) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, PrevFirstPersonTransform) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, InvFirstPersonTransform) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, ViewOriginHigh) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(FVector3f, ViewForward, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(FVector3f, ViewUp, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(FVector3f, ViewRight, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX(FVector3f, HMDViewNoRollUp, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX(FVector3f, HMDViewNoRollRight, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, InvDeviceZToWorldZTransform) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX(FVector4f, ScreenPositionScaleBias, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, ViewOriginLow) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, TranslatedWorldCameraOrigin) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, WorldViewOriginHigh) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, WorldViewOriginLow) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PreViewTranslationHigh) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PreViewTranslationLow) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, PrevViewToClip) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, PrevClipToView) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, PrevTranslatedWorldToClip) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, PrevTranslatedWorldToView) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, PrevViewToTranslatedWorld) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, PrevTranslatedWorldToCameraView) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, PrevCameraViewToTranslatedWorld) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PrevTranslatedWorldCameraOrigin) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PrevWorldCameraOriginHigh) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PrevWorldCameraOriginLow) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PrevWorldViewOriginHigh) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PrevWorldViewOriginLow) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PrevPreViewTranslationHigh) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PrevPreViewTranslationLow) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, ViewTilePosition) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, RelativeWorldCameraOriginTO) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, RelativeWorldViewOriginTO) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, RelativePreViewTranslationTO) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PrevRelativeWorldCameraOriginTO) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, PrevRelativeWorldViewOriginTO) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, RelativePrevPreViewTranslationTO) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, PrevClipToRelativeWorld) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, PrevScreenToTranslatedWorld) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, ClipToPrevClip) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FMatrix44f, ClipToPrevClipWithAA) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, TemporalAAJitter) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, GlobalClippingPlane) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector2f, FieldOfViewWideAngles) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector2f, PrevFieldOfViewWideAngles) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector2f, FirstPersonFieldOfViewWideAngles) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector2f, PrevFirstPersonFieldOfViewWideAngles) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX(FVector4f, ViewRectMin, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, ViewSizeAndInvSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FUintVector4, ViewRectMinAndSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, LightProbeSizeRatioAndInvSizeRatio) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, BufferSizeAndInvSize) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, BufferBilinearUVMinMax) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, ScreenToViewSpace) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector2f, BufferToSceneTextureScale) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector2f, ResolutionFractionAndInv) \
	VIEW_UNIFORM_BUFFER_MEMBER(int32, NumSceneColorMSAASamples) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(float, ProjectionDepthThicknessScale) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, PreExposure) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, OneOverPreExposure) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(FVector4f, DiffuseOverrideParameter, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(FVector4f, SpecularOverrideParameter, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(FVector4f, NormalOverrideParameter, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(FVector2f, RoughnessOverrideParameter, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(FVector2f, AmbientOcclusionOverrideParameter, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, PrevFrameGameTime) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, PrevFrameRealTime) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(float, OutOfBoundsMask, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector3f, WorldCameraMovementSinceLastFrame) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, CullingSign) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX(float, NearPlane, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(float, OrthoFarPlane) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, GameTime) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, RealTime) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DeltaTime) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, MaterialTextureMipBias) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, MaterialTextureDerivativeMultiply) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, Random) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, FrameNumber) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, FrameCounter) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, StateFrameIndexMod8) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, StateFrameIndex) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, StateOutputFrameIndex) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, DebugViewModeMask) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, WorldIsPaused) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(float, CameraCut, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(float, UnlitViewmodeMask, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(FLinearColor, DirectionalLightColor, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(FVector3f, DirectionalLightDirection, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FVector4f, TranslucencyLightingVolumeMin, TVC_MAX) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FVector4f, TranslucencyLightingVolumeInvSize, TVC_MAX) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, TemporalAAParams) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, CircleDOFParams) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DepthOfFieldSensorWidth) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DepthOfFieldFocalDistance) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DepthOfFieldScale) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DepthOfFieldFocalLength) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DepthOfFieldFocalRegion) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DepthOfFieldNearTransitionRegion) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DepthOfFieldFarTransitionRegion) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, MotionBlurNormalizedToPixel) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, GeneralPurposeTweak) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, GeneralPurposeTweak2) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(float, DemosaicVposOffset, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DecalDepthBias) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, SceneColorTextureFormatQuantizationError) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, IndirectLightingColorScale) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, PrecomputedIndirectLightingColorScale) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, PrecomputedIndirectSpecularColorScale) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FVector4f, AtmosphereLightDirection, NUM_ATMOSPHERE_LIGHTS) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FLinearColor, AtmosphereLightIlluminanceOnGroundPostTransmittance, NUM_ATMOSPHERE_LIGHTS) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FLinearColor, AtmosphereLightIlluminanceOuterSpace, NUM_ATMOSPHERE_LIGHTS) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FLinearColor, AtmosphereLightDiscLuminance, NUM_ATMOSPHERE_LIGHTS) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FVector4f, AtmosphereLightDiscCosHalfApexAngle_PPTrans, NUM_ATMOSPHERE_LIGHTS) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, SkyViewLutSizeAndInvSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, SkyCameraTranslatedWorldOrigin) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, SkyPlanetTranslatedWorldCenterAndViewHeight) \
	VIEW_UNIFORM_BUFFER_MEMBER(FMatrix44f, SkyViewLutReferential) \
	VIEW_UNIFORM_BUFFER_MEMBER(FLinearColor, SkyAtmosphereSkyLuminanceFactor) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyAtmospherePresentInScene) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyAtmosphereHeightFogContribution) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyAtmosphereBottomRadiusKm) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyAtmosphereTopRadiusKm) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyAtmosphereAerialPerspectiveStartDepthKm) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyAtmosphereApplyCameraAerialPerspectiveVolume) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, NormalCurvatureToRoughnessScaleBias) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, RenderingReflectionCaptureMask) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, RealTimeReflectionCapture) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, RealTimeReflectionCapturePreExposure) \
	VIEW_UNIFORM_BUFFER_MEMBER(FLinearColor, AmbientCubemapTint) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, AmbientCubemapIntensity) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyLightApplyPrecomputedBentNormalShadowingFlag) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyLightAffectReflectionFlag) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyLightAffectGlobalIlluminationFlag) \
	VIEW_UNIFORM_BUFFER_MEMBER(FLinearColor, SkyLightColor) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SkyLightVolumetricScatteringIntensity) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FVector4f, MobileSkyIrradianceEnvironmentMap, SKY_IRRADIANCE_ENVIRONMENT_MAP_VEC4_COUNT) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, MobilePreviewMode) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(float, HMDEyePaddingOffset) \
	VIEW_UNIFORM_BUFFER_MEMBER_EX(float, ReflectionCubemapMaxMip, EShaderPrecisionModifier::Half) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, ShowDecalsMask) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, DistanceFieldAOSpecularOcclusionMode) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, IndirectCapsuleSelfShadowingIntensity) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, ReflectionEnvironmentRoughnessMixingScaleBiasAndLargestWeight) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(int32, StereoPassIndex) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FVector4f, GlobalVolumeTranslatedCenterAndExtent, GlobalDistanceField::MaxClipmaps) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FVector4f, GlobalVolumeTranslatedWorldToUVAddAndMul, GlobalDistanceField::MaxClipmaps) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FVector4f, GlobalDistanceFieldMipTranslatedWorldToUVScale, GlobalDistanceField::MaxClipmaps) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FVector4f, GlobalDistanceFieldMipTranslatedWorldToUVBias, GlobalDistanceField::MaxClipmaps) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, GlobalDistanceFieldMipFactor) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, GlobalDistanceFieldMipTransition) \
	VIEW_UNIFORM_BUFFER_MEMBER(int32, GlobalDistanceFieldClipmapSizeInPages) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, GlobalDistanceFieldInvPageAtlasSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, GlobalDistanceFieldInvCoverageAtlasSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, GlobalVolumeDimension) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, GlobalVolumeTexelSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, MaxGlobalDFAOConeDistance) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, NumGlobalSDFClipmaps) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, CoveredExpandSurfaceScale) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, NotCoveredExpandSurfaceScale) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, NotCoveredMinStepScale) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DitheredTransparencyStepThreshold) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, DitheredTransparencyTraceThreshold) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, ViewportScaleUI) \
	VIEW_UNIFORM_BUFFER_MEMBER(FIntPoint, CursorPosition) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, bCheckerboardSubsurfaceProfileRendering) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, VolumetricFogGridSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, VolumetricFogInvGridSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, VolumetricFogGridZParams) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector2f, VolumetricFogSVPosToVolumeUV) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector2f, VolumetricFogViewGridUVToPrevViewRectUV) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector2f, VolumetricFogPrevViewGridRectUVToResourceUV) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector2f, VolumetricFogPrevUVMax) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector2f, VolumetricFogPrevUVMaxForTemporalBlend) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, VolumetricFogPrevResourceGridSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector2f, VolumetricFogScreenToResourceUV) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector2f, VolumetricFogUVMax) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, VolumetricFogMaxDistance) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, VolumetricLightmapWorldToUVScale) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, VolumetricLightmapWorldToUVAdd) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, VolumetricLightmapIndirectionTextureSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, VolumetricLightmapBrickSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, VolumetricLightmapBrickTexelSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, IndirectLightingCacheShowFlag) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, EyeToPixelSpreadAngle) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FVector4f, XRPassthroughCameraUVs, 2) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, GlobalVirtualTextureMipBias) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, VirtualTextureFeedbackBufferSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, VirtualTextureFeedbackShift) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, VirtualTextureFeedbackMask) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, VirtualTextureFeedbackJitterOffset) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, VirtualTextureFeedbackSampleOffset) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, VirtualTextureExtendedDebugBufferSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(int32, FarShadowStaticMeshLODBias) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, MinRoughness) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, HairRenderInfo) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, EnableSkyLight) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, HairRenderInfoBits) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, HairComponents) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, bSubsurfacePostprocessEnabled) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, SSProfilesTextureSizeAndInvSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, SSProfilesPreIntegratedTextureSizeAndInvSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, SpecularProfileTextureSizeAndInvSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector4f, ToonProfileTextureSizeAndInvSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector2f, ToonProfileDitherPatternSizeAndInvSize) \
	VIEW_UNIFORM_BUFFER_MEMBER(FVector3f, PhysicsFieldClipmapCenter) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, PhysicsFieldClipmapDistance) \
	VIEW_UNIFORM_BUFFER_MEMBER(int, PhysicsFieldClipmapResolution) \
	VIEW_UNIFORM_BUFFER_MEMBER(int, PhysicsFieldClipmapExponent) \
	VIEW_UNIFORM_BUFFER_MEMBER(int, PhysicsFieldClipmapCount) \
	VIEW_UNIFORM_BUFFER_MEMBER(int, PhysicsFieldTargetCount) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FIntVector4, PhysicsFieldTargets, MAX_PHYSICS_FIELD_TARGETS) \
	VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(FIntVector4, PhysicsFieldTargetMappings, MAX_PHYSICS_FIELD_TARGETS) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(uint32, GPUSceneViewId) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, ViewResolutionFraction) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, SubSurfaceColorAsTransmittanceAtDistanceInMeters) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, TanAndInvTanHalfFOV) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, PrevTanAndInvTanHalfFOV) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, FirstPersonTanAndInvTanHalfFOV) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, PrevFirstPersonTanAndInvTanHalfFOV) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector2f, WorldDepthToPixelWorldRadius) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, ScreenRayLengthMultiplier) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, GlintLUTParameters0) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(FVector4f, GlintLUTParameters1) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(float, FirstPersonScale) \
	VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(float, PrevFirstPersonScale) \
	VIEW_UNIFORM_BUFFER_MEMBER(float, MaterialMaxEmissiveValue) \
	VIEW_UNIFORM_BUFFER_MEMBER(int32, PostVolumeUserFlags) \
	VIEW_UNIFORM_BUFFER_MEMBER(FIntVector4, EnvironmentComponentsFlags) \
	VIEW_UNIFORM_BUFFER_MEMBER(uint32, bPrimitiveAlphaHoldoutEnabled) \

/** The uniform shader parameters associated with a view. */
#define VIEW_UNIFORM_BUFFER_MEMBER(type, identifier) \
	SHADER_PARAMETER(type, identifier)

#define VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(type, identifier) \
	SHADER_PARAMETER(type, identifier)

#define VIEW_UNIFORM_BUFFER_MEMBER_EX(type, identifier, precision) \
	SHADER_PARAMETER_EX(type, identifier, precision)

#define VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX(type, identifier, precision) \
	SHADER_PARAMETER_EX(type, identifier, precision)

#define VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(type, identifier, dimension) \
	SHADER_PARAMETER_ARRAY(type, identifier, [dimension])

#define VIEW_UNIFORM_BUFFER_MEMBER_ARRAY_PER_VIEW(type, identifier, dimension) \
	SHADER_PARAMETER_ARRAY(type, identifier, [dimension])

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT_WITH_CONSTRUCTOR(FViewUniformShaderParameters, ENGINE_API)

	VIEW_UNIFORM_BUFFER_MEMBER_TABLE

	// Same as Wrap_WorldGroupSettings and Clamp_WorldGroupSettings, but with mipbias=MaterialTextureMipBias.
	SHADER_PARAMETER_SAMPLER(SamplerState, MaterialTextureBilinearWrapedSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, MaterialTextureBilinearClampedSampler)

	SHADER_PARAMETER_TEXTURE(Texture3D<uint4>, VolumetricLightmapIndirectionTexture) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, VolumetricLightmapBrickAmbientVector) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, VolumetricLightmapBrickSHCoefficients0) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, VolumetricLightmapBrickSHCoefficients1) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, VolumetricLightmapBrickSHCoefficients2) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, VolumetricLightmapBrickSHCoefficients3) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, VolumetricLightmapBrickSHCoefficients4) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, VolumetricLightmapBrickSHCoefficients5) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, SkyBentNormalBrickTexture) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, DirectionalLightShadowingBrickTexture) // FPrecomputedVolumetricLightmapLightingPolicy

	SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricLightmapBrickAmbientVectorSampler) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricLightmapTextureSampler0) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricLightmapTextureSampler1) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricLightmapTextureSampler2) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricLightmapTextureSampler3) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricLightmapTextureSampler4) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricLightmapTextureSampler5) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, SkyBentNormalTextureSampler) // FPrecomputedVolumetricLightmapLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, DirectionalLightShadowingTextureSampler) // FPrecomputedVolumetricLightmapLightingPolicy

	SHADER_PARAMETER_TEXTURE(Texture3D, GlobalDistanceFieldPageAtlasTexture)
	SHADER_PARAMETER_TEXTURE(Texture3D, GlobalDistanceFieldCoverageAtlasTexture)
	SHADER_PARAMETER_TEXTURE(Texture3D<uint>, GlobalDistanceFieldPageTableTexture)
	SHADER_PARAMETER_TEXTURE(Texture3D, GlobalDistanceFieldMipTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, GlobalDistanceFieldPageAtlasTextureSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, GlobalDistanceFieldCoverageAtlasTextureSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, GlobalDistanceFieldMipTextureSampler)

	SHADER_PARAMETER_TEXTURE(Texture2D, AtmosphereTransmittanceTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, AtmosphereTransmittanceTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, AtmosphereIrradianceTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, AtmosphereIrradianceTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture3D, AtmosphereInscatterTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, AtmosphereInscatterTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, PerlinNoiseGradientTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, PerlinNoiseGradientTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture3D, PerlinNoise3DTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, PerlinNoise3DTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D<uint>, SobolSamplingTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SharedPointWrappedSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, SharedPointClampedSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, SharedBilinearWrappedSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, SharedBilinearClampedSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, SharedBilinearAnisoClampedSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, SharedTrilinearWrappedSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, SharedTrilinearClampedSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedBRDF)
	SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedBRDFSampler)
	SHADER_PARAMETER_SRV(StructuredBuffer<float4>, SkyIrradianceEnvironmentMap)
	// Atmosphere
	SHADER_PARAMETER_TEXTURE(Texture2D, TransmittanceLutTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, TransmittanceLutTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, SkyViewLutTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SkyViewLutTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture3D, CameraAerialPerspectiveVolume)
	SHADER_PARAMETER_SAMPLER(SamplerState, CameraAerialPerspectiveVolumeSampler)
	SHADER_PARAMETER_TEXTURE(Texture3D, CameraAerialPerspectiveVolumeMieOnly)
	SHADER_PARAMETER_SAMPLER(SamplerState, CameraAerialPerspectiveVolumeMieOnlySampler)
	SHADER_PARAMETER_TEXTURE(Texture3D, CameraAerialPerspectiveVolumeRayOnly)
	SHADER_PARAMETER_SAMPLER(SamplerState, CameraAerialPerspectiveVolumeRayOnlySampler)
	SHADER_PARAMETER_SRV(StructuredBuffer<float4>, DistantSkyLightLutBufferSRV)	
	SHADER_PARAMETER_SRV(Buffer<float4>, MobileDistantSkyLightLutBufferSRV)		
	// Hair
	SHADER_PARAMETER_TEXTURE(Texture3D, HairScatteringLUTTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, HairScatteringLUTSampler)
	// GGX/Sheen LTC textures
	SHADER_PARAMETER_TEXTURE(Texture2D, GGXLTCMatTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, GGXLTCMatSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, GGXLTCAmpTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, GGXLTCAmpSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, SheenLTCTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SheenLTCSampler)
	// Energy conservation
	SHADER_PARAMETER(uint32, bShadingEnergyConservation)
	SHADER_PARAMETER(uint32, bShadingEnergyPreservation)
	SHADER_PARAMETER_TEXTURE(Texture2D<float2>, ShadingEnergyGGXSpecTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D<float4>,  ShadingEnergyGGXMaxSpecTexture)
	SHADER_PARAMETER_TEXTURE(Texture3D<float2>, ShadingEnergyGGXGlassTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D<float2>, ShadingEnergyClothSpecTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D<float>, ShadingEnergyDiffuseTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ShadingEnergySampler)
	// Glints
	SHADER_PARAMETER_TEXTURE(Texture2DArray<float4>, GlintTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, GlintSampler)
	// Simple volume texture
	SHADER_PARAMETER_TEXTURE(Texture3D<float>, SimpleVolumeTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SimpleVolumeTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture3D<float>, SimpleVolumeEnvTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SimpleVolumeEnvTextureSampler)
	// SSS
	SHADER_PARAMETER_TEXTURE(Texture2D, SSProfilesTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SSProfilesSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, SSProfilesTransmissionSampler)
	SHADER_PARAMETER_TEXTURE(Texture2DArray, SSProfilesPreIntegratedTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SSProfilesPreIntegratedSampler)
	// Specular Profile
	SHADER_PARAMETER_TEXTURE(Texture2DArray, SpecularProfileTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SpecularProfileSampler)
	// Toon Profile texture
	SHADER_PARAMETER_TEXTURE(Texture2DArray, ToonProfileTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ToonProfileSampler)
	// Water
	SHADER_PARAMETER_SRV(Buffer<float4>, WaterIndirection)
	SHADER_PARAMETER_SRV(Buffer<float4>, WaterData)
	SHADER_PARAMETER(int32, WaterInfoTextureViewIndex)
	// Rect light atlas
	SHADER_PARAMETER(FVector4f, RectLightAtlasSizeAndInvSize)
	SHADER_PARAMETER(float, RectLightAtlasMaxMipLevel)
	SHADER_PARAMETER_TEXTURE(Texture2D<float4>, RectLightAtlasTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, RectLightAtlasSampler)
	// IES atlas
	SHADER_PARAMETER(FVector4f, IESAtlasSizeAndInvSize)
	SHADER_PARAMETER_TEXTURE(Texture2DArray<float>, IESAtlasTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, IESAtlasSampler)
	// Landscape
	SHADER_PARAMETER_SAMPLER(SamplerState, LandscapeWeightmapSampler)
	SHADER_PARAMETER_SRV(Buffer<uint>, LandscapeIndirection)
	SHADER_PARAMETER_SRV(Buffer<float>, LandscapePerComponentData)

	SHADER_PARAMETER_UAV(RWStructuredBuffer<uint>, VTFeedbackBuffer)

	// Physics Field
	SHADER_PARAMETER_SRV(Buffer<float>, PhysicsFieldClipmapBuffer)
	SHADER_PARAMETER_SRV(Buffer<float>, PhysicsFieldNodesParams)
	SHADER_PARAMETER_SRV(Buffer<int>, PhysicsFieldNodesOffsets)
	SHADER_PARAMETER_SRV(Buffer<int>, PhysicsFieldTargetsOffsets)

	// Blue noise texture
	SHADER_PARAMETER_TEXTURE(Texture2D<float>, BlueNoiseScalarTexture)
	SHADER_PARAMETER(FIntVector, BlueNoiseDimensions)
	SHADER_PARAMETER(FIntVector, BlueNoiseModuloMasks)

	// Ray tracing
	SHADER_PARAMETER(FVector3f, TLASPreViewTranslationHigh)
	SHADER_PARAMETER(FVector3f, TLASPreViewTranslationLow)

END_GLOBAL_SHADER_PARAMETER_STRUCT()

#undef VIEW_UNIFORM_BUFFER_MEMBER
#undef VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW
#undef VIEW_UNIFORM_BUFFER_MEMBER_EX
#undef VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX
#undef VIEW_UNIFORM_BUFFER_MEMBER_ARRAY
#undef VIEW_UNIFORM_BUFFER_MEMBER_ARRAY_PER_VIEW

/** Copy of the view uniform shader parameters associated with a view for instanced stereo. */
#define INSTANCED_VIEW_COUNT 2

#define VIEW_UNIFORM_BUFFER_MEMBER(type, identifier) \
	SHADER_PARAMETER(type, identifier)

#define VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(type, identifier) \
	SHADER_PARAMETER_ARRAY(TShaderParameterTypeInfo<type>::TInstancedType, identifier, [INSTANCED_VIEW_COUNT])

#define VIEW_UNIFORM_BUFFER_MEMBER_EX(type, identifier, precision) \
	SHADER_PARAMETER_EX(type, identifier, precision)

#define VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX(type, identifier, precision) \
	SHADER_PARAMETER_ARRAY_EX(TShaderParameterTypeInfo<type>::TInstancedType, identifier, [INSTANCED_VIEW_COUNT], precision)

#define VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(type, identifier, dimension) \
	SHADER_PARAMETER_ARRAY(type, identifier, [dimension])

#define VIEW_UNIFORM_BUFFER_MEMBER_ARRAY_PER_VIEW(type, identifier, dimension) \
	SHADER_PARAMETER_ARRAY(type, identifier, [dimension * INSTANCED_VIEW_COUNT])

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT_WITH_CONSTRUCTOR(FInstancedViewUniformShaderParameters, ENGINE_API)
	VIEW_UNIFORM_BUFFER_MEMBER_TABLE
END_GLOBAL_SHADER_PARAMETER_STRUCT()

#undef VIEW_UNIFORM_BUFFER_MEMBER
#undef VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW
#undef VIEW_UNIFORM_BUFFER_MEMBER_EX
#undef VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX
#undef VIEW_UNIFORM_BUFFER_MEMBER_ARRAY
#undef VIEW_UNIFORM_BUFFER_MEMBER_ARRAY_PER_VIEW

#define VIEW_UNIFORM_BUFFER_MEMBER(type, identifier) \
	if ( CopyViewId == 0 ) \
	{ \
		FMemory::Memcpy(&InstancedViewParameters.identifier, &ViewParameters.identifier, sizeof(type)); \
	}

// since per-view attributes are of type type, but the backing structure is TInstancedType for 16 byte alignment, we'd need to mem0 the rest,
// but we rely on FInstancedViewUniformShaderParameters zeroing itself in the constructor
#define VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(type, identifier) \
	FMemory::Memcpy(&InstancedViewParameters.identifier[CopyViewId], &ViewParameters.identifier, sizeof(type)); \
	check(FMemory::MemIsZero(((uint8*) &InstancedViewParameters.identifier[CopyViewId]) + sizeof(type), sizeof(TShaderParameterTypeInfo<type>::TInstancedType) - sizeof(type)));

#define VIEW_UNIFORM_BUFFER_MEMBER_EX(type, identifier, precision) \
	if ( CopyViewId == 0 ) \
	{ \
		FMemory::Memcpy(&InstancedViewParameters.identifier, &ViewParameters.identifier, sizeof(type)); \
	}

#define VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX(type, identifier, precision) \
	FMemory::Memcpy(&InstancedViewParameters.identifier[CopyViewId], &ViewParameters.identifier, sizeof(type)); \
	check(FMemory::MemIsZero(((uint8*)&InstancedViewParameters.identifier[CopyViewId]) + sizeof(type), sizeof(TShaderParameterTypeInfo<type>::TInstancedType) - sizeof(type)));

#define VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(type, identifier, dimension) \
	if ( CopyViewId == 0 ) \
	{ \
		for (uint32 ElementIndex = 0; ElementIndex < dimension; ElementIndex++) \
		{ \
			FMemory::Memcpy(&InstancedViewParameters.identifier[CopyViewId * dimension + ElementIndex], &ViewParameters.identifier[ElementIndex], sizeof(type)); \
		} \
	}

#define VIEW_UNIFORM_BUFFER_MEMBER_ARRAY_PER_VIEW(type, identifier, dimension) \
	for (uint32 ElementIndex = 0; ElementIndex < dimension; ElementIndex++) \
	{ \
		FMemory::Memcpy(&InstancedViewParameters.identifier[CopyViewId * dimension + ElementIndex], &ViewParameters.identifier[ElementIndex], sizeof(type)); \
		check(FMemory::MemIsZero(((uint8*) &InstancedViewParameters.identifier[CopyViewId * dimension + ElementIndex]) + sizeof(type), sizeof(TShaderParameterTypeInfo<type>::TInstancedType) - sizeof(type))); \
	}

namespace InstancedViewParametersUtils
{
	inline void CopyIntoInstancedViewParameters(FInstancedViewUniformShaderParameters& InstancedViewParameters, const FViewUniformShaderParameters& ViewParameters, uint32 CopyViewId)
	{
		VIEW_UNIFORM_BUFFER_MEMBER_TABLE
	}
}

#undef VIEW_UNIFORM_BUFFER_MEMBER
#undef VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW
#undef VIEW_UNIFORM_BUFFER_MEMBER_EX
#undef VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX
#undef VIEW_UNIFORM_BUFFER_MEMBER_ARRAY
#undef VIEW_UNIFORM_BUFFER_MEMBER_ARRAY_PER_VIEW

/** Copy of a base view uniform shader parameters structure into another. But only parameters, not resources, and not instanced views. */
#define VIEW_UNIFORM_BUFFER_MEMBER(type, identifier) \
	FMemory::Memcpy(&DstView.identifier, &SrcView.identifier, sizeof(type));

#define VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW(type, identifier) \
	FMemory::Memcpy(&DstView.identifier, &SrcView.identifier, sizeof(type));

#define VIEW_UNIFORM_BUFFER_MEMBER_EX(type, identifier, precision) \
	FMemory::Memcpy(&DstView.identifier, &SrcView.identifier, sizeof(type));

#define VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX(type, identifier, precision) \
	FMemory::Memcpy(&DstView.identifier, &SrcView.identifier, sizeof(type));

#define VIEW_UNIFORM_BUFFER_MEMBER_ARRAY(type, identifier, dimension) \
	for (uint32 ElementIndex = 0; ElementIndex < dimension; ElementIndex++) \
	{ \
		FMemory::Memcpy(&DstView.identifier[ElementIndex], &SrcView.identifier[ElementIndex], sizeof(type)); \
	} \

#define VIEW_UNIFORM_BUFFER_MEMBER_ARRAY_PER_VIEW(type, identifier, dimension) \
	for (uint32 ElementIndex = 0; ElementIndex < dimension; ElementIndex++) \
	{ \
		FMemory::Memcpy(&DstView.identifier[ElementIndex], &SrcView.identifier[ElementIndex], sizeof(type)); \
	} \

namespace ViewParametersUtils
{
	// This only copy the base view, not any instanced view data.
	inline void CopyViewParametersOnly(FViewUniformShaderParameters& DstView, const FViewUniformShaderParameters& SrcView)
	{
		VIEW_UNIFORM_BUFFER_MEMBER_TABLE
	}
}

#undef VIEW_UNIFORM_BUFFER_MEMBER
#undef VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW
#undef VIEW_UNIFORM_BUFFER_MEMBER_EX
#undef VIEW_UNIFORM_BUFFER_MEMBER_PER_VIEW_EX
#undef VIEW_UNIFORM_BUFFER_MEMBER_ARRAY
#undef VIEW_UNIFORM_BUFFER_MEMBER_ARRAY_PER_VIEW

#undef INSTANCED_VIEW_COUNT
#undef VIEW_UNIFORM_BUFFER_MEMBER_TABLE

BEGIN_SHADER_PARAMETER_STRUCT(FViewShaderParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_STRUCT_REF(FInstancedViewUniformShaderParameters, InstancedView)
END_SHADER_PARAMETER_STRUCT()

namespace EDrawDynamicFlags
{
	enum Type : int32
	{
		None = 0,
		ForceLowestLOD = 0x1,
		FarShadowCascade = 0x2,
	};
}

enum class EViewWaterIntersection : uint8
{
	OutsideWater,
	PossiblyIntersectingWater,
	InsideWater
};

enum class EColorChannelMask : int8
{
	All = INDEX_NONE,
	Red = 0,
	Green = 1,
	Blue = 2,
	Alpha = 3
};

// Hacky base class to avoid 8 bytes of padding after the vtable
class FSceneViewFixLayout
{
public:
	virtual ~FSceneViewFixLayout() = default;
};

/**
 * A projection from scene space into a 2D screen region.
 */
class FSceneView : public FSceneViewFixLayout
{
public:
	const FSceneViewFamily* Family;
	/** can be 0 (thumbnail rendering) */
	FSceneViewStateInterface* State;

	/** The uniform buffer for the view's parameters. This is only initialized in the rendering thread's copies of the FSceneView. */
	TUniformBufferRef<FViewUniformShaderParameters> ViewUniformBuffer;

private:
	/** During GetDynamicMeshElements this will be the correct cull volume for shadow stuff */
	const FConvexVolume* DynamicMeshElementsShadowCullFrustum;
	/** If the above is non-null, a translation that is applied to world-space before transforming by one of the shadow matrices. */
	FVector		PreShadowTranslation;

	TSharedPtr<FConvexVolume> StereoCullingFrustum;
	void SetupViewFrustum();

public:
	FSceneViewInitOptions SceneViewInitOptions;
	
	/** The actor which is being viewed from. */
	FSceneViewOwner ViewActor; // TODO: Rename to ViewOwner
	 
	/** Player index this view is associated with or INDEX_NONE. */
	int32 PlayerIndex;

	/** An interaction which draws the view's interaction elements. */
	FViewElementDrawer* Drawer;

	/* Final position of the view in the final render target (in pixels), potentially constrained by an aspect ratio requirement (black bars) */
	const FIntRect UnscaledViewRect;

	/* Raw view size (in pixels), used for screen space calculations */
	FIntRect UnconstrainedViewRect;

	/** Maximum number of shadow cascades to render with. */
	int32 MaxShadowCascades;

	FViewMatrices ViewMatrices;

	/** Variables used to determine the view matrix */
	FVector		ViewLocation;
	FRotator	ViewRotation;
	FQuat		BaseHmdOrientation;
	FVector		BaseHmdLocation;
	float		WorldToMetersScale;
	TOptional<FTransform> PreviousViewTransform;

	UE_DEPRECATED(5.6, "Use GetShadowViewMatrices instead")
	uint8 ShadowViewMatrices;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	ENGINE_API const FViewMatrices& GetShadowViewMatrices() const;
#else
	const FViewMatrices& GetShadowViewMatrices() const { return ViewMatrices; }
#endif

	FMatrix ProjectionMatrixUnadjustedForRHI;

	FLinearColor BackgroundColor;
	FLinearColor OverlayColor;

	/** Color scale multiplier used during post processing */
	FLinearColor ColorScale;

	/** Color scale multiplier applied to sky light color */
	FLinearColor SkylightScale;

	/** For stereoscopic rendering, whether or not this is a full pass, or a left / right eye pass */
	EStereoscopicPass StereoPass;

	/** For stereoscopic rendering, unique index identifying the view across view families */
	int32 StereoViewIndex;

	/** For stereoscopic rendering, view family index of the primary view associated with this view */
	int32 PrimaryViewIndex;

	/** Allow cross GPU transfer for this view */
	bool bAllowCrossGPUTransfer;

	/** Use custom GPUmask */
	bool bOverrideGPUMask;

	/** The GPU nodes on which to render this view. */
	FRHIGPUMask GPUMask;

	/** Whether this view should render the first instance only of any meshes using instancing. */
	bool bRenderFirstInstanceOnly;

	// Whether to use FOV when computing mesh LOD.
	bool bUseFieldOfViewForLOD;

	// Whether this view should use an HMD hidden area mask where appropriate.
	bool bHMDHiddenAreaMaskActive = false;

	/** Actual field of view and that desired by the camera originally */
	float FOV;
	float DesiredFOV;

	EDrawDynamicFlags::Type DrawDynamicFlags;

	/** Current buffer visualization mode */
	FName CurrentBufferVisualizationMode;

	/** Current Nanite visualization mode */
	FName CurrentNaniteVisualizationMode;

	/** Current Lumen visualization mode */
	FName CurrentLumenVisualizationMode;

	/** Current MegaLights visualization mode */
	FName CurrentMegaLightsVisualizationMode;

	/** Current Substrate visualization mode */
	FName CurrentSubstrateVisualizationMode;

	/** Current Groom visualization mode */
	FName CurrentGroomVisualizationMode;

	/** Current Virtual Shadow Map visualization mode */
	FName CurrentVirtualShadowMapVisualizationMode;

	/** Current Virtual Texture visualization mode */
	FName CurrentVirtualTextureVisualizationMode;

	/** Current visualize calibration color material name */
	FName CurrentVisualizeCalibrationColorMaterialName;

	/** Current visualize calibration grayscale material name */
	FName CurrentVisualizeCalibrationGrayscaleMaterialName;

	/** Current visualize calibration custom material name */
	FName CurrentVisualizeCalibrationCustomMaterialName;

	/** Current GPU Skin Cache visualization mode*/
	FName CurrentGPUSkinCacheVisualizationMode;

#if WITH_EDITOR
	/* Whether to use the pixel inspector */
	bool bUsePixelInspector;
#endif //WITH_EDITOR

	/**
	* These can be used to override material parameters across the scene without recompiling shaders.
	* The last component is how much to include of the material's value for that parameter, so 0 will completely remove the material's value.
	*/
	FVector4f DiffuseOverrideParameter;
	FVector4f SpecularOverrideParameter;
	FVector4f NormalOverrideParameter;
	FVector2f RoughnessOverrideParameter;
	FVector2f AmbientOcclusionOverrideParameter;

	/** Mip bias to apply in material's samplers. */
	float MaterialTextureMipBias;

	/** The primitives which are hidden for this view. */
	TSet<FPrimitiveComponentId> HiddenPrimitives;

	/** The primitives which are visible for this view. If the array is not empty, all other primitives will be hidden. */
	TOptional<TSet<FPrimitiveComponentId>> ShowOnlyPrimitives;

	// Derived members.

	bool bAllowTemporalJitter;

	FConvexVolume ViewFrustum;

	UE_DEPRECATED(5.6, "Use GetCullingFrustum() instead.")
	uint8 CullingFrustum;

	const FConvexVolume& GetCullingFrustum() const
	{
		return StereoCullingFrustum ? *StereoCullingFrustum : ViewFrustum;
	}

	bool bHasNearClippingPlane;

	FPlane NearClippingPlane;

	float NearClippingDistance;

	/** Monoscopic culling origin, same as the view matrix origin in case of non-stereo */
	FVector CullingOrigin;

	/** true if ViewMatrix.Determinant() is negative. */
	bool bReverseCulling;

	/* Vector used by shaders to convert depth buffer samples into z coordinates in world space */
	FVector4f InvDeviceZToWorldZTransform;

	/** World origin offset value. Non-zero only for a single frame when origin is rebased */
	FVector OriginOffsetThisFrame;

	/** Multiplier for cull distance on objects */
	float LODDistanceFactor;

	/** Whether we did a camera cut for this view this frame. */
	bool bCameraCut;
	
	// -1,-1 if not setup
	FIntPoint CursorPos;

	/** True if this scene was created from a game world. */
	bool bIsGameView;

	/** For sanity checking casts that are assumed to be safe. */
	bool bIsViewInfo;

	class FCustomRenderPassBase* CustomRenderPass = nullptr;

	/** Whether this view is being used to render a scene capture. */
	bool bIsSceneCapture;

	/** Whether the scene capture is a cube map (bIsSceneCapture will also be set). */
	bool bIsSceneCaptureCube;

	/** Whether to generate histogram eye adaptation from all view pixels, useful with a shared EyeAdaptationViewState.  Pixels for views must form a contiguous rect. */
	bool bEyeAdaptationAllViewPixels;

	/**
	 * Whether this view may use ray tracing, for views that are used to render a scene capture.  Use IsRayTracingAllowedForView()
	 * to test for ray tracing, not this bool.  Filled in based on the bUseRayTracingIfEnabled field in the scene capture component
	 * and r.RayTracing.SceneCaptures CVar (-1 == use value from component, 0 == disable globally, 1 == enable globally).
	 */
	bool bSceneCaptureUsesRayTracing;

	/** Whether this view is being used to render a reflection capture. */
	bool bIsReflectionCapture;

	/** Whether this view is being used to render a planar reflection. */
	bool bIsPlanarReflection;

	/** Whether this view is being used to render a runtime virtual texture. */
	bool bIsVirtualTexture;

	/** Whether this view is being used to render a high quality offline render */
	bool bIsOfflineRender;

	/** Whether to force two sided rendering for this view. */
	bool bRenderSceneTwoSided;

	/** Whether this view was created from a locked viewpoint. */
	bool bIsLocked;

	/** 
	 * Whether to only render static lights and objects.  
	 * This is used when capturing the scene for reflection captures, which aren't updated at runtime. 
	 */
	bool bStaticSceneOnly;

	/** DEPRECATED use IsInstancedStereo() instead. 
	 * True if instanced stereo is enabled. */
	bool bIsInstancedStereoEnabled;

	/** DEPRECATED all instanced stereo is now what used to be called multi viewport.
	 * use IsInstancedStereo() instead.
	 * True if mobile multi-viewport is enabled. */
	UE_DEPRECATED(5.9, "all instanced stereo is now what used to be called multi viewport. IsInstancedStereo() should be used")
	bool bIsMultiViewportEnabled;
	
	/* True if multi-view instanced stereo rendering is enabled. */
	bool bIsMobileMultiViewEnabled;

public:
	/** Number of views when using mobile multi view. */
	[[nodiscard]] ENGINE_API int32 GetMultiViewCount() const;
	
	/** True if the view should render as an instanced stereo pass */
	[[nodiscard]] ENGINE_API bool IsInstancedStereo() const;
	
	/** Whether the shaders are compiled with mobile multi view == true and thus expect texture arrays instead of a single texture */
	[[nodiscard]] ENGINE_API bool UsesMobileMultiViewShaders() const;
	
	/** Whether the shaders are compiled with instanced stereo == true and thus expect a single texture and an instanced view */
	[[nodiscard]] ENGINE_API bool UsesInstancedStereoShaders() const;
	
	/** Creates a texture or a texture array of the right size based on setup (texture array for mobile multi view shaders) */
	[[nodiscard]] ENGINE_API FRDGTextureDesc CreateStereoTextureDesc(FIntPoint Size, EPixelFormat Format, FClearValueBinding Clear, ETextureCreateFlags Flags) const;
	
private:
	/** Number of views for multiview scenarios. */
	uint32 MultiViewCount = 0;
	
public:

	/** True if we need to bind the instanced view uniform buffer parameters. */
	bool bShouldBindInstancedViewUB;

	/** True if any single-pass stereo mechanism is enabled (mobile multi view, instanced stereo multi-layer, or instanced stereo multi-viewport).
	 * Not related to the deprecated NVidia feature known as Single-Pass Stereo. */
	bool bIsSinglePassStereo;

	/** True if we need to force the camera to discard previous frames occlusion. Necessary for overlapped tile rendering
	 * where we discard previous frame occlusion because the projection matrix changes.
	 */
	bool bForceCameraVisibilityReset;

	/** Whether we should disable distance-based fade transitions for this frame (usually after a large camera movement.) */
	bool bDisableDistanceBasedFadeTransitions;

	/** Whether this view should be excluded from extent history tracking. */
	bool bExcludeFromSceneTextureExtents = false;

	/** True if we should force the path tracer to reset its internal accumulation state */
	bool bForcePathTracerReset;

	/** Sets an offset to the seed the path tracer is using. */
	int32 PathTracerSeedOffset;

	/** How far below the water surface this view is. -1 means the view is out of water. */
	float UnderwaterDepth;

	/** Whether the view intersects water. */
	EViewWaterIntersection WaterIntersection = EViewWaterIntersection::OutsideWater;

	/** Global clipping plane being applied to the scene, or all 0's if disabled.  This is used when rendering the planar reflection pass. */
	FPlane GlobalClippingPlane;

	/** Aspect ratio constrained view rect. In the editor, when attached to a camera actor and the camera black bar showflag is enabled, the normal viewrect 
	  * remains as the full viewport, and the black bars are just simulated by drawing black bars. This member stores the effective constrained area within the
	  * bars.
	 **/
	FIntRect CameraConstrainedViewRect;

	/** Sort axis for when TranslucentSortPolicy is SortAlongAxis */
	FVector TranslucentSortAxis;

	/** Translucent sort mode */
	TEnumAsByte<ETranslucentSortPolicy::Type> TranslucentSortPolicy;
	
	/** The frame index to override, useful for keeping determinism when rendering sequences. **/
	TOptional<uint32> OverrideFrameIndexValue;
	/** 
	* If set, overrides the Output Frame Index. Useful for Halton-like offsets that should produce the same results in multi-sample accumulation when rendering sequences. 
	* If unset, renderer defaults to FrameIndex.
	*/
	TOptional<uint32> OverrideOutputFrameIndexValue;

	/** In some cases, the principal point of the lens is not at the center of the screen, especially for overlapped tile
	 *  rendering. So given a UV in [-1,1] viewport space, convert it to the [-1,1] viewport space of the lens using
	 *  LensUV = LensPrincipalPointOffsetScale.xy ScreenUV * LensPrincipalPointOffsetScale.zw;
	 *  This value is FVector4(0,0,1,1) unless overridden.
	 */
	FVector4f LensPrincipalPointOffsetScale;

	/** Whether to enable motion blur caused by camera movements */
	TOptional<bool> bCameraMotionBlur;

	/** Matrix overrides PrevViewToClip in the view uniform buffer.
		Used for replacing motion blur caused by the current camera movements with a custom transform.*/
	TOptional<FMatrix> ClipToPrevClipOverride;

#if WITH_EDITOR
	/** The set of (the first 64) groups' visibility info for this view */
	uint64 EditorViewBitflag;

	/** True if we should draw translucent objects when rendering hit proxies */
	bool bAllowTranslucentPrimitivesInHitProxy;

	/** BitArray representing the visibility state of the various sprite categories in the editor for this view */
	TBitArray<> SpriteCategoryVisibility;
	/** Selection color for the editor (used by post processing) */
	FLinearColor SelectionOutlineColor;
	/** Selection color for use in the editor with inactive primitives */
	FLinearColor SubduedSelectionOutlineColor;
	/** Additional selection colors for the editor (used by post processing) */
	TStaticArray<FLinearColor, 6> AdditionalSelectionOutlineColors;
	/** True if any components are selected in isolation (independent of actor selection) */
	bool bHasSelectedComponents;

	struct FPostProcessCompositePrimitiveParams
	{
		/** Size (in pixels) of the dithering pattern applied to occluded primitives. */
		float OccludedDitheringPatternSize = 2.0f;
		/** The strength of the dithering applied to occluded primitives. */
		float OccludedDithering = 0.8f;
		/** The brightness of occluded primitives. */
		float OccludedBrightness = 0.7f;
	} CompositePrimitiveParams;

	/** In editor, this is used to feed rendering info back to the viewport. May be null. */
	TSharedPtr<UE::RenderViewportFeedback::FCollector> ViewportFeedback;
#endif

	/**
	 * The final settings for the current viewer position (blended together from many volumes).
	 * Setup by the main thread, passed to the render thread and never touched again by the main thread.
	 */
	FFinalPostProcessSettings FinalPostProcessSettings;
#if DEBUG_POST_PROCESS_VOLUME_ENABLE
	FName PostProcessDebugPropertyName;
	TArray<FPostProcessSettingsDebugInfo> FinalPostProcessDebugInfo;
#endif

	// The antialiasing method.
	EAntiAliasingMethod AntiAliasingMethod;

	// Primary screen percentage method to use.
	EPrimaryScreenPercentageMethod PrimaryScreenPercentageMethod;

	/** Parameters for atmospheric fog. */
	FTextureRHIRef AtmosphereTransmittanceTexture;
	FTextureRHIRef AtmosphereIrradianceTexture;
	FTextureRHIRef AtmosphereInscatterTexture;

	/** Water rendering related data */
	FShaderResourceViewRHIRef WaterIndirectionBuffer;
	FShaderResourceViewRHIRef WaterDataBuffer;
	/** Index of the water info texture slice corresponding to this view. */
	int32 WaterInfoTextureViewIndex = INDEX_NONE;

	struct FWaterInfoTextureRenderingParams
	{
		FRenderTarget* RenderTarget = nullptr;
		FVector ViewLocation = FVector::Zero();
		FMatrix ViewRotationMatrix = FMatrix(EForceInit::ForceInit);
		FMatrix ProjectionMatrix = FMatrix(EForceInit::ForceInit);
		TSet<FPrimitiveComponentId> TerrainComponentIds;
		TSet<FPrimitiveComponentId> WaterBodyComponentIds;
		TSet<FPrimitiveComponentId> DilatedWaterBodyComponentIds;
		FVector WaterZoneExtents = FVector::Zero();
		FVector2f WaterHeightExtents = FVector2f::ZeroVector;
		float GroundZMin = 0.0f;
		float CaptureZ = 0.0f;
		int32 VelocityBlurRadius = 0;
		int32 RenderTargetArrayLayer = 0;
	};
	TArray<FWaterInfoTextureRenderingParams> WaterInfoTextureRenderingParams;

	/** Landscape rendering related data */
	FShaderResourceViewRHIRef LandscapeIndirectionBuffer;
	FShaderResourceViewRHIRef LandscapePerComponentDataBuffer;

	/** Enables a CoC offset whose offset value changes with pixel's scene depth */
	bool bEnableDynamicCocOffset = false;

	/** When dynamic CoC offset is enabled, this is the distance from camera at which objects will be in perfect focus (when the CoC offset is maximum so that the final CoC is 0) */
	float InFocusDistance = 0.0;

	/** Lookup table for the dynamic CoC offset */
	FTextureRHIRef DynamicCocOffsetLUT;

	/** Feature level for this scene */
	const ERHIFeatureLevel::Type FeatureLevel;

	/** When using mobile multi view fallback path we need to instance draw calls ourselves to cover both eyes instead of letting the drivers do it for us. */
	uint32 InstanceFactor = 1;

	/** Set to false to disable ray tracing on this view.  Use IsRayTracingAllowedForView() function to test for ray tracing, not this bool. */
	bool bAllowRayTracing = true;

	/** View light masking support.  Some systems assume the same mask setting for all views within a family, with potential artifacts if views use different masks. */
	uint8 ViewLightingChannelMask = FViewLightingChannels::GetMaskDefault();

	/**  Stereo aspects of the shader pipeline based on this view's shader platform */
	UE::StereoRenderUtils::FStereoShaderAspects Aspects;

protected:
	friend class FSceneRenderer;

	/** Some views get cloned for certain renders, like shadows (see FViewInfo::CreateSnapshot()). If that's the case, this will point to the view this one originates from */
	const FSceneView* SnapshotOriginView = nullptr;

public:

	/** Initialization constructor. */
	ENGINE_API FSceneView(const FSceneViewInitOptions& InitOptions);

	/** These are only needed because of deprecated members being accessed in them */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSceneView(const FSceneView& Other) = default;
	FSceneView(FSceneView&& Other) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Default destructor */
	ENGINE_API virtual ~FSceneView();

#if DO_CHECK || USING_CODE_ANALYSIS
	/** Verifies all the assertions made on members. */
	ENGINE_API bool VerifyMembersChecks() const;
#endif

	inline bool AllowGPUParticleUpdate() const { return !bIsPlanarReflection && !bIsSceneCapture && !bIsReflectionCapture; }

	/** Transforms a point from world-space to the view's screen-space. */
	ENGINE_API FVector4 WorldToScreen(const FVector& WorldPoint) const;

	/** Transforms a point from the view's screen-space to world-space. */
	ENGINE_API FVector ScreenToWorld(const FVector4& ScreenPoint) const;

	/** Transforms a point from the view's screen-space into pixel coordinates relative to the view's X,Y. */
	ENGINE_API bool ScreenToPixel(const FVector4& ScreenPoint,FVector2D& OutPixelLocation) const;

	/** Transforms a point from pixel coordinates relative to the view's X,Y (left, top) into the view's screen-space. */
	ENGINE_API FVector4 PixelToScreen(float X,float Y,float Z) const;

	/** Transforms a cursor location in render target pixel coordinates into the view's screen-space, taking into account the viewport rectangle. */
	ENGINE_API FVector4 CursorToScreen(float X, float Y, float Z, bool bClamped = true) const;

	/** Transforms a point from the view's world-space into pixel coordinates relative to the view's X,Y (left, top). */
	ENGINE_API bool WorldToPixel(const FVector& WorldPoint,FVector2D& OutPixelLocation) const;

	/** Transforms a point from pixel coordinates relative to the view's X,Y (left, top) into the view's world-space. */
	ENGINE_API FVector4 PixelToWorld(float X,float Y,float Z) const;

	/** 
	 * Transforms a point from the view's world-space into the view's screen-space. 
	 * Divides the resulting X, Y, Z by W before returning. 
	 */
	ENGINE_API FPlane Project(const FVector& WorldPoint) const;

	/** 
	 * Transforms a point from the view's screen-space into world coordinates
	 * multiplies X, Y, Z by W before transforming. 
	 */
	ENGINE_API FVector Deproject(const FPlane& ScreenPoint) const;

	/** 
	 * Transforms 2D screen coordinates into a 3D world-space origin and direction 
	 * @param ScreenPos - screen coordinates in pixels
	 * @param out_WorldOrigin (out) - world-space origin vector
	 * @param out_WorldDirection (out) - world-space direction vector
	 */
	ENGINE_API void DeprojectFVector2D(const FVector2D& ScreenPos, FVector& out_WorldOrigin, FVector& out_WorldDirection) const;

	/** 
	 * Transforms 2D screen coordinates into a 3D world-space origin and direction 
	 * @param ScreenPos - screen coordinates in pixels
	 * @param ViewRect - view rectangle
	 * @param InvViewMatrix - inverse view matrix
	 * @param InvProjMatrix - inverse projection matrix
	 * @param out_WorldOrigin (out) - world-space origin vector
	 * @param out_WorldDirection (out) - world-space direction vector
	 */
	static ENGINE_API void DeprojectScreenToWorld(const FVector2D& ScreenPos, const FIntRect& ViewRect, const FMatrix& InvViewMatrix, const FMatrix& InvProjMatrix, FVector& out_WorldOrigin, FVector& out_WorldDirection);

	/** Overload to take a single combined view projection matrix. */
	static ENGINE_API void DeprojectScreenToWorld(const FVector2D& ScreenPos, const FIntRect& ViewRect, const FMatrix& InvViewProjMatrix, FVector& out_WorldOrigin, FVector& out_WorldDirection);

	/** 
	 * Transforms 3D world-space origin into 2D screen coordinates
	 * @param WorldPosition - the 3d world point to transform
	 * @param ViewRect - view rectangle
	 * @param ViewProjectionMatrix - combined view projection matrix
	 * @param out_ScreenPos (out) - screen coordinates in pixels
	 * @param bShouldCalcOutsideViewPosition - if enabled, calculates the out_ScreenPos if the WorldPosition is outside of ViewProjectionMatrix
	 */
	static ENGINE_API bool ProjectWorldToScreen(const FVector& WorldPosition, const FIntRect& ViewRect, const FMatrix& ViewProjectionMatrix, FVector2D& out_ScreenPos, bool bShouldCalcOutsideViewPosition = false);

	inline FVector GetViewRight() const { return ViewMatrices.GetWorldToView().GetColumn(0); }
	inline FVector GetViewUp() const { return ViewMatrices.GetWorldToView().GetColumn(1); }
	inline FVector GetViewDirection() const { return ViewMatrices.GetWorldToView().GetColumn(2); }

	inline const FConvexVolume* GetDynamicMeshElementsShadowCullFrustum() const { return DynamicMeshElementsShadowCullFrustum; }
	inline void SetDynamicMeshElementsShadowCullFrustum(const FConvexVolume* InDynamicMeshElementsShadowCullFrustum) { DynamicMeshElementsShadowCullFrustum = InDynamicMeshElementsShadowCullFrustum; }

	inline const FVector& GetPreShadowTranslation() const { return PreShadowTranslation; }
	inline void SetPreShadowTranslation(const FVector& InPreShadowTranslation) { PreShadowTranslation = InPreShadowTranslation; }

	/** @return true:perspective, false:orthographic */
	inline bool IsPerspectiveProjection() const { return ViewMatrices.IsPerspectiveProjection(); }

	bool IsUnderwater() const { return UnderwaterDepth > 0.0f; }

	/** Returns the location used as the origin for LOD computations
	 * @param Index, 0 or 1, which LOD origin to return
	 * @return LOD origin
	 */
	ENGINE_API FVector GetTemporalLODOrigin(int32 Index, bool bUseLaggedLODTransition = true) const;

	/** 
	 * Returns the blend factor between the last two LOD samples
	 */
	ENGINE_API float GetTemporalLODTransition() const;

	/**
	 * returns the distance field temporal sample index or 0 if there is no view state
	 */
	ENGINE_API uint32 GetDistanceFieldTemporalSampleIndex() const;

	/** 
	 * returns a unique key for the view state if one exists, otherwise returns zero
	 */
	ENGINE_API uint32 GetViewKey() const;

	/** 
	 * returns a the occlusion frame counter or MAX_uint32 if there is no view state
	 */
	ENGINE_API uint32 GetOcclusionFrameCounter() const;

	ENGINE_API void UpdateProjectionMatrix(const FMatrix& NewProjectionMatrix);

	/** Allow things like HMD displays to update the view matrix at the last minute, to minimize perceived latency */
	ENGINE_API void UpdateViewMatrix();

	/** If we late update a view, we need to also late update any planar reflection views derived from it */
	ENGINE_API void UpdatePlanarReflectionViewMatrix(const FSceneView& SourceView, const FMirrorMatrix& MirrorMatrix);

	/** Setup defaults and depending on view position (postprocess volumes) */
	ENGINE_API void StartFinalPostprocessSettings(FVector InViewLocation);

	/**
	 * custom layers can be combined with the existing settings
	 * @param Weight usually 0..1 but outside range is clamped
	 */
	ENGINE_API void OverridePostProcessSettings(const FPostProcessSettings& Src, float Weight, bool bAllowBlendables = true);

	/** applied global restrictions from show flags */
	ENGINE_API void EndFinalPostprocessSettings(const FSceneViewInitOptions& ViewInitOptions);

	ENGINE_API void SetupAntiAliasingMethod();

	/** Configure post process settings for the buffer visualization system */
	ENGINE_API void ConfigureBufferVisualizationSettings();

#if !(UE_BUILD_SHIPPING)
	/** Configure post process settings for calibration material */
	ENGINE_API void ConfigureVisualizeCalibrationSettings();
#endif

	/** Get the feature level for this view (cached from the scene so this is not different per view) **/
	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }

	/** Get the feature level for this view **/
	ENGINE_API EShaderPlatform GetShaderPlatform() const;

	/** True if the view should render as an instanced stereo pass */
	UE_DEPRECATED(5.9, "IsInstancedStereoPass() is deprecated, use IsInstancedStereo() instead")
	ENGINE_API bool IsInstancedStereoPass() const;

	/** Instance factor for a stereo pass (normally 2 for ISR views, but see IStereoRendering::GetDesiredNumberOfViews()). Returns 1 for non-instanced stereo views or regular (split screen etc) views. */
	int32 GetStereoPassInstanceFactor() const { return InstanceFactor; }

	/** Sets up the view rect parameters in the view's uniform shader parameters */
	ENGINE_API void SetupViewRectUniformBufferParameters(FViewUniformShaderParameters& ViewUniformShaderParameters, 
		const FIntPoint& InBufferSize,
		const FIntRect& InEffectiveViewRect,
		const FViewMatrices& InViewMatrices,
		const FViewMatrices& InPrevViewMatrice) const;

	static ENGINE_API FVector4f GetScreenPositionScaleBias(const FIntPoint& BufferSize, const FIntRect& ViewRect);

	/** 
	 * Populates the uniform buffer prameters common to all scene view use cases
	 * View parameters should be set up in this method if they are required for the view to render properly.
	 * This is to avoid code duplication and uninitialized parameters in other places that create view uniform parameters (e.g Slate) 
	 */
	ENGINE_API void SetupCommonViewUniformBufferParameters(FViewUniformShaderParameters& ViewUniformShaderParameters,
		const FIntPoint& InBufferSize,
		int32 NumMSAASamples,
		const FIntRect& InEffectiveViewRect,
		const FViewMatrices& InViewMatrices,
		const FViewMatrices& InPrevViewMatrices) const;

	/** Current ray tracing debug visualization mode */
	FName CurrentRayTracingDebugVisualizationMode;

	/** Tells if the eye adaptation buffer exists without attempting to allocate it. */
	ENGINE_API bool HasValidEyeAdaptationBuffer() const;

	/** Returns the eye adaptation buffer or null if it doesn't exist. */
	ENGINE_API FRDGPooledBuffer* GetEyeAdaptationBuffer() const;

	/** Returns the eye adaptation exposure or 0.0f if it doesn't exist. */
	ENGINE_API float GetLastEyeAdaptationExposure() const;

	/** Returns the average luminance or 0.0f if it doesn't exist. */
	ENGINE_API float GetLastAverageSceneLuminance() const;

	/** Get the primary view associated with the secondary view. */
	ENGINE_API const FSceneView* GetPrimarySceneView() const;

	/** Checks whether this is the primary view of a stereo pair (important in instanced stereo rendering). Will also be true for any view that isn't stereo. */
	inline bool IsPrimarySceneView() const
	{
		return GetPrimarySceneView() == this;
	}

	/** Get the first secondary view associated with the primary view. */
	ENGINE_API const FSceneView* GetInstancedSceneView() const;

	/** Checks whether this is the instanced view of a stereo pair. If the technique supports instanced rendering stereo and ISR is enabled, such views can be skipped. */
	inline bool IsInstancedSceneView() const
	{
		return GetInstancedSceneView() == this;
	}

	/** Get all secondary views associated with the primary view. */
	ENGINE_API TArray<const FSceneView*> GetSecondaryViews() const;

	/** Returns uniform buffer that is used to access (first) instanced view's properties. Note: it is not the same as that view's ViewUniformBuffer, albeit it contains all relevant data from it. */
	const TUniformBufferRef<FInstancedViewUniformShaderParameters>& GetInstancedViewUniformBuffer() const
	{
		return InstancedViewUniformBuffer;
	}

	/** Returns the load action to use when overwriting all pixels of a target that you intend to read from. Takes into account the HMD hidden area mesh. */
	inline ERenderTargetLoadAction GetOverwriteLoadAction() const
	{
		return bHMDHiddenAreaMaskActive ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ENoAction;
	}

	const FSceneView* GetSnapshotOriginView() const { return SnapshotOriginView; }

	inline bool IsRayTracingAllowedForView() const
	{
		return bAllowRayTracing && (!bIsSceneCapture || bSceneCaptureUsesRayTracing);
	}

	inline bool HasEyeAdaptationViewState() const
	{
		return EyeAdaptationViewState != nullptr;
	}

	inline bool IsRuntimeReflectionCapture() const
	{
		return bIsSceneCaptureCube && bIsReflectionCapture;
	}

	/** Buffer to the shader binding data used for inline raytracing - only valid when inline raytracing is enabled. */
	virtual FRDGBufferRef GetInlineRayTracingBindingDataBuffer() const { return nullptr; }

protected:
	friend class FViewSnapshotCache;

	FSceneViewStateInterface* EyeAdaptationViewState = nullptr;

	/** Instanced view uniform buffer held on the primary view. */
	TUniformBufferRef<FInstancedViewUniformShaderParameters> InstancedViewUniformBuffer;
};

//////////////////////////////////////////////////////////////////////////

// for r.DisplayInternals (allows for easy passing down data from main to render thread)
struct FDisplayInternalsData
{
	//
	int32 DisplayInternalsCVarValue;
	// -1 if not set, from IStreamingManager::Get().StreamAllResources(Duration) in FStreamAllResourcesLatentCommand
	uint32 NumPendingStreamingRequests;

	FDisplayInternalsData()
		: DisplayInternalsCVarValue(0)
		, NumPendingStreamingRequests(-1)
	{
		check(!IsValid());
	}

	// called on main thread
	// @param World may be 0
	void Setup(UWorld *World);

	bool IsValid() const { return DisplayInternalsCVarValue != 0; }
};

//////////////////////////////////////////////////////////////////////////

/**
 * Generic plugin extension that have a lifetime of the FSceneViewFamily
 */
class ISceneViewFamilyExtention
{
protected:
	/** 
	 * Called by the destructor of the view family.
	 * Can be called on game or rendering thread.
	 */
	virtual ~ISceneViewFamilyExtention() = default;

	friend class FSceneViewFamily;
};

/**
 * Generic plugin extension that have a lifetime of the FSceneViewFamily that can contain arbitrary data to passdown from game thread to render thread.
 */
class ISceneViewFamilyExtentionData : public ISceneViewFamilyExtention
{
public:
	/** Returns a const TCHAR* to uniquely identify what implementation ISceneViewFamilyExtentionData is. */
	virtual const TCHAR* GetSubclassIdentifier() const = 0;
};


/*
 * Game thread and render thread interface that takes care of a FSceneViewFamily's screen percentage.
 *
 * The renderer reserves the right to delete and replace the view family's screen percentage interface
 * for testing purposes with the r.Test.OverrideScreenPercentageInterface CVar.
 */
class ISceneViewFamilyScreenPercentage : ISceneViewFamilyExtention
{
public:
	// Sets the minimal and max screen percentage.
	static constexpr float kMinResolutionFraction = 0.01f;
	static constexpr float kMaxResolutionFraction = 4.0f;

	// Sets the minimal and maximal screen percentage for TSR.
	static constexpr float kMinTSRResolutionFraction = 0.25f;
	static constexpr float kMaxTSRResolutionFraction = 2.0f;

	// Sets the minimal and maximal screen percentage for TAAU.
	static constexpr float kMinTAAUpsampleResolutionFraction = 0.5f;
	static constexpr float kMaxTAAUpsampleResolutionFraction = 2.0f;

#if DO_CHECK || USING_CODE_ANALYSIS
	static bool IsValidResolutionFraction(float ResolutionFraction)
	{
		return ResolutionFraction >= kMinResolutionFraction && ResolutionFraction <= kMaxResolutionFraction;
	}
#endif

	/** 
	 * Method to know the maximum value that can be returned by GetPrimaryResolutionFraction_RenderThread().
	 * Can be called on game or rendering thread. This should return >= 1 if screen percentage show flag is disabled.
	 */
	virtual DynamicRenderScaling::TMap<float> GetResolutionFractionsUpperBound() const = 0;

	/** Create a new screen percentage interface for a new view family. */
	virtual ISceneViewFamilyScreenPercentage* Fork_GameThread(const class FSceneViewFamily& ViewFamily) const = 0;

protected:
	/**
	 * Setup view family's view's screen percentage on rendering thread.
	 * This should leave ResolutionFraction == 1 if screen percentage show flag is disabled.
	 */
	virtual DynamicRenderScaling::TMap<float> GetResolutionFractions_RenderThread() const = 0;

	friend class FSceneViewFamily;
	friend class FSceneRenderer;
};


//////////////////////////////////////////////////////////////////////////

/**
 * A set of views into a scene which only have different view transforms and owner actors.
 */
class FSceneViewFamily
{
public:
	/**
	* Helper struct for creating FSceneViewFamily instances
	* If created with specifying a time it will retrieve them from the world in the given scene.
	* 
	* @param InRenderTarget		The render target which the views are being rendered to.
	* @param InScene			The scene being viewed.
	* @param InShowFlags		The show flags for the views.
	*
	*/
	struct ConstructionValues
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ConstructionValues(ConstructionValues&&) = default;
		ConstructionValues(const ConstructionValues& Other) = default;
		ConstructionValues& operator=(ConstructionValues&&) = default;
		ConstructionValues& operator=(const ConstructionValues& Other) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		ENGINE_API ConstructionValues(
			const FRenderTarget* InRenderTarget,
			FSceneInterface* InScene,
			const FEngineShowFlags& InEngineShowFlags
			);

		/** The views which make up the family. */
		const FRenderTarget* RenderTarget;
		const FRenderTarget* RenderTargetDepth;

		/** The render target which the views are being rendered to. */
		FSceneInterface* Scene;

		/** The engine show flags for the views. */
		FEngineShowFlags EngineShowFlags;

		/** Additional view params related to the current viewmode (example : texcoord index) */
		int32 ViewModeParam;
		/** An name bound to the current viewmode param. (example : texture name) */
		FName ViewModeParamName;

		/** The current time. */
		FGameTime Time;

		/** Indicates whether the view family is additional. */
		uint32 bAdditionalViewFamily : 1;

		/** Indicates whether the view family is updated in real-time. */
		uint32 bRealtimeUpdate:1;
		
		/** Used to defer the back buffer clearing to just before the back buffer is drawn to */
		uint32 bDeferClear:1;
		
		/** If true then results of scene rendering are copied/resolved to the RenderTarget. */
		uint32 bResolveScene:1;		
		
		/** Safety check to ensure valid times are set either from a valid world/scene pointer or via the SetWorldTimes function */
		uint32 bTimesSet:1;

		/** True if scene color and depth should be multiview-allocated */
		uint32 bRequireMultiView:1;

		ConstructionValues& SetRenderTargetDepth(const FRenderTarget* InRenderTargetDepth)
		{
			RenderTargetDepth = InRenderTargetDepth;
			return *this;
		}

		/** Set the world time and real time independently to handle time dilation. */
		ConstructionValues& SetTime(const FGameTime& InTime)
		{
			Time = InTime;
			bTimesSet = true;
			return *this;
		}

		/** Set  whether the view family is additional. */
		ConstructionValues& SetAdditionalViewFamily(const bool Value) { bAdditionalViewFamily = Value; return *this; }

		/** Set  whether the view family is updated in real-time. */
		ConstructionValues& SetRealtimeUpdate(const bool Value) { bRealtimeUpdate = Value; return *this; }
		
		/** Set whether to defer the back buffer clearing to just before the back buffer is drawn to */
		ConstructionValues& SetDeferClear(const bool Value) { bDeferClear = Value; return *this; }
		
		/** Setting to if true then results of scene rendering are copied/resolved to the RenderTarget. */
		ConstructionValues& SetResolveScene(const bool Value) { bResolveScene = Value; return *this; }

		/** Setting to true results in scene color and depth being multiview-allocated. */
		ConstructionValues& SetRequireMobileMultiView(const bool Value) { bRequireMultiView = Value; return *this; }

		/** Set the view param. */
		ConstructionValues& SetViewModeParam(const int InViewModeParam, const FName& InViewModeParamName) { ViewModeParam = InViewModeParam; ViewModeParamName = InViewModeParamName; return *this; }		
	};
	
	/** The views which make up the family. */
	TArray<const FSceneView*> Views;

	/** All views include main camera views and scene capture views. */
	TArray<const FSceneView*> AllViews;

	/** View mode of the family. */
	EViewModeIndex ViewMode;

	/** The render target which the views are being rendered to. */
	const FRenderTarget* RenderTarget;
	const FRenderTarget* RenderTargetDepth;

	/** The scene being viewed. */
	FSceneInterface* Scene;

	/** The new show flags for the views (meant to replace the old system). */
	FEngineShowFlags EngineShowFlags;

	/** The current time. */
	FGameTime Time;

	/** Copy from main thread GFrameNumber to be accessible on render thread side. UINT_MAX before CreateSceneRenderer() or BeginRenderingViewFamily() was called */
	uint32 FrameNumber;

	/** Copy from main thread GFrameCounter to be accessible on render thread side. GFrameCounter is incremented once per engine tick, so multi views of the same frame have the same value. */
	uint64 FrameCounter = 0;

	/** Indicates this view family is an additional one. */
	bool bAdditionalViewFamily;

	/** Indicates whether the view family is updated in realtime. */
	bool bRealtimeUpdate;

	/** Used to defer the back buffer clearing to just before the back buffer is drawn to */
	bool bDeferClear;

	/** if true then results of scene rendering are copied/resolved to the RenderTarget. */
	bool bResolveScene;

	/** if true then each view is not rendered using the same GPUMask. */
	bool bMultiGPUForkAndJoin;

#if WITH_MGPU
	/** Force a cross-GPU copy of all persistent resources after rendering this view family (except those marked MultiGPUGraphIgnore) */
	bool bForceCopyCrossGPU = false;
#endif

	UE_DEPRECATED(5.6, "bIsMultipleViewFamily is no longer used")
	bool bIsMultipleViewFamily = false;
	
	/** 
	* Whether this view is the first of multiple view families rendered in a single frame.
	* Setting this correctly helps make better and more consistent streaming decisions.
	* This should be a temporary workaround until we have an API for executing code before/after all view rendering.
	*/
	bool bIsFirstViewInMultipleViewFamily = true;

	bool bIsViewFamilyInfo = false;

	/** Whether this view allows split screen debug -- we only want it for editor and game viewports, not other random scene renders */
	bool bSplitScreenDebugAllowed = false;

	/**
	 * Whether this is the "main" view family.  Should be set to true for the view family of the main editor or game viewport.  Affects
	 * where rendering occurs for Scene Captures with the "bMainViewFamily" flag set.
	 */
	bool bIsMainViewFamily = false;

	/**
	 * Which component of the scene rendering should be output to the final render target.
	 * If SCS_FinalColorLDR this indicates do nothing.
	 */
	ESceneCaptureSource SceneCaptureSource;
	

	/** When enabled, the scene capture will composite into the render target instead of overwriting its contents. */
	ESceneCaptureCompositeMode SceneCaptureCompositeMode;

	/** Whether this view is for thumbnail rendering */
	bool bThumbnailRendering = false;

	/**
	 * GetWorld->IsPaused() && !Simulate
	 * Simulate is excluded as the camera can move which invalidates motionblur
	 */
	bool bWorldIsPaused;

	/** When enabled, the post processing will output in HDR space */
	bool bIsHDR;

	/** True if scenecolor and depth should be multiview-allocated */
	bool bRequireMultiView;

	/** Parameters for masking specific color channels for visualization */
	struct FColorChannelMaskParams
	{
		/** The color channel to mask to, whose value will be output as a grayscale color */
		EColorChannelMask ColorChannelMask = EColorChannelMask::All;

		/** When true, the alpha channel is inverted when being visualized; does not effect the actual alpha value, just its visualization */
		bool bInvertAlphaChannelMask = false;

		/** Indicates that a checkerboard should be alpha blended underneath the viewport to help visualize the view's alpha */
		bool bDrawAlphaBlendedCheckerboard = false;
	} ChannelMaskParams;
	
	/** DPI scale to be used for debugging font. */
	float DebugDPIScale = 1.0f;

	/** Scale between the current rendering resolution and the final output resolution.
	 *  Used e.g. by high-res tiled screenshot capture where individual tiles are rendered
	 *  at a smaller resolution and composited into a larger final image */
	float FinalResolutionScale = 1.0f;

	/** Editor setting to allow designers to override the automatic expose. 0:Automatic, following indices: -4 .. +4 */
	FExposureSettings ExposureSettings;

	/** Extensions that can modify view parameters on the render thread. */
	TArray<TSharedRef<class ISceneViewExtension, ESPMode::ThreadSafe> > ViewExtensions;

	// for r.DisplayInternals (allows for easy passing down data from main to render thread)
	FDisplayInternalsData DisplayInternalsData;

	/**
	 * Secondary view fraction to support High DPI monitor still with same primary screen percentage range for temporal
	 * upscale to test content consistently in editor no mater of the HighDPI scale. 
	 */
	float SecondaryViewFraction;
	ESecondaryScreenPercentageMethod SecondaryScreenPercentageMethod;

	// Override the LOD of landscape in this viewport
	int8 LandscapeLODOverride;

	/** Whether the scene is currently being edited, which can be used to speed up lighting propagation. */
	bool bCurrentlyBeingEdited;

	/** Whether to disable virtual texture update throttling. */
	bool bOverrideVirtualTextureThrottle;

	/** Size in pixels of a virtual texture feedback tile. */
	int32 VirtualTextureFeedbackFactor;

#if WITH_EDITOR
	/** Indicates whether, or not, the base attachment volume should be drawn. */
	bool bDrawBaseInfo;

	/**
	 * Indicates whether the shader world space position should be forced to 0. Also sets the view vector to (0,0,1) for all pixels.
	 * This is used in the texture streaming build when computing material tex coords scale.
	 * Because the material are rendered in tiles, there is no actual valid mapping for world space position.
	 * World space mapping would require to render mesh with the level transforms to be valid.
	 */
	bool bNullifyWorldSpacePosition;
#endif

	// Optional description of view family for Unreal Insights profiling
	FString ProfileDescription;

	UE_DEPRECATED(5.6, "This member is no longer used")
	float* ProfileSceneRenderTime = nullptr;

	/** Views' origin of the (optional) streaming views. Used for prefetching rendering data ahead of time. */
	TArray<FVector, TInlineAllocator<2>> StreamingViewOrigins;

	/** Initialization constructor. */
	ENGINE_API FSceneViewFamily( const ConstructionValues& CVS );
	ENGINE_API FSceneViewFamily(const FSceneViewFamily&);
	ENGINE_API virtual ~FSceneViewFamily();

	ENGINE_API ERHIFeatureLevel::Type GetFeatureLevel() const;

	EShaderPlatform GetShaderPlatform() const { return GetFeatureLevelShaderPlatform(GetFeatureLevel()); }

#if WITH_DEBUG_VIEW_MODES
	UE_DEPRECATED(5.7, "Use GetDebugViewShaderMode()")
	mutable EDebugViewShaderMode DebugViewShaderMode = DVSM_MAX; // should be private
	UE_DEPRECATED(5.7, "Use GetViewModeParam()")
	int32 ViewModeParam;
	UE_DEPRECATED(5.7, "Use GetViewModeParamName()")
	FName ViewModeParamName;

	UE_DEPRECATED(5.7, "Use GetDebugViewShaderMode()")
	ENGINE_API EDebugViewShaderMode ChooseDebugViewShaderMode() const;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Return the debug viewmode last set by SetDebugViewShaderMode.
	// If no debug viewmode was set, this function will set one based on showflags.
	ENGINE_API EDebugViewShaderMode GetDebugViewShaderMode() const;
	void SetDebugViewShaderMode(EDebugViewShaderMode InMode) { DebugViewShaderMode = InMode; }
	inline int32 GetViewModeParam() const { return ViewModeParam; }
	inline const FName& GetViewModeParamName() const { return ViewModeParamName; }	
	inline bool UseDebugViewPS() const { return GetDebugViewShaderMode() != DVSM_None; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#else
	inline EDebugViewShaderMode GetDebugViewShaderMode() const { return DVSM_None; }
	void SetDebugViewShaderMode(EDebugViewShaderMode InMode) { }
	inline int32 GetViewModeParam() const { return -1; }
	inline FName GetViewModeParamName() const { return NAME_None; }
	inline bool UseDebugViewPS() const { return false; }
#endif

	/** Returns whether the screen percentage show flag is supported or not for this view family. */
	ENGINE_API bool SupportsScreenPercentage() const;

	inline bool AllowTranslucencyAfterDOF() const { return bAllowTranslucencyAfterDOF; }

	inline bool AllowStandardTranslucencySeparated() const { return bAllowStandardTranslucencySeparated; }

	inline const ISceneViewFamilyScreenPercentage* GetScreenPercentageInterface() const
	{
		return ScreenPercentageInterface;
	}

	/**
	 * Safely sets the view family's screen percentage interface.
	 * This is meant to be set by one of the ISceneViewExtension::BeginRenderViewFamily(). And collision will
	 * automatically be detected. If no extension sets it, that is fine since the renderer is going to use an
	 * internal default one.
	 *
	 * The renderer reserves the right to delete and replace the view family's screen percentage interface
	 * for testing purposes with the r.Test.OverrideScreenPercentageInterface CVar.
	 */
	inline void SetScreenPercentageInterface(ISceneViewFamilyScreenPercentage* InScreenPercentageInterface)
	{
		check(InScreenPercentageInterface);
		checkf(ScreenPercentageInterface == nullptr, TEXT("View family already had a screen percentage interface assigned."));
		ScreenPercentageInterface = InScreenPercentageInterface;
	}

	inline void SetScreenPercentageInterface_Unchecked(ISceneViewFamilyScreenPercentage* InScreenPercentageInterface)
	{
		ScreenPercentageInterface = InScreenPercentageInterface;
	}

	// View family assignment operator is not allowed because of ScreenPercentageInterface lifetime.
	void operator = (const FSceneViewFamily&) = delete;

	// Allow moving view family as long as no screen percentage interface are set.
	ENGINE_API FSceneViewFamily(FSceneViewFamily&& InViewFamily);

	template<typename TExtensionData> const TExtensionData* GetExtentionData() const
	{
		static_assert(TIsDerivedFrom<TExtensionData, ISceneViewFamilyExtentionData>::Value, "TExtensionData is not derived from ISceneViewFamilyExtentionData.");

		for (const TSharedRef<class ISceneViewFamilyExtentionData, ESPMode::ThreadSafe>& ViewExtensionData : ViewExtentionDatas)
		{
			if (ViewExtensionData->GetSubclassIdentifier() == TExtensionData::GSubclassIdentifier)
			{
				return static_cast<const TExtensionData*>(&ViewExtensionData.Get());
			}
		}
		return nullptr;
	}

	template<typename TExtensionData> TExtensionData* GetExtentionData()
	{
		return const_cast<TExtensionData*>(AsConst(*this).GetExtentionData<TExtensionData>());
	}

	template<typename TExtensionData> TExtensionData* GetOrCreateExtentionData()
	{
		TExtensionData* ViewExtensionData = GetExtentionData<TExtensionData>();
		if (!ViewExtensionData)
		{
			ViewExtentionDatas.Push(MakeShared<TExtensionData>());
			ViewExtensionData = static_cast<TExtensionData*>(&ViewExtentionDatas.Last().Get());
			check(ViewExtensionData->GetSubclassIdentifier() == TExtensionData::GSubclassIdentifier);
		}
		return ViewExtensionData;
	}

	inline void SetTemporalUpscalerInterface(UE::Renderer::Private::ITemporalUpscaler* InTemporalUpscalerInterface)
	{
		check(InTemporalUpscalerInterface);
		checkf(TemporalUpscalerInterface == nullptr, TEXT("View family already had a temporal upscaler assigned."));
		TemporalUpscalerInterface = InTemporalUpscalerInterface;
	}

	inline const UE::Renderer::Private::ITemporalUpscaler* GetTemporalUpscalerInterface() const
	{
		return TemporalUpscalerInterface;
	}

	inline void SetMegaFiltersInterface(UE::Renderer::Private::IMegaFilters* InMegaFiltersInterface)
	{
		check(InMegaFiltersInterface);
		checkf(MegaFiltersInterface == nullptr, TEXT("View family already had mega filters assigned."));
		MegaFiltersInterface = InMegaFiltersInterface;
	}

	inline const UE::Renderer::Private::IMegaFilters* GetMegaFiltersInterface() const
	{
		return MegaFiltersInterface;
	}

	inline void SetPrimarySpatialUpscalerInterface(ISpatialUpscaler* InSpatialUpscalerInterface)
	{
		check(InSpatialUpscalerInterface);
		checkf(PrimarySpatialUpscalerInterface == nullptr, TEXT("View family already had a primary spatial upscaler assigned."));
		PrimarySpatialUpscalerInterface = InSpatialUpscalerInterface;
	}

	inline const ISpatialUpscaler* GetPrimarySpatialUpscalerInterface() const
	{
		return PrimarySpatialUpscalerInterface;
	}

	inline void SetSecondarySpatialUpscalerInterface(ISpatialUpscaler* InSpatialUpscalerInterface)
	{
		check(InSpatialUpscalerInterface);
		checkf(SecondarySpatialUpscalerInterface == nullptr, TEXT("View family already had a secondary spatial upscaler assigned."));
		SecondarySpatialUpscalerInterface = InSpatialUpscalerInterface;
	}

	inline const ISpatialUpscaler* GetSecondarySpatialUpscalerInterface() const
	{
		return SecondarySpatialUpscalerInterface;
	}

	inline bool GetIsInFocus() const				{ return bIsInFocus; }
	inline void SetIsInFocus(bool bInIsInFocus)		{ bIsInFocus = bInIsInFocus; }

	void SetSceneRenderer(ISceneRenderer* NewSceneRenderer) { SceneRenderer = NewSceneRenderer; }
	ISceneRenderer* GetSceneRenderer() const { check(SceneRenderer); return SceneRenderer; }

private:
	/** The scene renderer that is rendering this view family. This is only initialized in the rendering thread's copies of the FSceneViewFamily. */
	ISceneRenderer* SceneRenderer;

	/** Interface to handle screen percentage of the views of the family. */
	ISceneViewFamilyScreenPercentage* ScreenPercentageInterface;

	/** Renderer private interfaces, automatically have same lifetime as FSceneViewFamily. */
	UE::Renderer::Private::ITemporalUpscaler* TemporalUpscalerInterface;
	UE::Renderer::Private::IMegaFilters* MegaFiltersInterface;
	ISpatialUpscaler* PrimarySpatialUpscalerInterface;
	ISpatialUpscaler* SecondarySpatialUpscalerInterface;

	/** Arrays of standalone data that have safe lifetime as FSceneViewFamily. */
	TArray<TSharedRef<class ISceneViewFamilyExtentionData, ESPMode::ThreadSafe> > ViewExtentionDatas;

	/** whether the translucency are allowed to render after DOF, if not they will be rendered in standard translucency. */
	bool bAllowTranslucencyAfterDOF = false;

	/** whether the pre DOF translucency are allowed to be rendered in separated target from scene to allow for better composition with distortion.*/
	bool bAllowStandardTranslucencySeparated = false;

	/** True if this view is the current editing view or the active game view */
	bool bIsInFocus = true;

	friend class FSceneRenderer;
	friend class FViewFamilyInfo;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
 * A view family which deletes its views when it goes out of scope.
 */
class FSceneViewFamilyContext : public FSceneViewFamily
{
public:
	/** Initialization constructor. */
	ENGINE_API FSceneViewFamilyContext(const ConstructionValues& CVS);

	/** Destructor. */
	ENGINE_API virtual ~FSceneViewFamilyContext();
};

struct FSetupViewUniformParametersInputs
{
	static FSetupViewUniformParametersInputs Create(const FSceneView& View)
	{
		return FSetupViewUniformParametersInputs
		{
			  .EngineShowFlags = &View.Family->EngineShowFlags
			, .UnscaledViewRect = View.UnscaledViewRect
			, .Time = View.Family->Time
			, .CursorPosition = View.CursorPos
			, .GlobalClippingPlane = View.GlobalClippingPlane
			, .InvDeviceZToWorldZTransform = View.InvDeviceZToWorldZTransform
			, .DiffuseOverrideParameter = View.DiffuseOverrideParameter
			, .NormalOverrideParameter = View.NormalOverrideParameter
			, .SpecularOverrideParameter = View.SpecularOverrideParameter
			, .RoughnessOverrideParameter = View.RoughnessOverrideParameter
			, .AmbientOcclusionOverrideParameter = View.AmbientOcclusionOverrideParameter
			, .DebugViewShaderMode = View.Family->GetDebugViewShaderMode()
			, .FrameCounter = View.Family->FrameCounter
			, .FrameNumber = View.Family->FrameNumber
			, .FOV = View.FOV
			, .MotionBlurMax = View.FinalPostProcessSettings.MotionBlurMax
#if WITH_EDITOR
			, .bNullifyWorldSpacePosition = View.Family->bNullifyWorldSpacePosition
#endif
			, .bReverseCulling = View.bReverseCulling
			, .bCameraCut = View.bCameraCut
			, .bWorldIsPaused = View.Family->bWorldIsPaused
		};
	}

	const FEngineShowFlags* EngineShowFlags  = nullptr;
	FIntRect UnscaledViewRect;
	FGameTime Time;
	FIntPoint CursorPosition                 = FIntPoint::ZeroValue;
	FPlane GlobalClippingPlane               = FPlane(0, 0, 0, 0);
	FVector4f InvDeviceZToWorldZTransform    = FVector4f(0, 0, 0, 0);
	FVector4f DiffuseOverrideParameter       = FVector4f(0, 0, 0, 1);
	FVector4f NormalOverrideParameter        = FVector4f(0, 0, 0, 1);
	FVector4f SpecularOverrideParameter      = FVector4f(0, 0, 0, 1);
	FVector2f RoughnessOverrideParameter     = FVector2f(0, 1);
	FVector2f AmbientOcclusionOverrideParameter = FVector2f(0, 1);
	EDebugViewShaderMode DebugViewShaderMode = EDebugViewShaderMode::DVSM_None;
	uint64 FrameCounter                      = 0;
	uint32 FrameNumber                       = 0;
	float FOV                                = 0.0f;
	float MotionBlurMax                      = 0.0f;
	bool bNullifyWorldSpacePosition          = false;
	bool bReverseCulling                     = false;
	bool bCameraCut                          = false;
	bool bWorldIsPaused                      = false;
};

ENGINE_API void SetupCommonViewUniformBufferParameters(FViewUniformShaderParameters& ViewUniformShaderParameters,
	const FIntPoint& InBufferSize,
	int32 NumMSAASamples,
	const FIntRect& InEffectiveViewRect,
	const FViewMatrices& InViewMatrices,
	const FViewMatrices& InPrevViewMatrices,
	const FSetupViewUniformParametersInputs& Inputs);

ENGINE_API void SetupViewRectUniformBufferParameters(FViewUniformShaderParameters& ViewUniformShaderParameters,
	const FIntPoint& InBufferSize,
	const FIntRect& InEffectiveViewRect,
	const FViewMatrices& InViewMatrices,
	const FViewMatrices& InPrevViewMatrice,
	const FSetupViewUniformParametersInputs& Inputs);

ENGINE_API FVector4f CreateInvDeviceZToWorldZTransform(const FMatrix& ProjMatrix);