// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneViewExtension.h: Allow changing the view parameters on the render thread
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "SceneViewExtensionContext.h"
#include "SceneRenderingAllocator.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "RendererInterface.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

/**
 *  SCENE VIEW EXTENSIONS
 *  -----------------------------------------------------------------------------------------------
 *
 *  This system lets you hook various aspects of UE rendering.
 *  To create a view extension, it is advisable to inherit
 *  from FSceneViewExtensionBase, which implements the
 *  ISceneViewExtension interface.
 *
 *
 *
 *  INHERITING, INSTANTIATING, LIFETIME
 *  -----------------------------------------------------------------------------------------------
 *
 *  In order to inherit from FSceneViewExtensionBase, do the following:
 *
 *      class FMyExtension : public FSceneViewExtensionBase
 *      {
 *          public:
 *          FMyExtension( const FAutoRegister& AutoRegister, FYourParam1 Param1, FYourParam2 Param2 )
 *          : FSceneViewExtensionBase( AutoRegister )
 *          {
 *          }
 *      };
 *
 *  Notice that your first argument must be FAutoRegister, and you must pass it
 *  to FSceneViewExtensionBase constructor. To instantiate your extension and register
 *  it, do the following:
 *
 *      FSceneViewExtensions::NewExtension<FMyExtension>(Param1, Param2);
 *
 *  You should maintain a reference to the extension for as long as you want to
 *  keep it registered.
 *
 *      TSharedRef<FMyExtension,ESPMode::ThreadSafe> MyExtension;
 *      MyExtension = FSceneViewExtensions::NewExtension<FMyExtension>(Param1, Param2);
 *
 *  If you follow this pattern, the cleanup of the extension will be safe and automatic
 *  whenever the `MyExtension` reference goes out of scope. In most cases, the `MyExtension`
 *  variable should be a member of the class owning the extension instance.
 *
 *  The engine will keep the extension alive for the duration of the current frame to allow
 *  the render thread to finish.
 *
 *
 *
 *  OPTING OUT of RUNNING
 *  -----------------------------------------------------------------------------------------------
 *
 *  Each frame, the engine will invoke ISceneVewExtension::IsActiveThisFrame() to determine
 *  if your extension wants to run this frame. Returning false will cause none of the methods
 *  to be called this frame. The IsActiveThisFrame() method will be invoked again next frame.
 *
 *  If you need fine grained control over individual methods, your IsActiveThisFrame should
 *  return `true` and gate each method as needed.
 *
 *
 *
 *  PRIORITY
 *  -----------------------------------------------------------------------------------------------
 *  Extensions are executed in priority order. Higher priority extensions run first.
 *  To determine the priority of your extension, override ISceneViewExtension::GetPriority();
 *
 */

#include "SceneTexturesConfig.h"

class APlayerController;
class FRHICommandListImmediate;
class FSceneView;
class FSceneViewFamily;
struct FMinimalViewInfo;
struct FSceneViewProjectionData;
class FRDGBuilder;
struct FPostProcessingInputs;
struct FMobilePostProcessingInputs;
struct FPostProcessMaterialInputs;
struct FScreenPassTexture;
class FViewport;
class ISceneRenderer;

/** This is used to add more flexibility to Post Processing, so that users can subscribe to any after Post Processing Pass events. */
using FPostProcessingPassDelegate = TDelegate<FScreenPassTexture(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs)>;
using FPostProcessingPassDelegateArray = TArray<FPostProcessingPassDelegate, SceneRenderingAllocator>;

// Soft deprecation of the previous after pass delegate names
using FAfterPassCallbackDelegate = FPostProcessingPassDelegate;
using FAfterPassCallbackDelegateArray = FPostProcessingPassDelegateArray;

/** Misc flags for communicating requirements or state externally. */
enum class ESceneViewExtensionFlags : uint32
{
	None = 0,

	SubscribesToPostTLASBuild = 1 << 0,
	RequiresHardwareInlineRayTracing = 1 << 1
};
ENUM_CLASS_FLAGS(ESceneViewExtensionFlags);

class ISceneViewExtension
{
public:

	// Each post-processing pass immediately precedes a PPM blend location, if it exists. See comments below.
	enum class EPostProcessingPass : uint32
	{
		BeforeDOF,				// BL_SceneColorBeforeDOF
		AfterDOF,				// BL_SceneColorAfterDOF
		TranslucencyAfterDOF,	// BL_TranslucencyAfterDOF
		SSRInput,				// BL_SSRInput

		// The following post-processing passes may be last, and therefore receive a valid OverrideOutput render target.
		ReplacingTonemapper,	// BL_ReplacingTonemapper
		MotionBlur,				// BL_SceneColorBeforeBloom
		Tonemap,				// BL_SceneColorAfterTonemapping
		FXAA,
		SMAA,
		VisualizeDepthOfField,

		MAX
	};

public:

	/**
     * Called on game thread when creating the view family.
     */
	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) {};

	/**
	 * Called on game thread when creating the view.
	 */
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) {}

	/**
	* Called when creating the viewpoint, before culling, in case an external tracking device needs to modify the base location of the view
	*/
	virtual void SetupViewPoint(APlayerController* Player, FMinimalViewInfo& InViewInfo) {}

    /**
	 * Called when creating the view, in case non-stereo devices need to update projection matrix.
	 */
	virtual void SetupViewProjectionMatrix(FSceneViewProjectionData& InOutProjectionData) {}

    /**
     * Called on game thread when view family is about to be rendered.
     */
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) {};

	/**
	* Called on game thread after the scene renderers have been created
	*/
	virtual void PostCreateSceneRenderer(const FSceneViewFamily& InViewFamily, ISceneRenderer* Renderer) {}

    /**
     * Called on render thread at the start of rendering.
     */
	virtual void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) {}

	/**
     * Called on render thread at the start of rendering, for each view, after PreRenderViewFamily_RenderThread call.
	 */
	virtual void PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) {}

	/**
	 * Called on render thread prior to initializing views.
	 */
	virtual void PreInitViews_RenderThread(FRDGBuilder& GraphBuilder) {}

	/**
	 * Called on render thread right before Base Pass rendering. bDepthBufferIsPopulated is true if anything has been rendered to the depth buffer. This does not need to be a full depth prepass.
	 */
	virtual void PreRenderBasePass_RenderThread(FRDGBuilder& GraphBuilder, bool bDepthBufferIsPopulated) {}

	/**
	 * Called right after Base Pass rendering finished when using the deferred renderer.
	 */
	virtual void PostRenderBasePassDeferred_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView, const FRenderTargetBindingSlots& RenderTargets, TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures) {}

	/**
	 * Called right after Base Pass rendering finished when using the mobile renderer.
	 */
	virtual void PostRenderBasePassMobile_RenderThread(FRHICommandList& RHICmdList, FSceneView& InView) {}

	/**
	 * Called after all the TLAS has been built.
	 */
	virtual void PostTLASBuild_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) {}

	/**
	 * Called right before Post Processing rendering begins
	 */
	virtual void PrePostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPostProcessingInputs& Inputs) {};

	/**
	 * Called right before Post Processing rendering begins for the mobile renderer
	 */
	virtual void PrePostProcessPassMobile_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FMobilePostProcessingInputs& Inputs) {};

	/**
	* This will be called at the beginning of post processing to make sure that each view extension gets a chance to subscribe to a post-processing pass event.
	*  - The pass MUST write to the override output texture if it is active (this occurs when the pass is the last in the post processing chain writing to the back buffer).
	*    For performance reasons it is recommended to only subscribe to a pass when the pass will produce a GPU resource.
	*/
	virtual void SubscribeToPostProcessingPass(EPostProcessingPass Pass, const FSceneView& InView, FPostProcessingPassDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) {}

	/**
	 * Allows to render content after the 3D content scene, useful for debugging
	 */
	virtual void PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) {}

	/**
	 * Allows to render content after the 3D content scene, useful for debugging
	 */
	virtual void PostRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) {}

	/**
     * Called to determine view extensions priority in relation to other view extensions, higher comes first
     */
	virtual int32 GetPriority() const { return 0; }

	/**
	 * Returning false disables the extension for the current frame in the given context. This will be queried each frame to determine if the extension wants to run.
	 */
	virtual bool IsActiveThisFrame(const FSceneViewExtensionContext& Context) const { return IsActiveThisFrame_Internal(Context); }

	/**
	 * Query any flags to communicate state or execution requirements externally.
	 */
	virtual ESceneViewExtensionFlags GetFlags() const { return ESceneViewExtensionFlags::None; }

	///////////////////////////////////////////////////////////////////////////////////////
	//! Deprecated APIs - These are no longer called and must be converted to restore functionality.

	UE_DEPRECATED(5.5, "SubscribeToPostProcessingPass now takes a SceneView")
	virtual void SubscribeToPostProcessingPass(EPostProcessingPass Pass, FPostProcessingPassDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) {}

	///////////////////////////////////////////////////////////////////////////////////////

protected:
	/**
	 * Called if no IsActive functors returned a definitive answer to whether this extension should be active this frame.
	 */
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const { return true; }
};



/**
 * Used to ensure that all extensions are constructed
 * via FSceneViewExtensions::NewExtension<T>(Args).
 */
class FAutoRegister
{
	friend class FSceneViewExtensions;
	FAutoRegister(){}
};


/** Inherit from this class to make a view extension. */
class FSceneViewExtensionBase : public ISceneViewExtension, public TSharedFromThis<FSceneViewExtensionBase, ESPMode::ThreadSafe>
{
public:
	ENGINE_API FSceneViewExtensionBase(const FAutoRegister&);
	ENGINE_API virtual ~FSceneViewExtensionBase();

	// Array of Functors that can be used to activate an extension for the current frame and given context.
	TArray<FSceneViewExtensionIsActiveFunctor> IsActiveThisFrameFunctions;

	// Determines if the extension should be active for the current frame and given context.
	ENGINE_API virtual bool IsActiveThisFrame(const FSceneViewExtensionContext& Context) const override final;
};

/** Scene View Extension which is enabled for all Viewports/Scenes which have the same world. */
class FWorldSceneViewExtension : public FSceneViewExtensionBase
{
public:
	ENGINE_API FWorldSceneViewExtension(const FAutoRegister& AutoReg, UWorld* InWorld);
protected:
	ENGINE_API virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
	UWorld* GetWorld() const { return World.Get(); }
private:
	/** The world of this view extension. */
	TWeakObjectPtr<UWorld> World;
};

/** Scene View Extension which is enabled for all HMDs to unify IsActiveThisFrame_Internal. */
class FHMDSceneViewExtension : public FSceneViewExtensionBase
{
public:
	FHMDSceneViewExtension(const FAutoRegister& AutoReg) : FSceneViewExtensionBase(AutoReg)
	{
	}
protected:
	ENGINE_API virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
};

using FSceneViewExtensionRef = TSharedRef<ISceneViewExtension, ESPMode::ThreadSafe>;

/**
 * Repository of all registered scene view extensions.
 */
class FSceneViewExtensions
{
	friend class FSceneViewExtensionBase;

public:
	/**
	 * Create a new extension of type ExtensionType.
	 */
	template<typename ExtensionType, typename... TArgs>
	static TSharedRef<ExtensionType, ESPMode::ThreadSafe> NewExtension( TArgs&&... Args )
	{
		TSharedRef<ExtensionType, ESPMode::ThreadSafe> NewExtension = MakeShareable(new ExtensionType( FAutoRegister(), Forward<TArgs>(Args)... ));
		RegisterExtension(NewExtension);
		return NewExtension;
	}

	/**
	 * Executes a function on each view extension which is active in a given context.
	 */
	static ENGINE_API void ForEachActiveViewExtension(
		const TArray<TWeakPtr<ISceneViewExtension, ESPMode::ThreadSafe>>& InExtensions, 
		const FSceneViewExtensionContext& InContext,
		const TFunctionRef<void(const FSceneViewExtensionRef&)>& Func);

	/**
     * Gathers all ViewExtensions that want to be active in a given context (@see ISceneViewExtension::IsActiveThisFrame()).
     * The list is sorted by priority (@see ISceneViewExtension::GetPriority())
     */
	ENGINE_API const TArray<FSceneViewExtensionRef> GatherActiveExtensions(const FSceneViewExtensionContext& InContext) const;

private:
	static ENGINE_API void RegisterExtension(const FSceneViewExtensionRef& RegisterMe);
	TArray< TWeakPtr<ISceneViewExtension, ESPMode::ThreadSafe> > KnownExtensions;
};

