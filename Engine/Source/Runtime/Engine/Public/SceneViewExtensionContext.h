// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneInterface.h"
#include "UnrealClient.h"
#include "ViewportClient.h"
#include "Engine/Engine.h"
#include "SceneViewExtensionContext.generated.h"

class FViewport;
class ISceneViewExtension;
class FViewport;

/** 
 * Contains information about the context in which this scene view extension will be used. 
 */
struct FSceneViewExtensionContext
{
private:
	// A quick and dirty way to determine which Context (sub)class this is. Every subclass should implement it.
	virtual FName GetRTTI() const { return TEXT("FSceneViewExtensionContext"); }

public:
	// The scene view extension can be defined with either a Viewport or a Scene

	FViewport* Viewport = nullptr;
	FSceneInterface* Scene = nullptr;
	bool bStereoEnabled = false;

	FSceneViewExtensionContext() : Viewport(nullptr), Scene(nullptr) {}
	explicit FSceneViewExtensionContext(FViewport* InViewport) : Viewport(InViewport) {}
	explicit FSceneViewExtensionContext(FSceneInterface* InScene) : Scene(InScene) {}

	virtual ~FSceneViewExtensionContext() = default;
	
	// Returns true if the given object is of the same type.
	bool IsA(const FSceneViewExtensionContext& Other) const
	{ 
		return GetRTTI() == Other.GetRTTI(); 
	}

	// Returns true if the given object is of the same type.
	bool IsA(const FSceneViewExtensionContext&& Other) const
	{
		return IsA(static_cast<const FSceneViewExtensionContext&>(Other));
	}

	/** Retrieve the world pointer for this context */
	UWorld* GetWorld() const
	{
		if (Viewport != nullptr)
		{
			if (const FViewportClient* ViewportClient = Viewport->GetClient())
			{
				return ViewportClient->GetWorld();
			}
		}
		
		if (Scene != nullptr)
		{
			return Scene->GetWorld();
		}

		return nullptr;
	}

	bool IsStereoSupported() const
	{
		return bStereoEnabled && GEngine && GEngine->IsStereoscopic3D(Viewport);
	}

	// Return true, if HMD supported
	virtual bool IsHMDSupported() const
	{
		return true;
	}
};


/**
 * Convenience type definition of a function that gives an opinion of whether the scene view extension should be active in the given context for the current frame.
 */
using TSceneViewExtensionIsActiveFunction = TFunction<TOptional<bool>(const ISceneViewExtension* SceneViewExtension, const FSceneViewExtensionContext& Context)>;

/**
 * Contains the TFunction that determines if a scene view extension should be valid in the given context given for the current frame.
 * It also contains Guid to help identify it, given that we can't directly compare TFunctions.
 */
USTRUCT(BlueprintType)
struct FSceneViewExtensionIsActiveFunctor
{
	GENERATED_BODY()

private:

	// The Guid is a way to identify the lambda in case it you want to later find it and remove it.
	FGuid Guid;

public:

	// Constructor
	FSceneViewExtensionIsActiveFunctor()
		: Guid(FGuid::NewGuid())
	{}

	// Returns the Guid of this Functor.
	FGuid GetGuid() const
	{
		return Guid;
	}

	// This is the lambda function used to determine if the Scene View Extension should be active or not.
	TSceneViewExtensionIsActiveFunction IsActiveFunction;

	// Make this a functor so that it behaves like the lambda it carries.
	TOptional<bool> operator () (const ISceneViewExtension* SceneViewExtension, const FSceneViewExtensionContext& Context) const
	{
		// If there is no lambda assigned, simply return an unset optional.
		if (!IsActiveFunction)
		{
			return TOptional<bool>();
		}

		// Evaluate the lambda function with the given arguments
		return IsActiveFunction(SceneViewExtension, Context);
	}
};


