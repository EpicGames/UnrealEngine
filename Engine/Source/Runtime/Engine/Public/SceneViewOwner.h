// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EngineDefines.h"

class AActor;

// Owner of view
struct FSceneViewOwner
{
	ENGINE_API FSceneViewOwner(const AActor* InActor = nullptr);
	bool IsPartOf(const TArray<uint32>& InArray) const { return ActorUniqueId ? InArray.Contains(ActorUniqueId) : false; }
	bool IsSet() const { return ActorUniqueId != 0; }
	uint32 ActorUniqueId;

	#if !WITH_STATE_STREAM
	const AActor* Get() const { return Actor; }
	operator const AActor*() const { return Actor; }
	const AActor* operator->() const { return Actor; }
	ENGINE_API void operator=(const AActor* InActor);
	friend bool operator==(const FSceneViewOwner& A, const AActor* B) { return A.Actor == B; }
	const AActor* Actor;
	#endif
};
