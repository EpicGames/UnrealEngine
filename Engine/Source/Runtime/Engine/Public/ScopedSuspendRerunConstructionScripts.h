// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"

#define UE_API ENGINE_API

#if WITH_EDITOR

class FScopedSuspendRerunConstructionScripts
{
public:
    UE_API FScopedSuspendRerunConstructionScripts();
    UE_API ~FScopedSuspendRerunConstructionScripts();

    // Called by AActor::RerunConstructionScripts() to see if it's suspended
    static bool IsSuspended()
    {
        return SuspensionCount > 0;
    }

    // Queues an actor for a deferred rerun
    static UE_API void DeferRerun(AActor* Actor);

private:
    // How many nested scopes are active
    static UE_API int32 SuspensionCount;

    // All actors that attempted to rerun while suspended
    static UE_API TSet<TWeakObjectPtr<AActor>> PendingActors;
};

#endif

#undef UE_API
