// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenRendering.h: Screen rendering definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "SceneView.h"
#include "StereoRenderUtils.h"

struct FScreenVertex
{
	FVector2f Position;
	FVector2f UV;
};

inline bool operator== (const FScreenVertex &a, const FScreenVertex &b)
{
	return a.Position == b.Position && a.UV == b.UV;
}

inline bool operator!= (const FScreenVertex &a, const FScreenVertex &b)
{
	return !(a == b);
}

/** The filter vertex declaration resource type. */
class FScreenVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	// Destructor.
	virtual ~FScreenVertexDeclaration() = default;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		uint16 Stride = sizeof(FScreenVertex);
		Elements.Add(FVertexElement(0,STRUCT_OFFSET(FScreenVertex,Position),VET_Float2,0,Stride));
		Elements.Add(FVertexElement(0,STRUCT_OFFSET(FScreenVertex,UV),VET_Float2,1,Stride));
		VertexDeclarationRHI = RHICreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

extern ENGINE_API TGlobalResource<FScreenVertexDeclaration> GScreenVertexDeclaration;

/**
 * A pixel shader for rendering a textured screen element.
 */
class FScreenPS : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FScreenPS,Global,ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FScreenPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FGlobalShader(Initializer)
	{
		InTexture.Bind(Initializer.ParameterMap,TEXT("InTexture"), SPF_Mandatory);
		InTextureSampler.Bind(Initializer.ParameterMap,TEXT("InTextureSampler"));
	}
	FScreenPS() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture)
	{
		SetTextureParameter(BatchedParameters,InTexture,InTextureSampler,Texture);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI)
	{
		SetTextureParameter(BatchedParameters,InTexture,InTextureSampler,SamplerStateRHI,TextureRHI);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
};

/**
 * A pixel shader for rendering a textured screen element, taking only the first slice of the array
 */
class FScreenFromSlice0PS : public FScreenPS
{
	DECLARE_EXPORTED_SHADER_TYPE(FScreenFromSlice0PS, Global, ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{ 
		if (FScreenPS::ShouldCompilePermutation(Parameters))
		{
			UE::StereoRenderUtils::FStereoShaderAspects Aspects(Parameters.Platform);
			return Aspects.IsMobileMultiViewEnabled();
		}
		return false;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FScreenPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SCREEN_PS_FROM_SLICE0"), 1);
	}

	FScreenFromSlice0PS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FScreenPS(Initializer)
	{
	}
	FScreenFromSlice0PS() {}
};

/**
 * A pixel shader for rendering side-by-side textures from the array
 */
class FScreenUnwrapSlicesPS : public FScreenPS
{
	DECLARE_EXPORTED_SHADER_TYPE(FScreenUnwrapSlicesPS, Global, ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
#if PLATFORM_DESKTOP
		// this is a desktop-only shader so far (really only needed in the editor)
		if (FScreenPS::ShouldCompilePermutation(Parameters))
		{
			UE::StereoRenderUtils::FStereoShaderAspects Aspects(Parameters.Platform);
			return Aspects.IsMobileMultiViewEnabled();
		}
#endif
		return false;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FScreenPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SCREEN_PS_UNWRAP_SLICES"), 1);
	}

	FScreenUnwrapSlicesPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FScreenPS(Initializer)
	{
	}
	FScreenUnwrapSlicesPS() {}
};

/**
 * A pixel shader for rendering a textured screen element.
 */
class FScreenPSInvertAlpha : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FScreenPSInvertAlpha, Global, ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FScreenPSInvertAlpha(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{
		InTexture.Bind(Initializer.ParameterMap, TEXT("InTexture"), SPF_Mandatory);
		InTextureSampler.Bind(Initializer.ParameterMap, TEXT("InTextureSampler"));
	}
	FScreenPSInvertAlpha() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, Texture);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, SamplerStateRHI, TextureRHI);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
};

/**
* A pixel shader for rendering a textured screen element.
*/
class FScreenPSsRGBSource : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FScreenPSsRGBSource, Global, ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FScreenPSsRGBSource(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{
		InTexture.Bind(Initializer.ParameterMap, TEXT("InTexture"), SPF_Mandatory);
		InTextureSampler.Bind(Initializer.ParameterMap, TEXT("InTextureSampler"));
	}
	FScreenPSsRGBSource() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, Texture);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, SamplerStateRHI, TextureRHI);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
};

/**
 * A pixel shader for rendering a textured screen element from an array slice.
 */
class FScreenPSArraySlice : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FScreenPSArraySlice, Global, ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SCREEN_PS_SINGLE_SLICE"), 1);
	}

	FScreenPSArraySlice(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{
		InTexture.Bind(Initializer.ParameterMap, TEXT("InTexture"), SPF_Mandatory);
		InTextureSampler.Bind(Initializer.ParameterMap, TEXT("InTextureSampler"));
		InArraySliceParameter.Bind(Initializer.ParameterMap, TEXT("ArraySlice"));
	}
	FScreenPSArraySlice() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture, uint32 ArraySlice = 0)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, Texture);
		SetShaderValue(BatchedParameters, InArraySliceParameter, ArraySlice);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI, uint32 ArraySlice = 0)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, SamplerStateRHI, TextureRHI);
		SetShaderValue(BatchedParameters, InArraySliceParameter, ArraySlice);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
	LAYOUT_FIELD(FShaderParameter, InArraySliceParameter);
};

/**
 * A pixel shader for rendering a textured screen element with mip maps.
 */
class FScreenPSMipLevel : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FScreenPSMipLevel, Global, ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FScreenPSMipLevel(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{
		InTexture.Bind(Initializer.ParameterMap, TEXT("InTexture"), SPF_Mandatory);
		InTextureSampler.Bind(Initializer.ParameterMap, TEXT("InTextureSampler"));
		InMipLevelParameter.Bind(Initializer.ParameterMap, TEXT("MipLevel"));
	}
	FScreenPSMipLevel() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture, int MipLevel = 0)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, Texture);
		SetShaderValue(BatchedParameters, InMipLevelParameter, MipLevel);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI, int MipLevel = 0)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, SamplerStateRHI, TextureRHI);
		SetShaderValue(BatchedParameters, InMipLevelParameter, MipLevel);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
	LAYOUT_FIELD(FShaderParameter, InMipLevelParameter);
};

/**
* A pixel shader for rendering a textured screen element with mip maps.
*/
class FScreenPSsRGBSourceMipLevel : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FScreenPSsRGBSourceMipLevel, Global, ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FScreenPSsRGBSourceMipLevel(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{
		InTexture.Bind(Initializer.ParameterMap, TEXT("InTexture"), SPF_Mandatory);
		InTextureSampler.Bind(Initializer.ParameterMap, TEXT("InTextureSampler"));
		InMipLevelParameter.Bind(Initializer.ParameterMap, TEXT("MipLevel"));
	}
	FScreenPSsRGBSourceMipLevel() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture, int MipLevel = 0)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, Texture);
		SetShaderValue(BatchedParameters, InMipLevelParameter, MipLevel);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI, int MipLevel = 0)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, SamplerStateRHI, TextureRHI);
		SetShaderValue(BatchedParameters, InMipLevelParameter, MipLevel);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
	LAYOUT_FIELD(FShaderParameter, InMipLevelParameter);
};

class FScreenPS_OSE : public FGlobalShader
{
    DECLARE_EXPORTED_SHADER_TYPE(FScreenPS_OSE,Global,ENGINE_API);
public:

    static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

    FScreenPS_OSE(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
        FGlobalShader(Initializer)
    {
        InTexture.Bind(Initializer.ParameterMap,TEXT("InTexture"), SPF_Mandatory);
        InTextureSampler.Bind(Initializer.ParameterMap,TEXT("InTextureSampler"));
    }

    FScreenPS_OSE() {}

    void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture)
    {
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, Texture);
	}

    void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI)
    {
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, SamplerStateRHI, TextureRHI);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
};

/**
 * A vertex shader for rendering a textured screen element.
 */
class FScreenVS : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FScreenVS,Global,ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FScreenVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FGlobalShader(Initializer)
	{
	}
	FScreenVS() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHIUniformBuffer* ViewUniformBuffer)
	{
		FGlobalShader::SetParameters<FViewUniformShaderParameters>(BatchedParameters, ViewUniformBuffer);
	}
};


