// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Shader/PreshaderTypes.h"
#include "Serialization/MemoryImage.h"
#include "Materials/MaterialLayersFunctions.h"

class FUniformExpressionSet;
struct FMaterialRenderContext;
class FMaterial;
class FXxHash64Builder;

namespace UE
{
namespace Shader
{

enum class EPreshaderOpcode : uint8
{
	Nop,
	ConstantZero,
	Constant,
	Parameter,
	Add,
	Sub,
	Mul,
	Div,
	Fmod,
	Modulo,
	Min,
	Max,
	Clamp,
	Sin,
	Cos,
	Tan,
	Asin,
	Acos,
	Atan,
	Atan2,
	Dot,
	Cross,
	Sqrt,
	Rcp,
	Length,
	Normalize,
	Saturate,
	Abs,
	Floor,
	Ceil,
	Round,
	Trunc,
	Sign,
	Frac,
	Fractional,
	Log2,
	Log10,
	ComponentSwizzle,
	AppendVector,
	TextureSize,
	TexelSize,
	ExternalTextureCoordinateScaleRotation,
	ExternalTextureCoordinateOffset,
	RuntimeVirtualTextureUniform,
	SparseVolumeTextureUniform,
	Neg,
	PushValue,
	Less,
	Assign,
	Greater,
	LessEqual,
	GreaterEqual,
	Exp,
	Exp2,
	Log
};

struct FPreshaderStructType
{
	DECLARE_TYPE_LAYOUT(FPreshaderStructType, NonVirtual);
	LAYOUT_FIELD(uint64, Hash);
	LAYOUT_FIELD(int32, ComponentTypeIndex);
	LAYOUT_FIELD(int32, NumComponents);
};

struct FPreshaderLabel
{
	explicit FPreshaderLabel(int32 InOffset = INDEX_NONE) : Offset(InOffset) {}

	int32 Offset;
};

class FPreshaderData
{
	DECLARE_TYPE_LAYOUT(FPreshaderData, NonVirtual);
public:
	friend inline bool operator==(const FPreshaderData& Lhs, const FPreshaderData& Rhs)
	{
		return Lhs.bPreshader2 == Rhs.bPreshader2 && Lhs.bPreFixup == Rhs.bPreFixup && Lhs.Preshader2TemporarySize == Rhs.Preshader2TemporarySize &&
			Lhs.Names == Rhs.Names && Lhs.Data == Rhs.Data;
	}

	friend inline bool operator!=(const FPreshaderData& Lhs, const FPreshaderData& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	void AppendHash(FXxHash64Builder& OutHasher) const;

	FPreshaderValue Evaluate(FUniformExpressionSet* UniformExpressionSet, const struct FMaterialRenderContext& Context, FPreshaderStack& Stack) const;
	FPreshaderValue EvaluateConstant(const FMaterial& Material, FPreshaderStack& Stack) const;

	const int32 Num() const { return Data.Num(); }

	void WriteData(const void* Value, uint32 Size);
	void WriteName(const FScriptName& Name);
	void WriteValue(const FValue& Value);

	template<typename T>
	FPreshaderData& Write(const T& Value) { WriteData(&Value, sizeof(T)); return *this; }

	template<>
	FPreshaderData& Write<FScriptName>(const FScriptName& Value) { WriteName(Value); return *this; }

	template<>
	FPreshaderData& Write<FValue>(const FValue& Value) { WriteValue(Value); return *this; }

	/** Writing FType isn't supported -- use its ValueType enum member instead */
	template<>
	FPreshaderData& Write<FType>(const FType& Value) = delete;

	/** Can't write FName, use FScriptName instead */
	template<>
	FPreshaderData& Write<FName>(const FName& Value) = delete;

	template<>
	FPreshaderData& Write<FHashedMaterialParameterInfo>(const FHashedMaterialParameterInfo& Value) { return Write(Value.Name).Write(Value.Index).Write(Value.Association); }

	inline FPreshaderData& WriteOpcode(EPreshaderOpcode Op) { ensure(Op != EPreshaderOpcode::Nop && bPreshader2 == false); return Write<uint8>((uint8)Op); }

	LAYOUT_FIELD_INITIALIZED(bool, bPreshader2, false);
	LAYOUT_FIELD_INITIALIZED(bool, bPreFixup, false);
	LAYOUT_FIELD_INITIALIZED(uint16, Preshader2TemporarySize, 0);		// Number of extra temporary float4s required for Preshader2 evaluation
	LAYOUT_FIELD(TMemoryImageArray<FScriptName>, Names);
	LAYOUT_FIELD(TMemoryImageArray<uint8>, Data);
};

} // namespace Shader
} // namespace UE