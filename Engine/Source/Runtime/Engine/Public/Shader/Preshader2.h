// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Preshader2.h: New Preshader interface.
=============================================================================*/

#pragma once

#include "Materials/MaterialParameters.h"
#include "Shader/Preshader.h"
#include "UObject/NameTypes.h"

namespace UE::Shader
{
struct FPreshaderDataContext;
}

#define PRESHADER2_UNARY_OP(OP_NAME, ALLOW_INTEGER)		\
	OP_NAME,											\
	OP_NAME ## _InPlace,								\
	OP_NAME ## _Reuse,

#define PRESHADER2_BINARY_OP(OP_NAME, ALLOW_INTEGER)	\
	OP_NAME,											\
	OP_NAME ## _InPlace,								\
	OP_NAME ## _Reuse,									\
	OP_NAME ## _Const,									\
	OP_NAME ## _ConstInPlace,							\
	OP_NAME ## _ConstReuse,								\
	OP_NAME ## _ConstReuseScalar,

#define PRESHADER2_UNARY_OP_CAST_REUSE(OP_NAME) OP_NAME,
#define PRESHADER2_UNARY_OP_CAST_REUSE_LAST(OP_NAME) OP_NAME,
#define PRESHADER2_UNARY_OP_CUSTOM(OP_NAME) OP_NAME,
#define PRESHADER2_BINARY_OP_CUSTOM(OP_NAME) OP_NAME,
#define PRESHADER2_OP(OP_NAME) OP_NAME,

enum class EPreshader2Opcode : uint8
{
	Invalid,

	#include "Preshader2Opcodes.inl"

	// Standard unary ops have three variations:  normal, in place (input == output), and reuse (input == output == last output)
	UnaryFirst = Abs,
	UnaryNoInt = ACos,			// First unary op that doesn't support integer types
	UnaryCustom = Length,		// First custom unary op with a single opcode variant instead of 3
	UnaryLast = Length,

	// Standard binary ops have seven variations:  normal, in place (first input == output), reuse (first input == output == last output),
	// plus four variations with inline constant second arguments (normal, in place, reuse, reuse with scalar constant).
	BinaryFirst = Add,
	BinaryNoInt = ATan2,		// First binary op that doesn't support integer types
	BinaryCustom = Cross,		// First binary op with a single opcode variant instead of 7
	BinaryLast = Dot,

	TextureFirst = RuntimeVirtualTextureUniform,
	TextureLast = TextureSize,
};

#undef PRESHADER2_UNARY_OP
#undef PRESHADER2_BINARY_OP
#undef PRESHADER2_UNARY_OP_CAST_REUSE
#undef PRESHADER2_UNARY_OP_CAST_REUSE_LAST
#undef PRESHADER2_UNARY_OP_CUSTOM
#undef PRESHADER2_BINARY_OP_CUSTOM
#undef PRESHADER2_OP

enum class EPreshader2Type : uint8
{
	Float,
	Double,
	Integer,
	LWC				// Large World Coordinates.  Not supported by math ops, only as an output format for the Cast op.
};

namespace UE::Preshader2
{

struct FPrintParameterInfo
{
	FScriptName Name;
	uint16 Offset;
	uint16 ComponentCount;
	bool bDouble;
};

// Preshader2 uses buffer offsets specified in 4-byte words.  Operations on Double types must use even aligned word offsets, so the 8-byte type ends up aligned.

// "DestType" only applies to the Cast op.  Reuse option uses the result offset of the previous instruction for both ResultOffset and AOffset, but some opcodes
// may not support this, so a valid ResultOffset and AOffset should still be specified.
void EmitUnaryOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Opcode Op, EPreshader2Type Type, uint8 Dimension, uint16 ResultOffset, uint16 AOffset, bool bReuse = false, EPreshader2Type DestType = EPreshader2Type::Float);

// Binary ops support either or both operands being scalars instead of vectors.
void EmitBinaryOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Opcode Op, EPreshader2Type Type, uint8 Dimension, uint16 ResultOffset, uint16 AOffset, uint16 BOffset, bool bAScalar, bool bBScalar, bool bReuse = false);

// Similar to above, but second argument is a constant inlined into the bytecode.  Note that "custom" binary opcodes (ones in the numeric range BinaryCustom to BinaryLast)
// don't work with this function.  Currently this includes Cross and Dot opcodes.  Returns size of the constant data that was emitted.
int32 EmitBinaryConstOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Opcode Op, EPreshader2Type Type, uint8 Dimension, uint16 ResultOffset, uint16 AOffset, const void* BData, bool bAScalar, bool bBScalar, bool bReuse = false);

// Loads an inlined constant to the given ResultOffset.
void EmitConstantOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Type Type, uint8 Dimension, uint16 ResultOffset, const void* Data);

// SourceOffsets contains the offsets of individual components to be moved.  If any are adjacent, the moves can be combined.  EmitMoveOpForFixup
// below must be used instead if PreshaderData.bPreFixup is true.
void EmitMoveOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Type Type, uint16 ResultOffset, TConstArrayView<uint16> SourceOffsets);

// When using FixupAndCompress (where unique "registers" are defined for intermediates, and later remapped to offsets), a different version of the move op must be
// emitted, as register indices by themselves don't allow for selection of which components in those registers are being moved.  A separate array of component
// indices is provided, and it is stored in a less compressed format than the normal move op.  Requires PreshaderData.bPreFixup to be true.
void EmitMoveOpForFixup(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Type Type, uint16 ResultRegister, uint8 ResultComponent, TConstArrayView<uint16> Registers, TConstArrayView<uint8> Components);

// Emit texture related opcode (TexelSize, TextureSize, etc).
void EmitTextureOp(UE::Shader::FPreshaderData& PreshaderData, EPreshader2Opcode Op, EPreshader2Type Type, uint8 Dimension, uint16 ResultOffset, const FHashedMaterialParameterInfo& HashedSourceParameterInfo, int32 TextureIndex, int32 UniformIndex = INDEX_NONE);

// Fix up offsets and compress opcodes.  Useful for initially emitting opcodes with unique "registers" per intermediate, then once register lifetimes and usages
// are known (after analyzing the whole tree), go back and remap registers and compress opcodes where in-place or reused destination execution is possible.  When
// using this function, in-place and reuse variations of opcodes are not supported!  These won't be possible if each intermediate is unique.  PreshaderOut should
// be a different empty buffer than PreshaderIn.
void FixupAndCompress(UE::Shader::FPreshaderData& PreshaderOut, const UE::Shader::FPreshaderData& PreshaderIn, TConstArrayView<uint16> Remap);

// Evaluate opcodes.  FMaterialRenderContext is required if any texture ops are used (TexelSize, TextureSize, etc).
void Evaluate(UE::Shader::FPreshaderDataContext& DataContext, const FMaterialRenderContext* MaterialContext = nullptr);

// Each Preshader instruction is printed on its own line to the output array.  ParameterMap is an optional list of parameter offsets and sizes to improve readability.
// 
void Print(const UE::Shader::FPreshaderData& Preshader, TConstArrayView<FPrintParameterInfo> ParameterInfos, TArray<FString>& OutInstructions);

// Unit test of system.
void UnitTest();

}  // namespace UE::Preshader2