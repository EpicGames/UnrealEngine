// Copyright Epic Games, Inc. All Rights Reserved.

// Unary ops with integer support -- each has 3 opcode variants, with Cast having 3 extra custom "Reuse" opcode variants for different destination types.
// The three extra opcodes conveniently maintain the 3-element stride for surrounding standard unary opcode variants.
PRESHADER2_UNARY_OP(Abs, true)
PRESHADER2_UNARY_OP(Cast, true)
PRESHADER2_UNARY_OP_CAST_REUSE(Cast_ReuseDouble)
PRESHADER2_UNARY_OP_CAST_REUSE(Cast_ReuseInteger)
PRESHADER2_UNARY_OP_CAST_REUSE_LAST(Cast_ReuseLWC)
PRESHADER2_UNARY_OP(Negate, true)
PRESHADER2_UNARY_OP(Saturate, true)
PRESHADER2_UNARY_OP(Sign, true)

// Unary ops without integer support
PRESHADER2_UNARY_OP(ACos, false)
PRESHADER2_UNARY_OP(ASin, false)
PRESHADER2_UNARY_OP(ATan, false)
PRESHADER2_UNARY_OP(Ceil, false)
PRESHADER2_UNARY_OP(Cos, false)
PRESHADER2_UNARY_OP(Exponential, false)
PRESHADER2_UNARY_OP(Exponential2, false)
PRESHADER2_UNARY_OP(Floor, false)
PRESHADER2_UNARY_OP(Frac, false)
PRESHADER2_UNARY_OP(Fractional, false)
PRESHADER2_UNARY_OP(Logarithm, false)
PRESHADER2_UNARY_OP(Logarithm10, false)
PRESHADER2_UNARY_OP(Logarithm2, false)
PRESHADER2_UNARY_OP(Normalize, false)
PRESHADER2_UNARY_OP(Reciprocal, false)
PRESHADER2_UNARY_OP(Round, false)
PRESHADER2_UNARY_OP(Sin, false)
PRESHADER2_UNARY_OP(Sqrt, false)
PRESHADER2_UNARY_OP(Tan, false)
PRESHADER2_UNARY_OP(Truncate, false)

// Non-standard unary ops -- single opcode variant
PRESHADER2_UNARY_OP_CUSTOM(Length)

// Binary ops with integer support -- each has 7 opcode variants
PRESHADER2_BINARY_OP(Add, true)
PRESHADER2_BINARY_OP(Divide, true)
PRESHADER2_BINARY_OP(Fmod, true)
PRESHADER2_BINARY_OP(Max, true)
PRESHADER2_BINARY_OP(Min, true)
PRESHADER2_BINARY_OP(Multiply, true)
PRESHADER2_BINARY_OP(Subtract, true)

// Binary ops without integer support
PRESHADER2_BINARY_OP(ATan2, false)

// Non-standard binary ops -- single opcode variant
PRESHADER2_BINARY_OP_CUSTOM(Cross)
PRESHADER2_BINARY_OP_CUSTOM(Dot)

// Ops that don't fall under unary or binary categories
PRESHADER2_OP(Move)				// Opcode(1), MoveCount+Type+FirstMoveSize+SecondMoveSize(1), ResultOffset+ThirdMoveSize(2), SourceOffset(2*N)
PRESHADER2_OP(MoveForFixup)		// Opcode(1), MoveCount+Type(1), ResultOffset(2), ComponentStart(1), SourceRegister(2*N), SourceComponent(1*N)
PRESHADER2_OP(Constant)			// Opcode(1), ResultOffset(2), Dimension+Type(1), Data(N)

// Texture inputs
PRESHADER2_OP(RuntimeVirtualTextureUniform)
PRESHADER2_OP(TexelSize)
PRESHADER2_OP(TextureSize)

// External inputs (TODO:  Unimplemented)
PRESHADER2_OP(ExternalTextureCoordinateScaleRotation)
PRESHADER2_OP(ExternalTextureCoordinateOffset)
PRESHADER2_OP(SparseVolumeTextureUniform)

// NOTE:  Clamp op is not implemented, caller should generate Min+Max.  Challenge of implementing Clamp as a single op is that there are many
// permutations of register and constant input arguments that need to be dealt with, so it's easier to split in two.  It should be plenty
// efficient with reuse of previous op's result, taking only one extra byte in storage, and one extra op.  Clamp is 0.01% of ops.
