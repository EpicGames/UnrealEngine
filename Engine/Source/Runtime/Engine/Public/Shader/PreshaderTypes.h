// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Shader/ShaderTypes.h"
#include "Containers/ArrayView.h"

namespace UE
{
namespace Shader
{

/**
 * Mirrors Shader::FType, but stores StructType as a hash rather than a pointer to facilitate serialization
 * Struct's flattened component types are stored directly, as that is the primary thing needed at runtime
 */
struct FPreshaderType
{
	FPreshaderType() = default;
	FPreshaderType(EValueType InType);

	int32 GetNumComponents() const { return GetValueTypeDescription(ValueType).NumComponents; }
	EValueComponentType GetComponentType(int32 Index) const;

	EValueType ValueType = EValueType::Void;
};

/** Mirrors Shader::FValue, except 'Component' array references memory owned by stack, rather than inline storage */
struct FPreshaderValue
{
	FPreshaderType Type;
	TArrayView<FValueComponent> Component;

	/** Converts to a regular 'FValue', uses the type registry to resolve StructType hashes into pointers */
	FValue AsShaderValue(const FStructTypeRegistry* TypeRegistry = nullptr) const;
};

class FPreshaderStack
{
public:
	int32 Num() const { return Values.Num(); }

	void CheckEmpty() const
	{
		check(Values.Num() == 0);
		check(Components.Num() == 0);
	}

	void PushValue(const FValue& InValue);
	void PushValue(const FPreshaderValue& InValue);
	void PushValue(const FPreshaderType& InType, TArrayView<const FValueComponent> InComponents);
	TArrayView<FValueComponent> PushEmptyValue(const FPreshaderType& InType);
	FValueComponent* PushEmptyValue(EValueType InType, int32 NumComponents);

	/** Returned values are invalidated when anything is pushed onto the stack */
	FPreshaderValue PopValue();
	FPreshaderValue PeekValue(int32 Offset = 0);

	/** Utility functions used for in-place swizzle operation */
	inline const FPreshaderType& PeekType()
	{
		return Values.Last();
	}
	inline FValueComponent* PeekComponents(int32 NumComponents)
	{
		return &Components[Components.Num() - NumComponents];
	}

	/** Pops the final result from the stack, with reduced overhead relative to PopValue -- must be zero or one items left! */
	void PopResult(FPreshaderValue& OutValue);

	/** Override top value type (after an in place unary operation) */
	inline void OverrideTopType(EValueType ValueType)
	{
		Values.Last().ValueType = ValueType;
	}

	/** Merge top two values (after an in place binary operation) */
	inline void MergeTopTwoValues(EValueType ValueType, int32 ComponentsConsumed)
	{
		Values.RemoveAt(Values.Num() - 1, EAllowShrinking::No);
		Components.RemoveAt(Components.Num() - ComponentsConsumed, ComponentsConsumed, EAllowShrinking::No);
		Values.Last().ValueType = ValueType;
	}

	// Adjust component count, by adding or removing components by the specified amount
	inline void AdjustComponentCount(int32 NumComponentChange)
	{
		if (NumComponentChange > 0)
		{
			Components.AddUninitialized(NumComponentChange);
		}
		else if (NumComponentChange < 0)
		{
			Components.RemoveAt(Components.Num() + NumComponentChange, -NumComponentChange, EAllowShrinking::No);
		}
	}

	void Reset()
	{
		Values.Reset();
		Components.Reset();
	}

private:
	TArray<FPreshaderType, TNonRelocatableInlineAllocator<8>> Values;
	TArray<FValueComponent, TNonRelocatableInlineAllocator<64>> Components;
};

} // namespace Shader
} // namespace UE