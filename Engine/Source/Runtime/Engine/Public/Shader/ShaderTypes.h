// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "Serialization/MemoryLayout.h"

class FMemStackBase;
class UTexture;

namespace UE::Shader
{

struct FStructType;

enum class EComponentBound : uint8
{
	NegDoubleMax,
	NegFloatMax,
	IntMin,
	NegOne,
	Zero,
	One,
	IntMax,
	FloatMax,
	DoubleMax,
};

inline EComponentBound MinBound(EComponentBound Lhs, EComponentBound Rhs) { return (EComponentBound)FMath::Min((uint8)Lhs, (uint8)Rhs); }
inline EComponentBound MaxBound(EComponentBound Lhs, EComponentBound Rhs) { return (EComponentBound)FMath::Max((uint8)Lhs, (uint8)Rhs); }

struct FComponentBounds
{
	FComponentBounds() = default;
	FComponentBounds(EComponentBound InMin, EComponentBound InMax) : Min(InMin), Max(InMax) {}

	EComponentBound Min = EComponentBound::NegDoubleMax;
	EComponentBound Max = EComponentBound::DoubleMax;
};
inline bool operator==(const FComponentBounds& Lhs, const FComponentBounds& Rhs)
{
	return Lhs.Min == Rhs.Min && Lhs.Max == Rhs.Max;
}
inline bool operator!=(const FComponentBounds& Lhs, const FComponentBounds& Rhs)
{
	return !operator==(Lhs, Rhs);
}

inline FComponentBounds MinBound(FComponentBounds Lhs, FComponentBounds Rhs) { return FComponentBounds(MinBound(Lhs.Min, Rhs.Min), MinBound(Lhs.Max, Rhs.Max)); }
inline FComponentBounds MaxBound(FComponentBounds Lhs, FComponentBounds Rhs) { return FComponentBounds(MaxBound(Lhs.Min, Rhs.Min), MaxBound(Lhs.Max, Rhs.Max)); }
inline bool IsWithinBounds(FComponentBounds Lhs, FComponentBounds Rhs) { return (uint8)Lhs.Min >= (uint8)Rhs.Min && (uint8)Lhs.Max <= (uint8)Rhs.Max; }

enum class EValueComponentType : uint8
{
	Void,
	Float,
	Double,
	Int,
	Bool,

	// May be any numeric type, stored internally as 'double' within FValue
	Numeric,

	Num,
};
static constexpr int32 NumValueComponentTypes = (int32)EValueComponentType::Num;

struct FValueComponentTypeDescription
{
	FValueComponentTypeDescription() = default;
	FValueComponentTypeDescription(const TCHAR* InName, uint32_t InSizeInBytes, EComponentBound InMin, EComponentBound InMax) : Name(InName), SizeInBytes(InSizeInBytes), Bounds(InMin, InMax) {}

	const TCHAR* Name = nullptr;
	uint32_t SizeInBytes = 0u;
	FComponentBounds Bounds;
};

ENGINE_API FValueComponentTypeDescription GetValueComponentTypeDescription(EValueComponentType Type);
inline const TCHAR* GetComponentTypeName(EValueComponentType Type) { return GetValueComponentTypeDescription(Type).Name; }
inline uint32 GetComponentTypeSizeInBytes(EValueComponentType Type) { return GetValueComponentTypeDescription(Type).SizeInBytes; }
inline bool IsNumericType(EValueComponentType Type) { return Type != EValueComponentType::Void; }
inline bool IsComponentTypeWithinBounds(EValueComponentType Type, FComponentBounds Bounds) { return IsWithinBounds(GetValueComponentTypeDescription(Type).Bounds, Bounds); }

ENGINE_API EValueComponentType CombineComponentTypes(EValueComponentType Lhs, EValueComponentType Rhs);

inline EValueComponentType MakeNonLWCType(EValueComponentType Type) { return Type == EValueComponentType::Double ? EValueComponentType::Float : Type; }
inline EValueComponentType MakeConcreteType(EValueComponentType Type) { return Type == EValueComponentType::Numeric ? EValueComponentType::Float : Type; }
inline bool IsGenericType(EValueComponentType Type) { return Type == EValueComponentType::Numeric; }
inline bool IsLWCType(EValueComponentType Type) { return Type == EValueComponentType::Double; }

enum class EValueType : uint8
{
	Void,

	Float1,
	Float2,
	Float3,
	Float4,

	Double1,
	Double2,
	Double3,
	Double4,

	Int1,
	Int2,
	Int3,
	Int4,

	Bool1,
	Bool2,
	Bool3,
	Bool4,

	// Any scalar/vector type
	Numeric1,
	Numeric2,
	Numeric3,
	Numeric4,

	// float4x4
	Float4x4,

	// Both of these are double4x4 on CPU
	// On GPU, they map to FDFMatrix and FDFInverseMatrix
	Double4x4,
	DoubleInverse4x4,

	// Any matrix type
	Numeric4x4,

	Struct,
	Object,
	Any,

	Num,
};
inline constexpr int32 NumValueTypes = (int32)EValueType::Num;

struct FValueTypeDescription
{
	const TCHAR* Name;
	EValueType ValueType;					// Useful for validation
	EValueComponentType ComponentType;
	int8 NumComponents;
	int8 ComponentSizeInBytes;
};

ENGINE_API const FValueTypeDescription& GetValueTypeDescription(EValueType Type);
ENGINE_API EValueType FindValueType(FName Name);
inline bool IsLWCType(EValueType Type) { return IsLWCType(GetValueTypeDescription(Type).ComponentType); }
inline bool IsGenericType(EValueType Type) { return Type == EValueType::Any || IsGenericType(GetValueTypeDescription(Type).ComponentType); }
ENGINE_API EValueType MakeValueType(EValueComponentType ComponentType, int32 NumComponents);
ENGINE_API EValueType MakeValueType(EValueType BaseType, int32 NumComponents);
ENGINE_API EValueType MakeValueTypeWithRequestedNumComponents(EValueType BaseType, int8 RequestedNumComponents);
ENGINE_API EValueType MakeNonLWCType(EValueType Type);
ENGINE_API EValueType MakeConcreteType(EValueType Type);
ENGINE_API EValueType MakeDerivativeType(EValueType Type);

inline bool IsNumericType(EValueType Type) { return IsNumericType(GetValueTypeDescription(Type).ComponentType); }

inline bool IsNumericScalarType(EValueType Type)
{
	const FValueTypeDescription TypeDesc = GetValueTypeDescription(Type);
	return IsNumericType(TypeDesc.ComponentType) && TypeDesc.NumComponents == 1;
}
inline bool IsNumericVectorType(EValueType Type)
{
	const FValueTypeDescription TypeDesc = GetValueTypeDescription(Type);
	return IsNumericType(TypeDesc.ComponentType) && TypeDesc.NumComponents <= 4;
}
inline bool IsNumericMatrixType(EValueType Type)
{
	const FValueTypeDescription TypeDesc = GetValueTypeDescription(Type);
	return IsNumericType(TypeDesc.ComponentType) && TypeDesc.NumComponents == 16;
}

struct FType
{
	FType() : ValueType(EValueType::Void) {}
	FType(EValueType InValueType) : ValueType(InValueType) { check(InValueType != EValueType::Struct && InValueType != EValueType::Object); }
	FType(const FStructType* InStruct) : StructType(InStruct), ValueType(InStruct ? EValueType::Struct : EValueType::Void) {}
	FType(FName InObjectType) : ObjectType(InObjectType), ValueType(!InObjectType.IsNone() ? EValueType::Object : EValueType::Void) {}

	const TCHAR* GetName() const;
	FType GetDerivativeType() const;
	FType GetNonLWCType() const { return IsNumericLWC() ? FType(MakeNonLWCType(ValueType)) : *this; }
	FType GetConcreteType() const { return IsNumeric() ? FType(MakeConcreteType(ValueType)) : *this; }
	bool IsVoid() const { return ValueType == EValueType::Void; }
	bool IsStruct() const { return ValueType == EValueType::Struct; }
	bool IsObject() const { return ValueType == EValueType::Object; }
	bool IsAny() const { return ValueType == EValueType::Any; }
	bool IsGeneric() const { return !IsStruct() && !IsObject() && IsGenericType(ValueType); }
	bool IsNumeric() const { return !IsStruct() && !IsObject() && IsNumericType(ValueType); }
	bool IsNumericScalar() const { return !IsStruct() && !IsObject() && IsNumericScalarType(ValueType); }
	bool IsNumericVector() const { return !IsStruct() && !IsObject() && IsNumericVectorType(ValueType); }
	bool IsNumericMatrix() const { return !IsStruct() && !IsObject() && IsNumericMatrixType(ValueType); }
	bool IsNumericLWC() const { return IsNumeric() && IsLWCType(ValueType); }
	int32 GetNumComponents() const;
	int32 GetNumFlatFields() const;
	EValueComponentType GetComponentType(int32 Index) const;
	EValueType GetFlatFieldType(int32 Index) const;

	inline operator EValueType() const { return ValueType; }
	inline operator bool() const { return !IsVoid(); }
	inline bool operator!() const { return IsVoid(); }

	const FStructType* StructType = nullptr;
	FName ObjectType;
	EValueType ValueType;
};

inline bool operator==(const FType& Lhs, const FType& Rhs)
{
	if (Lhs.ValueType != Rhs.ValueType) return false;
	if (Lhs.ValueType == EValueType::Struct && Lhs.StructType != Rhs.StructType) return false;
	if (Lhs.ValueType == EValueType::Object && Lhs.ObjectType != Rhs.ObjectType) return false;
	return true;
}
inline bool operator!=(const FType& Lhs, const FType& Rhs)
{
	return !operator==(Lhs, Rhs);
}

inline bool operator==(const FType& Lhs, const EValueType& Rhs)
{
	return !Lhs.IsStruct() && Lhs.ValueType == Rhs;
}
inline bool operator!=(const FType& Lhs, const EValueType& Rhs)
{
	return !operator==(Lhs, Rhs);
}

inline bool operator==(const EValueType& Lhs, const FType& Rhs)
{
	return !Rhs.IsStruct() && Lhs == Rhs.ValueType;
}
inline bool operator!=(const EValueType& Lhs, const FType& Rhs)
{
	return !operator==(Lhs, Rhs);
}

ENGINE_API FType CombineTypes(const FType& Lhs, const FType& Rhs, bool bMergeMatrixTypes = false);

struct FStructField
{
	const TCHAR* Name;
	FType Type;
	int32 ComponentIndex;
	int32 FlatFieldIndex;

	int32 GetNumComponents() const { return Type.GetNumComponents(); }
};

struct FStructType
{
	uint64 Hash;
	const TCHAR* Name;
	const FStructType* DerivativeType = nullptr;
	TArrayView<const FStructField> Fields;

	/**
	 * Fields with basic types are represented directly. Fields with struct types are recursively flattened into this list
	 */
	TArrayView<const EValueComponentType> ComponentTypes;

	/**
	 * Type may be viewed as a flat list of fields, rather than of individual components
	 */
	TArrayView<const EValueType> FlatFieldTypes;

	bool IsExternal() const { return Fields.Num() == 0; }
	int32 GetNumComponents() const { return ComponentTypes.Num(); }

	const FStructField* FindFieldByName(const TCHAR* InName) const;
};

struct FStructFieldInitializer
{
	FStructFieldInitializer() = default;
	FStructFieldInitializer(const FStringView& InName, const FType& InType) : Name(InName), Type(InType) {}

	FStringView Name;
	FType Type;
};

struct FStructTypeInitializer
{
	FStringView Name;
	TArrayView<const FStructFieldInitializer> Fields;
	bool bIsDerivativeType = false;
};

class FStructTypeRegistry
{
public:
	explicit FStructTypeRegistry(FMemStackBase& InAllocator);

	void EmitDeclarationsCode(FStringBuilderBase& OutCode) const;

	const FStructType* NewType(const FStructTypeInitializer& Initializer);

	/** An external struct type is opaque is far as the system is concerned, it's expected to be defined by the engine's HLSL code */
	const FStructType* NewExternalType(FStringView Name);
	
	const FStructType* FindType(uint64 Hash) const;

private:
	FMemStackBase* Allocator;
	TMap<uint64, const FStructType*> Types;
};

template<typename T>
struct TValue
{
	inline T& operator[](int32 i) { check(i >= 0 && i < 4); return Component[i]; }
	inline const T& operator[](int32 i) const { check(i >= 0 && i < 4); return Component[i]; }

	T Component[4];
};
using FFloatValue = TValue<float>;
using FDoubleValue = TValue<double>;
using FIntValue = TValue<int32>;
using FBoolValue = TValue<bool>;

enum class EValueStringFormat
{
	Description,
	HLSL,
};

struct FMemoryImageValue
{
	static const uint32 MaxSize = sizeof(double) * 16;
	uint8 Bytes[MaxSize];
	uint32 Size;
};

/** The value of a single component within FValue.  The actual type of the component is only known by FValue::Type */
union FValueComponent
{
	FValueComponent() : Packed(0u) {}
	FValueComponent(float InValue) : Packed(0u) { Float = InValue; }
	FValueComponent(double InValue) : Packed(0u) { Double = InValue; }
	FValueComponent(int32 InValue) : Packed(0u) { Int = InValue; }
	FValueComponent(bool InValue) : Packed(0u) { Bool = InValue ? 1 : 0; }

	// 'Bool' is stored as uint8 to avoid changing on different compilers
	bool AsBool() const { return Bool != 0u; }

	const TCHAR* ToString(EValueComponentType Type, FStringBuilderBase& OutString) const;

	uint64 Packed;
	double Double;
	float Float;
	int32 Int;
	uint8 Bool;
};
static_assert(sizeof(FValueComponent) == sizeof(uint64), "bad packing");

/**
 * Can store a numeric/struct shader value of any type
 * Values are represented as a flat list of components
 */
struct FValue
{
	FValue() = default;

	explicit FValue(const FType& InType) : Type(InType)
	{
		Component.AddDefaulted(InType.GetNumComponents());
	}

	inline FValue(EValueComponentType InComponentType, int8 InNumComponents) : Type(MakeValueType(InComponentType, InNumComponents))
	{
		Component.AddDefaulted(InNumComponents);
	}

	inline FValue(float v) : Type(EValueType::Float1)
	{
		Component.Add(v);
	}

	inline FValue(float X, float Y) : Type(EValueType::Float2)
	{
		Component.Add(X);
		Component.Add(Y);
	}

	inline FValue(float X, float Y, float Z) : Type(EValueType::Float3)
	{
		Component.Add(X);
		Component.Add(Y);
		Component.Add(Z);
	}

	inline FValue(float X, float Y, float Z, float W) : Type(EValueType::Float4)
	{
		Component.Add(X);
		Component.Add(Y);
		Component.Add(Z);
		Component.Add(W);
	}

	inline FValue(double v) : Type(EValueType::Double1)
	{
		Component.Add(v);
	}

	inline FValue(double X, double Y) : Type(EValueType::Double2)
	{
		Component.Add(X);
		Component.Add(Y);
	}

	inline FValue(double X, double Y, double Z) : Type(EValueType::Double3)
	{
		Component.Add(X);
		Component.Add(Y);
		Component.Add(Z);
	}

	inline FValue(double X, double Y, double Z, double W) : Type(EValueType::Double4)
	{
		Component.Add(X);
		Component.Add(Y);
		Component.Add(Z);
		Component.Add(W);
	}

	inline FValue(const FLinearColor& Value) : Type(EValueType::Float4)
	{
		Component.Add(Value.R);
		Component.Add(Value.G);
		Component.Add(Value.B);
		Component.Add(Value.A);
	}

	inline FValue(const FVector2f& Value) : Type(EValueType::Float2)
	{
		Component.Add(Value.X);
		Component.Add(Value.Y);
	}

	inline FValue(const FVector3f& Value) : Type(EValueType::Float3)
	{
		Component.Add(Value.X);
		Component.Add(Value.Y);
		Component.Add(Value.Z);
	}

	inline FValue(const FVector3d& Value) : Type(EValueType::Double3)
	{
		Component.Add(Value.X);
		Component.Add(Value.Y);
		Component.Add(Value.Z);
	}

	inline FValue(const FVector4f& Value) : Type(EValueType::Float4)
	{
		Component.Add(Value.X);
		Component.Add(Value.Y);
		Component.Add(Value.Z);
		Component.Add(Value.W);
	}

	inline FValue(const FVector4d& Value) : Type(EValueType::Double4)
	{
		Component.Add(Value.X);
		Component.Add(Value.Y);
		Component.Add(Value.Z);
		Component.Add(Value.W);
	}

	inline FValue(bool v) : Type(EValueType::Bool1)
	{
		Component.Add(v);
	}

	inline FValue(bool X, bool Y, bool Z, bool W) : Type(EValueType::Bool4)
	{
		Component.Add(X);
		Component.Add(Y);
		Component.Add(Z);
		Component.Add(W);
	}

	inline FValue(int32 v) : Type(EValueType::Int1)
	{
		Component.Add(v);
	}

	inline const FType& GetType() const { return Type; }
	inline int32 GetNumComponents() const { return Type.GetNumComponents(); }

	inline FValueComponent GetComponent(int32 Index) const
	{
		// Return scalar value for any request to xyzw
		const int32 ComponentIndex = (Type.IsNumericScalar() && Index >= 0 && Index < 4) ? 0 : Index;
		if (Component.IsValidIndex(ComponentIndex))
		{
			return Component[ComponentIndex];
		}
		return FValueComponent();
	}

	static FValue FromMemoryImage(EValueType Type, const void* Data, uint32* OutSizeInBytes = nullptr);
	FMemoryImageValue AsMemoryImage() const;

	FFloatValue AsFloat() const;
	FDoubleValue AsDouble() const;
	FIntValue AsInt() const;
	FBoolValue AsBool() const;

	FLinearColor AsLinearColor() const;
	FVector4d AsVector4d() const;
	float AsFloatScalar() const;
	bool AsBoolScalar() const;

	bool IsZero() const;

	const TCHAR* ToString(EValueStringFormat Format, FStringBuilderBase& OutString) const;

	FType Type;
	TArray<FValueComponent, TInlineAllocator<16>> Component;
};

ENGINE_API bool operator==(const FValue& Lhs, const FValue& Rhs);
inline bool operator!=(const FValue& Lhs, const FValue& Rhs) { return !operator==(Lhs, Rhs); }
ENGINE_API uint32 GetTypeHash(const FType& Type);
ENGINE_API uint32 GetTypeHash(const FValue& Value);

ENGINE_API FValue Neg(const FValue& Value);
ENGINE_API FValue Abs(const FValue& Value);
ENGINE_API FValue Saturate(const FValue& Value);
ENGINE_API FValue Floor(const FValue& Value);
ENGINE_API FValue Ceil(const FValue& Value);
ENGINE_API FValue Round(const FValue& Value);
ENGINE_API FValue Trunc(const FValue& Value);
ENGINE_API FValue Sign(const FValue& Value);
ENGINE_API FValue Frac(const FValue& Value);
ENGINE_API FValue Fractional(const FValue& Value);
ENGINE_API FValue Sqrt(const FValue& Value);
ENGINE_API FValue Rcp(const FValue& Value);
ENGINE_API FValue Exp(const FValue& Value);
ENGINE_API FValue Exp2(const FValue& Value);
ENGINE_API FValue Log(const FValue& Value);
ENGINE_API FValue Log2(const FValue& Value);
ENGINE_API FValue Log10(const FValue& Value);
ENGINE_API FValue Sin(const FValue& Value);
ENGINE_API FValue Cos(const FValue& Value);
ENGINE_API FValue Tan(const FValue& Value);
ENGINE_API FValue Asin(const FValue& Value);
ENGINE_API FValue Acos(const FValue& Value);
ENGINE_API FValue Atan(const FValue& Value);

ENGINE_API FValue Add(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Sub(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Mul(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Div(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Less(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Greater(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue LessEqual(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue GreaterEqual(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Min(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Max(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Clamp(const FValue& Value, const FValue& Low, const FValue& High);
ENGINE_API FValue Fmod(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Modulo(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Atan2(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Dot(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Cross(const FValue& Lhs, const FValue& Rhs);
ENGINE_API FValue Append(const FValue& Lhs, const FValue& Rhs);

ENGINE_API FValue Cast(const FValue& Value, EValueType Type);

// In place versions of certain functions.  Only works on numeric types.  Overwrites input components in place,
// possibly changing their type (returns the new type).
ENGINE_API EValueType NegInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType AbsInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType SaturateInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType FloorInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType CeilInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType RoundInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType TruncInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType SignInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType FracInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType FractionalInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType SqrtInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType RcpInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType ExpInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType Exp2InPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType LogInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType Log2InPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType Log10InPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType SinInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType CosInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType TanInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType AsinInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType AcosInPlace(EValueType Type, TArrayView<FValueComponent> Component);
ENGINE_API EValueType AtanInPlace(EValueType Type, TArrayView<FValueComponent> Component);

ENGINE_API EValueType AddInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed);
ENGINE_API EValueType SubInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed);
ENGINE_API EValueType MulInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed);
ENGINE_API EValueType DivInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed);
ENGINE_API EValueType MinInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed);
ENGINE_API EValueType MaxInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed);
ENGINE_API EValueType FmodInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed);
ENGINE_API EValueType ModuloInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed);
ENGINE_API EValueType Atan2InPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed);
ENGINE_API EValueType AppendInPlace(EValueType LhsType, EValueType RhsType, TArrayView<FValueComponent> Component, int32& OutComponentsConsumed);

} // namespace UE::Shader

DECLARE_INTRINSIC_TYPE_LAYOUT(UE::Shader::EValueType);
DECLARE_INTRINSIC_TYPE_LAYOUT(UE::Shader::EValueComponentType);
