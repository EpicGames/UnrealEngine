// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompiler.h: Platform independent shader compilation definitions.
=============================================================================*/

#pragma once

#include "Templates/RefCounting.h"
#include "HAL/PlatformProcess.h"
#include "ShaderCore.h"
#include "ShaderCompilerCore.h"
#include "ShaderCompilerJobTypes.h"
#include "Shader.h"
#include "HAL/RunnableThread.h"
#include "HAL/Runnable.h"
#include "Templates/Atomic.h"
#include "Templates/UniquePtr.h"
#include "HAL/ThreadSafeCounter.h"
#include "RHIDefinitions.h"
#include "GBufferInfo.h"
#include "ShaderMaterial.h"
#include "Math/Interval.h"
#include "Misc/ScopeRWLock.h"
#include "IAssetCompilingManager.h"
#include "Containers/HashTable.h"
#include "Containers/List.h"
#include "Containers/Deque.h"
#include "Hash/Blake3.h"
#include "Hash/CityHash.h"
#include "SceneTypes.h"
#include "UObject/StrongObjectPtr.h"

#include "ShaderCompiler.generated.h"

class FAsyncCompilationNotification;
class FCbObjectView;
class FCbWriter;
class FVertexFactoryType;
class IDistributedBuildController;
class FMaterialShaderMap;
class FShaderCompileJob;
struct FShaderCompilerStats;
class FShaderKeyGenerator;
class FShaderPipelineCompileJob;
class UMaterialInterface;
class FJsonObject;
struct FAnalyticsEventAttribute;

DECLARE_LOG_CATEGORY_EXTERN(LogShaderCompilers, Log, All);

bool AreShaderErrorsFatal();
bool CreateShadersOnLoad();
bool UseMaterialVSUniformBuffer();

extern ENGINE_API bool IsShaderJobCacheDDCEnabled();
extern ENGINE_API bool IsMaterialMapDDCEnabled();

/**
 * Returns true if we should compile shaders that are only compiled in on demand shader compilation
 * modes and not stored in cooked shader maps. This can be useful for debug shaders.
 */
extern ENGINE_API bool ShouldCompileODSCOnlyShaders();

struct FShaderCompilerJobMemoryLimitInfo;

struct FShaderJobCacheStoredOutput;
class FShaderJobCache;

class FShaderCompileJobCollection
{
public:
	FShaderCompileJobCollection(FCriticalSection& InCompileQueueSection);

	FShaderCompileJob* PrepareJob(uint32 InId, const FShaderCompileJobKey& InKey, EShaderCompileJobPriority InPriority);
	FShaderPipelineCompileJob* PrepareJob(uint32 InId, const FShaderPipelineCompileJobKey& InKey, EShaderCompileJobPriority InPriority);
	void RemoveJob(FShaderCommonCompileJob* InJob);

	int32 RemoveAllPendingJobsWithId(uint32 InId);

	void SubmitJobs(const TArray<FShaderCommonCompileJobPtr>& InJobs);
	void TransferJobsToLocalWorker(const TArray<FShaderCommonCompileJobPtr>& InJobs);
	
	/** Called for all completed jobs, including those that were cache hits, duplicates of other in flight jobs, or skipped due to failed preprocessing.
	 * Can be called from multiple threads.
	 */
	void ProcessFinishedJob(FShaderCommonCompileJob* FinishedJob, EShaderCompileJobStatus Status);

	/** Adds the job to cache. */
	void AddToCacheAndProcessPending(FShaderCommonCompileJob* FinishedJob);

	/** Retrieve caching statistics. */
	void GetCachingStats(FShaderCompilerStats& OutStats) const;

	int32 GetNumPendingJobs(EShaderCompileJobPriority InPriority) const;

	int32 GetNumOutstandingJobs() const;

	int32 GetNumPendingJobs() const;

	/** Despite being named a getter, this is drawing entries out of the list of pending compile jobs, i.e. the returned jobs will be removed from the list of pending jobs. */
	int32 GetPendingJobs(EShaderCompilerWorkerType InWorkerType, EShaderCompileJobPriority InPriority, int32 MinNumJobs, int32 MaxNumJobs, TArray<FShaderCommonCompileJobPtr>& OutJobs);

private:
	/** Handles the console command to log shader compiler stats */
	void HandlePrintStats();

	/** Cache for in flight and completed jobs.*/
	TPimplPtr<FShaderJobCache> JobsCache;

	/** Debugging - console command to print stats. */
	class IConsoleObject* PrintStatsCmd;
};

#if WITH_EDITOR
class FGlobalShaderTypeCompiler
{
public:
	/**
	* Enqueues compilation of a shader of this type.
	*/
	ENGINE_API static void BeginCompileShader(const FGlobalShaderType* ShaderType, int32 PermutationId, EShaderPlatform Platform, EShaderPermutationFlags PermutationFlags, TArray<FShaderCommonCompileJobPtr>& NewJobs);

	/**
	* Enqueues compilation of a shader pipeline of this type.
	*/
	ENGINE_API static void BeginCompileShaderPipeline(EShaderPlatform Platform, EShaderPermutationFlags PermutationFlags, const FShaderPipelineType* ShaderPipeline, TArray<FShaderCommonCompileJobPtr>& NewJobs);

	/** Either returns an equivalent existing shader of this type, or constructs a new instance. */
	static FShader* FinishCompileShader(const FGlobalShaderType* ShaderType, const FShaderCompileJob& CompileJob, const FShaderPipelineType* ShaderPipelineType);
};
#endif // WITH_EDITOR

struct FShaderCompileMemoryUsage
{
	/**
	* The amount of virtual memory used (committed on Windows)
	*/
	uint64 VirtualMemory;
	/**
	* The amount of physical memory used.
	*/
	uint64 PhysicalMemory;

	/**
	 * The max memory currently used across workers
	 */
	uint64 MaxWorkerMemory;

	/**
	 * The number of active shader compiler workers
	 */
	int32 ActiveWorkerCount;
};

/** Function signature to iterate over shader compile jobs (single and pipeline jobs). */
using FShaderCompileJobCallback = TFunction<bool(FShaderCommonCompileJob*)>;

class FShaderCompileThreadRunnableBase : public FRunnable
{
	friend class FShaderCompilingManager;

	/** 64-bit hash value of the worker states to detect hung shader compile jobs. */
	uint64 WorkerStateHash;

	/** Timestamp of the last point in time the worker states have changed. */
	double WorkerStateChangeTimestamp;

	/** Priority index range is RW lock protected so that we can change it for all threads when the distributed compiling thread is shutting down and transfers its jobs to local workers. */
	mutable FRWLock PriorityRangeLock;
	TInterval<int32> PriorityIndexRange;

protected:
	/** The manager for this thread */
	class FShaderCompilingManager* Manager;
	/** The runnable thread */
	FRunnableThread* Thread;
	
	TAtomic<bool> bForceFinish;

	/**
	* Tries to print out the memory usage of all shader compile workers. When called during an out-of-memory event, it is useful to allow this process
	* to wait for any locks, so we can rule out deadlocks while reporting out-of-memory errors.
	* Returns whether the memory usage was successfully printed.
	*/
	virtual bool PrintWorkerMemoryUsage(bool bAllowToWaitForLock=true) { return false; }
	/** Returns the amount of memory (in bytes) used by external processes related to this, if any. */
	virtual FShaderCompileMemoryUsage GetExternalWorkerMemoryUsage() { return FShaderCompileMemoryUsage{}; }

	// Returns a name for this thread instance. Defaults to "ShaderCompilingThread".
	virtual const TCHAR* GetThreadName() const
	{
		return TEXT("ShaderCompilingThread");
	}

	/**
	Returns false if the new worker state hash hasn't changed in a certain period of time. If false, a warning is printed.
	An input hash of 0 will always succeed and effectively reset the timer. A value of 0 should be used when no jobs are pending.
	*/
	bool WorkerStateHeartbeat(uint64 InWorkerStateHash);

public:
	FShaderCompileThreadRunnableBase(class FShaderCompilingManager* InManager);
	virtual ~FShaderCompileThreadRunnableBase() = default;

	void SetPriorityRange(EShaderCompileJobPriority MinPriority, EShaderCompileJobPriority MaxPriority);
	TInterval<int32> GetPriorityRange() const;

	void StartThread();

	// FRunnable interface.
	virtual void Stop() { bForceFinish = true; }
	virtual uint32 Run();
	inline void WaitForCompletion() const
	{
		if( Thread )
		{
			Thread->WaitForCompletion();
		}
	}

	/** Main work loop. */
	virtual int32 CompilingLoop() = 0;

	/** Returns the type of shader workers this thread represents. */
	virtual EShaderCompilerWorkerType GetWorkerType() const = 0;

	/** Events from the manager */
	virtual void OnMachineResourcesChanged() {}

	/**
	 * Invokes the specified callback on each pending job this shader compile thread is processing.
	 * Use this to find compile jobs that are already sent to a worker, since once a worker processes them,
	 * they are no longer in the "pending list" of the shader compile manager (i.e. FShaderCompileJobCollection specifically).
	 * @param PendingJobCallback Function callback that is invoked on each pending job. Returns false to exit the iteration early.
	 */
	virtual void ForEachPendingJob(const FShaderCompileJobCallback& PendingJobCallback) const {}
};

struct FWorkerMemoryUsage
{
	// The index of the worker in the pool
	int32 WorkerIndex;

	// The amount of virtual memory used (committed on Windows)
	uint64 VirtualMemory;

	// The amount of physical memory used
	uint64 PhysicalMemory;
};

/** 
 * Shader compiling thread
 * This runs in the background while UE is running, launches shader compile worker processes when necessary, and feeds them inputs and reads back the outputs.
 */
class FShaderCompileThreadRunnable : public FShaderCompileThreadRunnableBase
{
	friend class FShaderCompilingManager;
private:

	/** Information about the active workers that this thread is tracking. */
	TArray<TUniquePtr<struct FShaderCompileWorkerInfo>> WorkerInfos;
	mutable FCriticalSection WorkerInfosLock;
	mutable FCriticalSection PerWorkerMemUsageLock;

	/** Tracks the last time that this thread checked if the workers were still active. */
	double LastCheckForWorkersTime = 0.0;

	/** Tracks the last time the per worker memory usage was checked. This is expensive to query, so if recent enough not queried again. */
	double LastCheckForWorkersMemoryUsageTime = 0.0;

	/** Whether to read/write files for SCW in parallel (can help situations when this takes too long for a number of reasons) */
	bool bParallelizeIO = false;

	/** List of jobs that have been backlogged when workers had to be closed due to reaching memory limits. These jobs will be picked up first before new jobs are pulled from the manager job queue. */
	TArray<FShaderCommonCompileJobPtr> BackloggedJobs;

	/** Is dynamic SCW budget enabled? When enabled, the SCW budget is adjusted based on current memory needs */
	bool bDynamicSCWBudgetEnabled;
	/** Memory to reserve in MiB when running the dynamic budget */
	int32 DynamicSCWBudgetReserveInMiB;
	/** Granularity of SCW budget. This is to reduce the frequency of changes, limiting it to significant changes */
	int32 DynamicSCWBudgetGranularityInMiB;

	struct FMemoryMonitoringState
	{
		double LastTimeOfMemoryLimitPoll = 0.0;
		double LastTimeOfSuspendingOrResumingWorkers = 0.0;
		bool bHasFailedToSuspendWorkers = false;
	}
	MemoryMonitoringState;

	/** Most recent capture of per-worker memory usage. Note: Not updated every tick. */
	TArray<FWorkerMemoryUsage> PerWorkerMemoryUsage;

public:
	/** Initialization constructor. */
	FShaderCompileThreadRunnable(class FShaderCompilingManager* InManager);
	virtual ~FShaderCompileThreadRunnable();

	virtual EShaderCompilerWorkerType GetWorkerType() const override final
	{
		return EShaderCompilerWorkerType::LocalThread;
	}

	/** Returns most recent capture of per-worker memory usage */
	TArray<FWorkerMemoryUsage> GetRecentPerWorkerMemoryUsage() const
	{
		FScopeLock PerWorkerMemScopeLock(&PerWorkerMemUsageLock);
		return PerWorkerMemoryUsage;
	};

protected:
	virtual bool PrintWorkerMemoryUsage(bool bAllowToWaitForLock = true) override;
	virtual FShaderCompileMemoryUsage GetExternalWorkerMemoryUsage() override;

private:

	/** 
	 * Grabs tasks from Manager->CompileQueue in a thread safe way and puts them into QueuedJobs of available workers. 
	 */
	int32 PullTasksFromQueue();

	/**
	 * Writes completed jobs to Manager->ShaderMapJobs.
	 */
	void PushCompletedJobsToManager();

	/** Used when compiling through workers, writes out the worker inputs for any new tasks in WorkerInfos.QueuedJobs. */
	void WriteNewTasks();

	/** Used when compiling through workers, launches worker processes if needed. */
	bool LaunchWorkersIfNeeded();

	/** Used when compiling through workers, attempts to open the worker output file if the worker is done and read the results. Returns number of results processed. */
	int32 ReadAvailableResults();

	/** Used when compiling directly through the console tools dll. */
	void CompileDirectlyThroughDll();

	/** Main work loop. */
	virtual int32 CompilingLoop() override;

	virtual void OnMachineResourcesChanged() override;

	virtual void ForEachPendingJob(const FShaderCompileJobCallback& PendingJobCallback) const override;

	void PrintWorkerMemoryUsageWithLockTaken();

	/** Returns the total number of workers this thread is handling. */
	int32 GetNumberOfWorkers() const;

	/** Returns the number of available workers. Only call inside the critical section WorkerInfosLock. */
	int32 GetNumberOfAvailableWorkersUnsafe() const;
	int32 GetNumberOfAvailableWorkers() const;

	/** Returns the number of suspended workers. Only call inside the critical section WorkerInfosLock. */
	int32 GetNumberOfSuspendedWorkersUnsafe() const;

	/** Caches memory usage of each individual worker into PerWorkerMemoryUsage if stale */
	void CachePerWorkerMemoryUsage();

	/** Calculates median virtual memory usage from worker memory list */
	uint64 GetPerWorkerVirtualMemoryMedian(const TArray<FWorkerMemoryUsage>& InWorkerMemoryUsage) const;

	/** Writes per-worker summary to insights and log */
	void TracePerWorkerSummary(const TArray<FWorkerMemoryUsage>& InPerWorkerMemoryUsage, const bool bInHasMemoryViolation) const;

	/**
	 * Suspends the specified number of workers and moves all their compile jobs to the backlog queue.
	 * Returns the number of workers that have been suspended. The last worker cannot be suspended.
	 */
	int32 SuspendWorkersAndBacklogJobs(int32 NumWorkers, int32* OutNumBackloggedJobs = nullptr);

	/**
	 * Makes the specified number of workers available again after they have been suspended.
	 * Returns the number of workers that have been resumed. If all workers were already available, the return value is 0.
	 */
	int32 ResumeSuspendedWorkers(int32 NumWorkers);

	/** Deletes the output file of the specified worker if it exists and discards its content. This is called when a worker output is considered stale because it was previously suspended. */
	void DiscardWorkerOutputFile(int32 WorkerIndex);

	/** Returns the working directory for the specified shader compile worker. */
	FString GetWorkingDirectoryForWorker(int32 WorkerIndex, bool bRelativePath = false) const;

	/** Calculates memory totals from per-worker memory usage array */
	FShaderCompileMemoryUsage CalcPerWorkerMemoryUsageTotal(const TArray<FWorkerMemoryUsage>& InPerWorkerMemoryUsage) const;

	/** Checks it the memory limit for shader compile workers has been exceeded and suspend workers as needed. */
	void CheckMemoryLimitViolation();

	/** Return an estimate for how much memory is required for a single worker. Used for both suspension and resumption */
	int64 GetEstimatedMemoryRequiredPerWorker(const FShaderCompileMemoryUsage& LimitInfo);

	/** Queries the status if the worker processes have violated the memory limitation. */
	bool QueryMemoryLimitViolationStatus(const FShaderCompileMemoryUsage& InSCMemoryUsage, FShaderCompilerJobMemoryLimitInfo& OutInfo);

	/** Dynamically adapts the SCW memory budget to fit within current memory constraints. */
	void UpdateDynamicSCWMemoryBudget(const FShaderCompileMemoryUsage& InSCMemoryUsage);
};

class FShaderCompileUtilities
{
public:
	/** Execute the specified (single or pipeline) shader compile job. */
	static void ExecuteShaderCompileJob(FShaderCommonCompileJob& Job);

	static class FArchive* CreateFileHelper(const FString& Filename);
	static void MoveFileHelper(const FString& To, const FString& From);
	static void DeleteFileHelper(const FString& Filename);

	static ENGINE_API void GenerateBrdfHeaders(const EShaderPlatform Platform);
	static ENGINE_API void GenerateBrdfHeaders(const FName& ShaderFormat);
	static void ApplyDerivedDefines(FShaderCompilerEnvironment& OutEnvironment, FShaderCompilerEnvironment* SharedEnvironment, const EShaderPlatform Platform);
	static void AppendGBufferDDCKeyString(const EShaderPlatform Platform, FString& KeyString);
	static void AppendGBufferDDCKey(const EShaderPlatform Platform, FShaderKeyGenerator& KeyGen);
	static ENGINE_API void WriteGBufferInfoAutogen(EShaderPlatform TargetPlatform, ERHIFeatureLevel::Type FeatureLevel);

	static void ApplyFetchEnvironment(FShaderMaterialPropertyDefines& DefineData, const FShaderCompilerEnvironment& Environment);
	static void ApplyFetchEnvironment(FShaderGlobalDefines& DefineData, const FShaderCompilerEnvironment& Environment, const EShaderPlatform Platform);
	static void ApplyFetchEnvironment(FShaderLightmapPropertyDefines& DefineData, const FShaderCompilerEnvironment& Environment);
	static void ApplyFetchEnvironment(FShaderCompilerDefines& DefineData, const FShaderCompilerEnvironment& Environment);

	static ENGINE_API EGBufferLayout FetchGBufferLayout(const FShaderCompilerEnvironment& Environment);

	static ENGINE_API FGBufferParams FetchGBufferParamsRuntime(EShaderPlatform Platform, EGBufferLayout Layout); // this function is called from renderer
	static FGBufferParams FetchGBufferParamsPipeline(EShaderPlatform Platform, EGBufferLayout Layout);
};

class FShaderCompileDistributedThreadRunnable_Interface : public FShaderCompileThreadRunnableBase
{
	uint32 NumDispatchedJobs;

	TSparseArray<TUniquePtr<class FDistributedShaderCompilerTask>> DispatchedTasks;

public:
	/** Initialization constructor. */
	FShaderCompileDistributedThreadRunnable_Interface(class FShaderCompilingManager* InManager, class IDistributedBuildController& InController);
	virtual ~FShaderCompileDistributedThreadRunnable_Interface();

	virtual EShaderCompilerWorkerType GetWorkerType() const override final
	{
		return EShaderCompilerWorkerType::Distributed;
	}

	/** Main work loop. */
	virtual int32 CompilingLoop() override;

	static bool IsSupported();

	virtual void ForEachPendingJob(const FShaderCompileJobCallback& PendingJobCallback) const override;

protected:
	
	IDistributedBuildController& CachedController;
	TMap<EShaderPlatform, TArray<FString> >	PlatformShaderInputFilesCache;

private:

	void DispatchShaderCompileJobsBatch(TArray<FShaderCommonCompileJobPtr>& JobsToSerialize);

	virtual const TCHAR* GetThreadName() const override;
};

/** Results for a single compiled and finalized shader map. */
using FShaderMapFinalizeResults = FShaderMapCompileResults;

struct FDistributedBuildStats;

USTRUCT()
struct FShaderCompilerCounters
{
	GENERATED_BODY()
	/** This tracks accumulated wait time from local workers during the lifetime of the stats.
	 *
	 * Wait time is only counted for local workers that are alive and not between their invocations
	 */
	UPROPERTY()
	double AccumulatedLocalWorkerIdleTime = 0.0;

	/** How many times we registered idle time? */
	UPROPERTY()
	double TimesLocalWorkersWereIdle = 0;

	/** Number of jobs assigned to workers, no matter if they completed or not - used to average pending time. */
	UPROPERTY()
	int64 JobsAssigned = 0;

	/** Total number jobs completed. */
	UPROPERTY()
	int64 JobsCompleted = 0;

	/** Amount of time a job had to spent in pending queue (i.e. waiting to be assigned to a worker). */
	UPROPERTY()
	double AccumulatedPendingTime = 0;

	/** Max amount of time any single job was pending (waiting to be assigned to a worker). */
	UPROPERTY()
	double MaxPendingTime = 0;

	/** Amount of time job spent being processed by the worker. */
	UPROPERTY()
	double AccumulatedJobExecutionTime = 0;

	/** Max amount of time any single job spent being processed by the worker. */
	UPROPERTY()
	double MaxJobExecutionTime = 0;

	/** Amount of time job spent being processed overall. */
	UPROPERTY()
	double AccumulatedJobLifeTime = 0;

	/** Max amount of time any single job spent being processed overall. */
	UPROPERTY()
	double MaxJobLifeTime = 0;

	/** Number of local job batches seen. */
	UPROPERTY()
	int64 LocalJobBatchesSeen = 0;

	/** Total jobs in local job batches. */
	UPROPERTY()
	int64 TotalJobsReportedInLocalJobBatches = 0;

	/** Number of distributed job batches seen. */
	UPROPERTY()
	int64 DistributedJobBatchesSeen = 0;

	/** Total jobs in local job batches. */
	UPROPERTY()
	int64 TotalJobsReportedInDistributedJobBatches = 0;

	/** Size of the smallest output shader code. */
	UPROPERTY()
	int32 MinShaderCodeSize = 0;

	/** Size of the largest output shader code. */
	UPROPERTY()
	int32 MaxShaderCodeSize = 0;

	/** Total accumulated size of all output shader codes. */
	UPROPERTY()
	uint64 AccumulatedShaderCodeSize = 0;

	/** Number of accumulated output shader codes. */
	UPROPERTY()
	uint64 NumAccumulatedShaderCodes = 0;

	/** Total number of DDC misses on shader maps. */
	UPROPERTY()
	uint32 ShaderMapDDCMisses = 0;

	/** Total number of DDC hits on shader maps. */
	UPROPERTY()
	uint32 ShaderMapDDCHits = 0;

	/** Total number of job cache query attempts. */
	UPROPERTY()
	uint64 TotalCacheSearchAttempts = 0;

	/** Total number of hits in the job cache (i.e. input hashes seen >1 time) */
	UPROPERTY()
	uint64 TotalCacheHits = 0;

	/** Total number of duplicate jobs (input hash matches an in-flight job, processed when in-flight job completes) */
	UPROPERTY()
	uint32 TotalCacheDuplicates = 0;

	/** Total number of DDC queries in the job cache (per-shader DDC). */
	UPROPERTY()
	uint32 TotalCacheDDCQueries = 0;

	/** Total number of DDC hits in the job cache (per shader DDC, as opposed to shader map DDC stats above). */
	UPROPERTY()
	uint32 TotalCacheDDCHits = 0;

	/** Total number of unique input hashes seen in job cache queries */
	UPROPERTY()
	uint64 UniqueCacheInputHashes = 0;

	/** Total number of unique job outputs stored in the cache.
	  * Outputs are deduplicated based on a content hash so this number is in practice smaller than UniqueCacheInputHashes.
	  */
	UPROPERTY()
	uint64 UniqueCacheOutputs = 0;

	/** Total amount of memory currently used by the job cache */
	UPROPERTY()
	uint64 CacheMemUsed = 0;

	/** Memory budget allocated for the job cache */
	UPROPERTY()
	uint64 CacheMemBudget = 0;

	/** Maximum number of remote agents used during compilation. */
	UPROPERTY()
	uint32 MaxRemoteAgents = 0;

	/** Maximum number of CPU cores active across all remote agents. */
	UPROPERTY()
	uint32 MaxActiveAgentCores = 0;

	/** Largest violation of shader compiler memory. */
	UPROPERTY()
	int64 LargestViolationMemorySizeInBytes = 0;

	/** Largest violation of shader compiler memory - worker count */
	UPROPERTY()
	int32 LargestViolationWorkerCount = 0;

	FShaderCompilerCounters& operator+=(const FShaderCompilerCounters& Other)
	{
		AccumulatedLocalWorkerIdleTime += Other.AccumulatedLocalWorkerIdleTime;
		TimesLocalWorkersWereIdle += Other.TimesLocalWorkersWereIdle;
		JobsAssigned += Other.JobsAssigned;
		JobsCompleted += Other.JobsCompleted;
		AccumulatedPendingTime += Other.AccumulatedPendingTime;
		MaxPendingTime = FMath::Max(Other.MaxPendingTime, MaxPendingTime);
		AccumulatedJobExecutionTime += Other.AccumulatedJobExecutionTime;
		MaxJobExecutionTime = FMath::Max(Other.MaxJobExecutionTime, MaxJobExecutionTime);
		AccumulatedJobLifeTime += Other.AccumulatedJobLifeTime;
		MaxJobLifeTime = FMath::Max(Other.MaxJobLifeTime, MaxJobLifeTime);
		LocalJobBatchesSeen += Other.LocalJobBatchesSeen;
		TotalJobsReportedInLocalJobBatches += Other.TotalJobsReportedInLocalJobBatches;
		DistributedJobBatchesSeen += Other.DistributedJobBatchesSeen;
		TotalJobsReportedInDistributedJobBatches += Other.TotalJobsReportedInDistributedJobBatches;
		if (Other.MinShaderCodeSize > 0)
		{
			MinShaderCodeSize = (MinShaderCodeSize > 0 ? FMath::Min(MinShaderCodeSize, Other.MinShaderCodeSize) : Other.MinShaderCodeSize);
		}
		MaxShaderCodeSize = FMath::Max(Other.MaxShaderCodeSize, MaxShaderCodeSize);
		AccumulatedShaderCodeSize += Other.AccumulatedShaderCodeSize;
		NumAccumulatedShaderCodes += Other.NumAccumulatedShaderCodes;
		ShaderMapDDCMisses += Other.ShaderMapDDCMisses;
		ShaderMapDDCHits += Other.ShaderMapDDCHits;
		TotalCacheSearchAttempts += Other.TotalCacheSearchAttempts;
		TotalCacheHits += Other.TotalCacheHits;
		TotalCacheDuplicates += Other.TotalCacheDuplicates;
		TotalCacheDDCQueries += Other.TotalCacheDDCQueries;
		TotalCacheDDCHits += Other.TotalCacheDDCHits;
		UniqueCacheInputHashes += Other.UniqueCacheInputHashes;
		UniqueCacheOutputs += Other.UniqueCacheOutputs;
		CacheMemUsed += Other.CacheMemUsed;
		CacheMemBudget += Other.CacheMemBudget;
		MaxRemoteAgents += Other.MaxRemoteAgents;
		MaxActiveAgentCores += Other.MaxActiveAgentCores;
		if (Other.LargestViolationMemorySizeInBytes > LargestViolationMemorySizeInBytes)
		{
			LargestViolationMemorySizeInBytes = Other.LargestViolationMemorySizeInBytes;
			LargestViolationWorkerCount = Other.LargestViolationWorkerCount;
		}

		return *this;
	}
};

USTRUCT()
struct FShaderCompilerMaterialCounters
{
	GENERATED_BODY()
	/** The total number of materials cooked.  This corresponds to UMaterialInterface::Presave() */
	int32 NumMaterialsCooked = 0;

	/** The total number of materials that have been translated.  */
	UPROPERTY()
	int32 MaterialTranslateCalls = 0;

	/** The total time in seconds to translate all materials.  */
	UPROPERTY()
	double MaterialTranslateTotalTimeSec = 0.0;

	/** The total time spent actually translating materials (rather than for instance accessing the DDC cache). */
	UPROPERTY()
	double MaterialTranslateTranslationOnlyTimeSec = 0.0;

	/** The total time spent serializing DDC results. */
	UPROPERTY()
	double MaterialTranslateSerializationOnlyTimeSec = 0.0;

	/** The total number times a material translation was skipped because the the results were in the DDC. */
	UPROPERTY()
	int32 MaterialCacheHits = 0;

	FShaderCompilerMaterialCounters& operator+=(const FShaderCompilerMaterialCounters& Other)
	{
		NumMaterialsCooked += Other.NumMaterialsCooked;
		MaterialTranslateCalls += Other.MaterialTranslateCalls;
		MaterialTranslateTotalTimeSec += Other.MaterialTranslateTotalTimeSec;
		MaterialTranslateTranslationOnlyTimeSec += Other.MaterialTranslateTranslationOnlyTimeSec;
		MaterialTranslateSerializationOnlyTimeSec += Other.MaterialTranslateSerializationOnlyTimeSec;
		MaterialCacheHits += Other.MaterialCacheHits;

		return *this;
	}

	void WriteStatSummary(const TCHAR* AggregatedSuffix);
	void GatherAnalytics(TArray<FAnalyticsEventAttribute>& Attributes);
};

/** Structure used to describe compiling time of a shader type (for all the instances of it that we have seen).
	Can be dumped to CSV file via 'r.ShaderCompiler.DumpShaderTimeStats' CVar. */
USTRUCT()
struct FShaderTimings
{
	GENERATED_BODY()

	UPROPERTY()
	float MinCompileTime = 0.0f;
	UPROPERTY()
	float MaxCompileTime = 0.0f;
	UPROPERTY()
	float TotalCompileTime = 0.0f;
	UPROPERTY()
	float TotalPreprocessTime = 0.0f;
	UPROPERTY()
	int32 NumCompiled = 0;
	UPROPERTY()
	float AverageCompileTime = 0.0f;	// stored explicitly as an optimization

	FShaderTimings& operator+=(const FShaderTimings& Other)
	{
		MinCompileTime = FMath::Min(MinCompileTime, Other.MinCompileTime);
		MaxCompileTime = FMath::Max(MaxCompileTime, Other.MaxCompileTime);
		TotalCompileTime += Other.TotalCompileTime;
		TotalPreprocessTime += Other.TotalPreprocessTime;
		NumCompiled += Other.NumCompiled;
		if (NumCompiled)
		{
			AverageCompileTime = TotalCompileTime / static_cast<float>(NumCompiled);
		}
		return *this;
	}
};

USTRUCT()
struct FShaderCompilerSinglePermutationStat
{
	GENERATED_BODY()

	FShaderCompilerSinglePermutationStat()
		: PermutationHash(0)
		, Compiled(0)
		, Cooked(0)
		, CompiledDouble(0)
		, CookedDouble(0)
	{}

	FShaderCompilerSinglePermutationStat(uint64 PermutationHash, uint32 Compiled, uint32 Cooked)
		: PermutationHash(PermutationHash)
		, Compiled(Compiled)
		, Cooked(Cooked)
		, CompiledDouble(0)
		, CookedDouble(0)

	{}

	static uint64 GetPermutationHash(const FString& PermutationString)
	{
		return CityHash64(reinterpret_cast<const char*>(*PermutationString), PermutationString.Len() * sizeof(TCHAR));
	}

	UPROPERTY()
	uint64 PermutationHash;

	UPROPERTY()
	uint32 Compiled = 0u;

	UPROPERTY()
	uint32 Cooked = 0u;

	UPROPERTY()
	uint32 CompiledDouble = 0u;

	UPROPERTY()
	uint32 CookedDouble = 0u;
};

USTRUCT()
struct FShaderStats
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FShaderCompilerSinglePermutationStat> PermutationCompilations;

	UPROPERTY()
	uint32 Compiled = 0;

	UPROPERTY()
	uint32 Cooked = 0;

	UPROPERTY()
	uint32 CompiledDouble = 0;

	UPROPERTY()
	uint32 CookedDouble = 0;

	UPROPERTY()
	float CompileTime = 0.f;

	FShaderStats& operator+=(const FShaderStats& Other)
	{
		if (Compiled)
		{
			CompiledDouble += Other.Compiled;
		}
		else
		{
			Compiled += Other.Compiled;
		}

		if (Cooked)
		{
			CookedDouble += Other.Cooked;
		}
		else
		{
			Cooked += Other.Cooked;
		}

		CompiledDouble += Other.CompiledDouble;
		CookedDouble += Other.CookedDouble;
		CompileTime += Other.CompileTime;

		PermutationCompilations.Append(Other.PermutationCompilations);

		return *this;
	}
};

struct FShaderCompilerStats
{
public:
	using ShaderCompilerStats = TMap<FString, FShaderStats>;

	void IncrementMaterialCook();
	void IncrementMaterialTranslated(double InTotalTime, double InTranslationOnlyTime, double InSerializeTime);
	void IncrementMaterialCacheHit();

	ENGINE_API void RegisterCookedShaders(uint32 NumCooked, float CompileTime, EShaderPlatform Platform, const FString MaterialPath, FString PermutationString = FString());
	ENGINE_API void RegisterCompiledShaders(uint32 NumPermutations, EShaderPlatform Platform, const FString MaterialPath, FString PermutationString = FString());
	const TSparseArray<ShaderCompilerStats>& GetShaderCompilerStats() { return CompileStats; }
	ENGINE_API void WriteStats(class FOutputDevice* Ar = nullptr);
	ENGINE_API void WriteStatSummary();
	ENGINE_API uint32 GetTotalShadersCompiled();

	ENGINE_API void Aggregate(FShaderCompilerStats& Other);
	ENGINE_API void WriteToCompactBinary(FCbWriter& Writer);
	ENGINE_API void ReadFromCompactBinary(FCbObjectView& Reader);
	ENGINE_API TSharedPtr<FJsonObject> ToJson();
	inline void SetMultiProcessAggregated() { bMultiProcessAggregated = true; }

	void AddDDCMiss(uint32 NumMisses);
	uint32 GetDDCMisses() const;
	void AddDDCHit(uint32 NumHits);
	uint32 GetDDCHits() const;
	double GetTimeShaderCompilationWasActive();

	enum class EExecutionType
	{
		Local,
		Distributed
	};

	struct FWorkerDiagnosticsInfo
	{
		FShaderCompileWorkerDiagnostics WorkerDiagnosticsOutput;
		FString BatchLabel;
		int32 BatchSize = 0;
		uint32 WorkerId = 0; // Worker process ID or 0 if it came from a distributed job
	};

	/** Informs statistics about a time a local ShaderCompileWorker spent idle. */
	void RegisterLocalWorkerIdleTime(double IdleTime);

	/** Lets the stats to know about a newly added job. Job will be modified to include the current timestamp. */
	void RegisterNewPendingJob(FShaderCommonCompileJob& InOutJob);

	/** Marks the job as given out to a worker for execution for the stats purpose. Job will be modified to include the current timestamp. */
	void RegisterAssignedJob(FShaderCommonCompileJob& InOutJob);

	/** Marks the job as finished for the stats purpose. Job will be modified to include the current timestamp. */
	void RegisterFinishedJob(FShaderCommonCompileJob& InOutJob);

	/** Informs statistics about a new job batch, so we can tally up batches. */
	void RegisterJobBatch(int32 NumJobs, EExecutionType ExecType);

	/** Informs about current distributed build statistics. */
	void RegisterDistributedBuildStats(const FDistributedBuildStats& InStats);

	/** Informs statistics about a new worker diagnostics for a finished job batch. */
	void RegisterWorkerDiagnostics(const FShaderCompileWorkerDiagnostics& InDiagnostics, FString&& InBatchLabel, int32 InBatchSize, uint32 InWorkerId);

	/** Informs statistics about shader memory violation */
	void RegisterMemoryViolation(const int64 InMemorySizeInBytes, const int32 InWorkerCount);

	ENGINE_API void GatherAnalytics(const FString& BaseName, TArray<FAnalyticsEventAttribute>& Attributes);

private:
	friend class FShaderJobCache;
	FCriticalSection CompileStatsLock;
	TSparseArray<ShaderCompilerStats> CompileStats;

	FShaderCompilerCounters Counters;
	FShaderCompilerMaterialCounters MaterialCounters;

	/** Accumulates the job lifetimes without overlaps */
	TArray<TInterval<double>> JobLifeTimeIntervals;

	/** Map of shader names to their compilation timings */
	TMap<FString, FShaderTimings> ShaderTimings;

	/** Array of diagnostics information per batch. */
	TArray<FWorkerDiagnosticsInfo> WorkerDiagnostics;

	bool bMultiProcessAggregated = false;
};

struct FShaderCompilerMemoryLimits
{
	int64 CurrentLimit;
	int64 BaseLimit;
};

extern ENGINE_API FShaderCompilerMemoryLimits GetShaderCompilerCurrentMemoryLimits();
extern ENGINE_API void SetShaderCompilerTargetMemoryLimitInMiB(int32 NewLimit);
extern ENGINE_API void SetShaderCompilerAssetMemoryLoanInMiB(int32 NewLoan);

/**  
 * Manager of asynchronous and parallel shader compilation.
 * This class contains an interface to enqueue and retreive asynchronous shader jobs, and manages a FShaderCompileThreadRunnable.
 */
class FShaderCompilingManager : IAssetCompilingManager
{
	friend class FShaderCompileThreadRunnableBase;
	friend class FShaderCompileThreadRunnable;

#if PLATFORM_WINDOWS
	friend class FShaderCompileXGEThreadRunnable_XmlInterface;
#endif // PLATFORM_WINDOWS
	friend class FShaderCompileDistributedThreadRunnable_Interface;
	friend class FShaderCompileFASTBuildThreadRunnable;

public:
	/** Get the name of the asset type this compiler handles */
	ENGINE_API static FName GetStaticAssetTypeName();

private:
	FName GetAssetTypeName() const override;
	FTextFormat GetAssetNameFormat() const override;
	TArrayView<FName> GetDependentTypeNames() const override;
	int32 GetNumRemainingAssets() const override;
	void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;
	void ProcessAsyncTasks(const AssetCompilation::FProcessAsyncTaskParams& Params) override;

	//////////////////////////////////////////////////////
	// Thread shared properties: These variables can only be read from or written to when a lock on CompileQueueSection is obtained, since they are used by both threads.

	/** Tracks whether we are compiling while the game is running.  If true, we need to throttle down shader compiling CPU usage to avoid starving the runtime threads. */
	bool bCompilingDuringGame;

	/** Map from shader map Id to the compile results for that map, used to gather compiled results. */
	TMap<int32, FPendingShaderMapCompileResultsPtr> ShaderMapJobs;

	/** Number of jobs currently being compiled.  This includes CompileQueue and any jobs that have been assigned to workers but aren't complete yet. */
	int32 NumExternalJobs;

	void ReleaseJob(FShaderCommonCompileJobPtr& Job);
	void ReleaseJob(FShaderCommonCompileJob* Job);

	/** Critical section used to gain access to the variables above that are shared by both the main thread and the FShaderCompileThreadRunnable. */
	FCriticalSection CompileQueueSection;

	/** Collection of all outstanding jobs */
	FShaderCompileJobCollection AllJobs;

	//////////////////////////////////////////////////////
	// Main thread state - These are only accessed on the main thread and used to track progress

	/** Map from shader map id to results being finalized.  Used to track shader finalizations over multiple frames. */
	TMap<int32, FShaderMapFinalizeResults> PendingFinalizeShaderMaps;

	/** The threads spawned for shader compiling. */
	TArray<TUniquePtr<FShaderCompileThreadRunnableBase>> Threads;

	//////////////////////////////////////////////////////
	// Configuration properties - these are set only on initialization and can be read from either thread

	/** Number of busy threads to use for shader compiling while loading. */
	uint32 NumShaderCompilingThreads;
	/** Number of busy threads to use for shader compiling while in game. */
	uint32 NumShaderCompilingThreadsDuringGame;
	/** Largest number of jobs that can be put in the same batch. */
	int32 MaxShaderJobBatchSize;
	/** Number of runs through single-threaded compiling before we can retry to compile through workers. -1 if not used. */
	int32 NumSingleThreadedRunsBeforeRetry;
	/** Number of preprocessed shader sources that are dumped due to a crash of the shader compiler. */
	/** This state is updated when a job is completed, which happens on a worker thread.  The state is read on the Game Thread to see if we should dump the source to begin with. */
	std::atomic<int32> NumDumpedShaderSources = 0;
	/** Process Id of UE. */
	uint32 ProcessId;
	/** Whether to allow compiling shaders through the worker application, which allows multiple cores to be used. */
	bool bAllowCompilingThroughWorkers;
	/** Whether to allow shaders to compile in the background or to block after each material. */
	bool bAllowAsynchronousShaderCompiling;
	/** Whether shaders are compiled exclusively through the distributed shader controller and no local shader compiling thread is allocated. Ignored if 'bAllowCompilingThroughWorkers' is false. */
	bool bUseOnlyDistributedCompilationThread = false;
	/** Whether to ask to retry a failed shader compile error. */
	bool bPromptToRetryFailedShaderCompiles;
	/** If enabled when we enter the prompt to retry we will break in the debugger if one is attached rather than prompting. */
	bool bDebugBreakOnPromptToRetryShaderCompile = false;
	/** Whether to log out shader job completion times on the worker thread.  Useful for tracking down which global shader is taking a long time. */
	bool bLogJobCompletionTimes;
	/** Target execution time for ProcessAsyncResults.  Larger values speed up async shader map processing but cause more hitchiness while async compiling is happening. */
	float ProcessGameThreadTargetTime;
	/** Base directory where temporary files are written out during multi core shader compiling. */
	FString ShaderBaseWorkingDirectory;
	/** Absolute version of ShaderBaseWorkingDirectory. */
	FString AbsoluteShaderBaseWorkingDirectory;
	/** Absolute path to the directory to dump shader debug info to. */
	FString AbsoluteShaderDebugInfoDirectory;
	/** Name of the shader worker application. */
	FString ShaderCompileWorkerName;
	/** Last value of GetNumRemainingAssets */
	int32 LastNumRemainingAssets = 0;
	/** If dumping crash logs for workers is enabled and an absolute path is used (i.e. -AbsLog), this contains the base directory path. */
	FString WorkerCrashLogBaseDirectory;
	/** When the shader compiling manager is shutting down, don't respond to notifications from the compiling threads anymore. */
	bool bIsManagerShuttingDown = false;

	/** 
	 * Tracks the total time that shader compile workers have been busy since startup.  
	 * Useful for profiling the shader compile worker thread time.
	 */
	double WorkersBusyTime;

	/** 
	 * Tracks which opt-in shader platforms have their warnings suppressed.
	 */
	uint64 SuppressedShaderPlatforms;

	/** Cached Engine loop initialization state */
	bool bIsEngineLoopInitialized;

	/** Interface to the build distribution controller (XGE/SN-DBS) */
	IDistributedBuildController* BuildDistributionController = nullptr;

	/** Opt out of material shader compilation and instead place an empty shader map. */
	bool bNoShaderCompilation;

	/** If we are using ODSC (On Demand Shader Compilation) we should allow for incomplete maps to still be processed. */
	bool bAllowForIncompleteShaderMaps;

	/** Used to show a notification accompanying progress. */
	TUniquePtr<FAsyncCompilationNotification> Notification;

	/** Delegate handle for delegate used to report memory usage during out-of-memory conditions. */
	FDelegateHandle OutOfMemoryDelegateHandle;

#if WITH_EDITOR
	TMap<FString, FDelegateHandle> DirectoryWatcherHandles;
#endif // WITH_EDITOR

	/** Calculate NumShaderCompilingThreads, during construction or OnMachineResourcesChanged */
	void CalculateNumberOfCompilingThreads(int32 NumberOfCores, int32 NumberOfCoresIncludingHyperthreads);

	/** Launches the worker, returns the launched process handle. */
	FProcHandle LaunchWorker(const FString& WorkingDirectory, uint32 InParentProcessId, uint32 ThreadId, const FString& WorkerInputFile, const FString& WorkerOutputFile, uint32* OutWorkerProcessId = nullptr);

	/** Blocks on completion of the given shader maps. */
	void BlockOnShaderMapCompletion(const TArray<int32>& ShaderMapIdsToFinishCompiling, TMap<int32, FShaderMapFinalizeResults>& CompiledShaderMaps);

	/** Blocks on completion of all shader maps. */
	void BlockOnAllShaderMapCompletion(TMap<int32, FShaderMapFinalizeResults>& CompiledShaderMaps);

	/** Adds compiled results to the CompiledShaderMaps, merging with the existing ones as necessary. */
	void AddCompiledResults(TMap<int32, FShaderMapFinalizeResults>& CompiledShaderMaps, int32 ShaderMapIdx, const FShaderMapFinalizeResults& Results);

	/** Finalizes the given shader map results and optionally assigns the affected shader maps to materials, while attempting to stay within an execution time budget. */
	void ProcessCompiledShaderMaps(TMap<int32, FShaderMapFinalizeResults>& CompiledShaderMaps, float TimeBudget);

	/** Finalizes the given Niagara shader map results and assigns the affected shader maps to Niagara scripts, while attempting to stay within an execution time budget. */
	void ProcessCompiledNiagaraShaderMaps(TMap<int32, FShaderMapFinalizeResults>& CompiledShaderMaps, float TimeBudget);

	/** Propagate the completed compile to primitives that might be using the materials compiled. */
	void PropagateMaterialChangesToPrimitives(TMap<TRefCountPtr<FMaterial>, TRefCountPtr<FMaterialShaderMap>>& MaterialsToUpdate);

	/** Recompiles shader jobs if requested (either interactively for failed jobs via msgbox in editor, or for jobs with errors/warnings due to r.DumpShaderDebugInfo=2/3), and returns true if a retry was needed. */
	bool HandlePotentialRetry(TMap<int32, FShaderMapFinalizeResults>& CompletedShaderMaps);
	
	/** Checks if any target platform down't support remote shader compiling */
	bool AllTargetPlatformSupportsRemoteShaderCompiling();

	/** Take some action whenever the number of remaining asset changes. */
	void UpdateNumRemainingAssets();

	/** Returns the first remote compiler controller found */
	IDistributedBuildController* FindRemoteCompilerController() const;

	/** Prints out the memory usage for shader compile worker processes, if they exist. */
	void ReportMemoryUsage();

	/** Takes the ownership of the new shader compiling thread and returns its non-owning pointer. If 'bDelayThreadExecution' is false, the thread is started immediately. */
	FShaderCompileThreadRunnableBase* LaunchShaderCompilingThread(TUniquePtr<FShaderCompileThreadRunnableBase>&& NewShaderCompilingThread, bool bDelayThreadExecution = false);

	/** Launches the thread for remote shader compilation. */
	FShaderCompileThreadRunnableBase* LaunchRemoteShaderCompilingThread(bool bDelayThreadExecution = false);

	/** Launches the thread for local shader compilation. If the distributed shader controller supports local workers, this *might not* be invoked until this condition changes. */
	FShaderCompileThreadRunnableBase* LaunchLocalShaderCompilingThread(bool bDelayThreadExecution = false);

	/** Returns the shader compiling thread of the specified kind or null if there is none. */
	FShaderCompileThreadRunnableBase* FindShaderCompilingThread(EShaderCompilerWorkerType InWorkerType);

	/**
	 * Notifies the manager that one of the shader compiling thread has started.
	 * This updates the job priority range for all threads depending on how many threads the manager handles.
	 */
	void OnShaderCompilingThreadStarted(FShaderCompileThreadRunnableBase* Thread);

	/**
	 * Notifies the manager that one of the shader compiling thread has stopped and must be removed from the list.
	 * This updates the job priority range for all threads depending on how many threads the manager handles.
	 */
	void OnShaderCompilingThreadStopped(FShaderCompileThreadRunnableBase* Thread);

	/** Assigns the job priorities each shader compiling thread is responsible for depending on how many threads there are. */
	void AssignShaderCompilingThreadJobPriorities();

public:
	
	ENGINE_API FShaderCompilingManager();
	ENGINE_API ~FShaderCompilingManager();

	/** Called by external systems that have updated the number of worker threads available. */
	ENGINE_API void OnMachineResourcesChanged(int32 NumberOfCores, int32 NumberOfCoresIncludingHyperthreads);

	/** Called when CVars are changed at runtime that determine whether or not the distributed shader compiler supports local workers (e.g. 'r.XGEController.AvoidUsingLocalMachine' can be changed at runtime). */
	ENGINE_API void OnDistributedShaderCompilingChanged();

	ENGINE_API int32 GetNumPendingJobs() const;
	ENGINE_API int32 GetNumOutstandingJobs() const;
	
	ENGINE_API FShaderCompilerMemoryLimits GetCurrentMemoryLimits() const;
	ENGINE_API void SetTargetMemoryLimitInMiB(int32 NewLimit);

	/** 
	 * Returns whether to display a notification that shader compiling is happening in the background. 
	 * Note: This is dependent on NumOutstandingJobs which is updated from another thread, so the results are non-deterministic.
	 */
	bool ShouldDisplayCompilingNotification() const 
	{ 
		// Heuristic based on the number of jobs outstanding
		return GetNumOutstandingJobs() > 80 || GetNumPendingJobs() > 80 || NumExternalJobs > 10;
	}

	bool AllowAsynchronousShaderCompiling() const 
	{
		return bAllowAsynchronousShaderCompiling;
	}

	/** 
	 * Returns whether async compiling is happening.
	 * Note: This is dependent on NumOutstandingJobs which is updated from another thread, so the results are non-deterministic.
	 */
	bool IsCompiling() const
	{
		return GetNumOutstandingJobs() > 0 || HasShaderJobs() || GetNumPendingJobs() > 0 || NumExternalJobs > 0;
	}

	/**
	 * Returns whether remote compiling is enabled. Otherwise, only the local machine is used for shader compilation.
	 * This depends on command line argument '-NoRemoteShaderCompile', CVar 'r.ShaderCompiler.AllowDistributedCompilation',
	 * and whether the target platform supports remote compilin; see ITargetPlatform::CanSupportRemoteShaderCompile().
	 */
	bool IsRemoteCompilingEnabled() const
	{
		return BuildDistributionController != nullptr;
	}

	/**
	 * Returns whether shaders are exclusively compiled through distributed controller, i.e. no local shader compiling thread is allocated.
	 * If this is true, the distributed controller must not refuse job batches below a minimum threshold, otherwise those jobs will never get picked up.
	 */
	bool IsExclusiveDistributedCompilingEnabled() const
	{
		return bUseOnlyDistributedCompilationThread;
	}

	/**
	 * Returns whether normal throttling settings should be ignored because shader compilation is at the moment the only action blocking the critical path.
	 * An example of such situation is startup compilation of global shaders and default materials, but there may be more cases like that.
	 */
	bool IgnoreAllThrottling() const
	{
		return !bIsEngineLoopInitialized;
	}

	/**
	 * return true if we have shader jobs in any state
	 * shader jobs are removed when they are applied to the gamethreadshadermap
	 * accessable from gamethread
	 */
	bool HasShaderJobs() const
	{
		return ShaderMapJobs.Num() > 0 || PendingFinalizeShaderMaps.Num() > 0;
	}

	/** 
	 * Returns the number of outstanding compile jobs.
	 * Note: This is dependent on NumOutstandingJobs which is updated from another thread, so the results are non-deterministic.
	 */
	int32 GetNumRemainingJobs() const
	{
		return GetNumOutstandingJobs() + NumExternalJobs;
	}

	/**
	 * Returns the (current) number of local workers.
	 */
	int32 GetNumLocalWorkers() const
	{
		return bCompilingDuringGame ? NumShaderCompilingThreadsDuringGame : NumShaderCompilingThreads;
	}

	void SetExternalJobs(int32 NumJobs)
	{
		NumExternalJobs = NumJobs;
	}

	enum class EDumpShaderDebugInfo : int32
	{
		Never				= 0,
		Always				= 1,
		OnError				= 2,
		OnErrorOrWarning	= 3
	};

	ENGINE_API EDumpShaderDebugInfo GetDumpShaderDebugInfo() const;
	ENGINE_API EShaderDebugInfoFlags GetDumpShaderDebugInfoFlags() const;
	ENGINE_API FString CreateShaderDebugInfoPath(const FShaderCompilerInput& ShaderCompilerInput) const;
	ENGINE_API bool ShouldRecompileToDumpShaderDebugInfo(const FShaderCompileJob& Job) const;
	ENGINE_API bool ShouldRecompileToDumpShaderDebugInfo(const FShaderCompilerInput& Input, const FShaderCompilerOutput& Output, bool bSucceeded) const;


	void IncrementNumDumpedShaderSources()
	{
		NumDumpedShaderSources++;
	}

	const FString& GetAbsoluteShaderDebugInfoDirectory() const
	{
		return GetShaderDebugInfoPath();
	}

	bool AreWarningsSuppressed(const EShaderPlatform Platform) const
	{
		return (SuppressedShaderPlatforms & (static_cast<uint64>(1) << Platform)) != 0;
	}

	void SuppressWarnings(const EShaderPlatform Platform)
	{
		SuppressedShaderPlatforms |= static_cast<uint64>(1) << Platform;
	}

	bool IsShaderCompilationSkipped() const
	{
		return bNoShaderCompilation;
	}

	void SkipShaderCompilation(bool toggle)
	{
		if (AllowShaderCompiling())
		{
			bNoShaderCompilation = toggle;
		}
	}

	void SetAllowForIncompleteShaderMaps(bool toggle)
	{
		bAllowForIncompleteShaderMaps = toggle;
	}

	bool AllowForIncompleteShaderMaps() const
	{
		return bAllowForIncompleteShaderMaps;
	}

	ENGINE_API bool IsCompilingShaderMap(uint32 Id);

	/** Prepares a job of the given type for compilation.  If a job with the given Id/Key already exists, it will attempt to adjust to the higher priority if possible, and nullptr will be returned.
	  * If a non-nullptr is returned, the given job should be filled out with relevant information, then passed to SubmitJobs() when ready
	  */
	ENGINE_API FShaderCompileJob* PrepareShaderCompileJob(uint32 Id, const FShaderCompileJobKey& Key, EShaderCompileJobPriority Priority);
	ENGINE_API FShaderPipelineCompileJob* PreparePipelineCompileJob(uint32 Id, const FShaderPipelineCompileJobKey& Key, EShaderCompileJobPriority Priority);

	/** This is an entry point for all jobs that have finished the compilation. Can be called from multiple threads.*/
	ENGINE_API void ProcessFinishedJob(FShaderCommonCompileJob* FinishedJob, EShaderCompileJobStatus Status);

	/** 
	 * Adds shader jobs to be asynchronously compiled. 
	 * FinishCompilation or ProcessAsyncResults must be used to get the results.
	 */
	ENGINE_API void SubmitJobs(TArray<FShaderCommonCompileJobPtr>& NewJobs, const FString MaterialBasePath, FString PermutationString = FString());

	/**
	 * Transfers the specified jobs to local workers. If no local shader compiling thread exists, it will be launched now.
	 */
	ENGINE_API void TransferJobsToLocalWorker(TArray<FShaderCommonCompileJobPtr>& JobsToTransfer);

	/**
	* Removes all outstanding compile jobs for the passed shader maps.
	*/
	ENGINE_API void CancelCompilation(const TCHAR* MaterialName, const TArray<int32>& ShaderMapIdsToCancel);

	/** 
	 * Blocks until completion of the requested shader maps.  
	 * This will not assign the shader map to any materials, the caller is responsible for that.
	 */
	ENGINE_API void FinishCompilation(const TCHAR* MaterialName, const TArray<int32>& ShaderMapIdsToFinishCompiling);

	/** 
	 * Blocks until completion of all async shader compiling, and assigns shader maps to relevant materials.
	 * This should be called before exit if the DDC needs to be made up to date. 
	 */
	ENGINE_API void FinishAllCompilation() override;

	/** 
	 * Shutdown the shader compiler manager, this will shutdown immediately and not process any more shader compile requests. 
	 */
	ENGINE_API void Shutdown() override;

	/**
	 * Prints stats related to shader compilation to the log.
	 */
	ENGINE_API void PrintStats();

	/** Retrieve compiler statistics for all compilation done in this process.
	  *	Note that this will not include stats for any compilation that occurs in worker processes in a multiprocess cook. 
	  */
	ENGINE_API void GetLocalStats(FShaderCompilerStats & OutStats) const;

	/**
	 * Returns the current memory usage of external local compilation processes in bytes.
	 */
	ENGINE_API FShaderCompileMemoryUsage GetExternalMemoryUsage();

	/** 
	 * Processes completed asynchronous shader maps, and assigns them to relevant materials.
	 * @param TimeSlice - When more than 0, ProcessAsyncResults will be bandwidth throttled by the given timeslice, to limit hitching.
	 *		ProcessAsyncResults will then have to be called often to finish all shader maps (eg from Tick).  Otherwise, all compiled shader maps will be processed.
	 * @param bBlockOnGlobalShaderCompletion - When enabled, ProcessAsyncResults will block until global shader maps are complete.
	 *		This must be done before using global shaders for rendering.
	 */
	ENGINE_API void ProcessAsyncResults(float TimeSlice, bool bBlockOnGlobalShaderCompletion);

	/** Version of ProcessAsyncResults that specifies use of ProcessGameThreadTargetTime for the timeslice. */
	ENGINE_API void ProcessAsyncResults(bool bLimitExecutionTime, bool bBlockOnGlobalShaderCompletion);

	/** Dumps the debug information for each pending job for which PendingJobCallback returns true. */
	ENGINE_API void DumpDebugInfoForEachPendingJob(const FShaderCompileJobCallback& PendingJobCallback) const;

	/**
	 * Returns true if the given shader compile worker is still running.
	 */
	static bool IsShaderCompilerWorkerRunning(FProcHandle & WorkerHandle);
};

/** The global shader compiling thread manager. */
extern ENGINE_API FShaderCompilingManager* GShaderCompilingManager;

/** The global shader compiling stats */
extern ENGINE_API FShaderCompilerStats* GShaderCompilerStats;

#if WITH_EDITOR

/** Enqueues a shader compile job with GShaderCompilingManager. */
extern ENGINE_API void GlobalBeginCompileShader(
	const FString& DebugGroupName,
	const class FVertexFactoryType* VFType,
	const class FShaderType* ShaderType,
	const class FShaderPipelineType* ShaderPipelineType,
	int32 PermutationId,
	const TCHAR* SourceFilename,
	const TCHAR* FunctionName,
	FShaderTarget Target,
	FShaderCompilerInput& Input,
	bool bAllowDevelopmentShaderCompile,
	const FString& DebugDescription,
	const FString& DebugExtension
	);

/** Enqueues a shader compile job with GShaderCompilingManager. */
extern ENGINE_API void GlobalBeginCompileShader(
	const FString& DebugGroupName,
	const class FVertexFactoryType* VFType,
	const class FShaderType* ShaderType,
	const class FShaderPipelineType* ShaderPipelineType,
	int32 PermutationId,
	const TCHAR* SourceFilename,
	const TCHAR* FunctionName,
	FShaderTarget Target,
	FShaderCompilerInput& Input,
	bool bAllowDevelopmentShaderCompile = true,
	const TCHAR* DebugDescription = nullptr,
	const TCHAR* DebugExtension = nullptr
);

#endif // WITH_EDITOR

extern void GetOutdatedShaderTypes(TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes);

/** Implementation of the 'recompileshaders' console command.  Recompiles shaders at runtime based on various criteria. */
extern bool RecompileShaders(const TCHAR* Cmd, FOutputDevice& Ar);

/** Returns whether all global shader types containing the substring are complete and ready for rendering. if type name is null, check everything */
extern ENGINE_API bool IsGlobalShaderMapComplete(const TCHAR* TypeNameSubstring = nullptr);

#if WITH_EDITORONLY_DATA
/** Returns the delegate triggered when global shaders compilation jobs start. */
DECLARE_MULTICAST_DELEGATE(FOnGlobalShadersCompilation);
extern ENGINE_API FOnGlobalShadersCompilation& GetOnGlobalShaderCompilation();
#endif // WITH_EDITORONLY_DATA

/**
* Makes sure all global shaders are loaded and/or compiled for the passed in platform.
* Note: if compilation is needed, this only kicks off the compile.
*
* @param	Platform						Platform to verify global shaders for
* @param	bLoadedFromCacheFile			Load the shaders from cache, will error out and not compile shaders if missing
* @param	OutdatedShaderTypes				Optional list of shader types, will trigger compilation job for shader types found in this list even if the map already has the shader.
* @param	OutdatedShaderPipelineTypes		Optional list of shader pipeline types, will trigger compilation job for shader pipeline types found in this list even if the map already has the pipeline.
*/
extern ENGINE_API void VerifyGlobalShaders(EShaderPlatform Platform, bool bLoadedFromCacheFile, const TArray<const FShaderType*>* OutdatedShaderTypes = nullptr, const TArray<const FShaderPipelineType*>* OutdatedShaderPipelineTypes = nullptr);
extern ENGINE_API void VerifyGlobalShaders(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, bool bLoadedFromCacheFile, const TArray<const FShaderType*>* OutdatedShaderTypes = nullptr, const TArray<const FShaderPipelineType*>* OutdatedShaderPipelineTypes = nullptr, const FShaderCompilerFlags& InExtraCompilerFlags = {});

/** Precreates compute PSOs for global shaders. Separate from Compile/VerifyGlobalShaders to avoid blocking loads. */
extern ENGINE_API void PrecacheComputePipelineStatesForGlobalShaders(ERHIFeatureLevel::Type FeatureLevel, const ITargetPlatform* TargetPlatform);

/**
* Forces a recompile of the global shaders.
*/
extern ENGINE_API void RecompileGlobalShaders();

/**
* Recompiles global shaders and material shaders
* rebuilds global shaders and also
* clears the cooked platform data for all materials if there is a global shader change detected
* can be slow
*/
extern ENGINE_API bool RecompileChangedShadersForPlatform(const FString& PlatformName);

/**
* Begins recompiling the specified global shader types, and flushes their bound shader states.
* FinishRecompileGlobalShaders must be called after this and before using the global shaders for anything.
*/
extern ENGINE_API void BeginRecompileGlobalShaders(const TArray<const FShaderType*>& OutdatedShaderTypes, const TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, EShaderPlatform ShaderPlatform, const ITargetPlatform* TargetPlatform = nullptr, const FShaderCompilerFlags& InExtraCompilerFlags = {});

/** Finishes recompiling global shaders.  Must be called after BeginRecompileGlobalShaders. */
extern ENGINE_API void FinishRecompileGlobalShaders();

#if WITH_EDITOR
/** Called by the shader compiler to process completed global shader jobs. */
extern ENGINE_API void ProcessCompiledGlobalShaders(const TArray<FShaderCommonCompileJobPtr>& CompilationResults);

/** Serializes a global shader map to an archive (used with recompiling shaders for a remote console) */
extern ENGINE_API void SaveGlobalShadersForRemoteRecompile(FArchive& Ar, EShaderPlatform ShaderPlatform);
#endif // WITH_EDITOR

/** Serializes a global shader map to an archive (used with recompiling shaders for a remote console) */
extern ENGINE_API void LoadGlobalShadersForRemoteRecompile(FArchive& Ar, EShaderPlatform ShaderPlatform);

/**
* Saves the global shader map as a file for the target platform.
* @return the name of the file written
*/
extern ENGINE_API FString SaveGlobalShaderFile(EShaderPlatform Platform, FString SavePath, class ITargetPlatform* TargetPlatform = nullptr);

struct FODSCRequestPayload
{
	/** The shader platform to compile for. */
	EShaderPlatform ShaderPlatform;

	/** Which feature level to compile for. */
	ERHIFeatureLevel::Type FeatureLevel;

	/** Which material quality level to compile for. */
	EMaterialQualityLevel::Type QualityLevel;

	/** Which material do we compile for?. */
	FString MaterialName;

	/** The vertex factory type name to compile shaders for. */
	FString VertexFactoryName;

	/** The name of the pipeline to compile shaders for. */
	FString PipelineName;

	/** An array of shader type names for each stage in the Pipeline. */
	TArray<FString> ShaderTypeNames;

	/** The permutation ID to compile. */
	int32 PermutationId;

	/** A hash of the above information to uniquely identify a Request. */
	FString RequestHash;

	FODSCRequestPayload() {};
	ENGINE_API FODSCRequestPayload(
		EShaderPlatform InShaderPlatform,
		ERHIFeatureLevel::Type InFeatureLevel,
		EMaterialQualityLevel::Type InQualityLevel,
		const FString& InMaterialName,
		const FString& InVertexFactoryName,
		const FString& InPipelineName,
		const TArray<FString>& InShaderTypeNames,
		int32 InPermutationId,
		const FString& InRequestHash
	);

	/**
	* Serializes FODSCRequestPayload value from or into this archive.
	*
	* @param Ar The archive to serialize to.
	* @param Value The value to serialize.
	* @return The archive.
	*/
	ENGINE_API friend FArchive& operator<<(FArchive& Ar, FODSCRequestPayload& Elem);
};

enum class ODSCRecompileCommand
{
	None,
	Changed,
	Global,
	Material,
	SingleShader,
	ResetMaterialCache
};

extern ENGINE_API const TCHAR* ODSCCmdEnumToString(ODSCRecompileCommand Cmd);

struct FShaderRecompileData
{
	/** The platform name to compile for. */
	FString PlatformName;

	/** Shader platform */
	EShaderPlatform ShaderPlatform = SP_NumPlatforms;

	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::SM5;

	EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::High;

	/** Additional compiler flags for debugging, e.g. for ODSC shader requests to skip optimizations for individual shaders or enable bindless during development. */
	FShaderCompilerFlags ExtraCompilerFlags = FShaderCompilerFlags();

	/** All filenames that have been changed during the shader compilation. */
	TArray<FString>* ModifiedFiles = nullptr;

	/** Mesh materials, returned to the caller.  */
	TArray<uint8>* MeshMaterialMaps = nullptr;

	/** Materials to load. */
	TArray<FString> MaterialsToLoad;

	/** The names of shader type file names to compile shaders for. */
	FString ShaderTypesToLoad;

	/** What type of shaders to recompile. All, Changed, Global, or Material? */
	ODSCRecompileCommand CommandType = ODSCRecompileCommand::Changed;

	/** Global shader map, returned to the caller.  */
	TArray<uint8>* GlobalShaderMap = nullptr;

	/** On-demand shader compiler payload.  */
	TArray<FODSCRequestPayload> ShadersToRecompile;

	/** Optional Array of the loaded materials  */
	TArray<TStrongObjectPtr<UMaterialInterface>>* LoadedMaterialsToRecompile = nullptr;

#if WITH_EDITOR
	TFunction<UMaterialInterface*(const FString&)> ODSCCustomLoadMaterial;
#endif

	/** Default constructor. */
	FShaderRecompileData() {};

	/** Recompile all the changed shaders for the current platform. */
	ENGINE_API FShaderRecompileData(const FString& InPlatformName, TArray<FString>* OutModifiedFiles, TArray<uint8>* OutMeshMaterialMaps, TArray<uint8>* OutGlobalShaderMap);

	/** For recompiling just global shaders. */
	ENGINE_API FShaderRecompileData(const FString& InPlatformName, EShaderPlatform InShaderPlatform, ODSCRecompileCommand InCommandType, TArray<FString>* OutModifiedFiles, TArray<uint8>* OutMeshMaterialMaps, TArray<uint8>* OutGlobalShaderMap);

	ENGINE_API friend FArchive& operator<<(FArchive& Ar, FShaderRecompileData& Elem);
};

#if WITH_EDITOR

/**
* Recompiles global shaders
*
* @param Args					Arguments and configuration for issuing recompiles.
* @param OutputDirectory		The directory the compiled data will be stored to
**/
extern ENGINE_API void RecompileShadersForRemote(FShaderRecompileData& Args, const FString& OutputDirectory);

/** Frees resources and finalizes artifacts created during the cook of the given platforms. */
extern ENGINE_API void ShutdownShaderCompilers(TConstArrayView<const ITargetPlatform*> TargetPlatforms);

#endif // WITH_EDITOR

extern ENGINE_API void CompileGlobalShaderMap(bool bRefreshShaderMap=false);
extern ENGINE_API void CompileGlobalShaderMap(ERHIFeatureLevel::Type InFeatureLevel, bool bRefreshShaderMap=false);
extern ENGINE_API void CompileGlobalShaderMap(EShaderPlatform Platform, bool bRefreshShaderMap = false);
extern ENGINE_API void CompileGlobalShaderMap(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, bool bRefreshShaderMap);
extern ENGINE_API void ShutdownGlobalShaderMap();

extern ENGINE_API const FGuid& GetGlobalShaderMapDDCGuid();
extern ENGINE_API const FGuid& GetMaterialShaderMapDDCGuid();

extern ENGINE_API bool ShouldDumpShaderDDCKeys();
extern ENGINE_API void DumpShaderDDCKeyToFile(const EShaderPlatform InPlatform, bool bWithEditor, const TCHAR* DebugGroupName, const FString& DDCKey);

/**
* Handles serializing in MeshMaterialMaps or GlobalShaderMap from a CookOnTheFly command and applying them to the in-memory shadermaps.
*
* @param MeshMaterialMaps				Byte array that contains the serialized material shadermap from across the network.
* @param MaterialsToLoad				The materials contained in the MeshMaterialMaps
* @param GlobalShaderMap				Byte array that contains the serialized global shadermap from across the network.
**/
extern ENGINE_API void ProcessCookOnTheFlyShaders(bool bReloadGlobalShaders, const TArray<uint8>& MeshMaterialMaps, const TArray<FString>& MaterialsToLoad, const TArray<uint8>& GlobalShaderMap);
