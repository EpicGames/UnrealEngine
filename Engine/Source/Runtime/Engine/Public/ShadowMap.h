// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "RenderDeferredCleanup.h"
#include "SceneManagement.h"

class FShadowMap2D;
class UInstancedStaticMeshComponent;
class ULightComponent;
class UMapBuildDataRegistry;
class UShadowMapTexture2D;

struct FSignedDistanceFieldShadowSample
{
	/** Normalized and encoded distance to the nearest shadow transition, in the range [0, 1], where .5 is at the transition. */
	float Distance;
	/** Normalized penumbra size, in the range [0,1]. */
	float PenumbraSize;

	/** True if this sample maps to a valid point on a surface. */
	bool IsMapped;
};

struct FQuantizedSignedDistanceFieldShadowSample
{
	enum { NumFilterableComponents = 2 };
	uint8 Distance;
	uint8 PenumbraSize;
	uint8 Coverage;

	float GetFilterableComponent(int32 Index) const
	{
		if (Index == 0)
		{
			return Distance * (1.f / 255.0f);
		}
		else
		{
			checkSlow(Index == 1);
			return PenumbraSize * (1.f / 255.0f);
		}
	}
	
	// this is not a correct U8 quantizer (should round, not trunc); see below for correct version
	void SetFilterableComponent(float InComponent, int32 Index)
	{
		if (Index == 0)
		{
			Distance = (uint8)FMath::Clamp<int32>(FMath::TruncToInt(InComponent * 255.0f), 0, 255);
		}
		else
		{
			checkSlow(Index == 1);
			PenumbraSize = (uint8)FMath::Clamp<int32>(FMath::TruncToInt(InComponent * 255.0f), 0, 255);
		}
	}

	#if 0
	// correct quantizer; needs to be tested to make sure other biases aren't compensating for the error
	void SetFilterableComponent(float InComponent, int32 Index)
	{
		uint8 Quantized = FColor::QuantizeUNormFloatTo8(InComponent);

		if (Index == 0)
		{
			Distance = Quantized;
		}
		else
		{
			checkSlow(Index == 1);
			PenumbraSize = Quantized;
		}
	}
	#endif

	/** Equality operator */
	bool operator==(const FQuantizedSignedDistanceFieldShadowSample& RHS) const
	{
		return Distance == RHS.Distance &&
			PenumbraSize == RHS.PenumbraSize &&
			Coverage == RHS.Coverage;
	}

	FQuantizedSignedDistanceFieldShadowSample() {}
	FQuantizedSignedDistanceFieldShadowSample(const FSignedDistanceFieldShadowSample& InSample)
	{
		Distance = (uint8)FMath::Clamp<int32>(FMath::TruncToInt(InSample.Distance * 255.0f), 0, 255);
		PenumbraSize = (uint8)FMath::Clamp<int32>(FMath::TruncToInt(InSample.PenumbraSize * 255.0f), 0, 255);
		Coverage = InSample.IsMapped ? 255 : 0;
	}
};

/**
 * The raw data which is used to construct a 2D shadowmap.
 */
class FShadowMapData2D
{
public:
	virtual ~FShadowMapData2D() = default;

	// Accessors.
	uint32 GetSizeX() const { return SizeX; }
	uint32 GetSizeY() const { return SizeY; }

	// USurface interface
	virtual float GetSurfaceWidth() const { return (float)SizeX; }
	virtual float GetSurfaceHeight() const { return (float)SizeY; }

	enum ShadowMapDataType {
		UNKNOWN,
		SHADOW_FACTOR_DATA,
		SHADOW_FACTOR_DATA_QUANTIZED,
		SHADOW_SIGNED_DISTANCE_FIELD_DATA,
		SHADOW_SIGNED_DISTANCE_FIELD_DATA_QUANTIZED,
	};
	virtual ShadowMapDataType GetType() const { return UNKNOWN; }

	virtual void Quantize(TArray<FQuantizedSignedDistanceFieldShadowSample>& OutData) const {}

	virtual void Serialize(FArchive* OutShadowMap) {}

protected:

	FShadowMapData2D(uint32 InSizeX, uint32 InSizeY) :
		SizeX(InSizeX),
		SizeY(InSizeY)
	{
	}

	/** The width of the shadow-map. */
	uint32 SizeX;

	/** The height of the shadow-map. */
	uint32 SizeY;
};

/**
 * A 2D signed distance field map, which consists of FSignedDistanceFieldShadowSample's.
 */
class FShadowSignedDistanceFieldData2D : public FShadowMapData2D
{
public:

	FShadowSignedDistanceFieldData2D(uint32 InSizeX, uint32 InSizeY)
		: FShadowMapData2D(InSizeX, InSizeY)
	{
		Data.Empty(SizeX * SizeY);
		Data.AddZeroed(SizeX * SizeY);
	}

	const FSignedDistanceFieldShadowSample& operator()(uint32 X, uint32 Y) const { return Data[SizeX * Y + X]; }
	FSignedDistanceFieldShadowSample& operator()(uint32 X, uint32 Y) { return Data[SizeX * Y + X]; }

	virtual ShadowMapDataType GetType() const override { return SHADOW_SIGNED_DISTANCE_FIELD_DATA; }
	virtual void Quantize(TArray<FQuantizedSignedDistanceFieldShadowSample>& OutData) const override
	{
		OutData.Empty(Data.Num());
		OutData.AddUninitialized(Data.Num());
		for (int32 Index = 0; Index < Data.Num(); Index++)
		{
			OutData[Index] = Data[Index];
		}
	}

	virtual void Serialize(FArchive* OutShadowMap) override
	{
		for (int32 Index = 0; Index < Data.Num(); Index++)
		{
			OutShadowMap->Serialize(&Data[Index], sizeof(FSignedDistanceFieldShadowSample));
		}
	}

private:
	TArray<FSignedDistanceFieldShadowSample> Data;
};

/**
 * A 2D signed distance field map, which consists of FQuantizedSignedDistanceFieldShadowSample's.
 */
class FQuantizedShadowSignedDistanceFieldData2D : public FShadowMapData2D
{
public:

	FQuantizedShadowSignedDistanceFieldData2D(uint32 InSizeX, uint32 InSizeY)
		: FShadowMapData2D(InSizeX, InSizeY)
	{
		Data.Empty(SizeX * SizeY);
		Data.AddZeroed(SizeX * SizeY);
	}

	FQuantizedSignedDistanceFieldShadowSample* GetData() { return Data.GetData(); }

	const FQuantizedSignedDistanceFieldShadowSample& operator()(uint32 X, uint32 Y) const { return Data[SizeX * Y + X]; }
	FQuantizedSignedDistanceFieldShadowSample& operator()(uint32 X, uint32 Y) { return Data[SizeX * Y + X]; }

	virtual ShadowMapDataType GetType() const override { return SHADOW_SIGNED_DISTANCE_FIELD_DATA_QUANTIZED; }
	virtual void Quantize(TArray<FQuantizedSignedDistanceFieldShadowSample>& OutData) const override
	{
		OutData = Data;
	}

	virtual void Serialize(FArchive* OutShadowMap) override
	{
		for (int32 Index = 0; Index < Data.Num(); Index++)
		{
			OutShadowMap->Serialize(&Data[Index], sizeof(FQuantizedSignedDistanceFieldShadowSample));
		}
	}

private:
	TArray<FQuantizedSignedDistanceFieldShadowSample> Data;
};

class FFourDistanceFieldSamples
{
public:
	FQuantizedSignedDistanceFieldShadowSample Samples[4];
};

/**
 * The abstract base class of 1D and 2D shadow-maps.
 */
class FShadowMap : private FDeferredCleanupInterface
{
public:
	enum
	{
		SMT_None = 0,
		SMT_2D = 2,
	};

	/** The GUIDs of lights which this shadow-map stores. */
	TArray<FGuid> LightGuids;

	/** Default constructor. */
	FShadowMap() :
		NumRefs(0)
	{
	}

	FShadowMap(TArray<FGuid> InLightGuids)
		: LightGuids(MoveTemp(InLightGuids))
		, NumRefs(0)
	{
	}

	/** Destructor. */
	virtual ~FShadowMap() { check(NumRefs == 0); }

	/**
	 * Checks if a light is stored in this shadow-map.
	 * @param	LightGuid - The GUID of the light to check for.
	 * @return	True if the light is stored in the light-map.
	 */
	bool ContainsLight(const FGuid& LightGuid) const
	{
		return LightGuids.Find(LightGuid) != INDEX_NONE;
	}

	// FShadowMap interface.
	virtual void AddReferencedObjects(FReferenceCollector& Collector) {}
	ENGINE_API virtual void Serialize(FArchive& Ar);
	virtual FShadowMapInteraction GetInteraction() const = 0;

	// Runtime type casting.
	virtual FShadowMap2D* GetShadowMap2D() { return NULL; }
	virtual const FShadowMap2D* GetShadowMap2D() const { return NULL; }

	// Reference counting.
	void AddRef()
	{
		check(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		NumRefs++;
	}
	void Release()
	{
		check(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		checkSlow(NumRefs > 0);
		if (--NumRefs == 0)
		{
			Cleanup();
		}
	}

protected:

	/**
	 * Called when the light-map is no longer referenced.  Should release the lightmap's resources.
	 */
	ENGINE_API virtual void Cleanup();

private:
	int32 NumRefs;
};

class FShadowMap2D : public FShadowMap
{
public:

	/**
	 * Executes all pending shadow-map encoding requests.
	 * @param	LightingContext				Context in which the textures exist
	 * @param	bLightingSuccessful	Whether the lighting build was successful or not.
	 */
	static ENGINE_API void EncodeTextures(const FStaticLightingBuildContext* LightingContext, bool bLightingSuccessful, bool bMultithreadedEncode =false );

	/**
	 * Constructs mip maps for a single shadowmap texture.
	 */
	static ENGINE_API int32 EncodeSingleTexture(const FStaticLightingBuildContext* LightingContext, struct FShadowMapPendingTexture& PendingTexture, UShadowMapTexture2D* Texture, TArray< TArray<FFourDistanceFieldSamples>>& MipData);

	static ENGINE_API TRefCountPtr<FShadowMap2D> AllocateShadowMap(
		UObject* LightMapOuter,
		const TMap<ULightComponent*, FShadowMapData2D*>& ShadowMapData,
		const FBoxSphereBounds& Bounds,
		ELightMapPaddingType InPaddingType,
		EShadowMapFlags InShadowmapFlags);

	/**
	 * Allocates texture space for the shadow-map and stores the shadow-map's raw data for deferred encoding.
	 * If the shadow-map has no shadows in it, it will return NULL.
	 * @param	Component - The component that owns this shadowmap
	 * @param	InstancedShadowMapData - The raw shadow-map data to fill the texture with.
	 * @param	Bounds - The bounds of the primitive the shadow-map will be rendered on.  Used as a hint to pack shadow-maps on nearby primitives in the same texture.
	 * @param	InPaddingType - the method for padding the shadowmap.
	 * @param	ShadowmapFlags - flags that determine how the shadowmap is stored (e.g. streamed or not)
	 */
	static ENGINE_API TRefCountPtr<FShadowMap2D> AllocateInstancedShadowMap(UObject* LightMapOuter, UInstancedStaticMeshComponent* Component, TArray<TMap<ULightComponent*, TUniquePtr<FShadowMapData2D>>>&& InstancedShadowMapData,
		UMapBuildDataRegistry* Registry, FGuid MapBuildDataId, const FBoxSphereBounds& Bounds, ELightMapPaddingType PaddingType, EShadowMapFlags ShadowmapFlags);

	ENGINE_API FShadowMap2D();

	ENGINE_API FShadowMap2D(const TMap<ULightComponent*, FShadowMapData2D*>& ShadowMapData);
	ENGINE_API FShadowMap2D(TArray<FGuid> LightGuids);

	// Accessors.
	ENGINE_API UTexture2D* GetTexture();
	ENGINE_API const UTexture2D* GetTexture() const;
	const FVector2D& GetCoordinateScale() const { check(IsValid()); return CoordinateScale; }
	const FVector2D& GetCoordinateBias() const { check(IsValid()); return CoordinateBias; }
	bool IsValid() const { return Texture != NULL; }
	bool IsShadowFactorTexture() const { return false; }

	// FShadowMap interface.
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector);
	ENGINE_API virtual void Serialize(FArchive& Ar);
	ENGINE_API virtual FShadowMapInteraction GetInteraction() const;

	// Runtime type casting.
	virtual FShadowMap2D* GetShadowMap2D() { return this; }
	virtual const FShadowMap2D* GetShadowMap2D() const { return this; }

#if WITH_EDITOR
	/** Call to enable/disable status update of LightMap encoding */
	static void SetStatusUpdate(bool bInEnable)
	{
		bUpdateStatus = bInEnable;
	}

	static bool GetStatusUpdate()
	{
		return bUpdateStatus;
	}
#endif

protected:

	/** The texture which contains the shadow-map data. */
	TObjectPtr<UShadowMapTexture2D> Texture;

	/** The scale which is applied to the shadow-map coordinates before sampling the shadow-map textures. */
	FVector2D CoordinateScale;

	/** The bias which is applied to the shadow-map coordinates before sampling the shadow-map textures. */
	FVector2D CoordinateBias;

	/** Tracks which of the 4 channels has valid texture data. */
	bool bChannelValid[4];

	/** Stores the inverse of the penumbra size, normalized.  Stores 1 to interpret the shadowmap as a shadow factor directly, instead of as a distance field. */
	FVector4f InvUniformPenumbraSize;

#if WITH_EDITOR
	/** If true, update the status when encoding light maps */
	static ENGINE_API bool bUpdateStatus;
#endif
};



/**
 * Shadowmap reference serializer
 * Intended to be used by TRefCountPtr's serializer, not called directly
 */
extern ENGINE_API FArchive& operator<<(FArchive& Ar, FShadowMap*& R);
