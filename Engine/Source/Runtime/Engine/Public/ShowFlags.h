// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShowFlags.h: Show Flag Definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "EngineDefines.h"

/** for FShowFlagData, order is not important, adding new elements requires updating some editor code */
enum EShowFlagGroup
{
	SFG_Normal,				// currently one level higher in the hierarchy the the other groups
	SFG_Advanced,
	SFG_PostProcess,
	SFG_CollisionModes,
	SFG_Developer,
	SFG_Visualize,
	SFG_LightTypes,
	SFG_LightingComponents,
	SFG_LightingFeatures,
	SFG_Lumen,
	SFG_MegaLights,
	SFG_Nanite,
	SFG_Hidden,
	SFG_Transient, // Hidden, and don't serialize it
	SFG_Custom,
	SFG_Max
};

/** To customize the EngineShowFlags constructor */
enum EShowFlagInitMode
{
	ESFIM_Game,
	ESFIM_Editor,
	ESFIM_VREditing,
	ESFIM_All0
};

#if (UE_BUILD_SHIPPING && !WITH_EDITOR)
	#define UE_BUILD_OPTIMIZED_SHOWFLAGS 1
#else
	#define UE_BUILD_OPTIMIZED_SHOWFLAGS 0
#endif

/**
 * ShowFlags are a set of bits (some are fixed in SHIPPING) that are stored in the ViewFamily.
 * They can be used by the artist/designer/developer to debug/profile parts of the rendering.
 * Some ShowFlags are used to customize the rendering e.g. by the SceneCaptureActor, those should not be locked in shipping and not used in inner loops (for performance)
 * They should not be used for scalability (as some might be compiled out for SHIPPING), there we use console variables.
 * ViewModes are on a higher level, they can manipulate show flags before they get used.
 */
struct FEngineShowFlags
{
	// ---------------------------------------------------------
	// Define the showflags.
	// A show flag is either an uint32:1 or static const bool (if optimized out according to UE_BUILD_OPTIMIZED_SHOWFLAGS)

#if PLATFORM_USE_SHOWFLAGS_ALWAYS_BITFIELD
	#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,...) uint32 a : 1; void Set##a(bool bVal){ a = bVal ? 1:0; }
#else
	// broken bit field compilers will render the background black
	#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,...) bool a; void Set##a(bool bVal){ a = bVal; }
#endif

	#if UE_BUILD_OPTIMIZED_SHOWFLAGS 
		#if PLATFORM_USE_SHOWFLAGS_ALWAYS_BITFIELD
			#define SHOWFLAG_FIXED_IN_SHIPPING(v,a,...)  uint32 _padding_##a : 1; static const bool a = v; void Set##a(bool bVal){}
		#else
			#define SHOWFLAG_FIXED_IN_SHIPPING(v,a,...)  bool _padding_##a; static const bool a = v; void Set##a(bool bVal){} 
		#endif 
	#else
		#define SHOWFLAG_FIXED_IN_SHIPPING(v,a,b,c) SHOWFLAG_ALWAYS_ACCESSIBLE(a,b,c)
	#endif

	#include "ShowFlagsValues.inl"

	// ---------------------------------------------------------
	// Define an enum for all of the show flags.
	// Useful for matching against indices.
	#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,...) SF_##a,
	enum EShowFlag
	{
		#include "ShowFlagsValues.inl"
		SF_FirstCustom, 
	};
	// ---------------------------------------------------------

	// Strong type for custom show flag indices starting from 0 to make interface clearer
	enum class ECustomShowFlag : uint32
	{
		First = 0, // ECustomShowFlag + SF_FirstCustom -> EShowFlag

		None = 0xFFFFFFFF,
	};
	
	// NOTE: that the offset of this variable is used mark the end of the variables defined by ShowFlagsValues.inl, no other variables should be added before it.
	TBitArray<> CustomShowFlags;

	// Delegate fired when a new custom show flag is registered or re-registered so other systems can iterate all show flags and update any related data
	static ENGINE_API FSimpleMulticastDelegate OnCustomShowFlagRegistered;

	/** Retrieve the localized display name for a named show flag */
	static bool FindShowFlagDisplayName(const FString& InName, FText& OutText)
	{
		static TMap< FString, FText > LocNames;

		if (LocNames.Num() == 0)
		{
			#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,b,c) LocNames.Add( TEXT(UE_STRINGIZE(a)), c);
			#define SHOWFLAG_FIXED_IN_SHIPPING(v,a,b,c) LocNames.Add( TEXT(UE_STRINGIZE(a)), c);
			#include "ShowFlagsValues.inl"

			// Additional strings that don't correspond to a showflag variable:
			LocNames.Add(TEXT("BlockingVolume"), NSLOCTEXT("UnrealEd", "BlockingVolumeSF", "Blocking"));
			LocNames.Add(TEXT("CullDistanceVolume"), NSLOCTEXT("UnrealEd", "CullDistanceVolumeSF", "Cull Distance"));
			LocNames.Add(TEXT("LevelStreamingVolume"), NSLOCTEXT("UnrealEd", "LevelStreamingVolumeSF", "Level Streaming"));
			LocNames.Add(TEXT("LightmassCharacterIndirectDetailVolume"), NSLOCTEXT("UnrealEd", "LightmassCharacterIndirectDetailVolumeSF", "Lightmass Character Indirect Detail"));
			LocNames.Add(TEXT("LightmassImportanceVolume"), NSLOCTEXT("UnrealEd", "LightmassImportanceVolumeSF", "Lightmass Importance"));
			LocNames.Add(TEXT("MassiveLODOverrideVolume"), NSLOCTEXT("UnrealEd", "MassiveLODOverrideVolumeSF", "Massive LOD Override"));
			LocNames.Add(TEXT("NavMeshBoundsVolume"), NSLOCTEXT("UnrealEd", "NavMeshBoundsVolumeSF", "NavMesh Bounds"));
			LocNames.Add(TEXT("NavModifierVolume"), NSLOCTEXT("UnrealEd", "NavModifierVolumeSF", "Nav Modifier"));
			LocNames.Add(TEXT("PainCausingVolume"), NSLOCTEXT("UnrealEd", "PainCausingVolumeSF", "Pain Causing"));
			LocNames.Add(TEXT("PhysicsVolume"), NSLOCTEXT("UnrealEd", "PhysicsVolumeSF", "Physics"));
			LocNames.Add(TEXT("PostProcessVolume"), NSLOCTEXT("UnrealEd", "PostProcessVolumeSF", "Post Process"));
			LocNames.Add(TEXT("PrecomputedVisibilityOverrideVolume"), NSLOCTEXT("UnrealEd", "PrecomputedVisibilityOverrideVolumeSF", "Precomputed Visibility Override"));
			LocNames.Add(TEXT("PrecomputedVisibilityVolume"), NSLOCTEXT("UnrealEd", "PrecomputedVisibilityVolumeSF", "Precomputed Visibility"));
			LocNames.Add(TEXT("AudioVolume"), NSLOCTEXT("UnrealEd", "AudioVolumeSF", "Audio"));
			LocNames.Add(TEXT("TriggerVolume"), NSLOCTEXT("UnrealEd", "TriggerVolumeSF", "Trigger"));
			LocNames.Add(TEXT("CameraBlockingVolume"), NSLOCTEXT("UnrealEd", "CameraBlockingVolumeSF", "Camera Blocking"));
			LocNames.Add(TEXT("HierarchicalLODVolume"), NSLOCTEXT("UnrealEd", "HierarchicalLODVolumeSF", "Hierarchical LOD"));
			LocNames.Add(TEXT("KillZVolume"), NSLOCTEXT("UnrealEd", "KillZVolumeSF", "Kill Z"));
			LocNames.Add(TEXT("MeshMergeCullingVolume"), NSLOCTEXT("UnrealEd", "MeshMergeCullingVolumeSF", "Mesh Merge Culling"));
			LocNames.Add(TEXT("VolumetricLightmapDensityVolume"), NSLOCTEXT("UnrealEd", "VolumetricLightmapDensityVolumeSF", "Volumetric Lightmap Density"));
			LocNames.Add(TEXT("VisualizeAO"), NSLOCTEXT("UnrealEd", "VisualizeAOSF", "Ambient Occlusion"));
		}

		check(InName.Len() > 0);
		const FText* OutName = LocNames.Find(InName);
		if (OutName)
		{
			OutText = *OutName;
			return true;
		}
		if (FindCustomShowFlagDisplayName(InName, OutText))
		{
			return true;
		}

		OutText = FText::FromString(InName);
		return false;
	}

	/** this function defines the grouping in the editor. If a flag is not defined here it's assumed to be a postprocess flag */
	static EShowFlagGroup FindShowFlagGroup(const TCHAR* Name)
	{
		int32 FlagIndex = FindIndexByName(Name);

		if (FlagIndex >= SF_FirstCustom)
		{
			return GetCustomShowFlagGroup(ECustomShowFlag(FlagIndex - SF_FirstCustom));
		}

		switch (FlagIndex)
		{
			#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,b,c) case SF_##a: return b;
			#define SHOWFLAG_FIXED_IN_SHIPPING(v,a,b,c) case SF_##a: return b;
			#include "ShowFlagsValues.inl"

			default:
				// Every flag is defined, if we're here then an invalid index has been returned.
				checkNoEntry();
				return SFG_PostProcess;
		}
	}

	/** this constructor defines the default view settings */
	FEngineShowFlags(EShowFlagInitMode InitMode)
	{
		Init(InitMode);
	}

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	FEngineShowFlags()
	{
		EnsureRetrievingVTablePtrDuringCtor(TEXT("FEngineShowFlags()"));
		Init(ESFIM_Game);
	}

	static bool CanBeToggledInEditor(const TCHAR* Name)
	{
		// Order doesn't matter. All elements need to be comma separated, no spaces, last character should be no comma
		return !IsNameThere(Name,
			TEXT("Editor,LargeVertices,Selection,HitProxies,PropertyColoration,LightInfluences,LightRadius,Game"));
	}

	// for rendering thumbnails, cascade, material editor, static mesh editor
	// keeps bloom as material properties with HDR can be better perceived
	void DisableAdvancedFeatures()
	{
		SetLensFlares(false);
		SetOnScreenDebug(false);
		SetEyeAdaptation(false);
		SetColorGrading(false);
		SetCameraImperfections(false);
		SetDepthOfField(false);
		SetVignette(false);
		SetGrain(false);
		SetSeparateTranslucency(false);
		SetScreenPercentage(false);
		SetScreenSpaceReflections(false);
		SetTemporalAA(false);

		// might cause reallocation if we render rarely to it - for now off
		SetAmbientOcclusion(false);

		// Requires resources in the FScene, which get reallocated for every temporary scene if enabled
		SetIndirectLightingCache(false);

		SetLightShafts(false);
		SetPostProcessMaterial(false);
		SetHighResScreenshotMask(false);
		SetHMDDistortion(false);
		SetStereoRendering(false);
		SetDistanceFieldAO(false);
		SetVolumetricFog(false);
		SetVolumetricLightmap(false);
		SetLumenGlobalIllumination(false);
		SetLumenReflections(false);
		SetMegaLights(false);

		// Have VSM drop all persistent data each frame
		SetVirtualShadowMapPersistentData(false);

		SetShaderPrint(false);
	}

	void EnableAdvancedFeatures()
	{
		SetLensFlares(true);
		SetEyeAdaptation(true);
		SetColorGrading(true);
		SetCameraImperfections(true);
		SetDepthOfField(true);
		SetVignette(true);
		SetGrain(true);
		SetSeparateTranslucency(true);
		SetScreenSpaceReflections(true);
		SetTemporalAA(true);

		// might cause reallocation if we render rarely to it - for now off
		SetAmbientOcclusion(true);

		// Requires resources in the FScene, which get reallocated for every temporary scene if enabled
		SetIndirectLightingCache(true);

		SetLightShafts(true);
		SetPostProcessMaterial(true);
		SetDistanceFieldAO(true);
		SetLumenGlobalIllumination(false);
		SetLumenReflections(false);
		SetMegaLights(false);

		// Have VSM drop all persistent data each frame
		// TODO: Revisit some of the cases that trigger this; if they clean up the scene renderers this is not necessary
		SetVirtualShadowMapPersistentData(false);
	}

	// Features that make sense to disable together for unlit rendering (custom render passes or unlit scene captures)
	void DisableFeaturesForUnlit(bool bAllowAtmosphere = false)
	{
		SetDynamicShadows(false);		// Not supported for CRPs (shadows aren't used if lighting is disabled)
		SetLighting(false);				// Not supported for CRPs
		SetPostProcessing(false);		// Not supported for CRPs (post processing is skipped for unlit scene captures, via bResolveScene being set to false)
		SetRefraction(false);			// Not supported for CRPs (enables separate translucency, which requires post processing to run to composite it back in)

		SetLightFunctions(false);		// Needless performance cost (updates light function atlas, which is unused if lighting is disabled)
		SetUnlitRichView(false);		// Needless performance cost (expensive dynamic rendering code path, related to main view debug modes active when lighting is disabled)
		SetUnlitViewmode(false);		// Development shader feature that writes BaseColor to the emissive channel when lighting is disabled (typically only on non-shipping PC builds)

		if (!bAllowAtmosphere)
		{
			SetAtmosphere(false);		// Technically works with unlit, but costs performance, and is assumed to be undesirable by default
		}
	}

	bool IsVisualizeCalibrationEnabled() const
	{
		return (VisualizeCalibrationColor || VisualizeCalibrationGrayscale || VisualizeCalibrationCustom);
	}

	// ---------------------------------------------------------
	// The following methods are there for serialization, localization and in general to iterate and manipulate flags.
	// ---------------------------------------------------------
	
	/**
	 * O(1)
	 * @param bSet true:set the flag, false:clear the flag
	 */
	ENGINE_API void SetSingleFlag(uint32 Index, bool bSet);

	/**
	 * O(1)
	 * @param Index you can get the index from FindIndexByName() or IterateAllFlags()
	 */
	ENGINE_API bool GetSingleFlag(uint32 Index) const;

	/**
	 * O(1)
	 * Check if a CVar is forcing a particular flag on / off
	 * @param Index you can get the index from FindIndexByName() or IterateAllFlags()
	 * @return true if CVars are forcing flag on / off, false otherwise
	 */
	static ENGINE_API bool IsForceFlagSet(uint32 Index);

	/**
	 * not optimized for speed, good for serialization and debugging
	 * @return e.g. TEXT("PostProcessingTonemapper=0,AmbientCubemap=1")
	 */
	ENGINE_API FString ToString() const;

	/**
	 * More powerful than SetSingleFlag() as it allows to set/clear multiple flags
	 * not optimized but doesn't have to be
	 * Tolerates whitespace and user error, as user might manipulate the data
	 * @param In e.g. TEXT("PostProcessing,PostProcessingTonemapper=0,AmbientCubemap=1")
	 * @return true:success, false:parse error detected
	 */
	ENGINE_API bool SetFromString(const TCHAR* In);
	
	/**
	 * @param Name e.g. TEXT("EyeAdaptation")
	 * @param CommaSeparatedNames leave 0 for normal purpose, is used internally for the grouping feature
	 * @return -1 if not found
	 */
	ENGINE_API static int32 FindIndexByName(const TCHAR* Name, const TCHAR *CommaSeparatedNames = 0);
	
	/**
	 * @param Name e.g. TEXT("EyeAdaptation")
	 * @param CommaSeparatedNames leave 0 for normal purpose, is used internally for the grouping feature
	 * @return success
	 */
	static inline bool IsNameThere(const TCHAR* Name, const TCHAR *CommaSeparatedNames)
	{
		return FindIndexByName(Name, CommaSeparatedNames) != -1;
	}

	/**
	 * @param InIndex can be from FindIndexByName() or IterateAllFlags()
	 * @return empty string if not found, otherwise name of the flag e.g. FString("PostProcessing")
	 */
	ENGINE_API static FString FindNameByIndex(uint32 InIndex);
	
	/**
	 * Allows to iterate through all flags (if you want to change or read the actual flag settings you need to pass in your own instance of FEngineShowFlags).
	 * Use this function instead of writing your own parser.
	 * implement a class with
	 *    bool OnEngineShowFlag(uint32 InIndex, const FString& InName)
	 * which should return true to parse further
	 * 
	 */
	template <class T> static void IterateAllFlags(T& Sink)
	{
		#define SHOWFLAG_ALWAYS_ACCESSIBLE(a,...) if( !Sink.OnEngineShowFlag((uint32)SF_##a, TEXT(UE_STRINGIZE(a))) ) { return; }
		#include "ShowFlagsValues.inl"

		IterateCustomFlags([&](uint32 Index, const FString& Name) { return Sink.OnCustomShowFlag(Index, Name); });
	}

	// Used by TCustomShowFlag to register a show flag from another module 
	static ENGINE_API ECustomShowFlag RegisterCustomShowFlag(const TCHAR* Name, bool DefaultEnabled, EShowFlagGroup Group, FText DisplayName);
	// Iterate all registered custom show flags
	static ENGINE_API void IterateCustomFlags(TFunctionRef<bool(uint32, const FString&)> Functor);
	// Supports ShowFlag.* cvars for custom show flags to force custom show flags on/off
	void EngineOverrideCustomShowFlagsFromCVars();

private:
	/** Initializes the struct. */
	void Init(EShowFlagInitMode InitMode)
	{
		if (InitMode == ESFIM_All0)
		{
			FMemory::Memset(this, 0x00, STRUCT_OFFSET(FEngineShowFlags, CustomShowFlags));
			return;
		}

		// Most flags are on by default. With the following line we only need disable flags.
#if PLATFORM_USE_SHOWFLAGS_ALWAYS_BITFIELD
		FMemory::Memset(this, 0xff, STRUCT_OFFSET(FEngineShowFlags, CustomShowFlags));
#else
		FMemory::Memset(this, uint8(true), STRUCT_OFFSET(FEngineShowFlags, CustomShowFlags));
#endif
		InitCustomShowFlags(InitMode);

		// The following code sets what should be off by default.
		SetVisualizeHDR(false);
		SetVisualizeSkyLightIlluminance(false);
		SetVisualizeLocalFogVolumes(false);
		SetVisualizeLocalExposure(false);
		SetVisualizeShadingModels(false);
		SetOverrideDiffuseAndSpecular(false);
		SetLightingOnlyOverride(false);
		SetReflectionOverride(false);
		SetMaterialNormal(true);
		SetMaterialAmbientOcclusion(true);
		SetVisualizeBuffer(false);
		SetVisualizeNanite(false);
		SetVisualizeLumen(false);
		SetVisualizeSubstrate(false);
		SetVisualizeGroom(false);
		SetVisualizeVirtualShadowMap(false);
		SetVisualizeVirtualTexture(false);
		SetVectorFields(false);
		SetGBufferHints(false);
		SetVisualizeColorGrading(false);
		SetCompositeEditorPrimitives(InitMode == ESFIM_Editor || InitMode == ESFIM_VREditing);
#if UE_ENABLE_DEBUG_DRAWING
		SetCompositeDebugPrimitives(true);
#else
		SetCompositeDebugPrimitives(false);
#endif
		SetOpaqueCompositeEditorPrimitives(false);
		SetTestImage(false);
		SetVisualizeDOF(false);
		SetVertexColors(false);
		SetPhysicalMaterialMasks(false);
		SetVisualizeMotionBlur(false);
		SetVisualizeMotionVectors(false);
		SetVisualizeReprojection(false);
		SetVisualizeTemporalUpscaler(false);
		SetVisualizeTSR(false);
		SetEditingLevelInstance(false);
		SetSelectionOutline(false);
		SetDebugAI(false);
		SetNavigation(false);
		SetLightComplexity(false);
		SetShaderComplexity(false);
		SetQuadOverdraw(false);
		SetShaderComplexityWithQuadOverdraw(false);
		SetStationaryLightOverlap(false);
		SetLightMapDensity(false);
		SetLODColoration(false);
		SetHLODColoration(false);
		SetStreamingTextureDeficit(false);
		SetStreamingTextureResidency(false);
		SetStreamingMeshLODDeficit(false);
		SetStreamingMeshLODResidency(false);
		SetVisualizeGPUSkinCache(false);
		SetVisualizeLWCComplexity(false);
		SetVisualizeShadowCasters(false);
		SetStreamingBounds(false);
		SetHISMCOcclusionBounds(false);
		SetHISMCClusterTree(false);
		SetVisualizeInstanceUpdates(false);
		SetConstraints(false);
		SetMassProperties(false);
		SetCameraFrustums(false);
		SetAudioRadius(InitMode == ESFIM_Editor);
		SetBSPSplit(false);
		SetBrushes(false);
		SetEditor(InitMode == ESFIM_Editor || InitMode == ESFIM_VREditing);
		SetLargeVertices(false);
		SetGrid(InitMode == ESFIM_Editor);
		SetMeshEdges(false);
		SetCover(false);
		SetSplines(InitMode == ESFIM_Editor);
		SetSelection(InitMode == ESFIM_Editor || InitMode == ESFIM_VREditing);
		SetModeWidgets(InitMode == ESFIM_Editor);
		SetBounds(false);
		SetHitProxies(false);
		SetLightInfluences(false);
		SetPivot(InitMode == ESFIM_Editor || InitMode == ESFIM_VREditing);
		SetShadowFrustums(false);
		SetWireframe(false);
		SetLightRadius(InitMode == ESFIM_Editor);
		SetVolumes(InitMode == ESFIM_Editor);
		SetGame(InitMode != ESFIM_Editor && InitMode != ESFIM_VREditing);
		SetActorColoration(false);
		SetCollision(false);
		SetCollisionPawn(false);
		SetCollisionVisibility(false);
		SetCameraAspectRatioBars(false);
		SetCameraSafeFrames(false);
		SetVisualizeOutOfBoundsPixels(false);
		SetHighResScreenshotMask(false);
		SetHMDDistortion(false); // only effective if a HMD is in use
		SetAlphaInvert(false);
		SetStereoMotionVectors(false);
		SetStereoRendering(false);
		SetDistanceCulledPrimitives(InitMode == ESFIM_Editor);
		SetVisualizeLightCulling(false);
		SetPrecomputedVisibilityCells(false);
		SetVisualizeVolumetricLightmap(false);
		SetVolumeLightingSamples(false);
		// we enable it manually on the editor view ports
		SetSnap(false);
		SetVisualizeMeshDistanceFields(false);
		SetVisualizeGlobalDistanceField(false);
		SetVisualizeLightingOnProbes(false);
		SetVisualizeDistanceFieldAO(false);
		SetPhysicsField(false);
		SetVisualizeSSR(false);
		SetVisualizeSSS(false);
		SetPrimitiveDistanceAccuracy(false);
		SetMeshUVDensityAccuracy(false);
		SetMaterialTextureScaleAccuracy(false);
		SetOutputMaterialTextureScales(false);
		SetRequiredTextureResolution(false);
		SetMotionBlur(InitMode != ESFIM_Editor && InitMode != ESFIM_VREditing);
		SetBones(false);
		SetServerDrawDebug(false);
		SetScreenPercentage(InitMode != ESFIM_Editor && InitMode != ESFIM_VREditing);
		SetVREditing(InitMode == ESFIM_VREditing);
		SetOcclusionMeshes(false);
		SetDisableOcclusionQueries(false);
		SetPathTracing(false);
		SetRayTracingDebug(false);
		SetVisualizeSkyAtmosphere(false);
		SetVisualizeLightFunctionAtlas(false);
		SetVisualizeCalibrationColor(false);
		SetVisualizeCalibrationGrayscale(false);
		SetVisualizeCalibrationCustom(false);
		SetVisualizePostProcessStack(false);
		SetVirtualTexturePrimitives(false);
		SetVisualizeInstanceOcclusionQueries(false);
		SetVisualizeVolumetricCloudConservativeDensity(false);
		SetVisualizeVolumetricCloudEmptySpaceSkipping(false);
		SetDebugDrawDistantVirtualSMLights(false);
		SetSceneCaptureCopySceneDepth(true);

		SetLumenScreenTraces(true);
		SetLumenDetailTraces(true);
		SetLumenGlobalTraces(true);
		SetLumenFarFieldTraces(true);
		SetLumenSecondaryBounces(true);
		SetLumenShortRangeAmbientOcclusion(true);

		SetClay(false);
		SetZebra(false);
		SetFrontBackFace(false);
		SetRandomColor(false);
	}


	/**
	 * Allows to iterate through all flags (if you want to change or read the actual flag settings you need to pass in your own instance of FEngineShowFlags).
	 * Use this function instead of writing your own parser.
	 * not optimized for performance
	 * implement a class with
	 *    bool OnEngineShowFlag(uint32 InIndex, const FString& InName)
	 * which should return true to parse further
	 * @param CommaSeparatedNames leave 0 for normal purpose, is used internally for the grouping feature
	 */
	template <class T> static void IterateAllFlags(T& Sink, const TCHAR *CommaSeparatedNames )
	{
		const TCHAR* p = CommaSeparatedNames;
		
		check( p != nullptr );

		for(uint32 Index = 0; ; ++Index)
		{
			FString Name;

			// jump over ,
			while(*p && *p != (TCHAR)',')
			{
				Name += *p++;
			}

			// , in the end of the string? or ; between multiple lines of TEXT("")
			check(!Name.IsEmpty());

			if(!Sink.OnEngineShowFlag(Index, Name))
			{
				// stop iteration
				return;
			}

			if(*p == 0)
			{
				// end
				return;
			}

			// jump over ,
			++p;
		}
	}

	/** @return -1 if not found */
	static void AddNameByIndex(uint32 InIndex, FString& Out);

	// Set default values for custom flags on this FEngineShowFlags instance
	ENGINE_API void InitCustomShowFlags(EShowFlagInitMode InitMode);
	// If new custom show flags have been registered, add new default entries to CustomShowFlags
	void UpdateNewCustomShowFlags();
	
	// Lookup functions for custom show flags matching engine ones
	static ENGINE_API bool FindCustomShowFlagDisplayName(const FString& InName, FText& OutText);
	static bool IsRegisteredCustomShowFlag(ECustomShowFlag Index);
	static FString GetCustomShowFlagName(ECustomShowFlag Index);
	static FText GetCustomShowFlagDisplayName(ECustomShowFlag Index);
	static ENGINE_API EShowFlagGroup GetCustomShowFlagGroup(ECustomShowFlag Index);
	static ECustomShowFlag FindCustomShowFlagByName(const FString& Name);
};

/** Set ShowFlags and EngineShowFlags depending on view mode. */
ENGINE_API void ApplyViewMode(EViewModeIndex ViewModeIndex, bool bPerspective, FEngineShowFlags& EngineShowFlags);

/** Call each view rendering after game [engine] show flags for rendering a view have been set. */
ENGINE_API void EngineShowFlagOverride(EShowFlagInitMode ShowFlagInitMode, EViewModeIndex ViewModeIndex, FEngineShowFlags& EngineShowFlags, bool bCanDisableTonemapper);

/** Call each view rendering after game [engine] show flags for rendering a view have been set; disables effects that will not work in an orthographic projection due to shader limitations. */
ENGINE_API void EngineShowFlagOrthographicOverride(bool bIsPerspective, FEngineShowFlags& EngineShowFlags);

/** Find the view mode that first to the set show flags (to be backward compatible, can be removed on day). */
ENGINE_API EViewModeIndex FindViewMode(const FEngineShowFlags& EngineShowFlags);

/** @return 0 if outside of the range, e.g. TEXT("LightingOnly") for EVM_LightingOnly */
const TCHAR* GetViewModeName(EViewModeIndex ViewModeIndex);

// Enum to control behavior of custom show flags in shipping builds
enum class EShowFlagShippingValue
{
	Dynamic,		// Allow flag to be changed dynamically
	ForceEnabled,	// Always enabled in shipping builds
	ForceDisabled,	// Always disabled in shipping builds
};

// Statically register a custom show flag to control rendering - use TCustomShowFlag::IsEnabled to get current state for a given view.
template<EShowFlagShippingValue ShippingValue = EShowFlagShippingValue::Dynamic>
struct TCustomShowFlag
{
	TCustomShowFlag(const TCHAR* Name, bool DefaultEnabled, EShowFlagGroup Group = SFG_Custom, FText DisplayName = {})
	{
#if UE_BUILD_SHIPPING
		if constexpr (ShippingValue == EShowFlagShippingValue::Dynamic)
#endif
		{
			FlagIndex = FEngineShowFlags::RegisterCustomShowFlag(Name, DefaultEnabled, Group, DisplayName);
		}
	}

	bool IsEnabled(const FEngineShowFlags& ShowFlags)
	{
#if UE_BUILD_SHIPPING
		if constexpr (ShippingValue == EShowFlagShippingValue::ForceDisabled)
		{
			return false;
		}
		else if constexpr (ShippingValue == EShowFlagShippingValue::ForceEnabled)
		{
			return true;
		}
		else
#endif
		{
			if (FlagIndex == FEngineShowFlags::ECustomShowFlag::None)
			{
				return false;
			}
			else
			{
				return ShowFlags.GetSingleFlag((uint32)FlagIndex + FEngineShowFlags::SF_FirstCustom);
			}
		}
	}

	bool SetEnabled(FEngineShowFlags& ShowFlags, bool bSet)
	{
#if UE_BUILD_SHIPPING
		if constexpr (ShippingValue == EShowFlagShippingValue::ForceDisabled ||
			ShippingValue == EShowFlagShippingValue::ForceEnabled)
		{
			return false;
		}
		else
#endif
		{
			if (FlagIndex == FEngineShowFlags::ECustomShowFlag::None)
			{
				return false;
			}
			else
			{
				ShowFlags.SetSingleFlag((uint32)FlagIndex + FEngineShowFlags::SF_FirstCustom, bSet);
				return true;
			}
		}
	}

private:
	FEngineShowFlags::ECustomShowFlag FlagIndex = FEngineShowFlags::ECustomShowFlag::None;
};
