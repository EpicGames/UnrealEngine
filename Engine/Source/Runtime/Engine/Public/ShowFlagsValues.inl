// Copyright Epic Games, Inc. All Rights Reserved.

// usage
//
// General purpose showflag (always variable):
// SHOWFLAG_ALWAYS_ACCESSIBLE( <showflag name>, <showflag group>, <Localized TEXT stuff>)
// Fixed in shipping builds:
// SHOWFLAG_FIXED_IN_SHIPPING( <showflag name>, <fixed bool>, <showflag group>, <Localized TEXT stuff>)

// [[ IncludeTool: Inline ]] // Markup to tell IncludeTool that this file is state changing and cannot be optimized out.

#ifndef SHOWFLAG_ALWAYS_ACCESSIBLE
#error SHOWFLAG_ALWAYS_ACCESSIBLE macro is undefined.
#endif

// the default case for SHOWFLAG_FIXED_IN_SHIPPING is to give flag name.
#ifndef SHOWFLAG_FIXED_IN_SHIPPING
#define SHOWFLAG_FIXED_IN_SHIPPING(v,a,...) SHOWFLAG_ALWAYS_ACCESSIBLE(a,__VA_ARGS__)
#endif

/** Affects all postprocessing features, depending on viewmode this is on or off, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's used by ReflectionEnvironmentCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(PostProcessing, SFG_Hidden, NSLOCTEXT("UnrealEd", "PostProcessingSF", "Post-processing"))
/** Bloom, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(Bloom, SFG_PostProcess, NSLOCTEXT("UnrealEd", "BloomSF", "Bloom"))
/** Local Exposure, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(LocalExposure, SFG_PostProcess, NSLOCTEXT("UnrealEd", "LocalExposureSF", "Local Exposure"))
/** HDR->LDR conversion is done through a tone mapper (otherwise linear mapping is used) */
SHOWFLAG_FIXED_IN_SHIPPING(1, Tonemapper, SFG_PostProcess, NSLOCTEXT("UnrealEd", "TonemapperSF", "Tonemapper"))
/** Any Anti-aliasing e.g. FXAA, Temporal AA, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(AntiAliasing, SFG_Normal, NSLOCTEXT("UnrealEd", "AntiAliasingSF", "Anti-aliasing"))
/** Only used in AntiAliasing is on, true:uses Temporal AA, otherwise FXAA, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture  */
SHOWFLAG_ALWAYS_ACCESSIBLE(TemporalAA, SFG_Advanced, NSLOCTEXT("UnrealEd", "TemporalAASF", "Temporal AA (instead FXAA)"))
/** e.g. Ambient cube map, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(AmbientCubemap, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "AmbientCubemapSF", "Ambient Cubemap"))
/** Human like eye simulation to adapt to the brightness of the view, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(EyeAdaptation, SFG_PostProcess, NSLOCTEXT("UnrealEd", "EyeAdaptationSF", "Eye Adaptation"))
/** Display a histogram of the scene HDR color */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeHDR, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeHDRSF", "HDR (Eye Adaptation)"))
/** Display the illuminance debug view for the skylight */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeSkyLightIlluminance, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeSkyLightIlluminance", "Sky Light Illuminance"))
/** Display the illuminance debug view for the skylight */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeLocalFogVolumes, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeLocalFogVolumes", "Local Fog Volumes"))
/** Helper to tweak local expsoure settings */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeLocalExposure, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeLocalExposureSF", "Local Exposure"))
/** Image based lens flares (Simulate artifact of reflections within a camera system) */
SHOWFLAG_FIXED_IN_SHIPPING(1, LensFlares, SFG_PostProcess, NSLOCTEXT("UnrealEd", "LensFlaresSF", "Lens Flares"))
/** Lens distortion (Simulate the distortion of light through the lens onto the filmback) */
SHOWFLAG_FIXED_IN_SHIPPING(1, LensDistortion, SFG_PostProcess, NSLOCTEXT("UnrealEd", "LensDistortionSF", "Lens Distortion"))
/** show indirect lighting component, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's needed by r.GBuffer */
SHOWFLAG_ALWAYS_ACCESSIBLE(GlobalIllumination, SFG_LightingComponents, NSLOCTEXT("UnrealEd", "GlobalIlluminationSF", "Global Illumination"))
/** Darkens the screen borders (Camera artifact and artistic effect) */
SHOWFLAG_ALWAYS_ACCESSIBLE(Vignette, SFG_PostProcess, NSLOCTEXT("UnrealEd", "VignetteSF", "Vignette"))
/** Fine film grain */
SHOWFLAG_FIXED_IN_SHIPPING(1, Grain, SFG_PostProcess, NSLOCTEXT("UnrealEd", "GrainSF", "Grain"))
/** Screen Space Ambient Occlusion, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(AmbientOcclusion, SFG_LightingComponents, NSLOCTEXT("UnrealEd", "AmbientOcclusionSF", "Ambient Occlusion"))
/** Decal rendering, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(Decals, SFG_Normal, NSLOCTEXT("UnrealEd", "DecalsSF", "Decals"))
/** like bloom dirt mask */
SHOWFLAG_FIXED_IN_SHIPPING(1, CameraImperfections, SFG_PostProcess, NSLOCTEXT("UnrealEd", "CameraImperfectionsSF", "Camera Imperfections"))
/** to allow to disable visualizetexture for some editor rendering (e.g. thumbnail rendering) */
SHOWFLAG_ALWAYS_ACCESSIBLE(OnScreenDebug, SFG_Developer, NSLOCTEXT("UnrealEd", "OnScreenDebugSF", "On Screen Debug"))
/** needed for VMI_Lit_DetailLighting, Whether to override material diffuse and specular with constants, used by the Detail Lighting viewmode. */
SHOWFLAG_FIXED_IN_SHIPPING(0, OverrideDiffuseAndSpecular, SFG_Hidden, NSLOCTEXT("UnrealEd", "OverrideDiffuseAndSpecularSF", "Override Diffuse And Specular"))
/** needed for VMI_LightingOnly, Whether to override material diffuse with constants, used by the Lighting Only viewmode. */
SHOWFLAG_FIXED_IN_SHIPPING(0, LightingOnlyOverride, SFG_Hidden, NSLOCTEXT("UnrealEd", "LightingOnlyOverrideSF", "Lighting Only"))
/** needed for VMI_ReflectionOverride, Whether to override all materials to be smooth, mirror reflections. */
SHOWFLAG_FIXED_IN_SHIPPING(0, ReflectionOverride, SFG_Hidden, NSLOCTEXT("UnrealEd", "ReflectionOverrideSF", "Reflections"))
/** Whether to use material normal. */
SHOWFLAG_FIXED_IN_SHIPPING(1, MaterialNormal, SFG_Hidden, NSLOCTEXT("UnrealEd", "MaterialNormalSF", "Material Normal"))
/** Whether to use material ambient occlusion. */
SHOWFLAG_FIXED_IN_SHIPPING(1, MaterialAmbientOcclusion, SFG_Hidden, NSLOCTEXT("UnrealEd", "MaterialAmbientOcclusionSF", "Material Ambient Occlusion"))
/** needed for VMI_VisualizeBuffer, Whether to enable the buffer visualization mode. */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeBuffer, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeBufferSF", "Buffer Visualization"))
/** Needed for VMI_VisualizeNanite, Whether to enable the Nanite visualization mode. */
SHOWFLAG_ALWAYS_ACCESSIBLE(VisualizeNanite, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeNaniteSF", "Nanite Visualization"))
/** Needed for VMI_VisualizeLumen, Whether to enable the Lumen visualization mode. */
SHOWFLAG_ALWAYS_ACCESSIBLE(VisualizeLumen, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeLumenSF", "Lumen Visualization"))
/** Needed for VMI_VisualizeMegaLights, Whether to enable the MegaLights visualization mode. */
SHOWFLAG_ALWAYS_ACCESSIBLE(VisualizeMegaLights, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeMegaLightsSF", "MegaLights Visualization"))
/** Needed for VMI_VisualizeSubstrate, Whether to enable the Substrate visualization mode. */
SHOWFLAG_ALWAYS_ACCESSIBLE(VisualizeSubstrate, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeSubstrateSF", "Substrate Visualization"))
/** Needed for VMI_VisualizeGroom, Whether to enable the Groom visualization mode. */
SHOWFLAG_ALWAYS_ACCESSIBLE(VisualizeGroom, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeGroomSF", "Groom Visualization"))
/** Needed for VMI_VisualizeVirtualShadowMap, Whether to enable the virtual shadow map visualization mode. */
SHOWFLAG_ALWAYS_ACCESSIBLE(VisualizeVirtualShadowMap, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeVirtualShadowMapSF", "Virtual Shadow Map Visualization"))
/** Allows to disable all direct lighting (does not affect indirect light) */
SHOWFLAG_FIXED_IN_SHIPPING(1, DirectLighting, SFG_LightingComponents, NSLOCTEXT("UnrealEd", "DirectLightingSF", "Direct Lighting"))
/** Allows to disable lighting from Directional Lights */
SHOWFLAG_FIXED_IN_SHIPPING(1, DirectionalLights, SFG_LightTypes, NSLOCTEXT("UnrealEd", "DirectionalLightsSF", "Directional Lights"))
/** Allows to disable lighting from Point Lights, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(PointLights, SFG_LightTypes, NSLOCTEXT("UnrealEd", "PointLightsSF", "Point Lights"))
/** Allows to disable lighting from Spot Lights, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(SpotLights, SFG_LightTypes, NSLOCTEXT("UnrealEd", "SpotLightsSF", "Spot Lights"))
/** Allows to disable lighting from Rect Lights, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(RectLights, SFG_LightTypes, NSLOCTEXT("UnrealEd", "RectLightsSF", "Rect Lights"))
/** Color correction after tone mapping */
SHOWFLAG_FIXED_IN_SHIPPING(1, ColorGrading, SFG_PostProcess, NSLOCTEXT("UnrealEd", "ColorGradingSF", "Color Grading"))
/** Whether to show the color grading visualization displays. */
SHOWFLAG_ALWAYS_ACCESSIBLE(VisualizeColorGrading, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeColorGrading", "Visualize Color Grading"))
/** Visualize vector fields. */
SHOWFLAG_FIXED_IN_SHIPPING(0, VectorFields, SFG_Developer, NSLOCTEXT("UnrealEd", "VectorFieldsSF", "Vector Fields"))
/** Depth of Field */
SHOWFLAG_ALWAYS_ACCESSIBLE(DepthOfField, SFG_PostProcess, NSLOCTEXT("UnrealEd", "DepthOfFieldSF", "Depth Of Field"))
/** Highlight materials that indicate performance issues or show unrealistic materials */
SHOWFLAG_FIXED_IN_SHIPPING(0, GBufferHints, SFG_Developer, NSLOCTEXT("UnrealEd", "GBufferHintsSF", "GBuffer Hints (material attributes)"))
/** MotionBlur, for now only camera motion blur, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(MotionBlur, SFG_PostProcess, NSLOCTEXT("UnrealEd", "MotionBlurSF", "Motion Blur"))
/** Whether to render elements drawn through the debug PDI interface. */
SHOWFLAG_ALWAYS_ACCESSIBLE(CompositeDebugPrimitives, SFG_Developer, NSLOCTEXT("UnrealEd", "CompositeDebugPrimitivesSF", "Composite Debug Primitives"))
/** Whether to render the editor gizmos and other foreground editor widgets off screen and apply them after post process, only needed for the editor */
SHOWFLAG_FIXED_IN_SHIPPING(0, CompositeEditorPrimitives, SFG_Developer, NSLOCTEXT("UnrealEd", "CompositeEditorPrimitivesSF", "Composite Editor Primitives"))
/** When CompositeEditorPrimitives is set, determines whether occluded portions are drawn opaquely on top, or dithered when they are occluded */
SHOWFLAG_FIXED_IN_SHIPPING(0, OpaqueCompositeEditorPrimitives, SFG_Developer, NSLOCTEXT("UnrealEd", "OpaqueCompositeEditorPrimitives", "Make Composite Editor Primitives Opaque"))
/** Shows a test image that allows to tweak the monitor colors, borders and allows to judge image and temporal aliasing  */
SHOWFLAG_FIXED_IN_SHIPPING(0, TestImage, SFG_Developer, NSLOCTEXT("UnrealEd", "TestImageSF", "Test Image"))
/** Helper to tweak depth of field */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeDOF, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeDOFSF", "Depth of Field Layers"))
/** Show Vertex Colors */
SHOWFLAG_FIXED_IN_SHIPPING(0, VertexColors, SFG_Advanced, NSLOCTEXT("UnrealEd", "VertexColorsSF", "Vertex Colors"))
/** Show Physical Material Masks */
SHOWFLAG_FIXED_IN_SHIPPING(0, PhysicalMaterialMasks, SFG_Advanced, NSLOCTEXT("UnrealEd", "PhysicalMaterialMasksSF", "Physical Material Masks"))
/** Render Post process (screen space) distortion/refraction */
SHOWFLAG_FIXED_IN_SHIPPING(1, Refraction, SFG_Developer, NSLOCTEXT("UnrealEd", "RefractionSF", "Refraction"))
/** Usually set in game or when previewing cinematics but not in editor, used for motion blur or any kind of rendering features that rely on the former frame */
SHOWFLAG_ALWAYS_ACCESSIBLE(CameraInterpolation, SFG_Hidden, NSLOCTEXT("UnrealEd", "CameraInterpolationSF", "Camera Interpolation"))
/** Post processing color fringe (chromatic aberration) */
SHOWFLAG_FIXED_IN_SHIPPING(1, SceneColorFringe, SFG_PostProcess, NSLOCTEXT("UnrealEd", "SceneColorFringeSF", "Scene Color Fringe"))
/** Post processing filmic tone curve and expand gamut */
SHOWFLAG_ALWAYS_ACCESSIBLE(ToneCurve, SFG_PostProcess, NSLOCTEXT("UnrealEd", "ToneCurveSF", "Tone Curve"))
/** If Translucency should be rendered into a separate RT and composited without DepthOfField, can be disabled in the materials (affects sorting), SHOWFLAG_ALWAYS_ACCESSIBLE for now because USceneCaptureComponent needs that */
SHOWFLAG_ALWAYS_ACCESSIBLE(SeparateTranslucency, SFG_Advanced, NSLOCTEXT("UnrealEd", "SeparateTranslucencySF", "Separate Translucency"))
/** If Screen Percentage should be applied.  */
SHOWFLAG_ALWAYS_ACCESSIBLE(ScreenPercentage, SFG_Hidden, NSLOCTEXT("UnrealEd", "ScreenPercentageSF", "Screen Percentage"))
/** Helper to tweak motion blur settings */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeMotionBlur, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeMotionBlurSF", "Motion Blur"))
/** DEPRECATED in 5.2: Use VisualizeReprojection instead. */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeMotionVectors, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeMotionVectorsSF", "Motion Vectors"))
/** Helper to ensure previous frame's temporal history reprojections using motion vectors are working correctly. */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeReprojection, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeReprojectionSF", "Previous frame's reprojection"))
/** Helper to diagnose temporal upscaler's inputs and outputs. */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeTemporalUpscaler, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeTemporalUpscalerSF", "Temporal Upscaler I/O (TSR, TAAU or third party plugins)"))
/** Helper to diagnose the internals of TSR. */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeTSR, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeTSRSF", "Temporal Super Resolution"))
/** Whether to display the MegaLights feature, which replaces deferred direct lighting and Ray Traced Shadows with a stochastic sampling and denoising algorithm */
SHOWFLAG_ALWAYS_ACCESSIBLE(MegaLights, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "MegaLightsSF", "MegaLights"))
/** Whether to display the Reflection Environment feature, which has local reflections from Reflection Capture actors, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(ReflectionEnvironment, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "ReflectionEnvironmentSF", "Reflection Environment"))
/** Visualize pixels that are outside of their object's bounding box (content error). */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeOutOfBoundsPixels, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeOutOfBoundsPixelsSF", "Out of Bounds Pixels"))
/** Whether to display the scene's diffuse. */
SHOWFLAG_FIXED_IN_SHIPPING(1, Diffuse, SFG_LightingComponents, NSLOCTEXT("UnrealEd", "DiffuseSF", "Diffuse"))
/** Whether to display the scene's specular, including reflections. */
SHOWFLAG_ALWAYS_ACCESSIBLE(Specular, SFG_LightingComponents, NSLOCTEXT("UnrealEd", "SpecularSF", "Specular"))
/** Outline around selected objects in the editor */
SHOWFLAG_FIXED_IN_SHIPPING(0, SelectionOutline, SFG_Hidden, NSLOCTEXT("UnrealEd", "SelectionOutlineSF", "Selection Outline"))
/** If screen space reflections are enabled, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(ScreenSpaceReflections, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "ScreenSpaceReflectionsSF", "Screen Space Reflections"))
SHOWFLAG_ALWAYS_ACCESSIBLE(LumenReflections, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "LumenReflectionsSF", "Lumen Reflections"))
/** If Screen space contact shadows are enabled. */
SHOWFLAG_ALWAYS_ACCESSIBLE(ContactShadows, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "ContactShadows", "Screen Space Contact Shadows"))
/** If RTDF shadows are enabled. */
SHOWFLAG_ALWAYS_ACCESSIBLE(RayTracedDistanceFieldShadows, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "RayTracedDistanceFieldShadows", "Distance Field Shadows"))
/** If Capsule shadows are enabled. */
SHOWFLAG_ALWAYS_ACCESSIBLE(CapsuleShadows, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "CapsuleShadows", "Capsule Shadows"))
/** If Screen Space Subsurface Scattering enabled */
SHOWFLAG_FIXED_IN_SHIPPING(1, SubsurfaceScattering, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "SubsurfaceScatteringSF", "Subsurface Scattering (Screen Space)"))
/** If Screen Space Subsurface Scattering visualization is enabled */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeSSS, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeSSSSF", "Subsurface Scattering (Screen Space)"))
/** Whether to apply volumetric lightmap lighting, when present. */
SHOWFLAG_ALWAYS_ACCESSIBLE(VolumetricLightmap, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "VolumetricLightmapSF", "Volumetric Lightmap"))
/** If the indirect lighting cache is enabled, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(IndirectLightingCache, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "IndirectLightingCacheSF", "Indirect Lighting Cache"))
/** calls debug drawing for AIs */
SHOWFLAG_FIXED_IN_SHIPPING(0, DebugAI, SFG_Developer, NSLOCTEXT("UnrealEd", "DebugAISF", "AI Debug"))
/** calls debug drawing for whatever LogVisualizer wants to draw */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisLog, SFG_Developer, NSLOCTEXT("UnrealEd", "VisLogSF", "Log Visualizer"))
/** whether to draw navigation data */
SHOWFLAG_FIXED_IN_SHIPPING(0, Navigation, SFG_Normal, NSLOCTEXT("UnrealEd", "NavigationSF", "Navigation"))
/** used by gameplay debugging components to debug-draw on screen */
SHOWFLAG_FIXED_IN_SHIPPING(0, GameplayDebug, SFG_Developer, NSLOCTEXT("UnrealEd", "GameplayDebugSF", "Gameplay Debug"))
/** LightProfiles, usually 1d textures to have a light (IES), for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(TexturedLightProfiles,  SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "TexturedLightProfilesSF", "Textured Light Profiles (IES Texture)"))
/** LightFunctions (masking light sources with a material), for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(LightFunctions, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "LightFunctionsSF", "Light Functions"))
/** Draws instanced static meshes that are not foliage or grass, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(InstancedStaticMeshes, SFG_Advanced, NSLOCTEXT("UnrealEd", "InstancedStaticMeshesSF", "Instanced Static Meshes"))
/** Draws instanced foliage, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(InstancedFoliage, SFG_Advanced, NSLOCTEXT("UnrealEd", "InstancedFoliageSF", "Foliage"))
/** Allow to see the foliage bounds used in the occlusion test */
SHOWFLAG_FIXED_IN_SHIPPING(0, HISMCOcclusionBounds, SFG_Advanced, NSLOCTEXT("UnrealEd", "HISMOcclusionBoundsSF", "HISM/Foliage Occlusion Bounds"))
/** Allow to see the cluster tree bounds used used to generate the occlusion bounds and in the culling */
SHOWFLAG_FIXED_IN_SHIPPING(0, HISMCClusterTree, SFG_Advanced, NSLOCTEXT("UnrealEd", "HISMClusterTreeSF", "HISM/Foliage Cluster Tree"))
/** Allow to see where each instance is in each ISM Component */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeInstanceUpdates, SFG_Advanced, NSLOCTEXT("UnrealEd", "VisualizeInstanceUpdatesSF", "Visualize Instance Updates"))
/** Draws instanced grass, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(InstancedGrass, SFG_Advanced, NSLOCTEXT("UnrealEd", "InstancedGrassSF", "Grass"))
/** non baked shadows, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(DynamicShadows, SFG_LightingComponents, NSLOCTEXT("UnrealEd", "DynamicShadowsSF", "Dynamic Shadows"))
/** Particles, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(Particles, SFG_Normal, NSLOCTEXT("UnrealEd", "ParticlesSF", "Particle Sprites"))
/** Niagara, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_FIXED_IN_SHIPPING(1, Niagara, SFG_Advanced, NSLOCTEXT("UnrealEd", "NiagaraSF", "Niagara Renderers"))
/** HeterogeneousVolumes, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_FIXED_IN_SHIPPING(1, HeterogeneousVolumes, SFG_Advanced, NSLOCTEXT("UnrealEd", "HeterogeneousVolumesSF", "Heterogeneous Volumes"))
/** if SkeletalMeshes are getting rendered, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(SkeletalMeshes, SFG_Normal, NSLOCTEXT("UnrealEd", "SkeletalMeshesSF", "Skeletal Meshes"))
/** if the builder brush (editor) is getting rendered */
SHOWFLAG_FIXED_IN_SHIPPING(0, BuilderBrush, SFG_Hidden, NSLOCTEXT("UnrealEd", "BuilderBrushSF", "Builder Brush"))
/** Render translucency, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(Translucency, SFG_Normal, NSLOCTEXT("UnrealEd", "TranslucencySF", "Translucency"))
/** Draw billboard components */
SHOWFLAG_FIXED_IN_SHIPPING(1, BillboardSprites, SFG_Advanced, NSLOCTEXT("UnrealEd", "BillboardSpritesSF", "Billboard Sprites"))
/** Use LOD parenting, MinDrawDistance, etc. If disabled, will show LOD parenting lines */
SHOWFLAG_ALWAYS_ACCESSIBLE(LOD, SFG_Advanced, NSLOCTEXT("UnrealEd", "LODSF", "LOD Parenting"))
/** needed for VMI_LightComplexity */
SHOWFLAG_FIXED_IN_SHIPPING(0, LightComplexity, SFG_Hidden, NSLOCTEXT("UnrealEd", "LightComplexitySF", "Light Complexity"))
/** needed for VMI_ShaderComplexity, render world colored by shader complexity */
SHOWFLAG_FIXED_IN_SHIPPING(0, ShaderComplexity, SFG_Hidden, NSLOCTEXT("UnrealEd", "ShaderComplexitySF", "Shader Complexity"))
/** needed for VMI_StationaryLightOverlap, render world colored by stationary light overlap */
SHOWFLAG_FIXED_IN_SHIPPING(0, StationaryLightOverlap,  SFG_Hidden, NSLOCTEXT("UnrealEd", "StationaryLightOverlapSF", "Stationary Light Overlap"))
/** needed for VMI_LightmapDensity and VMI_LitLightmapDensity, render checkerboard material with UVs scaled by lightmap resolution w. color tint for world-space lightmap density */
SHOWFLAG_FIXED_IN_SHIPPING(0, LightMapDensity, SFG_Hidden, NSLOCTEXT("UnrealEd", "LightMapDensitySF", "Light Map Density"))
/** Render streaming bounding volumes for the currently selected texture */
SHOWFLAG_FIXED_IN_SHIPPING(0, StreamingBounds, SFG_Advanced, NSLOCTEXT("UnrealEd", "StreamingBoundsSF", "Streaming Bounds"))
/** Render joint limits */
SHOWFLAG_FIXED_IN_SHIPPING(0, Constraints, SFG_Advanced, NSLOCTEXT("UnrealEd", "ConstraintsSF", "Constraints"))
/** Render mass debug data */
SHOWFLAG_FIXED_IN_SHIPPING(0, MassProperties, SFG_Advanced, NSLOCTEXT("UnrealEd", "MassPropertiesSF", "Mass Properties"))
/** Draws camera frustums */
SHOWFLAG_FIXED_IN_SHIPPING(0, CameraFrustums, SFG_Advanced, NSLOCTEXT("UnrealEd", "CameraFrustumsSF", "Camera Frustums"))
/** Draw sound actor radii */
SHOWFLAG_FIXED_IN_SHIPPING(0, AudioRadius, SFG_Advanced, NSLOCTEXT("UnrealEd", "AudioRadiusSF", "Audio Radius"))
/** Draw force feedback radii */
SHOWFLAG_FIXED_IN_SHIPPING(0, ForceFeedbackRadius, SFG_Advanced, NSLOCTEXT("UnrealEd", "ForceFeedbackSF", "Force Feedback Radius"))
/** Colors BSP based on model component association */
SHOWFLAG_FIXED_IN_SHIPPING(0, BSPSplit, SFG_Advanced, NSLOCTEXT("UnrealEd", "BSPSplitSF", "BSP Split"))
/** show editor (wireframe) brushes, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_FIXED_IN_SHIPPING(0, Brushes, SFG_Hidden, NSLOCTEXT("UnrealEd", "BrushesSF", "Brushes"))
/** Show the usual material light interaction */
SHOWFLAG_ALWAYS_ACCESSIBLE(Lighting, SFG_Hidden, NSLOCTEXT("UnrealEd", "LightingSF", "Lighting"))
/** Enable rich view when lighting is disabled, forcing a slower dynamic render code path.  Used for unlit debug view modes that override shaders. */
SHOWFLAG_ALWAYS_ACCESSIBLE(UnlitRichView, SFG_Hidden, NSLOCTEXT("UnrealEd", "UnlitRichViewSF", "UnlitRichView"))
/** Enable view mode that causes shaders to output Base Color as Scene Color (emissive) when lighting is disabled.  Only works when development shaders are enabled, typically only on PC. */
SHOWFLAG_ALWAYS_ACCESSIBLE(UnlitViewmode, SFG_Hidden, NSLOCTEXT("UnrealEd", "UnlitViewmodeSF", "UnlitViewmode"))
/** Execute the deferred light passes, can be disabled for debugging, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(DeferredLighting, SFG_Advanced, NSLOCTEXT("UnrealEd", "DeferredLightingSF", "DeferredLighting"))
/** Special: Allows to hide objects in the editor, is evaluated per primitive */
SHOWFLAG_FIXED_IN_SHIPPING(0, Editor, SFG_Hidden, NSLOCTEXT("UnrealEd", "EditorSF", "Editor"))
/** needed for VMI_BrushWireframe and VMI_LitLightmapDensity, Draws BSP triangles */
SHOWFLAG_FIXED_IN_SHIPPING(1, BSPTriangles, SFG_Hidden, NSLOCTEXT("UnrealEd", "BSPTrianglesSF", "BSP Triangles"))
/** Displays large clickable icons on static mesh vertices, only needed for the editor */
SHOWFLAG_FIXED_IN_SHIPPING(0, LargeVertices, SFG_Advanced, NSLOCTEXT("UnrealEd", "LargeVerticesSF", "Large Vertices"))
/** To show the grid in editor (grey lines and red dots) */
SHOWFLAG_FIXED_IN_SHIPPING(0, Grid, SFG_Normal, NSLOCTEXT("UnrealEd", "GridSF", "Grid"))
/** To show the snap in editor (only for editor view ports, red dots) */
SHOWFLAG_FIXED_IN_SHIPPING(0, Snap, SFG_Hidden, NSLOCTEXT("UnrealEd", "SnapSF", "Snap"))
/** In the filled view modeModeWidgetss, render mesh edges as well as the filled surfaces. */
SHOWFLAG_FIXED_IN_SHIPPING(0, MeshEdges, SFG_Advanced, NSLOCTEXT("UnrealEd", "MeshEdgesSF", "Mesh Edges"))
/** Complex cover rendering */
SHOWFLAG_FIXED_IN_SHIPPING(0, Cover, SFG_Hidden, NSLOCTEXT("UnrealEd", "CoverSF", "Cover"))
/** Spline rendering */
SHOWFLAG_FIXED_IN_SHIPPING(0, Splines, SFG_Advanced, NSLOCTEXT("UnrealEd", "SplinesSF", "Splines"))
/** Selection rendering, could be useful in game as well */
SHOWFLAG_FIXED_IN_SHIPPING(0, Selection, SFG_Advanced, NSLOCTEXT("UnrealEd", "SelectionSF", "Selection"))
/** Grey out actors which do not belong to the currently edited LevelInstance */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeLevelInstanceEditing, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeLevelInstanceEditingSF", "Visualize Level Instance Editing"))
/** Used to determine whether there are any editing LevelInstances to enable the post processing pass */
SHOWFLAG_FIXED_IN_SHIPPING(0, EditingLevelInstance, SFG_Transient, NSLOCTEXT("UnrealEd", "EditingLevelInstanceSF", "Editing LevelInstance"))
/** Draws mode specific gizmos and controls in the viewports (should only be set on viewport clients that are editing the level itself) */
SHOWFLAG_FIXED_IN_SHIPPING(0, ModeWidgets, SFG_Advanced, NSLOCTEXT("UnrealEd", "ModeWidgetsSF", "Mode Gizmos"))
/**  */
SHOWFLAG_FIXED_IN_SHIPPING(0, Bounds,  SFG_Advanced, NSLOCTEXT("UnrealEd", "BoundsSF", "Bounds"))
/** Draws each hit proxy in the scene with a different color, for now only available in the editor */
SHOWFLAG_FIXED_IN_SHIPPING(0, HitProxies, SFG_Developer, NSLOCTEXT("UnrealEd", "HitProxiesSF", "Hit Proxies"))
/** Draw lines to lights affecting this mesh if its selected. */
SHOWFLAG_FIXED_IN_SHIPPING(0, LightInfluences, SFG_Advanced, NSLOCTEXT("UnrealEd", "LightInfluencesSF", "Light Influences"))
/** for the Editor */
SHOWFLAG_FIXED_IN_SHIPPING(0, Pivot, SFG_Hidden, NSLOCTEXT("UnrealEd", "PivotSF", "Pivot"))
/** Draws un-occluded shadow frustums in wireframe */
SHOWFLAG_FIXED_IN_SHIPPING(0, ShadowFrustums, SFG_Advanced, NSLOCTEXT("UnrealEd", "ShadowFrustumsSF", "Shadow Frustums"))
/** needed for VMI_Wireframe and VMI_BrushWireframe */
SHOWFLAG_FIXED_IN_SHIPPING(0, Wireframe, SFG_Hidden, NSLOCTEXT("UnrealEd", "WireframeSF", "Wireframe"))
/**  */
SHOWFLAG_FIXED_IN_SHIPPING(1, Materials, SFG_Hidden, NSLOCTEXT("UnrealEd", "MaterialsSF", "Materials"))
/** for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(StaticMeshes, SFG_Normal, NSLOCTEXT("UnrealEd", "StaticMeshesSF", "Static Meshes"))
/** for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(Landscape, SFG_Normal, NSLOCTEXT("UnrealEd", "LandscapeSF", "Landscape"))
/**  */
SHOWFLAG_FIXED_IN_SHIPPING(0, LightRadius, SFG_Advanced, NSLOCTEXT("UnrealEd", "LightRadiusSF", "Light Radius"))
/** Draws fog, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(Fog, SFG_Normal, NSLOCTEXT("UnrealEd", "FogSF", "Fog"))
/** Draws Volumes */
SHOWFLAG_FIXED_IN_SHIPPING(0, Volumes, SFG_Advanced, NSLOCTEXT("UnrealEd", "VolumesSF", "Volumes"))
/** if this is a game viewport, needed? */
SHOWFLAG_ALWAYS_ACCESSIBLE(Game, SFG_Hidden, NSLOCTEXT("UnrealEd", "GameSF", "Game"))
/** Render objects with colors based on what the actors coloring handlers provides */
SHOWFLAG_FIXED_IN_SHIPPING(0, ActorColoration, SFG_Transient, NSLOCTEXT("UnrealEd", "ActorColorationSF", "Actor Coloration"))
/** Draws BSP brushes (in game or editor textured triangles usually with lightmaps), for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(BSP, SFG_Normal, NSLOCTEXT("UnrealEd", "BSPSF", "BSP"))
/** Collision drawing */
SHOWFLAG_FIXED_IN_SHIPPING(0, Collision, SFG_Normal, NSLOCTEXT("UnrealEd", "CollisionWireFrame", "Collision"))
/** Collision blocking visibility against complex **/
SHOWFLAG_FIXED_IN_SHIPPING(0, CollisionVisibility, SFG_Hidden, NSLOCTEXT("UnrealEd", "CollisionVisibility", "Visibility"))
/** Collision blocking pawn against simple collision **/
SHOWFLAG_FIXED_IN_SHIPPING(0, CollisionPawn, SFG_Hidden, NSLOCTEXT("UnrealEd", "CollisionPawn", "Pawn"))
/** Render LightShafts, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(LightShafts, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "LightShaftsSF", "Light Shafts"))
/** Render the PostProcess Material */
SHOWFLAG_FIXED_IN_SHIPPING(1, PostProcessMaterial, SFG_PostProcess, NSLOCTEXT("UnrealEd", "PostProcessMaterialSF", "Post Process Material"))
/** Render Sky and Atmospheric lighting, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(Atmosphere, SFG_Normal, NSLOCTEXT("UnrealEd", "AtmosphereSF", "Atmosphere"))
/** Disable the (main) deferred sky atmosphere render pass while preserving atmospheric lighting. */
SHOWFLAG_ALWAYS_ACCESSIBLE(DeferredAtmospherePass, SFG_Advanced, NSLOCTEXT("UnrealEd", "DeferredAtmospherePassSF", "Deferred Atmosphere Pass"))
/** Render Cloud, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(Cloud, SFG_Normal, NSLOCTEXT("UnrealEd", "CloudSF", "Cloud"))
/** Render safe frames bars*/
SHOWFLAG_FIXED_IN_SHIPPING(0, CameraAspectRatioBars, SFG_Advanced, NSLOCTEXT("UnrealEd", "CameraAspectRatioBarsSF", "Camera Aspect Ratio Bars"))
/** Render safe frames */
SHOWFLAG_FIXED_IN_SHIPPING(1, CameraSafeFrames, SFG_Advanced, NSLOCTEXT("UnrealEd", "CameraSafeFramesSF", "Camera Safe Frames"))
/** Render TextRenderComponents (3D text), for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(TextRender, SFG_Advanced, NSLOCTEXT("UnrealEd", "TextRenderSF", "Render (3D) Text"))
/** Any rendering/buffer clearing  (good for benchmarking and for pausing rendering while the app is not in focus to save cycles). */
SHOWFLAG_ALWAYS_ACCESSIBLE(Rendering, SFG_Hidden, NSLOCTEXT("UnrealEd", "RenderingSF", "Any Rendering")) // do not make it FIXED_IN_SHIPPING, used by Oculus plugin.
/** Show the current mask being used by the highres screenshot capture */
SHOWFLAG_FIXED_IN_SHIPPING(0, HighResScreenshotMask, SFG_Transient, NSLOCTEXT("UnrealEd", "HighResScreenshotMaskSF", "High Res Screenshot Mask"))
/** Distortion of output for HMD devices, SHOWFLAG_ALWAYS_ACCESSIBLE for now because USceneCaptureComponent needs that */
SHOWFLAG_ALWAYS_ACCESSIBLE(HMDDistortion, SFG_PostProcess, NSLOCTEXT("UnrealEd", "HMDDistortionSF", "HMD Distortion"))
/** Whether to render in stereoscopic 3d, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's used by StereoRendering */
SHOWFLAG_ALWAYS_ACCESSIBLE(StereoRendering, SFG_Hidden, NSLOCTEXT("UnrealEd", "StereoRenderingSF", "Stereoscopic Rendering"))
/** Show objects even if they should be distance culled, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(DistanceCulledPrimitives, SFG_Hidden, NSLOCTEXT("UnrealEd", "DistanceCulledPrimitivesSF", "Distance Culled Primitives"))
/** To visualize the culling in Tile Based Deferred Lighting, later for non tiled as well */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeLightCulling, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeLightCullingSF", "Light Culling"))
/** To disable precomputed visibility */
SHOWFLAG_FIXED_IN_SHIPPING(1, PrecomputedVisibility, SFG_Advanced, NSLOCTEXT("UnrealEd", "PrecomputedVisibilitySF", "Precomputed Visibility"))
/** Contribution from sky light, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(SkyLighting, SFG_LightTypes, NSLOCTEXT("UnrealEd", "SkyLightingSF", "Sky Lighting"))
/** Visualize preview shadow indicator */
SHOWFLAG_FIXED_IN_SHIPPING(0, PreviewShadowsIndicator, SFG_Visualize, NSLOCTEXT("UnrealEd", "PreviewShadowIndicatorSF", "Preview Shadows Indicator"))
/** Visualize precomputed visibility cells */
SHOWFLAG_FIXED_IN_SHIPPING(0, PrecomputedVisibilityCells, SFG_Visualize, NSLOCTEXT("UnrealEd", "PrecomputedVisibilityCellsSF", "Precomputed Visibility Cells"))
/** Visualize volumetric lightmap used for GI on dynamic objects */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeVolumetricLightmap, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeVolumetricLightmapSF", "Volumetric Lightmap"))
/** Visualize volume lighting samples used for GI on dynamic objects */
SHOWFLAG_FIXED_IN_SHIPPING(0, VolumeLightingSamples, SFG_Visualize, NSLOCTEXT("UnrealEd", "VolumeLightingSamplesSF", "Volume Lighting Samples"))
/** Render Paper2D sprites, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(Paper2DSprites, SFG_Advanced, NSLOCTEXT("UnrealEd", "Paper2DSpritesSF", "Paper 2D Sprites"))
/** Visualization of distance field AO */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeDistanceFieldAO, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeDistanceFieldAOSF", "Distance Field Ambient Occlusion"))
/** Mesh Distance fields */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeMeshDistanceFields, SFG_Visualize, NSLOCTEXT("UnrealEd", "MeshDistanceFieldsSF", "Mesh DistanceFields"))
/** Physics field */
SHOWFLAG_FIXED_IN_SHIPPING(0, PhysicsField, SFG_Visualize, NSLOCTEXT("UnrealEd", "PhysicsField", "Physics Field"))
/** Global Distance field */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeGlobalDistanceField, SFG_Visualize, NSLOCTEXT("UnrealEd", "GlobalDistanceFieldSF", "Global DistanceField"))
/** Enable the debug visualization of diffuse/specular lighting (direct and indirect) using probes */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeLightingOnProbes, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeLightingOnProbesSF", "Debug Lighting on Probes"))
/** Screen space AO, for now SHOWFLAG_ALWAYS_ACCESSIBLE because r.GBuffer need that */
SHOWFLAG_ALWAYS_ACCESSIBLE(ScreenSpaceAO, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "ScreenSpaceAOSF", "Screen Space Ambient Occlusion"))
/** Distance field AO, for now SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(DistanceFieldAO, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "DistanceFieldAOSF", "Distance Field Ambient Occlusion"))
/** Lumen GI */
SHOWFLAG_ALWAYS_ACCESSIBLE(LumenGlobalIllumination, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "LumenGlobalIlluminationSF", "Lumen Global Illumination"))
/** SHOWFLAG_ALWAYS_ACCESSIBLE because it's exposed in SceneCapture */
SHOWFLAG_ALWAYS_ACCESSIBLE(VolumetricFog, SFG_LightingFeatures, NSLOCTEXT("UnrealEd", "VolumetricFogSF", "Volumetric Fog"))
/** Visualize screen space reflections, for developer (by default off): */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeSSR, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeSSR", "Screen Space Reflections"))
/** Visualize the Shading Models, mostly or debugging and profiling */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeShadingModels, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeShadingModels", "Shading Models"))
/** Visualize the senses configuration of AIs' PawnSensingComponent */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeSenses, SFG_Advanced, NSLOCTEXT("UnrealEd", "VisualizeSenses", "Senses"))
/** Visualize LOD Coloration */
SHOWFLAG_FIXED_IN_SHIPPING(0, LODColoration, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeLODColoration", "Visualize LOD Coloration"))
/** Visualize HLOD Coloration */
SHOWFLAG_FIXED_IN_SHIPPING(0, HLODColoration, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeHLODColoration", "Visualize HLOD Coloration"))
/** Visualize streaming texture deficit (WantedMips - ResidentMips) */
SHOWFLAG_FIXED_IN_SHIPPING(0, StreamingTextureDeficit, SFG_Hidden, NSLOCTEXT("UnrealEd", "StreamingTextureDeficit", "Streaming Texture Deficit"))
/** Visualize streaming texture residency (MaxLODs - ResidentMips) */
SHOWFLAG_FIXED_IN_SHIPPING(0, StreamingTextureResidency, SFG_Hidden, NSLOCTEXT("UnrealEd", "StreamingTextureResidency", "Streaming Texture Residency"))
/** Visualize streaming mesh LOD deficit (WantedMips - ResidentMips) */
SHOWFLAG_FIXED_IN_SHIPPING(0, StreamingMeshLODDeficit, SFG_Hidden, NSLOCTEXT("UnrealEd", "StreamingMeshLODDeficit", "Streaming Mesh LOD Deficit"))
/** Visualize streaming mesh LOD residency (MaxLODs - ResidentLODs) */
SHOWFLAG_FIXED_IN_SHIPPING(0, StreamingMeshLODResidency, SFG_Hidden, NSLOCTEXT("UnrealEd", "StreamingMeshLODResidency", "Streaming Mesh LOD Residency"))
/** Visualize screen quads */
SHOWFLAG_FIXED_IN_SHIPPING(0, QuadOverdraw, SFG_Hidden, NSLOCTEXT("UnrealEd", "QuadOverdrawSF", "Quad Overdraw"))
/** Visualize the overhead of material quads */
SHOWFLAG_FIXED_IN_SHIPPING(0, ShaderComplexityWithQuadOverdraw, SFG_Hidden, NSLOCTEXT("UnrealEd", "ShaderComplexityWithQuadOverdraw", "Shader Complexity With Quad Overdraw"))
/** Visualize the accuracy of the primitive distance computed for texture streaming */
SHOWFLAG_FIXED_IN_SHIPPING(0, PrimitiveDistanceAccuracy, SFG_Hidden, NSLOCTEXT("UnrealEd", "PrimitiveDistanceAccuracy", "Primitive Distance Accuracy"))
/** Visualize the accuracy of the mesh UV density computed for texture streaming */
SHOWFLAG_FIXED_IN_SHIPPING(0, MeshUVDensityAccuracy, SFG_Hidden, NSLOCTEXT("UnrealEd", "MeshUVDensityAccuracy", "Mesh UV Densities Accuracy"))
/** Visualize the accuracy of CPU material texture scales when compared to the GPU values */
SHOWFLAG_FIXED_IN_SHIPPING(0, MaterialTextureScaleAccuracy, SFG_Hidden, NSLOCTEXT("UnrealEd", "MaterialTextureScaleAccuracy", "Material Texture Scales Accuracy"))
/** Outputs the material texture scales. */
SHOWFLAG_FIXED_IN_SHIPPING(0, OutputMaterialTextureScales, SFG_Hidden, NSLOCTEXT("UnrealEd", "OutputMaterialTextureScales", "Output Material Texture Scales"))
/** Compare the required texture resolution to the actual resolution. */
SHOWFLAG_FIXED_IN_SHIPPING(0, RequiredTextureResolution, SFG_Hidden, NSLOCTEXT("UnrealEd", "RequiredTextureResolution", "Required Texture Resolution"))
/** Compare the required mip level to the resident virtual texture mip level. */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeVirtualTexture, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeVirtualTextureSF", "Virtual Texture Visualization"))
/** If WidgetComponents should be rendered in the scene */
SHOWFLAG_ALWAYS_ACCESSIBLE(WidgetComponents, SFG_Normal, NSLOCTEXT("UnrealEd", "WidgetComponentsSF", "Widget Components"))
/** Draw the bones of all skeletal meshes */
SHOWFLAG_FIXED_IN_SHIPPING(0, Bones, SFG_Developer, NSLOCTEXT("UnrealEd", "BoneSF", "Bones"))
/** Draw debug rendering from the Dedicated Server during Play In Editor (with Use Dedicated Server checked) in this client viewport */
SHOWFLAG_FIXED_IN_SHIPPING(0, ServerDrawDebug, SFG_Developer, NSLOCTEXT("UnrealEd", "ServerDrawDebugSF", "Dedicated Server Debug Drawing"))
/** If media planes should be shown */
SHOWFLAG_ALWAYS_ACCESSIBLE(MediaPlanes, SFG_Normal, NSLOCTEXT("UnrealEd", "MediaPlanesSF", "Media Planes"))
/** if this is a vr editing viewport, needed? */
SHOWFLAG_FIXED_IN_SHIPPING(0, VREditing, SFG_Hidden, NSLOCTEXT("UnrealEd", "VREditSF", "VR Editing"))
/** Visualize Occlusion Query bounding meshes */
SHOWFLAG_FIXED_IN_SHIPPING(0, OcclusionMeshes, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeOcclusionQueries", "Visualize Occlusion Queries"))
/** Visualize Occlusion Query bounding meshes */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeInstanceOcclusionQueries, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeInstanceOcclusionQueriesSF", "Visualize Instance Culling Occlusion Queries"))
/** Disable hardware occlusion queries, similar to setting r.AllowOcclusionQueries=0, but just for this scene. */
SHOWFLAG_ALWAYS_ACCESSIBLE(DisableOcclusionQueries, SFG_Hidden, NSLOCTEXT("UnrealEd", "DisableOcclusionQueries", "Disable Hardware Occlusion Queries"))

// RHI_RAYTRACING begin
SHOWFLAG_ALWAYS_ACCESSIBLE(PathTracing, SFG_Hidden, NSLOCTEXT("UnrealEd", "PathTracing", "Path tracing"))
SHOWFLAG_FIXED_IN_SHIPPING(0, RayTracingDebug, SFG_Hidden, NSLOCTEXT("UnrealEd", "RayTracingDebug", "Ray tracing debug"))
// RHI_RAYTRACING end

/** Enable the SkyAtmosphere visualization to be drawn on screen */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeSkyAtmosphere, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeSkyAtmosphereSF", "Sky Atmosphere"))
/** Enable the light function atlas debug visualization to be drawn on screen */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeLightFunctionAtlas, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeLightFunctionAtlasSF", "Light Function Atlas"))
/** Shows a full-screen calibration color post process material defined in renderer config  */
SHOWFLAG_ALWAYS_ACCESSIBLE(VisualizeCalibrationColor, SFG_Developer, NSLOCTEXT("UnrealEd", "VisualizeCalibrationColorSF", "Visualize Calibration Color"))
/** Shows a full-screen calibration grayscale post process material defined in renderer config  */
SHOWFLAG_ALWAYS_ACCESSIBLE(VisualizeCalibrationGrayscale, SFG_Developer, NSLOCTEXT("UnrealEd", "VisualizeCalibrationGrayscaleSF", "Visualize Calibration Grayscale"))
/** Shows a full-screen calibration custom post process material defined in renderer config  */
SHOWFLAG_ALWAYS_ACCESSIBLE(VisualizeCalibrationCustom, SFG_Developer, NSLOCTEXT("UnrealEd", "VisualizeCalibrationCustomSF", "Visualize Calibration Custom"))
/** Visualise information about the post processes stacked on screen */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizePostProcessStack, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizePostProcessStackSF", "Post Process Stack"))

/** Draw in the main pass the primitives that would normally only contribute to runtime virtual textures. */
SHOWFLAG_FIXED_IN_SHIPPING(0, VirtualTexturePrimitives, SFG_Developer, NSLOCTEXT("UnrealEd", "VirtualTexturePrimitivesSF", "Virtual Texture Primitives"))

/** Visualize volumetric cloud conservative density. */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeVolumetricCloudConservativeDensity, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeVolumetricCloudConservativeDensitySF", "Volumetric Cloud Conservative Density"))

/** Visualize volumetric cloud density for empty space skipping. */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeVolumetricCloudEmptySpaceSkipping, SFG_Visualize, NSLOCTEXT("UnrealEd", "VisualizeVolumetricCloudEmptySpaceSkippingSF", "Volumetric Cloud Empty Space Skipping Density"))

SHOWFLAG_FIXED_IN_SHIPPING(1, VirtualShadowMapPersistentData, SFG_Developer, NSLOCTEXT("UnrealEd", "VirtualShadowMapPersistentData", "Virtual Shadow Map Persistent Data"))

SHOWFLAG_FIXED_IN_SHIPPING(0, DebugDrawDistantVirtualSMLights, SFG_Advanced, NSLOCTEXT("UnrealEd", "DebugDrawDistantVirtualSMLightsSF", "Debug Draw Distant VSM Lights"))

/** Debug the Virtual Texture System */
SHOWFLAG_FIXED_IN_SHIPPING(0, VirtualTextureResidency, SFG_Hidden, NSLOCTEXT("UnrealEd", "VirtualTextureResidency", "Virtual Texture Residency"))

SHOWFLAG_FIXED_IN_SHIPPING(0, MaterialCacheVirtualTextureAllocator, SFG_Hidden, NSLOCTEXT("UnrealEd", "MaterialCacheVirtualTextureAllocator", "Material Cache Virtual Texture Allocator"))

/** Input Debug Visualizer */
SHOWFLAG_FIXED_IN_SHIPPING(0, InputDebugVisualizer, SFG_Hidden, NSLOCTEXT("UnrealEd", "InputDebugVisualizer", "Input Debug Visualizer"))

/** Use screen space tracing in Lumen */
SHOWFLAG_ALWAYS_ACCESSIBLE(LumenScreenTraces, SFG_Lumen, NSLOCTEXT("UnrealEd", "LumenScreenTracesSF", "Screen Traces"))
/** Use detail tracing in Lumen */
SHOWFLAG_ALWAYS_ACCESSIBLE(LumenDetailTraces, SFG_Lumen, NSLOCTEXT("UnrealEd", "LumenDetailTracesSF", "Detail Traces"))
/** Use global traces in Lumen */
SHOWFLAG_ALWAYS_ACCESSIBLE(LumenGlobalTraces, SFG_Lumen, NSLOCTEXT("UnrealEd", "LumenGlobalTracesSF", "Global Traces"))
/** Use far field traces in Lumen */
SHOWFLAG_ALWAYS_ACCESSIBLE(LumenFarFieldTraces, SFG_Lumen, NSLOCTEXT("UnrealEd", "LumenFarFieldTracesSF", "Far Field Traces"))
/** Compute secondary bounces in Lumen */
SHOWFLAG_ALWAYS_ACCESSIBLE(LumenSecondaryBounces, SFG_Lumen, NSLOCTEXT("UnrealEd", "LumenSecondaryBouncesSF", "Secondary Bounces"))
/** Compute screen space directional occlusion in Lumen */
SHOWFLAG_ALWAYS_ACCESSIBLE(LumenShortRangeAmbientOcclusion, SFG_Lumen, NSLOCTEXT("UnrealEd", "LumenShortRangeAmbientOcclusionSF", "Short Range Ambient Occlusion"))

/** Use screen space traces in MegaLights */
SHOWFLAG_ALWAYS_ACCESSIBLE(MegaLightsScreenTraces, SFG_MegaLights, NSLOCTEXT("UnrealEd", "MegaLightsScreenTracesSF", "Screen Traces"))
/** Use world space traces in MegaLights */
SHOWFLAG_ALWAYS_ACCESSIBLE(MegaLightsWorldSpaceTraces, SFG_MegaLights, NSLOCTEXT("UnrealEd", "MegaLightsWorldSpaceTracesSF", "World Space Traces"))
/** Use far field traces in MegaLights */
SHOWFLAG_ALWAYS_ACCESSIBLE(MegaLightsFarFieldTraces, SFG_MegaLights, NSLOCTEXT("UnrealEd", "MegaLightsFarFieldTracesSF", "Far Field Traces"))
/** Use strand hair voxel traces in MegaLights */
SHOWFLAG_ALWAYS_ACCESSIBLE(MegaLightsHairVoxelTraces, SFG_MegaLights, NSLOCTEXT("UnrealEd", "MegaLightsHairVoxelTracesSF", "Hair Voxel Traces"))

/** Draw Nanite meshes */
SHOWFLAG_ALWAYS_ACCESSIBLE(NaniteMeshes, SFG_Nanite, NSLOCTEXT("UnrealEd", "NaniteMeshesSF", "Meshes"))
/** Draw Nanite streaming geometry */
SHOWFLAG_ALWAYS_ACCESSIBLE(NaniteStreamingGeometry, SFG_Nanite, NSLOCTEXT("UnrealEd", "NaniteStreamingGeometrySF", "Streaming Geometry"))

/** Visualize Skin Cache */
SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeGPUSkinCache, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeGPUSkinCache", "Visualize GPU Skin Cache"))

SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeLWCComplexity, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeLWCComplexity", "Visualize Material LWC Function Usage"))

SHOWFLAG_FIXED_IN_SHIPPING(0, VisualizeShadowCasters, SFG_Hidden, NSLOCTEXT("UnrealEd", "VisualizeShadowCasters", "Visualize VSM Shadow Casters"))

/** Enables ShaderPrint functionality that is used for debug printing from shaders (e.g. 'Nanitestats' command). Disable this for certain editor rendering (e.g. thumbnail rendering) */
SHOWFLAG_ALWAYS_ACCESSIBLE(ShaderPrint, SFG_Developer, NSLOCTEXT("UnrealEd", "ShaderPrint", "Shader Print"))

/** If true, scene depth is copied to the scene capture output depth texture. If false, other depth sources like editor primitives depth are still copied. */
SHOWFLAG_ALWAYS_ACCESSIBLE(SceneCaptureCopySceneDepth, SFG_Hidden, NSLOCTEXT("UnrealEd", "SceneCaptureCopySceneDepth", "Copy Scene Depth To Scene Capture"))

/** If false, camera's are not shown in the viewport (but are enabled and available for use). */
SHOWFLAG_ALWAYS_ACCESSIBLE(Cameras, SFG_Normal, NSLOCTEXT("UnrealEd", "CamerasSF", "Cameras"))

/** If false, hair geometry won't be rendered. */
SHOWFLAG_ALWAYS_ACCESSIBLE(Hair, SFG_Normal, NSLOCTEXT("UnrealEd", "HairSF", "Hair"))

/** Allows primitive alpha holdout rendering. Note that the "Support Primitive Alpha Holdout" project setting first needs to be enabled for the deferred renderer. */
SHOWFLAG_ALWAYS_ACCESSIBLE(AllowPrimitiveAlphaHoldout, SFG_Advanced, NSLOCTEXT("UnrealEd", "AllowPrimitiveAlphaHoldoutSF", "Allow Primitive Alpha Holdout"))

/** Invert the alpha channel at tne end of post processing. */
SHOWFLAG_ALWAYS_ACCESSIBLE(AlphaInvert, SFG_PostProcess, NSLOCTEXT("UnrealEd", "AlphaInvertSF", "Invert Alpha Channel"))

/** Directly render motion vectors and motion vector depth for OpenXR extensions when normal velocity rendering is unsupported. Motion vectors/depth must be in Vulkan NDC-space and can be requested at any size. */
SHOWFLAG_ALWAYS_ACCESSIBLE(StereoMotionVectors, SFG_Hidden, NSLOCTEXT("UnrealEd", "MotionVectorsSF", "Render Stereo Motion Vectors for OpenXR"))

/** Geometry Inspection modes. */
SHOWFLAG_FIXED_IN_SHIPPING(0, Clay, SFG_Hidden, NSLOCTEXT("UnrealEd", "Clay", "Clay"))
SHOWFLAG_FIXED_IN_SHIPPING(0, Zebra, SFG_Hidden, NSLOCTEXT("UnrealEd", "Zebra", "Zebra"))
SHOWFLAG_FIXED_IN_SHIPPING(0, FrontBackFace, SFG_Hidden, NSLOCTEXT("UnrealEd", "FrontBackFace", "Front/Back Face"))
SHOWFLAG_FIXED_IN_SHIPPING(0, RandomColor, SFG_Hidden, NSLOCTEXT("UnrealEd", "RandomColor", "RandomColor"))

#undef SHOWFLAG_ALWAYS_ACCESSIBLE
#undef SHOWFLAG_FIXED_IN_SHIPPING
