// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	SimpleElementShaders.h: Definitions for simple element shaders.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "SceneTypes.h"
#include "Engine/EngineTypes.h"
#include "DataDrivenShaderPlatformInfo.h"

class FSceneView;
struct FDFRelativeViewMatrices;

/**
 * A vertex shader for rendering a texture on a simple element.
 */
class FSimpleElementVS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FSimpleElementVS, ENGINE_API);
public:
	FSimpleElementVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	FSimpleElementVS() {}

	ENGINE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FMatrix& WorldToClipMatrix);
	ENGINE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FDFRelativeViewMatrices& Matrices, const FDFRelativeViewMatrices& InstancedViewMatrices);
	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FDFRelativeViewMatrices& Matrices)
	{
		SetParameters(BatchedParameters, Matrices, Matrices);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

private:
	LAYOUT_FIELD(FShaderParameter, RelativeTransform);
	LAYOUT_FIELD(FShaderParameter, TransformPositionHigh);
};

/**
 * Simple pixel shader that just reads from a texture
 * This is used for resolves and needs to be as efficient as possible
 */
class FSimpleElementPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FSimpleElementPS, Global);
public:
	FSimpleElementPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	FSimpleElementPS() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* TextureValue);
	/** Sets parameters for compositing editor primitives */
	void SetEditorCompositingParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView* View);

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView* View, const FTexture* TextureValue);

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture)
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler)
	LAYOUT_FIELD(FShaderParameter, TextureComponentReplicate)
	LAYOUT_FIELD(FShaderParameter, TextureComponentReplicateAlpha)
	LAYOUT_FIELD(FShaderParameter, EditorCompositeDepthTestParameter)
	LAYOUT_FIELD(FShaderParameter, ScreenToPixel)
};

/**
 * Simple pixel shader that just reads from an alpha-only texture
 */
class FSimpleElementAlphaOnlyPS : public FSimpleElementPS
{
	DECLARE_SHADER_TYPE(FSimpleElementAlphaOnlyPS, Global);
public:

	FSimpleElementAlphaOnlyPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	FSimpleElementAlphaOnlyPS() {}
};

/**
 * A pixel shader for rendering a texture on a simple element.
 */
class FSimpleElementGammaBasePS : public FSimpleElementPS
{
	DECLARE_TYPE_LAYOUT(FSimpleElementGammaBasePS, NonVirtual);
public:

	FSimpleElementGammaBasePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	FSimpleElementGammaBasePS() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture, float GammaValue, ESimpleElementBlendMode BlendMode);
	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView* View, const FTexture* Texture, float GammaValue, ESimpleElementBlendMode BlendMode);

private:
	LAYOUT_FIELD(FShaderParameter, Gamma)
};

template <bool bSRGBTexture>
class FSimpleElementGammaPS : public FSimpleElementGammaBasePS
{
	DECLARE_SHADER_TYPE(FSimpleElementGammaPS, Global);
public:
	FSimpleElementGammaPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FSimpleElementGammaBasePS(Initializer) {}
	FSimpleElementGammaPS() {}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// SRGB_INPUT_TEXTURE is now ignored, this has no effect
		// input texture gamma correction is done automatically by the sampler, controlled by TexCreate_SRGB
		OutEnvironment.SetDefine(TEXT("SRGB_INPUT_TEXTURE"), bSRGBTexture);
	}
};

/**
 * Simple pixel shader that just reads from an alpha-only texture and gamma corrects the output
 */
class FSimpleElementGammaAlphaOnlyPS : public FSimpleElementGammaBasePS
{
	DECLARE_SHADER_TYPE(FSimpleElementGammaAlphaOnlyPS, Global);
public:
	FSimpleElementGammaAlphaOnlyPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FSimpleElementGammaBasePS(Initializer) {}
	FSimpleElementGammaAlphaOnlyPS() {}
};

/**
 * A pixel shader for rendering a masked texture on a simple element.
 */
class FSimpleElementMaskedGammaBasePS : public FSimpleElementGammaBasePS
{
	DECLARE_TYPE_LAYOUT(FSimpleElementMaskedGammaBasePS, NonVirtual);
public:
	FSimpleElementMaskedGammaBasePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	FSimpleElementMaskedGammaBasePS() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture,float Gamma,float ClipRefValue,ESimpleElementBlendMode BlendMode);
	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView* View, const FTexture* Texture,float Gamma,float ClipRefValue,ESimpleElementBlendMode BlendMode);

private:
	LAYOUT_FIELD(FShaderParameter, ClipRef)
};

template <bool bSRGBTexture>
class FSimpleElementMaskedGammaPS : public FSimpleElementMaskedGammaBasePS
{
	DECLARE_SHADER_TYPE(FSimpleElementMaskedGammaPS, Global);
public:
	FSimpleElementMaskedGammaPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FSimpleElementMaskedGammaBasePS(Initializer) {}
	FSimpleElementMaskedGammaPS() {}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SRGB_INPUT_TEXTURE"), bSRGBTexture);
	}
};

/**
* A pixel shader for rendering a masked texture using signed distance filed for alpha on a simple element.
*/
class FSimpleElementDistanceFieldGammaPS : public FSimpleElementMaskedGammaBasePS
{
	DECLARE_SHADER_TYPE(FSimpleElementDistanceFieldGammaPS,Global);
public:
	FSimpleElementDistanceFieldGammaPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	FSimpleElementDistanceFieldGammaPS() {}

	/**
	* Sets all the constant parameters for this shader
	*
	* @param Texture - 2d tile texture
	* @param Gamma - if gamma != 1.0 then a pow(color,Gamma) is applied
	* @param ClipRef - reference value to compare with alpha for killing pixels
	* @param SmoothWidth - The width to smooth the edge the texture
	* @param EnableShadow - Toggles drop shadow rendering
	* @param ShadowDirection - 2D vector specifying the direction of shadow
	* @param ShadowColor - Color of the shadowed pixels
	* @param ShadowSmoothWidth - The width to smooth the edge the shadow of the texture
	* @param BlendMode - current batched element blend mode being rendered
	*/
	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FTexture* Texture,
		float Gamma,
		float ClipRef,
		float SmoothWidthValue,
		bool EnableShadowValue,
		const FVector2D& ShadowDirectionValue,
		const FLinearColor& ShadowColorValue,
		float ShadowSmoothWidthValue,
		const FDepthFieldGlowInfo& GlowInfo,
		ESimpleElementBlendMode BlendMode
		);

private:
	/** The width to smooth the edge the texture */
	LAYOUT_FIELD(FShaderParameter, SmoothWidth)
	/** Toggles drop shadow rendering */
	LAYOUT_FIELD(FShaderParameter, EnableShadow)
	/** 2D vector specifying the direction of shadow */
	LAYOUT_FIELD(FShaderParameter, ShadowDirection)
	/** Color of the shadowed pixels */
	LAYOUT_FIELD(FShaderParameter, ShadowColor)	
	/** The width to smooth the edge the shadow of the texture */
	LAYOUT_FIELD(FShaderParameter, ShadowSmoothWidth)
	/** whether to turn on the outline glow */
	LAYOUT_FIELD(FShaderParameter, EnableGlow)
	/** base color to use for the glow */
	LAYOUT_FIELD(FShaderParameter, GlowColor)
	/** outline glow outer radius */
	LAYOUT_FIELD(FShaderParameter, GlowOuterRadius)
	/** outline glow inner radius */
	LAYOUT_FIELD(FShaderParameter, GlowInnerRadius)
};

/**
 * A pixel shader for rendering a texture on a simple element.
 */
class FSimpleElementHitProxyPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FSimpleElementHitProxyPS,Global);
public:
	FSimpleElementHitProxyPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	FSimpleElementHitProxyPS() {}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* TextureValue);

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture)
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler)
};

/**
* A pixel shader for rendering a texture with the ability to weight the colors for each channel.
* The shader also features the ability to view alpha channels and desaturate any red, green or blue channels
* 
*/
class FSimpleElementColorChannelMaskPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FSimpleElementColorChannelMaskPS,Global);
public:
	FSimpleElementColorChannelMaskPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	FSimpleElementColorChannelMaskPS() {}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	/**
	* Sets all the constant parameters for this shader
	*
	* @param Texture - 2d tile texture
	* @param ColorWeights - reference value to compare with alpha for killing pixels
	* @param Gamma - if gamma != 1.0 then a pow(color,Gamma) is applied
	*/
	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* TextureValue, const FMatrix& ColorWeightsValue, float GammaValue);

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture)
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler)
	LAYOUT_FIELD(FShaderParameter, ColorWeights) 
	LAYOUT_FIELD(FShaderParameter, Gamma)
};

typedef FSimpleElementGammaPS<true> FSimpleElementGammaPS_SRGB;
typedef FSimpleElementGammaPS<false> FSimpleElementGammaPS_Linear;
typedef FSimpleElementMaskedGammaPS<true> FSimpleElementMaskedGammaPS_SRGB;
typedef FSimpleElementMaskedGammaPS<false> FSimpleElementMaskedGammaPS_Linear;
