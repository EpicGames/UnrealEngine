// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "SingleAnimationPlayData.generated.h"

class UAnimSingleNodeInstance;

USTRUCT(BlueprintType)
struct FSingleAnimationPlayData
{
	GENERATED_USTRUCT_BODY()

	// @todo in the future, we should make this one UObject
	// and have detail customization to display different things
	// The default sequence to play on this skeletal mesh
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	TObjectPtr<class UAnimationAsset> AnimToPlay;

	/** Default setting for looping for SequenceToPlay. This is not current state of looping. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (DisplayName = "Looping"))
	uint32 bSavedLooping : 1;

	/** Default setting for playing for SequenceToPlay. This is not current state of playing. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (DisplayName = "Playing"))
	uint32 bSavedPlaying : 1;

	/** Default setting for position of SequenceToPlay to play. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (DisplayName = "Initial Position"))
	float SavedPosition;

	/** Default setting for play rate of SequenceToPlay to play. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Animation, meta = (DisplayName = "PlayRate"))
	float SavedPlayRate;

	FSingleAnimationPlayData()
	{
		AnimToPlay = nullptr;
		SavedPlayRate = 1.0f;
		bSavedLooping = true;
		bSavedPlaying = true;
		SavedPosition = 0.f;
	}

	/** Called when initialized. */
	ENGINE_API void Initialize(class UAnimSingleNodeInstance* Instance);

	/** Populates this play data with the current state of the supplied instance. */
	ENGINE_API void PopulateFrom(UAnimSingleNodeInstance* Instance);

	ENGINE_API void ValidatePosition();
};
