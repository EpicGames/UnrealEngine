// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "BoneIndices.h"

struct FAnimNodePoseWatch;
class FPrimitiveDrawInterface;
struct FCompactHeapPose;
struct FReferenceSkeleton;
class HHitProxy;

namespace EBoneDrawMode
{
	enum Type
	{
		None,
		Selected,
		SelectedAndParents,
		SelectedAndChildren,
		SelectedAndParentsAndChildren,
		All,
		NumDrawModes
	};
};

struct FBoneAxisDrawConfig
{
	bool bDraw = false;
	float Thickness = 0.f;
	float Length = 0.f;
};

struct FSkelDebugDrawConfig
{
public:
	EBoneDrawMode::Type BoneDrawMode;
	float BoneDrawSize;
	bool bForceDraw;
	bool bAddHitProxy;
	bool bUseMultiColorAsDefaultColor;
	FLinearColor DefaultBoneColor;
	FLinearColor AffectedBoneColor;
	FLinearColor SelectedBoneColor;
	FLinearColor ParentOfSelectedBoneColor;
	FBoneAxisDrawConfig AxisConfig;
	
	/** 
	 * For methods that draw multiple bones, draws local rotation axes for all when set to true.
	 * For methods that draw a single bone this option has no effect.
	 */
	bool bDrawAllLocalRotationAxes = false;
};

namespace SkeletalDebugRendering
{
	static const int32 NumSphereSides = 10;
	static const int32 NumConeSides = 4;

/**
 * Draw a wireframe bone from InStart to InEnd 
 * @param	PDI					Primitive draw interface to use
 * @param	InStart				The start location of the bone
 * @param	InEnd				The end location of the bone
 * @param	InColor				The color to draw the bone with
 * @param	InDepthPriority		The scene depth priority group to use
 * @param	SphereRadius		SphereRadius
 */
ENGINE_API void DrawWireBone(
	FPrimitiveDrawInterface* PDI,
	const FVector& InStart,
	const FVector& InEnd,
	const FLinearColor& InColor,
	ESceneDepthPriorityGroup InDepthPriority,
	const float SphereRadius = 1.f);

/**
 * Draw a set of axes to represent a transform
 * @param	PDI					Primitive draw interface to use
 * @param	InTransform			The transform to represent
 * @param	InDepthPriority		The scene depth priority group to use
 * @param	Thickness			Thickness of Axes
 * @param	AxisLength			AxisLength
 */
ENGINE_API void DrawAxes(
	FPrimitiveDrawInterface* PDI,
	const FTransform& InTransform,
	ESceneDepthPriorityGroup InDepthPriority,
	const float Thickness = 0.f,
	const float AxisLength = 4.f);

/**
 * Draw a set of axes to represent a transform
 * @param	PDI					Primitive draw interface to use
 * @param	InBoneTransform		The bone transform
 * @param	InChildLocations	The positions of all the children of the bone
 * @param   InChildColors		The colors to use when drawing the cone to each child
 * @param	InColor				The color to use for the bone
 * @param	InDepthPriority		The scene depth priority group to use
 * @param	SphereRadius		Radius of the ball drawn at the bone location
 * @param	InAxisConfig		Draw configuration for small coordinate axes inside the joint sphere
 */
ENGINE_API	void DrawWireBoneAdvanced(
	FPrimitiveDrawInterface* PDI,
	const FTransform& InBoneTransform,
	const TArray<FVector>& InChildLocations,
	const TArray<FLinearColor>& InChildColors,
	const FLinearColor& InColor,
	ESceneDepthPriorityGroup InDepthPriority,
	const float SphereRadius,
	const FBoneAxisDrawConfig& InAxisConfig);

/**
 * Draw a cone showing offset from origin position to a given bone transform
 * Used to draw the root cone (always in red)
 * @param	PDI				Primitive draw interface to use
 * @param	Start			The position in world space of the start of the cone
 * @param	End				The position in world space of the tapered end of the cone
 * @param	SphereRadius	The radius of the root bone
 * @param	Color			The color to use for the cone
 */
ENGINE_API	void DrawConeConnection(
	FPrimitiveDrawInterface* PDI,
	const FVector& Start,
	const FVector& End,
	const float SphereRadius,
	const FLinearColor& Color);

#if WITH_EDITOR
ENGINE_API void DrawBonesFromPoseWatch(
	FPrimitiveDrawInterface* PDI,
	const FAnimNodePoseWatch& PoseWatch,
	const bool bUseWorldTransform);
#endif


ENGINE_API void DrawBones(
	FPrimitiveDrawInterface* PDI,
	const FVector& ComponentOrigin,
	const TArray<FBoneIndexType>& RequiredBones,
	const FReferenceSkeleton& RefSkeleton,
	const TArray<FTransform>& WorldTransforms,
	const TArray<int32>& InSelectedBones,
	const TArray<FLinearColor>& BoneColors,
	const TArray<TRefCountPtr<HHitProxy>>& HitProxies,
	const FSkelDebugDrawConfig& DrawConfig);

/* Alternative signature that allows overriding the drawn bones that're computed from EBoneDrawMode */
ENGINE_API void DrawBones(
	FPrimitiveDrawInterface* PDI,
	const FVector& ComponentOrigin,
	const TArray<FBoneIndexType>& RequiredBones,
	const FReferenceSkeleton& RefSkeleton,
	const TArray<FTransform>& WorldTransforms,
	const TArray<int32>& InSelectedBones,
	const TArray<FLinearColor>& BoneColors,
	const TArray<TRefCountPtr<HHitProxy>>& HitProxies,
	const FSkelDebugDrawConfig& DrawConfig,
	const TBitArray<>& BonesToDrawOverride);


void DrawBonesInternal(
	FPrimitiveDrawInterface* PDI,
	const FVector& ComponentOrigin,
	const TArray<FBoneIndexType>& RequiredBones,
	const TArray<int32>& ParentIndices,
	const TArray<FTransform>& WorldTransforms,
	const TArray<int32>& InSelectedBones,
	const TArray<FLinearColor>& BoneColors,
	const TArray<TRefCountPtr<HHitProxy>>& HitProxies,
	const FSkelDebugDrawConfig& DrawConfig,
	const TBitArray<>& BonesToDrawOverride);

ENGINE_API FLinearColor GetSemiRandomColorForBone(const int32 BoneIndex, float Value=1.0f, float Saturation=0.75f);
ENGINE_API void FillWithMultiColors(TArray<FLinearColor>& BoneColors, const int32 NumBones);

ENGINE_API void CalculateBonesToDraw(
	const TArray<int32>& ParentIndices,
	const TArray<int32>& InSelectedBones,
	const EBoneDrawMode::Type BoneDrawMode,
	TBitArray<>& OutBonesToDraw);
}
