// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SkinnedAssetCompiler.h"

class USkeletalMesh;
class USkeletalMeshComponent;

#if WITH_EDITOR
using FSkeletalMeshCompilingManager UE_DEPRECATED(5.1, "Use FSkinnedAssetCompilingManager instead.") = FSkinnedAssetCompilingManager;
#endif // #if WITH_EDITOR