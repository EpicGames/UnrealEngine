// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PrimitiveComponentId.h"
#include "RenderGraphResources.h"

class FRDGBuffer;
class FRDGExternalAccessQueue;
class FRHIShaderResourceView;
class FSceneInterface;
class FSkeletalMeshObject;

/** Functions that expose some internal functionality of FSkeletalMeshObject required by MeshDeformer systems. */
class FSkeletalMeshDeformerHelpers
{
public:
	static constexpr uint32 PosBufferBytesPerElement = 4;
	static constexpr uint32 PosBufferElementMultiplier= 3;
	static constexpr uint32 TangentBufferBytesPerElement = 8;
	static constexpr uint32 TangentBufferElementMultiplier= 2;
	static constexpr uint32 ColorBufferBytesPerElement = 4;
	
#pragma region GetInternals

	/** Get direct access to bone matrix buffer SRV. */
	ENGINE_API static FRHIShaderResourceView* GetBoneBufferForReading(
		FSkeletalMeshObject const* InMeshObject,
		int32 InLodIndex,
		int32 InSectionIndex,
		bool bInPreviousFrame);

	/** Get direct access to morph target buffer SRV. */
	ENGINE_API static FRHIShaderResourceView* GetMorphTargetBufferForReading(
		const FSkeletalMeshObject* InMeshObject,
		int32 InLodIndex,
		bool bInPreviousFrame);

	/** Buffer SRVs from the cloth system. */
	struct FClothBuffers
	{
		int32 ClothInfluenceBufferOffset = 0;
		FRHIShaderResourceView* ClothInfluenceBuffer = nullptr;
		FRHIShaderResourceView* ClothSimulatedPositionAndNormalBuffer = nullptr;
		FMatrix44f ClothToLocal = FMatrix44f::Identity;
	};

	/** Get direct access to cloth buffer SRVs. */
	ENGINE_API static FClothBuffers GetClothBuffersForReading(
		FSkeletalMeshObject const* InMeshObject,
		int32 InLodIndex,
		int32 InSectionIndex,
		uint32 InFrameNumber,
		bool bInPreviousFrame);

	/** Returns the allocated writable position buffer if one has been allocated */	
	ENGINE_API static FRDGBuffer* GetAllocatedPositionBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex);

	/** Returns the allocated writable tangent buffer if one has been allocated */	
	ENGINE_API static FRDGBuffer* GetAllocatedTangentBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex);

	/** Returns the allocated writable color buffer if one has been allocated */	
	ENGINE_API static FRDGBuffer* GetAllocatedColorBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex);

	/** Returns the index of the first section that is not disabled */
	ENGINE_API static int32 GetIndexOfFirstAvailableSection(FSkeletalMeshObject* InMeshObject, int32 InLodIndex);
	
	/** 
	 * Resolves and validates a FSkeletalMeshObject from a scene + component ID.
	 * Returns nullptr if the primitive is not found, not a skinned mesh, not yet updated, or CPU-skinned. 
	 */
	ENGINE_API static FSkeletalMeshObject* GetSkeletalMeshObject(FSceneInterface* InSceneInterface, FPrimitiveComponentId InComponentId);
	
#pragma endregion GetInternals

#pragma region SetInternals

	/** 
	 * Allocate and bind a new position buffer and return it for writing.
	 * Ownership is handled by the MeshObject.
	 * If we call this more than once for the same MeshObject in the same frame then we return the allocation from the first call.
	 */
	ENGINE_API static FRDGBuffer* AllocateVertexFactoryPositionBuffer(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccesQueue, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, TCHAR const* InBufferName);

	/**
	 * Allocate and bind a new tangent buffer and return it for writing.
	 * Ownership is handled by the MeshObject.
	 * If we call this more than once for the same MeshObject in the same frame then we return the allocation from the first call.
	 */
	ENGINE_API static FRDGBuffer* AllocateVertexFactoryTangentBuffer(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccesQueue, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, TCHAR const* InBufferName);

	/**
	 * Allocate and bind a new color buffer and return it for writing.
	 * Ownership is handled by the MeshObject.
	 * If we call this more than once for the same MeshObject in the same frame then we return the allocation from the first call.
	 */
	ENGINE_API static FRDGBuffer* AllocateVertexFactoryColorBuffer(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccesQueue, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, TCHAR const* InBufferName);

	/** 
	 * Update all of the MeshObject's passthrough vertex factories with the currently allocated vertex buffers. 
	 * Usually call this after all AllocateVertexFactory*() functions for a frame.
	 */
	ENGINE_API static void UpdateVertexFactoryBufferOverrides(FRHICommandListBase& RHICmdList, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, bool bInvalidatePreviousPosition);
	ENGINE_API static void UpdateVertexFactoryBufferOverrides(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, bool bInvalidatePreviousPosition);

	/** 
	 * Release all of the the buffers that have been allocated through the AllocateVertexFactory*() functions.
	 * Reset all of the MeshObject's passthrough vertex factories.  
	 */
	ENGINE_API static void ResetVertexFactoryBufferOverrides(FSkeletalMeshObject* InMeshObject, int32 InLodIndex);

#pragma endregion SetInternals

#pragma region Deprecated

	UE_DEPRECATED(5.6, "AllocateVertexFactoryPositionBuffer requires a FRDGExternalAccessQueue.")
	ENGINE_API static FRDGBuffer* AllocateVertexFactoryPositionBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, bool bInLodJustChanged, TCHAR const* InBufferName);

	UE_DEPRECATED(5.6, "AllocateVertexFactoryTangentBuffer requires a FRDGExternalAccessQueue.")
	ENGINE_API static FRDGBuffer* AllocateVertexFactoryTangentBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, TCHAR const* InBufferName);

	UE_DEPRECATED(5.6, "AllocateVertexFactoryColorBuffer requires a FRDGExternalAccessQueue.")
	ENGINE_API static FRDGBuffer* AllocateVertexFactoryColorBuffer(FRDGBuilder& GraphBuilder, FSkeletalMeshObject* InMeshObject, int32 InLodIndex, TCHAR const* InBufferName);

	UE_DEPRECATED(5.6, "UpdateVertexFactoryBufferOverrides requires a bInvalidatePreviousPosition flag.")
	ENGINE_API static void UpdateVertexFactoryBufferOverrides(FRHICommandListBase& RHICmdList, FSkeletalMeshObject* InMeshObject, int32 InLodIndex);

	UE_DEPRECATED(5.4, "UpdateVertexFactoryBufferOverrides requires a command list.")
	ENGINE_API static void UpdateVertexFactoryBufferOverrides(FSkeletalMeshObject* InMeshObject, int32 InLodIndex);

	UE_DEPRECATED(5.8, "GetMorphTargetBufferForReading no longer requires section index and frame number.")
	/** Get direct access to morph target buffer SRV. */
	ENGINE_API static FRHIShaderResourceView* GetMorphTargetBufferForReading(
		FSkeletalMeshObject const* InMeshObject,
		int32 InLodIndex,
		int32 InSectionIndex,
		uint32 InFrameNumber,
		bool bInPreviousFrame);

#pragma endregion Deprecated
};
