// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

// Custom serialization version for SkeletalMesh types
struct FSkeletalMeshCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,
		// Remove Chunks array in FStaticLODModel and combine with Sections array
		CombineSectionWithChunk = 1,
		// Remove FRigidSkinVertex and combine with FSoftSkinVertex array
		CombineSoftAndRigidVerts = 2,
		// Need to recalc max bone influences
		RecalcMaxBoneInfluences = 3,
		// Add NumVertices that can be accessed when stripping editor data
		SaveNumVertices = 4,
		// Regenerated clothing section shadow flags from source sections
		RegenerateClothingShadowFlags = 5,
		// Share color buffer structure with StaticMesh
		UseSharedColorBufferFormat = 6,
		// Use separate buffer for skin weights
		UseSeparateSkinWeightBuffer = 7,
		// Added new clothing systems
		NewClothingSystemAdded = 8,
		// Cached inv mass data for clothing assets
		CachedClothInverseMasses = 9,
		// Compact cloth vertex buffer, without dummy entries
		CompactClothVertexBuffer = 10,
		// Remove SourceData
		RemoveSourceData = 11,
		// Split data into Model and RenderData
		SplitModelAndRenderData = 12,
		// Remove triangle sorting support
		RemoveTriangleSorting = 13,
		// Remove the duplicated clothing sections that were a legacy holdover from when we didn't use our own render data
		RemoveDuplicatedClothingSections = 14,
		// Remove 'Disabled' flag from SkelMesh asset sections
		DeprecateSectionDisabledFlag = 15,
		// Add Section ignore by reduce
		SectionIgnoreByReduceAdded = 16,
		// Adding skin weight profile support
		SkinWeightProfiles = 17,
		// Remove uninitialized/deprecated enable cloth LOD flag
		RemoveEnableClothLOD = 18,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	ENGINE_API const static FGuid GUID;

private:
	FSkeletalMeshCustomVersion() {}
};

// Custom serialization version for RecomputeTangent
struct FRecomputeTangentCustomVersion
{
	enum Type
{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
		// We serialize the RecomputeTangent Option
		RuntimeRecomputeTangent = 1,
		// Choose which Vertex Color channel to use as mask to blend tangents
		RecomputeTangentVertexColorMask = 2,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
	
	// The GUID for this custom version number
	const static FGuid GUID;
	
private:
	FRecomputeTangentCustomVersion() {}
};

// custom version for overlapping vertcies code
struct FOverlappingVerticesCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
		// Converted to use HierarchicalInstancedStaticMeshComponent
		DetectOVerlappingVertices = 1,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FOverlappingVerticesCustomVersion() {}
};