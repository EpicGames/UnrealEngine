// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalMeshMerge.h: Merging of unreal skeletal mesh objects.
=============================================================================*/

/*-----------------------------------------------------------------------------
FSkeletalMeshMerge
-----------------------------------------------------------------------------*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "ReferenceSkeleton.h"
#include "Components.h"

#include "SkeletalMeshMerge.generated.h"

class UMaterialInterface;
class USkeletalMesh;
class USkeletalMeshSocket;
class USkeleton;
class FSkeletalMeshLODRenderData;
struct FSkelMeshRenderSection;

struct FRefPoseOverride
{
 public:

	enum EBoneOverrideMode
	{
		BoneOnly,			// Override the bone only.
		ChildrenOnly,		// Override the bone's children only.
		BoneAndChildren,	// Override both the bone & children.
		MAX,
	};

	/**
	 * Constructs a FRefPoseOverride.
	 */
	FRefPoseOverride(const USkeletalMesh* ReferenceMesh)
	{
		this->SkeletalMesh = ReferenceMesh;
	}

	/**
	 * Adds a bone to the list of poses to override.
	 */
	void AddOverride(FName BoneName, EBoneOverrideMode OverrideMode = BoneOnly)
	{
		FBoneOverrideInfo OverrideInfo;
		OverrideInfo.BoneName = BoneName;
		OverrideInfo.OverrideMode = OverrideMode;

		Overrides.Add(OverrideInfo);
	}

 private:

	 struct FBoneOverrideInfo
	 {
		 /** The names of the bone to override. */
		 FName BoneName;

		 /** Whether the override applies to the bone, bone's children, or both. */
		 EBoneOverrideMode OverrideMode;
	 };

	/** The skeletal mesh that contains the reference pose. */
	const USkeletalMesh* SkeletalMesh;

	/** The list of bone overrides. */
	TArray<FBoneOverrideInfo> Overrides;

	friend class FSkeletalMeshMerge;
};

/** 
* Info to map all the sections from a single source skeletal mesh to 
* a final section entry in the merged skeletal mesh
*/
USTRUCT(BlueprintType)
struct FSkelMeshMergeSectionMapping
{
	GENERATED_USTRUCT_BODY()
	
	/** indices to final section entries of the merged skel mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mesh Merge Parameters")
	TArray<int32> SectionIDs;
};

USTRUCT(BlueprintType)
struct FSkelMeshMergeMeshUVTransforms
{
	GENERATED_USTRUCT_BODY()

	/** A list of how UVs should be transformed on a given mesh, where index represents a specific UV channel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mesh Merge Parameters")
	TArray<FTransform> UVTransforms;
};

/** 
* Info to map all the sections about how to transform their UVs
*/
struct UE_DEPRECATED(5.0, "FSkelMeshMergeUVTransforms has been deprecated, use FSkelMeshMergeMeshUVTransforms instead") FSkelMeshMergeUVTransforms
{
	/** For each UV channel on each mesh, how the UVS should be transformed. */
	TArray<TArray<FTransform>> UVTransformsPerMesh;
};

/** 
* Info to map all the sections about how to transform their UVs
*/
USTRUCT(BlueprintType)
struct FSkelMeshMergeUVTransformMapping
{
	GENERATED_USTRUCT_BODY()
	
	/** UV coordinates transform datam one entry for each Skeletal Mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mesh Merge Parameters")
	TArray<FSkelMeshMergeMeshUVTransforms> UVTransformsPerMesh;
};

/** 
* Utility for merging a list of skeletal meshes into a single mesh.
*/
class FSkeletalMeshMerge
{
public:
	/**
	* Constructor
	* @param InMergeMesh - destination mesh to merge to
	* @param InSrcMeshList - array of source meshes to merge
	* @param InForceSectionMapping - optional array to map sections from the source meshes to merged section entries
	* @param StripTopLODs - number of high LODs to remove from input meshes
    * @param bMeshNeedsCPUAccess - (optional) if the resulting mesh needs to be accessed by the CPU for any reason (e.g. for spawning particle effects).
	* @param UVTransforms - optional array to transform the UVs in each mesh
	*/
	ENGINE_API FSkeletalMeshMerge( 
		USkeletalMesh* InMergeMesh, 
		const TArray<USkeletalMesh*>& InSrcMeshList, 
		const TArray<FSkelMeshMergeSectionMapping>& InForceSectionMapping,
		int32 StripTopLODs,
        EMeshBufferAccess MeshBufferAccess=EMeshBufferAccess::Default,
		const FSkelMeshMergeUVTransformMapping* InSectionUVTransforms = nullptr
		);

	UE_DEPRECATED(5.0, "FSkelMeshMergeUVTransforms has been replaced with FSkelMeshMergeMeshUVTransforms, use different signature")
	ENGINE_API FSkeletalMeshMerge( 
    		USkeletalMesh* InMergeMesh, 
    		const TArray<USkeletalMesh*>& InSrcMeshList, 
    		const TArray<FSkelMeshMergeSectionMapping>& InForceSectionMapping,
    		int32 StripTopLODs,
            EMeshBufferAccess MeshBufferAccess,
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
    		FSkelMeshMergeUVTransforms* InSectionUVTransforms
    		PRAGMA_ENABLE_DEPRECATION_WARNINGS
    		);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSkeletalMeshMerge(const FSkeletalMeshMerge&) = default;
	ENGINE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Merge/Composite skeleton and meshes together from the list of source meshes.
	 * @param RefPoseOverrides - An optional override for the merged skeleton's reference pose.
	 * @return true if succeeded
	 */
	bool DoMerge(TArray<FRefPoseOverride>* RefPoseOverrides = nullptr);

	/**
	 * Create the 'MergedMesh' reference skeleton from the skeletons in the 'SrcMeshList'.
	 * Use when the reference skeleton is needed prior to finalizing the merged meshes (do not use with DoMerge()).
	 * @param RefPoseOverrides - An optional override for the merged skeleton's reference pose.
	 */
	ENGINE_API void MergeSkeleton(const TArray<FRefPoseOverride>* RefPoseOverrides = nullptr);

	/**
	 * Creates the merged mesh from the 'SrcMeshList' (note, this should only be called after MergeSkeleton()).
 	 * Use when the reference skeleton is needed prior to finalizing the merged meshes (do not use with DoMerge()).
	 * @return 'true' if successful; 'false' otherwise.
	 */
	ENGINE_API bool FinalizeMesh();

private:
	/** Destination merged mesh */
	USkeletalMesh* MergeMesh;
	
	/** Array of source skeletal meshes  */
	TArray<USkeletalMesh*> SrcMeshList;

	/** Number of high LODs to remove from input meshes. */
	int32 StripTopLODs;

    /** Whether or not the resulting mesh needs to be accessed by the CPU (e.g. for particle spawning).*/
    EMeshBufferAccess MeshBufferAccess;

	/** Info about source mesh used in merge. */
	struct FMergeMeshInfo
	{
		/** Mapping from RefSkeleton bone index in source mesh to output bone index. */
		TArray<int32> SrcToDestRefSkeletonMap;
	};

	/** Array of source mesh info structs. */
	TArray<FMergeMeshInfo> SrcMeshInfo;

	/** New reference skeleton, made from creating union of each part's skeleton. */
	FReferenceSkeleton NewRefSkeleton;

	/** array to map sections from the source meshes to merged section entries */
	const TArray<FSkelMeshMergeSectionMapping>& ForceSectionMapping;

	/** optional array to transform UVs in each source mesh */
	const FSkelMeshMergeUVTransformMapping* SectionUVTransforms;

	UE_DEPRECATED(5.0, "Used to facilitate backwards compatibility with old constructor")
	FSkelMeshMergeUVTransformMapping DummySectionUVTransforms;

	/** Matches the Materials array in the final mesh - used for creating the right number of Material slots. */
	TArray<int32>	MaterialIds;

	/** keeps track of an existing section that need to be merged with another */
	struct FMergeSectionInfo
	{
		/** ptr to source skeletal mesh for this section */
		const USkeletalMesh* SkelMesh;
		/** ptr to source section for merging */
		const FSkelMeshRenderSection* Section;
		/** mapping from the original BoneMap for this sections chunk to the new MergedBoneMap */
		TArray<FBoneIndexType> BoneMapToMergedBoneMap;
		/** transform from the original UVs */
		TArray<FTransform> UVTransforms;

		FMergeSectionInfo( const USkeletalMesh* InSkelMesh,const FSkelMeshRenderSection* InSection, TArray<FTransform> & InUVTransforms )
			:	SkelMesh(InSkelMesh)
			,	Section(InSection)
			,	UVTransforms(InUVTransforms)
		{}
	};

	/** info needed to create a new merged section */
	struct FNewSectionInfo
	{
		/** array of existing sections to merge */
		TArray<FMergeSectionInfo> MergeSections;
		/** merged bonemap */
		TArray<FBoneIndexType> MergedBoneMap;
		/** material for use by this section */
		UMaterialInterface* Material;

		/** 
		* if -1 then we use the Material* to match new section entries
		* otherwise the MaterialId is used to find new section entries
		*/
		int32 MaterialId;

		/** material slot name, if multiple section use the same material we use the first slot name found */
		FName SlotName;

		/** Default UVChannelData for new sections. Will be recomputed if necessary */
		FMeshUVChannelInfo UVChannelData;

		FNewSectionInfo( UMaterialInterface* InMaterial, int32 InMaterialId, FName InSlotName, const FMeshUVChannelInfo& InUVChannelData )
			:	Material(InMaterial)
			,	MaterialId(InMaterialId)
			,	SlotName(InSlotName)
			,	UVChannelData(InUVChannelData)
		{}
	};

	/**
	* Merge a bonemap with an existing bonemap and keep track of remapping
	* (a bonemap is a list of indices of bones in the USkeletalMesh::RefSkeleton array)
	* @param MergedBoneMap - out merged bonemap
	* @param BoneMapToMergedBoneMap - out of mapping from original bonemap to new merged bonemap 
	* @param BoneMap - input bonemap to merge
	*/
	ENGINE_API void MergeBoneMap( TArray<FBoneIndexType>& MergedBoneMap, TArray<FBoneIndexType>& BoneMapToMergedBoneMap, const TArray<FBoneIndexType>& BoneMap );

	/**
	* Creates a new LOD model and adds the new merged sections to it. Modifies the MergedMesh.
	* @param LODIdx - current LOD to process
	*/
	template<typename VertexDataType>
	void GenerateLODModel( int32 LODIdx);

	/**
	* Generate the list of sections that need to be created along with info needed to merge sections
	* @param NewSectionArray - out array to populate
	* @param LODIdx - current LOD to process
	*/
	ENGINE_API void GenerateNewSectionArray( TArray<FNewSectionInfo>& NewSectionArray, int32 LODIdx );

	/**
	* (Re)initialize and merge skeletal mesh info from the list of source meshes to the merge mesh
	* @return true if succeeded
	*/
	ENGINE_API bool ProcessMergeMesh();

	/**
	 * Returns the number of LODs that can be supported by the meshes in 'SourceMeshList'.
	 */
	ENGINE_API int32 CalculateLodCount(const TArray<USkeletalMesh*>& SourceMeshList) const;

	/**
	 * Builds a new 'RefSkeleton' from the reference skeletons in the 'SourceMeshList'.
	 */
	static ENGINE_API void BuildReferenceSkeleton(const TArray<USkeletalMesh*>& SourceMeshList, FReferenceSkeleton& RefSkeleton, const USkeleton* SkeletonAsset);

	/**
	 * Overrides the 'TargetSkeleton' bone poses with the bone poses specified in the 'PoseOverrides' array.
	 */
	static ENGINE_API void OverrideReferenceSkeletonPose(const TArray<FRefPoseOverride>& PoseOverrides, FReferenceSkeleton& TargetSkeleton, const USkeleton* SkeletonAsset);

	/**
	 * Override the 'TargetSkeleton' bone pose with the pose from from the 'SourceSkeleton'.
	 * @return 'true' if the override was successful; 'false' otherwise.
	 */
	static ENGINE_API bool OverrideReferenceBonePose(int32 SourceBoneIndex, const FReferenceSkeleton& SourceSkeleton, FReferenceSkeletonModifier& TargetSkeleton);

	/**
	 * Releases any resources the 'MergeMesh' is currently holding.
	 */
	ENGINE_API void ReleaseResources(int32 Slack = 0);

	/**
	 * Copies and adds the 'NewSocket' to the MergeMesh's MeshOnlySocketList only if the socket does not already exist.
	 * @return 'true' if the socket is added; 'false' otherwise.
	 */
	ENGINE_API bool AddSocket(const USkeletalMeshSocket* NewSocket, bool bIsSkeletonSocket);

	/**
	 * Adds only the new sockets from the 'NewSockets' array to the 'ExistingSocketList'.
	 */
	ENGINE_API void AddSockets(const TArray<USkeletalMeshSocket*>& NewSockets, bool bAreSkeletonSockets);

	/**
	 * Builds a new 'SocketList' from the sockets in the 'SourceMeshList'.
	 */
	ENGINE_API void BuildSockets(const TArray<USkeletalMesh*>& SourceMeshList);

	//void OverrideSockets(const TArray<FRefPoseOverride>& PoseOverrides);

	/**
	 * Override the corresponding 'MergeMesh' socket with 'SourceSocket'.
	 */
	ENGINE_API void OverrideSocket(const USkeletalMeshSocket* SourceSocket);

	/**
	 * Overrides the sockets attached to 'BoneName' with the corresponding socket in the 'SourceSocketList'.
	 */
	ENGINE_API void OverrideBoneSockets(const FName& BoneName, const TArray<USkeletalMeshSocket*>& SourceSocketList);

	/**
	 * Overrides the sockets of overridden bones.
	 */
	ENGINE_API void OverrideMergedSockets(const TArray<FRefPoseOverride>& PoseOverrides);

	/*
	 * Copy Vertex Buffer from Source LOD Model
	 */
	template<typename VertexDataType>
	void CopyVertexFromSource(VertexDataType& DestVert, const FSkeletalMeshLODRenderData& SrcLODData, int32 SourceVertIdx, const FMergeSectionInfo& MergeSectionInfo);
};
