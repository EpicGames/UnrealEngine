// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalMeshReductionSettings.h: Skeletal Mesh Reduction Settings
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "SkeletalMeshReductionSettings.generated.h"

class FSkeletalMeshLODModel;

/** Enum specifying the reduction type to use when simplifying skeletal meshes with internal tool */
UENUM()
enum SkeletalMeshTerminationCriterion : int
{
	SMTC_NumOfTriangles UMETA(DisplayName = "Triangles", ToolTip = "Triangle count criterion will be used for simplification."),
	SMTC_NumOfVerts UMETA(DisplayName = "Vertices", ToolTip = "Vertex cont criterion will be used for simplification."),
	SMTC_TriangleOrVert UMETA(DisplayName = "First Percent Satisfied", ToolTip = "Simplification will continue until either Triangle or Vertex count criteria is met."),
	SMTC_AbsNumOfTriangles UMETA(DisplayName = "Max Triangles", ToolTip = "Triangle count criterion will be used for simplification."),
	SMTC_AbsNumOfVerts UMETA(DisplayName = "Max Vertices", ToolTip = "Vertex cont criterion will be used for simplification."),
	SMTC_AbsTriangleOrVert UMETA(DisplayName = "First Max Satisfied", ToolTip = "Simplification will continue until either Triangle or Vertex count criteria is met."),
	SMTC_MAX UMETA(Hidden),
};

/** Enum specifying the reduction type to use when simplifying skeletal meshes with Simmplygon */
UENUM()
enum SkeletalMeshOptimizationType : int
{
	SMOT_NumOfTriangles UMETA(DisplayName = "Triangles", ToolTip = "Triangle requirement will be used for simplification."),
	SMOT_MaxDeviation UMETA(DisplayName = "Accuracy", ToolTip = "Accuracy requirement will be used for simplification."),
	SMOT_TriangleOrDeviation UMETA(DisplayName = "Any", ToolTip = "Simplification will continue until either Triangle or Accuracy requirement is met."),
	SMOT_MAX UMETA(Hidden),
};

/** Enum specifying the importance of properties when simplifying skeletal meshes. */
UENUM()
enum SkeletalMeshOptimizationImportance : int
{
	SMOI_Off UMETA(DisplayName = "Off"),
	SMOI_Lowest UMETA(DisplayName = "Lowest"),
	SMOI_Low UMETA(DisplayName = "Low"),
	SMOI_Normal UMETA(DisplayName = "Normal"),
	SMOI_High UMETA(DisplayName = "High"),
	SMOI_Highest UMETA(DisplayName = "Highest"),
	SMOI_MAX UMETA(Hidden)
};

DECLARE_DELEGATE_OneParam(FOnDeleteLODModelOverride, FSkeletalMeshLODModel*);

/**
* FSkeletalMeshOptimizationSettings - The settings used to optimize a skeletal mesh LOD.
*/
USTRUCT()
struct FSkeletalMeshOptimizationSettings
{
	GENERATED_USTRUCT_BODY()

	/** The method to use when optimizing the skeletal mesh LOD */
	UPROPERTY(EditAnywhere, Category = ReductionMethod)
	TEnumAsByte<enum SkeletalMeshTerminationCriterion> TerminationCriterion;

	/** The percentage of triangles to retain as a ratio, e.g. 0.1 indicates 10 percent */
	UPROPERTY(EditAnywhere, Category = ReductionMethod, meta = (DisplayName = "Percent of Triangles"))
	float NumOfTrianglesPercentage;

	/** The percentage of vertices to retain as a ratio, e.g. 0.1 indicates 10 percent */
	UPROPERTY(EditAnywhere, Category = ReductionMethod, meta = (DisplayName = "Percent of Vertices"))
	float NumOfVertPercentage;
	
	/** The maximum number of triangles to retain */
	UPROPERTY(EditAnywhere, Category = ReductionMethod, meta = (DisplayName = "Max Triangle Count", ClampMin = 4))
	uint32 MaxNumOfTriangles;

	/** The maximum number of vertices to retain */
	UPROPERTY(EditAnywhere, Category = ReductionMethod, meta = (DisplayName = "Max Vertex Count", ClampMin = 6))
	uint32 MaxNumOfVerts;

#if WITH_EDITORONLY_DATA
	/** The maximum number of triangles to retain when using percentage termination criterion. */
	UPROPERTY(EditAnywhere, Category = ReductionMethod, meta = (DisplayName = "Max Triangle Count", ClampMin = 4, UIMin = "4"))
	uint32 MaxNumOfTrianglesPercentage;

	/** The maximum number of vertices to retain when using percentage termination criterion. */
	UPROPERTY(EditAnywhere, Category = ReductionMethod, meta = (DisplayName = "Max Vertex Count", ClampMin = 6, UIMin = "6"))
	uint32 MaxNumOfVertsPercentage;
#endif

	/**If ReductionMethod equals MaxDeviation this value is the maximum deviation from the base mesh as a percentage of the bounding sphere. 
	 * In code, it ranges from [0, 1]. In the editor UI, it ranges from [0, 100]
	 */
	UPROPERTY(EditAnywhere, Category = ReductionMethod)
	float MaxDeviationPercentage;

	/** The method to use when optimizing the skeletal mesh LOD */
	UPROPERTY(EditAnywhere, Category = ReductionMethod)
	TEnumAsByte<enum SkeletalMeshOptimizationType> ReductionMethod;

	/** How important the shape of the geometry is. */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Silhouette"))
	TEnumAsByte<enum SkeletalMeshOptimizationImportance> SilhouetteImportance;

	/** How important texture density is. */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Texture"))
	TEnumAsByte<enum SkeletalMeshOptimizationImportance> TextureImportance;

	/** How important shading quality is. */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Shading"))
	TEnumAsByte<enum SkeletalMeshOptimizationImportance> ShadingImportance;

	/** How important skinning quality is. */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Skinning"))
	TEnumAsByte<enum SkeletalMeshOptimizationImportance> SkinningImportance;

	/* Remap the morph targets from the base LOD onto the reduce LOD. */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings)
	uint8 bRemapMorphTargets:1;

	/** Whether Normal smoothing groups should be preserved. If true then Hard Edge Angle (NormalsThreshold) is used **/
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Recompute Normal"))
	uint8 bRecalcNormals:1;

	/** The welding threshold distance. Vertices under this distance will be welded. */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings)
	float WeldingThreshold;

	/** If the angle between two triangles are above this value, the normals will not be
	smooth over the edge between those two triangles. Set in degrees. This is only used when bRecalcNormals is set to true*/
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Hard Edge Angle", EditCondition = "bRecalcNormals"))
	float NormalsThreshold;

	/** Maximum number of bones that can be assigned to each vertex. */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Max Bones Influence", ClampMin = 1))
	int32 MaxBonesPerVertex;

	/** Penalize edge collapse between vertices that have different major bones.  This will help articulated segments like tongues but can lead to undesirable results under extreme simplification */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Enforce Bone Boundaries"))
	uint8 bEnforceBoneBoundaries : 1;

	/** If enabled this option make sure vertices that share the same location (e.g. UV boundaries) have the same bone weights. This can fix cracks when the characters animate. */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Merge Coincident Vertices Bones"))
	uint8 bMergeCoincidentVertBones : 1;

	/** Default value of 1 attempts to preserve volume.  Smaller values will loose volume by flattening curved surfaces, and larger values will accentuate curved surfaces.  */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Volumetric Correction", ClampMin = 0, ClampMax = 2))
	float VolumeImportance;

	/** Preserve cuts in the mesh surface by locking vertices in place.  Increases the quality of the simplified mesh at edges at the cost of more triangles*/
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Lock Mesh Edges"))
	uint8 bLockEdges : 1;

	/** Disallow edge collapse when the vertices do not have a common color*/
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings, meta = (DisplayName = "Lock Vertex Color Boundaries"))
	uint8 bLockColorBounaries : 1;

	/** Better distribution of triangles on 2d meshes, such as flat cloth, but at the cost of potentially worse UVs in those areas.  This generally has little or no effect for mesh regions that aren't laid out on a plane intersecting the origin such as the xy-plane. When this is disabled, the planar regions may simplify to fewer large triangles.*/
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings)
	uint8 bImproveTrianglesForCloth : 1;

	/** Base LOD index to generate this LOD. By default, we generate from LOD 0 */
	UPROPERTY(EditAnywhere, Category = FSkeletalMeshOptimizationSettings)
	int32 BaseLOD;


#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FBoneReference> BonesToRemove_DEPRECATED;

	UPROPERTY()
	TObjectPtr<class UAnimSequence> BakePose_DEPRECATED;

	//Transient mutable delegate. If the delegate is bound, the reduction will call it instead of deleting the replaced LODModel. It will be then the delegate owner responsible of the LODModel memory
	mutable FOnDeleteLODModelOverride OnDeleteLODModelDelegate;
#endif

	FSkeletalMeshOptimizationSettings()
		: TerminationCriterion(SMTC_NumOfTriangles)
		, NumOfTrianglesPercentage(0.5f)
		, NumOfVertPercentage(0.5f)
		, MaxNumOfTriangles(4)
		, MaxNumOfVerts(6)
#if WITH_EDITORONLY_DATA		
		, MaxNumOfTrianglesPercentage(MAX_uint32)
		, MaxNumOfVertsPercentage(MAX_uint32)
#endif
		, MaxDeviationPercentage(0.5f)
		, ReductionMethod(SMOT_NumOfTriangles)
		, SilhouetteImportance(SMOI_Normal)
		, TextureImportance(SMOI_Normal)
		, ShadingImportance(SMOI_Normal)
		, SkinningImportance(SMOI_Normal)
		, bRemapMorphTargets(false)
		, bRecalcNormals(true)
		, WeldingThreshold(0.1f)
		, NormalsThreshold(60.0f)
		, MaxBonesPerVertex(4)
		, bEnforceBoneBoundaries(false)
		, bMergeCoincidentVertBones(true)
		, VolumeImportance(1.f)
		, bLockEdges(false)
		, bLockColorBounaries(false)
		, bImproveTrianglesForCloth(true)
		, BaseLOD(0)
#if WITH_EDITORONLY_DATA
		, BakePose_DEPRECATED(nullptr)
#endif
	{
	}
};

