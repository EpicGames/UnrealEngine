// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PrimitiveSceneProxy.h"
#include "SkeletalMeshTypes.h"
#include "SkinningSceneExtensionProxy.h"

/*-----------------------------------------------------------------------------
FSkeletalMeshSceneProxy
-----------------------------------------------------------------------------*/

class FSkeletalMeshObject;

struct FSkinnedMeshSceneProxyDesc;
struct FSkelMeshRenderSection;

/**
 * A skeletal mesh component scene proxy.
 */
class FSkeletalMeshSceneProxy
	: public FPrimitiveSceneProxy
{
public:
	ENGINE_API SIZE_T GetTypeHash() const override;

	/** 
	 * Constructor. 
	 * @param	Component - skeletal mesh primitive being added
	 */
	ENGINE_API FSkeletalMeshSceneProxy(const USkinnedMeshComponent* Component, FSkeletalMeshRenderData* InSkelMeshRenderData);
	ENGINE_API FSkeletalMeshSceneProxy(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InSkelMeshRenderData);

	UE_DEPRECATED(5.8, "MeshDesc now has MinLODLevel and InClampedLODIndex is unused")
	ENGINE_API FSkeletalMeshSceneProxy(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InSkelMeshRenderData, int32 InClampedLODIndex);

	ENGINE_API virtual ~FSkeletalMeshSceneProxy();

#if WITH_EDITOR
	ENGINE_API virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
#endif
	ENGINE_API virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override;
	ENGINE_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	ENGINE_API virtual const FCardRepresentationData* GetMeshCardRepresentation() const override;

	ENGINE_API virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void DestroyRenderThreadResources() override;

#if RHI_RAYTRACING
	ENGINE_API virtual bool HasRayTracingRepresentation() const override;

	virtual bool IsRayTracingRelevant() const override { return true; }

	virtual bool IsRayTracingStaticRelevant() const override
	{
		return bRayTraceStatic;
	}

	ENGINE_API virtual RayTracing::FGeometryGroupHandle GetRayTracingGeometryGroupHandle() const override;

	ENGINE_API virtual TArray<FRayTracingGeometry*> GetStaticRayTracingGeometries() const override;
	ENGINE_API virtual void GetStaticRayTracingInstances(FStaticRayTracingInstances& OutRayTracingInstances) override;

	ENGINE_API virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override;
#endif // RHI_RAYTRACING

	ENGINE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	ENGINE_API virtual bool CanBeOccluded() const override;
	ENGINE_API virtual bool IsUsingDistanceCullFade() const override;
	
	ENGINE_API virtual bool HasDynamicIndirectShadowCasterRepresentation() const override;
	ENGINE_API virtual void GetShadowShapes(FVector PreViewTranslation, TArray<FCapsuleShape3f>& OutCapsuleShapes) const override;

	virtual bool CanApplyStreamableRenderAssetScaleFactor() const override { return true; }

	ENGINE_API virtual void GetStreamableRenderAssetInfo(const FBoxSphereBounds& InPrimitiveBounds, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamableRenderAssets) const override;

	/** Return the bounds for the pre-skinned primitive in local space */
	virtual void GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const override { OutBounds = PreSkinnedLocalBounds; }

	/** Returns a pre-sorted list of shadow capsules's bone indicies */
	const TArray<uint16>& GetSortedShadowBoneIndices() const
	{
		return ShadowCapsuleBoneIndices;
	}

	ENGINE_API FSkinningSceneExtensionProxy* GetSkinningSceneExtensionProxy() const override;

	FSkeletalMeshObject* GetMeshObject() const
	{
		return MeshObject;
	}

	USkinnedAsset* GetSkinnedAsset() const
	{
		return SkinnedAsset;
	}

	/**
	 * Returns the world transform to use for drawing.
	 * @param OutLocalToWorld - Will contain the local-to-world transform when the function returns.
	 * @param OutWorldToLocal - Will contain the world-to-local transform when the function returns.
	 * 
	 * @return true if out matrices are valid 
	 */
	ENGINE_API bool GetWorldMatrices( FMatrix& OutLocalToWorld, FMatrix& OutWorldToLocal ) const;

	/** Util for getting LOD index currently used by this SceneProxy. */
	ENGINE_API int32 GetCurrentLODIndex();

	/** 
	 * Render physics asset for debug display
	 */
	ENGINE_API virtual void DebugDrawPhysicsAsset(int32 ViewIndex, FMeshElementCollector& Collector, const FEngineShowFlags& EngineShowFlags) const;

	/** Render the bones of the skeleton for debug display */ 
	ENGINE_API void DebugDrawSkeleton(int32 ViewIndex, FMeshElementCollector& Collector, const FEngineShowFlags& EngineShowFlags) const;

#if WITH_EDITOR
	ENGINE_API void DebugDrawPoseWatchSkeletons(int32 ViewIndex, FMeshElementCollector& Collector, const FEngineShowFlags& EngineShowFlags) const;
#endif

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	SIZE_T GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() + LODSections.GetAllocatedSize() ); }

	/**
	* Updates morph material usage for materials referenced by each LOD entry
	*
	* @param bNeedsMorphUsage - true if the materials used by this skeletal mesh need morph target usage
	*/
	ENGINE_API void UpdateMorphMaterialUsage_GameThread(TArray<UMaterialInterface*>& MaterialUsingMorphTarget);


#if WITH_EDITORONLY_DATA
	ENGINE_API virtual bool GetPrimitiveDistance(int32 LODIndex, int32 SectionIndex, const FVector& ViewOrigin, float& PrimitiveDistance) const override;
	ENGINE_API virtual bool GetMeshUVDensities(int32 LODIndex, int32 SectionIndex, FVector4& WorldUVDensities) const override;
	ENGINE_API virtual bool GetMaterialTextureScales(int32 LODIndex, int32 SectionIndex, const FMaterialRenderProxy* MaterialRenderProxy, FVector4f* OneOverScales, FIntVector4* UVChannelIndices) const override;
#endif

	friend class FSkeletalMeshSectionIter;

	ENGINE_API virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override;

	ENGINE_API virtual bool GetInstanceWorldPositionOffsetDisableDistance(float& OutWPODisableDistance) const override;
	ENGINE_API virtual void SetWorldPositionOffsetDisableDistance_GameThread(int32 NewValue) override;

	virtual uint8 GetCurrentFirstLODIdx_RenderThread() const override
	{
		return GetCurrentFirstLODIdx_Internal();
	}

#if !UE_BUILD_SHIPPING
	ENGINE_API virtual uint8 GetStreamingLODWantedDeficit() const override;
#endif

	ENGINE_API virtual FDesiredLODLevel GetDesiredLODLevel_RenderThread(const FSceneView* View) const override;

	ENGINE_API bool GetCachedGeometry(FRDGBuilder& GraphBuilder, struct FCachedGeometry& OutCachedGeometry) const;

	UE_DEPRECATED(5.6, "GetCachedGeometry now requires a GraphBuilder.")
	bool GetCachedGeometry(struct FCachedGeometry& OutCachedGeometry) const { return false; }

protected:
	AActor* Owner;
	FSkeletalMeshObject* MeshObject;
	USkinnedAsset* SkinnedAsset;
	TUniquePtr<FSkinningSceneExtensionProxy> SceneExtensionProxy;
	FSkeletalMeshRenderData* RenderData;
	int32 MinLODLevel;
	int32 ForceStreamedLOD;
	uint8 NumLODs = 1u;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	class UPhysicsAsset* PhysicsAssetForDebug;
#endif

	UMaterialInterface* OverlayMaterial;
	float OverlayMaterialMaxDrawDistance;
	float WPODisableDistance = 0.f;
	TArray<TObjectPtr<UMaterialInterface>> MaterialSlotsOverlayMaterial;

public:
#if RHI_RAYTRACING
	bool bAnySegmentUsesWorldPositionOffset : 1;
#endif

protected:
	/** data copied for rendering */
	uint8 bForceWireframe : 1;
	uint8 bIsCPUSkinned : 1;
	uint8 bCanHighlightSelectedSections : 1;
	uint8 bRenderStatic:1;
	uint8 bMobileGPUScene:1;
#if RHI_RAYTRACING
	uint8 bHasRayTracingRepresentation : 1;
	uint8 bRayTraceStatic : 1;
#endif
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	uint8 bDrawDebugSkeleton:1;
#endif
	uint8 bAllowDynamicMeshBounds:1;
	uint8 bUseGpuLodSelection:1;
#if WITH_EDITOR
	bool bHasSelectedInstances = false;
#else
	static const bool bHasSelectedInstances = false;
#endif
	TEnumAsByte<ERHIFeatureLevel::Type> FeatureLevel;

	bool bMaterialsNeedMorphUsage_GameThread;

	FMaterialRelevance MaterialRelevance;


	/** info for section element in an LOD */
	struct FSectionElementInfo
	{
		FSectionElementInfo(UMaterialInterface* InMaterial, bool bInEnableShadowCasting, int32 InUseMaterialIndex, UMaterialInterface* InPerSectionOverlayMaterial)
		: Material( InMaterial )
		, bEnableShadowCasting( bInEnableShadowCasting )
		, UseMaterialIndex( InUseMaterialIndex )
#if WITH_EDITOR
		, HitProxy(NULL)
#endif
		, PerSectionOverlayMaterial(InPerSectionOverlayMaterial)
		{}
		
		UMaterialInterface* Material;
		
		/** Whether shadow casting is enabled for this section. */
		bool bEnableShadowCasting;
		
		/** Index into the materials array of the skel mesh or the component after LOD mapping */
		int32 UseMaterialIndex;

#if WITH_EDITOR
		/** The editor needs to be able to individual sub-mesh hit detection, so we store a hit proxy on each mesh. */
		HHitProxy* HitProxy;
#endif

		UMaterialInterface* PerSectionOverlayMaterial;
	};

	/** Section elements for a particular LOD */
	struct FLODSectionElements
	{
		TArray<FSectionElementInfo> SectionElements;
	};
	
	/** Array of section elements for each LOD */
	TArray<FLODSectionElements> LODSections;

	/** 
	 * BoneIndex->capsule pairs used for rendering sphere/capsule shadows 
	 * Note that these are in refpose component space, NOT bone space.
	 */
	TArray<TPair<int32, FCapsuleShape>> ShadowCapsuleData;
	TArray<uint16> ShadowCapsuleBoneIndices;

	/** Set of materials used by this scene proxy, safe to access from the game thread. */
	TSet<UMaterialInterface*> MaterialsInUse_GameThread;
	
	/** The primitive's pre-skinned local space bounds. */
	FBoxSphereBounds PreSkinnedLocalBounds;

	/** Lumen cards data */
	class FCardRepresentationData* CardRepresentationData = nullptr;

#if RHI_RAYTRACING
	RayTracing::FGeometryGroupHandle RayTracingGeometryGroupHandle;
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** The color we draw this component in if drawing debug bones */
	TOptional<FLinearColor> DebugDrawColor;
#endif

#if WITH_EDITORONLY_DATA
	/** The component streaming distance multiplier */
	float StreamingDistanceMultiplier;
#endif

	ENGINE_API void GetDynamicElementsSection(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap,
									const FSkeletalMeshLODRenderData& LODData, const int32 LODIndex, const int32 SectionIndex, bool bSectionSelected,
								   const FSectionElementInfo& SectionElementInfo, bool bInSelectable, FMeshElementCollector& Collector) const;

	ENGINE_API void GetMeshElementsConditionallySelectable(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, bool bInSelectable, uint32 VisibilityMap, FMeshElementCollector& Collector) const;

	/** Only call on render thread timeline */
	ENGINE_API uint8 GetCurrentFirstLODIdx_Internal() const;

	ENGINE_API void UpdateLumenCardsFromBounds();

private:
	ENGINE_API void CreateBaseMeshBatch(const FSceneView* View, const FSkeletalMeshLODRenderData& LODData, const int32 LODIndex, const int32 SectionIndex, const FSectionElementInfo& SectionElementInfo, FMeshBatch& Mesh, ESkinVertexFactoryMode VFMode = ESkinVertexFactoryMode::Default) const;
	
	ENGINE_API void GetStaticMeshBatch(
		const FSkeletalMeshLODRenderData& LODData,
		const int32 LODIndex,
		const FSkelMeshRenderSection& Section,
		const int32 SectionIndex,
		const FSectionElementInfo& SectionElementInfo,
		const FVertexFactory* VertexFactory,
		ESceneDepthPriorityGroup PrimitiveDPG,
		FMeshBatch& OutMeshBatch) const;
	ENGINE_API void GetOverlayMeshBatch(const FSkeletalMeshLODRenderData& LODData, UMaterialInterface* SectionOverlayMaterial, const FMeshBatch& MeshBatch, FMeshBatch& OutOverlayMeshBatch) const;

public:
#if WITH_EDITORONLY_DATA
	struct FPoseWatchDynamicData* PoseWatchDynamicData;
#endif
};
