// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialInterface.h"
#include "ComponentReregisterContext.h"
#include "SkeletalMeshLegacyCustomVersions.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "GPUSkinPublicDefs.h"

class ISkinnedMeshComponent;
class FMaterialRenderProxy;
class FMeshElementCollector;
class FPrimitiveDrawInterface;
class FRawStaticIndexBuffer16or32Interface;
class UMorphTarget;
class UPrimitiveComponent;
class USkinnedMeshComponent;
class USkeletalMesh;
class FSkeletalMeshRenderData;
class FSkeletalMeshLODRenderData;
class USkinnedAsset;

/** Flags used when building vertex buffers. */
enum class ESkeletalMeshVertexFlags : uint8
{
	None								= 0x0,
	UseFullPrecisionUVs					= 0x1,
	HasVertexColors						= 0x2,
	UseHighPrecisionTangentBasis		= 0x4,
	UseBackwardsCompatibleF16TruncUVs	= 0x8,
	UseHighPrecisionWeights				= 0x10,
};
ENUM_CLASS_FLAGS(ESkeletalMeshVertexFlags);

/** Name of vertex color channels, used by recompute tangents */
enum class ESkinVertexColorChannel : uint8
{
	// Use red channel as recompute tangents blending mask
	Red = 0,
	// Use green channel as recompute tangents blending mask
	Green = 1,
	// Use blue channel as recompute tangents blending mask
	Blue = 2,
	// Alpha channel not used by recompute tangents
	Alpha = 3,
	None = Alpha
};

enum class ESkinVertexFactoryMode
{
	Default,
	RayTracing
};

struct FSoftSkinVertex
{
	FVector3f			Position;

	// Tangent, U-direction
	FVector3f			TangentX;
	// Binormal, V-direction
	FVector3f			TangentY;
	// Normal
	FVector4f			TangentZ;

	// UVs
	FVector2f		UVs[MAX_TEXCOORDS];
	// VertexColor
	FColor			Color;
	uint16			InfluenceBones[MAX_TOTAL_INFLUENCES];
	uint16			InfluenceWeights[MAX_TOTAL_INFLUENCES];

	/** If this vert is rigidly weighted to a bone, return true and the bone index. Otherwise return false. */
	ENGINE_API bool GetRigidWeightBone(uint16& OutBoneIndex) const;

	/** Returns the maximum weight of any bone that influences this vertex. */
	ENGINE_API uint16 GetMaximumWeight() const;

	/**
	* Serializer
	*
	* @param Ar - archive to serialize with
	* @param V - vertex to serialize
	* @return archive that was used
	*/
	friend FArchive& operator<<(FArchive& Ar, FSoftSkinVertex& V);
};

/**
 * A structure for holding mesh-to-mesh triangle influences to skin one mesh to another (similar to a wrap deformer)
 */
struct FMeshToMeshVertData
{
	// Barycentric coords and distance along normal for the position of the final vert
	FVector4f PositionBaryCoordsAndDist; 

	// Barycentric coords and distance along normal for the location of the unit normal endpoint
	// Actual normal = ResolvedNormalPosition - ResolvedPosition
	FVector4f NormalBaryCoordsAndDist;

	// Barycentric coords and distance along normal for the location of the unit Tangent endpoint
	// Actual normal = ResolvedNormalPosition - ResolvedPosition
	FVector4f TangentBaryCoordsAndDist;

	// Contains the 3 indices for verts in the source mesh forming a triangle, the last element
	// is a flag to decide how the skinning works, 0xffff uses no simulation, and just normal
	// skinning, anything else uses the source mesh and the above skin data to get the final position
	uint16	 SourceMeshVertIndices[4];

	// For weighted averaging of multiple triangle influences
	float	 Weight = 0.0f;

	// Dummy for alignment
	uint32	 Padding = 0;

	friend ENGINE_API FArchive& operator<<(FArchive& Ar, FMeshToMeshVertData& V);
	
	bool operator==(const FMeshToMeshVertData& Other) const
	{
		return PositionBaryCoordsAndDist == Other.PositionBaryCoordsAndDist &&
			NormalBaryCoordsAndDist == Other.NormalBaryCoordsAndDist &&
			TangentBaryCoordsAndDist == Other.TangentBaryCoordsAndDist &&
			SourceMeshVertIndices[0] == Other.SourceMeshVertIndices[0] &&
			SourceMeshVertIndices[1] == Other.SourceMeshVertIndices[1] &&
			SourceMeshVertIndices[2] == Other.SourceMeshVertIndices[2] &&
			SourceMeshVertIndices[3] == Other.SourceMeshVertIndices[3] &&
			Weight == Other.Weight;
	}
};

class FSkeletalMeshUnifiedBoneIndices
{
public:
	FSkeletalMeshUnifiedBoneIndices(TConstArrayView<uint16> InUnifiedToSkeletonIndex)
		: UnifiedToSkeletonIndex(InUnifiedToSkeletonIndex)
	{
		int32 MaxSkeletonIndex = -1;

		for (int32 SkeletonIndex : UnifiedToSkeletonIndex)
		{
			MaxSkeletonIndex = FMath::Max(MaxSkeletonIndex, SkeletonIndex);
		}

		SkeletonToUnifiedBoneIndex.Init(INDEX_NONE, MaxSkeletonIndex + 1);

		for (int32 UnifiedBoneIndex = 0; UnifiedBoneIndex < UnifiedToSkeletonIndex.Num(); ++UnifiedBoneIndex)
		{
			SkeletonToUnifiedBoneIndex[UnifiedToSkeletonIndex[UnifiedBoneIndex]] = (uint16)UnifiedBoneIndex;
		}
	}

	bool IsValid() const
	{
		return !UnifiedToSkeletonIndex.IsEmpty();
	}

	int32 GetNumUnifiedBones() const
	{
		return UnifiedToSkeletonIndex.Num();
	}

	uint16 GetSkeletonToUnifiedIndex(int32 SkeletonIndex) const
	{
		check(IsValid());
		return SkeletonToUnifiedBoneIndex.IsValidIndex(SkeletonIndex) ? SkeletonToUnifiedBoneIndex[SkeletonIndex] : INDEX_NONE;
	}

	int32 GetUnifiedToSkeletonIndex(int32 UnifiedIndex) const
	{
		check(IsValid());
		return UnifiedToSkeletonIndex[UnifiedIndex];
	}

	TConstArrayView<uint16> GetBoneMap() const
	{
		return UnifiedToSkeletonIndex;
	}

	ENGINE_API void RemapVertex(TConstArrayView<uint16> BoneMap, FSoftSkinVertex& Vertex) const;

private:
	TConstArrayView<uint16> UnifiedToSkeletonIndex;
	TArray<uint16> SkeletonToUnifiedBoneIndex;
};

/**
* Structure to store the buffer offsets to the section's cloth deformer mapping data. 
* Also contains the stride to further cloth LODs data for cases where one section has
* multiple mappings so that it can be wrap deformed with cloth data from a different LOD.
* When using LOD bias in Raytracing for example.
*/
struct FClothBufferIndexMapping
{
	/** Section first index. */
	uint32 BaseVertexIndex;

	/** Offset in the buffer to the corresponding cloth mapping. */
	uint32 MappingOffset;

	/** Stride to the next LOD mapping if any. */
	uint32 LODBiasStride;

	friend ENGINE_API FArchive& operator<<(FArchive& Ar, FClothBufferIndexMapping& V);
};

struct FClothingSectionData
{
	FClothingSectionData()
		: AssetGuid()
		, AssetLodIndex(INDEX_NONE)
	{}

	bool IsValid() const
	{
		return AssetGuid.IsValid() && AssetLodIndex != INDEX_NONE;
	}

	/** Guid of the clothing asset applied to this section */
	FGuid AssetGuid;

	/** LOD inside the applied asset that is used */
	int32 AssetLodIndex;

	friend FArchive& operator<<(FArchive& Ar, FClothingSectionData& Data)
	{
		Ar << Data.AssetGuid;
		Ar << Data.AssetLodIndex;

		return Ar;
	}
};


/** Used to recreate all skinned mesh components for a given skinned asset. */
class FSkinnedMeshComponentRecreateRenderStateContext
{
public:

	/** Initialization constructor. */
	ENGINE_API FSkinnedMeshComponentRecreateRenderStateContext(USkinnedAsset* InSkinnedAsset, bool InRefreshBounds = false);

	/** Destructor: recreates render state for all components that had their render states destroyed in the constructor. */
	ENGINE_API ~FSkinnedMeshComponentRecreateRenderStateContext();
	
private:

	/** List of components to reset */
	TArray<TPair<TWeakObjectPtr<>, ISkinnedMeshComponent*>> SkinnedMeshComponents;

	/** Whether we'll refresh the component bounds as we reset */
	bool bRefreshBounds;
};

#if WITH_EDITOR

//Helper to scope skeletal mesh post edit change.
class FScopedSkeletalMeshPostEditChange
{
public:
	/*
	 * This constructor increment the skeletal mesh PostEditChangeStackCounter. If the stack counter is zero before the increment
	 * the skeletal mesh component will be unregister from the world. The component will also release there rendering resources.
	 * Parameters:
	 * @param InbCallPostEditChange - if we are the first scope PostEditChange will be called.
	 * @param InbReregisterComponents - if we are the first scope we will re register component from world and also component render data.
	 */
	ENGINE_API FScopedSkeletalMeshPostEditChange(USkeletalMesh* InSkeletalMesh, bool InbCallPostEditChange = true, bool InbReregisterComponents = true);

	/*
	 * This destructor decrement the skeletal mesh PostEditChangeStackCounter. If the stack counter is zero after the decrement,
	 * the skeletal mesh PostEditChange will be call. The component will also be register to the world and there render data resources will be rebuild.
	 */
	ENGINE_API ~FScopedSkeletalMeshPostEditChange();

	ENGINE_API void SetSkeletalMesh(USkeletalMesh* InSkeletalMesh);

private:
	USkeletalMesh* SkeletalMesh;
	bool bReregisterComponents;
	bool bCallPostEditChange;
	FSkinnedMeshComponentRecreateRenderStateContext* RecreateExistingRenderStateContext;
	TIndirectArray<FComponentReregisterContext> ComponentReregisterContexts;
};

//Helper to scope the component register context.
class FScopedSkeletalMeshReregisterContexts
{
public:
	/*
	 * This constructor will enregister all skeletal mesh component from the world. The component will also release there rendering resources.
	 * Parameters:
	 */
	ENGINE_API FScopedSkeletalMeshReregisterContexts(USkeletalMesh* InSkeletalMesh);

	/*
	 * This destructor will Reregister all unregistered component to the world and there render data resources will be rebuild.
	 */
	ENGINE_API ~FScopedSkeletalMeshReregisterContexts();


private:
	USkeletalMesh* SkeletalMesh;
	FSkinnedMeshComponentRecreateRenderStateContext* RecreateExistingRenderStateContext = nullptr;
	TIndirectArray<FComponentReregisterContext> ComponentReregisterContexts;
};

#endif
