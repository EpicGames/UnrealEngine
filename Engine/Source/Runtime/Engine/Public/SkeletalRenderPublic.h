// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalRenderPublic.h: Definitions and inline code for rendering SkeletalMeshComponent
=============================================================================*/
#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "PackedNormal.h"
#include "RenderDeferredCleanup.h"
#include "RenderUtils.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkinnedMeshComponent.h"
#include "MeshUVChannelInfo.h"
#include "SkeletalMeshTypes.h"
#include "RenderMath.h"
#include "Async/Mutex.h"
#include "Matrix3x4.h"
#include "GPUSkinVertexFactory.h"

class FPrimitiveDrawInterface;
class FVertexFactory;
class UMorphTarget;
struct FSkelMeshRenderSection;
struct FCachedGeometry;
struct FSkinningHeader;
struct FRWBuffer;
class FGPUSkinCacheEntry;
class FMeshDeformerGeometry;
class FRayTracingGeometry;
class FSkinningSceneExtensionProxy;
class FRHICommandList;
class FRDGBuilder;
namespace Nanite
{
	struct FMaterialAudit;
}

struct FSkinWeightVertexBufferInfo
{
	FShaderResourceViewRHIRef InputWeightStreamSRV;
	FShaderResourceViewRHIRef InputWeightLookupStreamSRV;
	uint32 InputWeightIndexSize = 0;
	uint32 InputWeightStride = 0;
	uint32 MaxBoneInfluences = 0;
	GPUSkinBoneInfluenceType BoneInfluenceType = DefaultBoneInfluence;
	uint8 bUse16BitBoneIndex  : 1 = 0;
	uint8 bUse16BitBoneWeight : 1 = 0;
};

namespace UE::SkeletalRender::Settings
{
	// Returns the maximum value allowed for morph targets blend weights, configured at RenderSettings
	ENGINE_API float GetMorphTargetMaxBlendWeight();
}

/** data for a single skinned skeletal mesh vertex */
struct FFinalSkinVertex
{
	FVector3f			Position;
	FPackedNormal	TangentX;
	FPackedNormal	TangentZ;
	float			U;
	float			V;
	FVector2D TextureCoordinates[MAX_TEXCOORDS];

	FVector3f GetTangentY() const
	{
		return FVector3f(GenerateYAxis(TangentX, TangentZ));
	};
};

enum class EPreviousBoneTransformUpdateMode
{
	None,

	UpdatePrevious,

	DuplicateCurrentToPrevious,
};

enum class EBoneTransformAllocationMode : uint8
{
	// Transforms are allocated and sourced from the scene extension buffer.
	Scene,

	// Transforms are allocated and sourced from individual section vertex buffers in the vertex factory.
	VertexFactory
};

// Returns the globally configured bone transform allocation mode.
EBoneTransformAllocationMode GetBoneTransformAllocationMode();

enum class EBoneTransformStorageMode : uint8
{
	// Transforms are stored directly without modification on the GPU. This may require a section bone map buffer indirection on the GPU.
	Direct,

	// Transforms are stored with a built-in bone map indirection for the specific LOD.
	BoneMap
};

// If true, the skinning scene extension supports GPU skin skeletal mesh objects.
ENGINE_API extern bool IsGPUSkinSceneExtensionEnabled();

struct FSkinBatchVertexFactoryUserData
{
	FGPUSkinCacheEntry* SkinCacheEntry = nullptr;
	FMeshDeformerGeometry* DeformerGeometry = nullptr;
	int32 SectionIndex = -1;
};

/**
* Interface for mesh rendering data
*/

struct FSkinnedMeshSceneProxyDesc;
struct FSkinnedMeshSceneProxyDynamicData;

class FSkeletalMeshObject : public FDeferredCleanupInterface
{
public:
	ENGINE_API FSkeletalMeshObject(const USkinnedMeshComponent* InMeshComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type FeatureLevel);
	ENGINE_API FSkeletalMeshObject(const FSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InSkelMeshRenderData, ERHIFeatureLevel::Type FeatureLevel);

	ENGINE_API virtual ~FSkeletalMeshObject();

	/** 
	 * Initialize rendering resources for each LOD 
	 */
	virtual void InitResources(const FSkinnedMeshSceneProxyDesc& InMeshDesc) = 0;

	/** 
	 * Release rendering resources for each LOD 
	 */
	virtual void ReleaseResources() = 0;

	/**
	 * Called by the game thread for any dynamic data updates for this skel mesh object
	 * @param	LODIndex - lod level to update
	 * @param	InSkeletalMeshComponen - parent prim component doing the updating
	 * @param	ActiveMorphs - morph targets to blend with during skinning
	 */

	ENGINE_API void Update(int32 LODIndex, const USkinnedMeshComponent* InMeshComponent, const FMorphTargetWeightMap& InActiveMorphTargets, const TArray<float>& MorphTargetWeights, EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode, const FExternalMorphWeightData& InExternalMorphWeightData);
	virtual void Update(int32 LODIndex, const FSkinnedMeshSceneProxyDynamicData& InDynamicData, const FPrimitiveSceneProxy* InSceneProxy, const USkinnedAsset* InSkinnedAsset, const FMorphTargetWeightMap& InActiveMorphTargets, const TArray<float>& InMorphTargetWeights, EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode, const FExternalMorphWeightData& InExternalMorphWeightData) = 0;

	/**
	 * Called by FSkeletalMeshObject prior to GDME. This allows the GPU skin version to update bones etc now that we know we are going to render
	 * @param FrameNumber from GFrameNumber
	 */
	UE_DEPRECATED(5.5, "PreGDMECallback is no longer used")
	virtual void PreGDMECallback(FRHICommandList& RHICmdList, class FGPUSkinCache* GPUSkinCache, uint32 FrameNumber) {}

	/**
	 * @param	View - View, must not be 0, allows to cull depending on showflags (normally not needed/used in SHIPPING)
	 * @param	LODIndex - each LOD has its own vertex data
	 * @param	ChunkIdx - not used
	 * @return	vertex factory for rendering the LOD, 0 to suppress rendering
	 */
	virtual const FVertexFactory* GetSkinVertexFactory(const FSceneView* View, int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode = ESkinVertexFactoryMode::Default) const = 0;

	/**
     * Called by DrawStaticElements to cache mesh draw commands for skeletal meshes.
     */
	virtual const FVertexFactory* GetStaticSkinVertexFactory(int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const = 0;

	UE_DEPRECATED(5.5, "This method is no longer in use")
	virtual const FSkinBatchVertexFactoryUserData* GetVertexFactoryUserData(const int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const { return nullptr; }

	/**
	 * Returns true if this mesh performs skinning on the CPU.
	 */
	virtual bool IsCPUSkinned() const { return false; }

	/**
	 * Returns true if this mesh is an FSkeletalMeshObjectGPUSkin
	 */
	virtual bool IsGPUSkinMesh() const { return false; }

	/**
	 * Returns true if this mesh is an FSkeletalMeshObjectNanite
	 */
	virtual bool IsNaniteMesh() const { return false; }

	/**
	 * Returns a valid object if IsNaniteMesh() == true
	 */
	virtual const Nanite::FMaterialAudit* GetNaniteMaterials() const { return nullptr; }

	/** 
	 *	Get the array of component-space bone transforms. May include transforms that are inherited by follower components.
	 *	Not safe to hold this point between frames, because it exists in dynamic data passed from main thread.
	 */
	virtual TArray<FTransform>* GetComponentSpaceTransforms() const = 0;

	/** 
	 *	Get the array of refpose->local matrices
	 *	Not safe to hold this reference between frames, because it exists in dynamic data passed from main thread.
	 */
	virtual TConstArrayView<FMatrix44f> GetReferenceToLocalMatrices() const = 0;

	/**
	 *	Get the array of previous refpose->local matrices
	 *	Not safe to hold this reference between frames, because it exists in dynamic data passed from main thread.
	 */
	virtual TConstArrayView<FMatrix44f> GetPrevReferenceToLocalMatrices() const
	{
		// Not implemented
		checkNoEntry();
		return GetReferenceToLocalMatrices();
	}

	UE_DEPRECATED(5.7, "Use GetReferenceToLocalMatrices instead")
	virtual const TArray<FMatrix3x4>* GetCurrentBoneTransforms() const
	{
		checkNoEntry();
		return nullptr;
	}

	UE_DEPRECATED(5.7, "Use GetPrevReferenceToLocalMatrices instead")
	virtual const TArray<FMatrix3x4>* GetPreviousBoneTransforms() const
	{
		checkNoEntry();
		return nullptr;
	}

	struct FBoneTransforms
	{
		TConstArrayView<FMatrix44f> CurrentTransforms;
		TConstArrayView<FMatrix44f> PreviousTransforms;
		uint32 CurrentRevisionNumber = 0;
		uint32 PreviousRevisionNumber = 0;
		EPreviousBoneTransformUpdateMode UpdateMode = EPreviousBoneTransformUpdateMode::None;
	};

	virtual FBoneTransforms GetBoneTransforms() const
	{
		return FBoneTransforms{};
	}

	struct FBoneBufferSRV
	{
		FRHIShaderResourceView* SRV = nullptr;
		uint32 Offset = 0;
	};

	/** Returns the bone buffer SRV for the given LOD / Section and frame. The offset points to the start of the section. */
	virtual FBoneBufferSRV GetSectionBoneBuffer(int32 InLODIndex, int32 InSectionIndex, bool bPreviousFrame) const
	{
		return FBoneBufferSRV{};
	}

	/** Called when creating a new scene extension proxy object. */
	virtual FSkinningSceneExtensionProxy* CreateSceneExtensionProxy(const USkinnedAsset* InSkinnedAsset, bool bAllowScaling) const { return nullptr; }

	/** Called when the scene extension updates the skinning header for the current mesh. */
	virtual void UpdateSceneExtensionHeader(FRHICommandListBase& RHICmdList, const FSkinningHeader& SkinningHeader, FRHIShaderResourceView* TransformDataBufferSRV) const {}

	/**
	 * If we are caching geometry deformation through skin-cache/mesh-deformers or other, then this returns the currently cached geoemtry.
	 */
	virtual bool GetCachedGeometry(FRDGBuilder& GraphBuilder, FCachedGeometry& OutCachedGeometry) const { return false; }

	UE_DEPRECATED(5.6, "GetCachedGeometry now requires a GraphBuilder")
	bool GetCachedGeometry(FCachedGeometry& OutCachedGeometry) const { return false; }

	/**
	*	Will force re-evaluating which Skin Weight buffer should be used for skinning, determined by checking for any override weights or a skin weight profile being set.
	*	This prevents re-creating the vertex factories, but rather updates the bindings in place. 
	*/
	void UpdateSkinWeightBuffer(const USkinnedMeshComponent* InMeshComponent);
	virtual void UpdateSkinWeightBuffer(const TArrayView<const FSkelMeshComponentLODInfo> InLODInfo) = 0;

	/** Get the LOD to render this mesh at. */
	virtual int32 GetLOD() const = 0;
	/** 
	 * Enable blend weight rendering in the editor
	 * @param bEnabled - turn on or off the rendering mode
	 * optional parameters will decide which one to draw
	 * @param (optional) BonesOfInterest - array of bone indices to capture weights for
	 * @param (optional) MorphTargetsOfInterest - array of morphtargets to render for
	 */
	virtual void EnableOverlayRendering(bool bEnabled, const TArray<int32>* InBonesOfInterest, const TArray<UMorphTarget*>* MorphTargetOfInterest) {}

	/** 
	* Draw Normals/Tangents based on skinned vertex data 
	* @param PDI			- Draw Interface
	* @param ToWorldSpace	- Transform from component space to world space
	* @param bDrawNormals	- Should draw vertex normals
	* @param bDrawTangents	- Should draw vertex tangents
	* @param bDrawBinormals	- Should draw vertex binormals
	*/
	virtual void DrawVertexElements(FPrimitiveDrawInterface* PDI, const FMatrix& ToWorldSpace, bool bDrawNormals, bool bDrawTangents, bool bDrawBinormals) const {}

	/** 
	 *	Given a set of views, update the MinDesiredLODLevel member to indicate the minimum (ie best) LOD we would like to use to render this mesh. 
	 *	This is called from the rendering thread (PreRender) so be very careful what you read/write to.
	 * @param FrameNumber from ViewFamily.FrameNumber
	 */
	void UpdateMinDesiredLODLevel(const FSceneView* View, const FBoxSphereBounds& Bounds);

	/**
	 *	Return true if this does have valid dynamic data to render
	 */
	virtual bool HaveValidDynamicData() const = 0;

	// allow access to mesh component
	friend class FDynamicSkelMeshObjectDataCPUSkin;
	friend class FDynamicSkelMeshObjectDataGPUSkin;
	friend class FSkeletalMeshSceneProxy;
	friend class FSkeletalMeshSectionIter;

	/** @return if per-bone motion blur is enabled for this object. This includes is the system overwrites the skeletal mesh setting. */
	bool ShouldUsePerBoneMotionBlur() const { return bUsePerBoneMotionBlur; }
	
	/** Returns the mode for storing transforms on the GPU. */
	EBoneTransformStorageMode GetBoneTransformStorageMode() const
	{
		return BoneTransformStorageMode;
	}

	/** Returns the size of memory allocated by render data */
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) = 0;

	/**
	 * List of sections to be rendered based on instance weight usage. Full swap of weights will render with its own sections.
	 * @return Sections to iterate over for rendering
	 */
	ENGINE_API const TArray<FSkelMeshRenderSection>& GetRenderSections(int32 InLODIndex) const;

	/**
	 * Update the hidden material section flags for an LOD entry
	 *
	 * @param InLODIndex - LOD entry to update hidden material flags for
	 * @param HiddenMaterials - array of hidden material sections
	 */
	void SetHiddenMaterials(int32 InLODIndex,const TArray<bool>& HiddenMaterials);
	
	/**
	 * Determine if the material section entry for an LOD is hidden or not
	 *
	 * @param InLODIndex - LOD entry to get hidden material flags for
	 * @param MaterialIdx - index of the material section to check
	 */
	bool IsMaterialHidden(int32 InLODIndex,int32 MaterialIdx) const;
	
	/**
	 * Initialize the array of LODInfo based on the settings of the current skel mesh component
	 */
	void InitLODInfos(const TArrayView<const FSkelMeshComponentLODInfo> InComponentLODInfo, const USkinnedAsset* InSkinnedAsset);

	/**
	 * Return the ID of the component to which the skeletal mesh object belongs to.
	 */
	inline uint32 GetComponentId() const { return ComponentId; }

	inline TStatId GetStatId() const { return StatId; }

	/** Get the skeletal mesh resource for which this mesh object was created. */
	inline FSkeletalMeshRenderData& GetSkeletalMeshRenderData() const { return *SkeletalMeshRenderData; }

	FColor GetSkinCacheVisualizationDebugColor(const FName& GPUSkinCacheVisualizationMode, uint32 SectionIndex) const;

	/** Helper function to return the asset path name, optionally joined with the LOD index if LODIndex > -1. */
	FName GetAssetPathName(int32 LODIndex = -1) const;

#if RHI_RAYTRACING
	/** Retrieve ray tracing geometry from the underlying mesh object */
	virtual FRayTracingGeometry* GetRayTracingGeometry() { return nullptr; }
	virtual const FRayTracingGeometry* GetRayTracingGeometry() const { return nullptr; }
	virtual const FRayTracingGeometry* GetStaticRayTracingGeometry() const { return nullptr; }
	virtual FRWBuffer* GetRayTracingDynamicVertexBuffer() { return nullptr; }
	virtual int32 GetRayTracingLOD() const { return GetLOD(); }

	virtual bool ShouldUseSeparateSkinCacheEntryForRayTracing() const { return GetLOD() != GetRayTracingLOD(); }
	virtual FGPUSkinCacheEntry* GetSkinCacheEntryForRayTracing() const { return ShouldUseSeparateSkinCacheEntryForRayTracing() ? SkinCacheEntryForRayTracing : SkinCacheEntry; }

	/** Request ray tracing geometry to be updated with specified LOD Vertex Buffers */
	virtual void UpdateRayTracingGeometry(FRHICommandListBase& RHICmdList, FSkeletalMeshLODRenderData& LODModel, uint32 LODIndex, TArray<FBufferRHIRef>& VertexBuffers) {}

	/** Queue ray tracing geometry update if pending */
	UE_DEPRECATED(5.6, "QueuePendingRayTracingGeometryUpdate is no longer used, but bRequiresRebuild flag on FRayTracingGeometry is used to check if ray tracing geometry needs to be updated during FRayTracingGeometry")
	void QueuePendingRayTracingGeometryUpdate(FRHICommandListBase& RHICmdList) {}
	   
#endif // RHI_RAYTRACING

	/** Called when that component transform has changed */
	virtual void SetTransform(const FMatrix& InNewLocalToWorld, uint32 FrameNumber) {};

	/** Called to notify clothing data that component transform has changed */
	virtual void RefreshClothingTransforms(const FMatrix& InNewLocalToWorld, uint32 FrameNumber) {};

	/** Setup for rendering a specific LOD entry of the component */
	struct FSkelMeshObjectLODInfo
	{
		/** Hidden Material Section Flags for rendering - That is Material Index, not Section Index  */
		TArray<bool>	HiddenMaterials;

		/** The bounding sphere's screen size ratio threshold for this LOD, duplicated from the corresponding FSkeletalMeshLODInfo entry */
		FPerPlatformFloat ScreenSize;

		/** The hysteresis value for determining the LOD switch boundary. A non-zero value indicates that
		 *  a change from LOD0 -> LOD1 will occur at ScreenSize level, but a change from LOD1 -> LOD0 will
		 *  occur at when the object size is ScreenSize + LODHysteresis. This avoids flickering at the boundary.
		 */
		float LODHysteresis;

#if RHI_ENABLE_RESOURCE_INFO
		FName AssetPathName;
#endif
	};

	TArray<FSkelMeshObjectLODInfo> LODInfo;

	TArray<FCapsuleShape3f> ShadowCapsuleShapes;

	/** 
	 *	Lowest (best) LOD that was desired for rendering this SkeletalMesh last frame.
	 *
	 *	Note that if LOD streaming is enabled, the desired LOD is not guaranteed to be currently loaded.
	 * 
	 *	This should only ever be WRITTEN by the RENDER thread (in FSkeletalMeshProxy::PreRenderView) and READ by the GAME thread (in USkeletalMeshComponent::UpdateSkelPose).
	 */
	int32 MinDesiredLODLevel;

	/** The minimum allowed LOD level for this mesh. */
	const int32 MinLODLevel;

	/** 
	 *	High (best) DistanceFactor that was desired for rendering this SkeletalMesh last frame. Represents how big this mesh was in screen space  
	 *	This should only ever be WRITTEN by the RENDER thread (in FSkeletalMeshProxy::PreRenderView) and READ by the GAME thread (in USkeletalMeshComponent::UpdateSkelPose).
	 */
	float MaxDistanceFactor;

	/** This frames min desired LOD level. This is copied (flipped) to MinDesiredLODLevel at the beginning of the next frame. */
	int32 WorkingMinDesiredLODLevel;

	/** This frames max distance factor. This is copied (flipped) to MaxDistanceFactor at the beginning of the next frame. */
	float WorkingMaxDistanceFactor;

	/** This is set to true when we have sent our Mesh data to the rendering thread at least once as it needs to have have a datastructure created there for each MeshObject **/
	bool bHasBeenUpdatedAtLeastOnce;

#if RHI_RAYTRACING
	bool bSupportRayTracing;
	bool bHiddenMaterialVisibilityDirtyForRayTracing;
	int32 RayTracingMinLOD;
#endif

#if UE_BUILD_SHIPPING
	FName GetDebugName() const { return FName(); }
#else
	FName GetDebugName() const { return DebugName; }
	FName DebugName;
#endif // !UE_BUILD_SHIPPING

#if WITH_EDITORONLY_DATA
	/** Index of the section to preview... If set to -1, all section will be rendered */
	int32 SectionIndexPreview;
	int32 MaterialIndexPreview;

	/** The section currently selected in the Editor. Used for highlighting */
	int32 SelectedEditorSection;
	/** The Material currently selected. need to remember this index for reimporting cloth */
	int32 SelectedEditorMaterial;
#endif

	/** returns the feature level this FSkeletalMeshObject was created with */
	ERHIFeatureLevel::Type GetFeatureLevel() const
	{
		return FeatureLevel;
	}

	bool SupportsStaticRelevance() const
	{
		return bSupportsStaticRelevance;
	}

	/**
	 * Returns the display factor for the given LOD level
	 *
	 * @Param LODIndex - The LOD to get the display factor for
	 */
	float GetScreenSize(int32 LODIndex) const;

	/** Get the weight buffer either from the component LOD info or the skeletal mesh LOD render data */
	static FSkinWeightVertexBuffer* GetSkinWeightVertexBuffer(FSkeletalMeshLODRenderData& LODData, const FSkelMeshComponentLODInfo* CompLODInfo);

	/** Get the color buffer either from the component LOD info or the skeletal mesh LOD render data */
	static FColorVertexBuffer* GetColorVertexBuffer(FSkeletalMeshLODRenderData& LODData, const FSkelMeshComponentLODInfo* CompLODInfo);

	/** Get the weight buffer for specific LOD index. Only needed for GPU skin cache */
	UE_DEPRECATED(5.8, "Use GetSkinWeightVertexBuffer_RenderThread instead")
	virtual const FSkinWeightVertexBuffer* GetSkinWeightVertexBuffer(int32 LODIndex) const { return nullptr; }

	/** Get the weight buffer for specific LOD index. Only needed for GPU skin cache */
	virtual FSkinWeightVertexBufferInfo GetSkinWeightVertexBuffer_RenderThread(int32 LODIndex) const
	{
		return FSkinWeightVertexBufferInfo();
	}

	/** Get the offset in the dynamic mesh bounds buffer for the current LOD. Only needed for GPU skin cache. */
	virtual int32 GetDynamicBoundsStartOffset(int32 LODIndex) const { return -1; }

	/** Private data for the skinning scene extension. */
	mutable class FSkinningSceneExtensionPrivateData
	{
	private:
		friend class FSkinningSceneExtension;
		int32 CachedPrimitiveIndex = INDEX_NONE;

	} SkinningSceneExtensionPrivateData;

	TConstArrayView<FBoneReference> GetCachedBonesToRemove(const USkinnedAsset* SkinnedAsset, int32 InLODIndex)
	{
		if (SkinnedAsset != BonesToRemoveCache.SkinnedAsset || InLODIndex != BonesToRemoveCache.LODIndex)
		{
			BonesToRemoveCache.SkinnedAsset = SkinnedAsset;
			BonesToRemoveCache.LODIndex = InLODIndex;
			BonesToRemoveCache.BonesToRemove = SkinnedAsset->GetLODInfo(InLODIndex)->BonesToRemove;
		}
		return BonesToRemoveCache.BonesToRemove;
	}

protected:
	/** The skeletal mesh resource with which to render. */
	FSkeletalMeshRenderData* SkeletalMeshRenderData;

	FGPUSkinCacheEntry* SkinCacheEntry;
	FGPUSkinCacheEntry* SkinCacheEntryForRayTracing;

	/** Used to keep track of the first call to UpdateMinDesiredLODLevel each frame. from ViewFamily.FrameNumber */
	uint32 LastFrameNumber;

	/** Guards the call to UpdateMinDesiredLODLevel */
	UE::FMutex DesiredLODLevelMutex;

	/** 
	 *	If true, per-bone motion blur is enabled for this object. This includes is the system overwrites the skeletal mesh setting.
	 */
	bool bUsePerBoneMotionBlur;

	/** If true, the skeletal mesh will take the static relevance path using cached mesh draw commands. */
	bool bSupportsStaticRelevance = false;

	EBoneTransformStorageMode BoneTransformStorageMode = EBoneTransformStorageMode::Direct;

	/** Used for dynamic stats */
	TStatId StatId;

	/** Feature level to render for. */
	ERHIFeatureLevel::Type FeatureLevel;

	/** Component ID to which belong this  mesh object  */
	uint32 ComponentId;

	FVector WorldScale = FVector::OneVector;

#if RHI_ENABLE_RESOURCE_INFO
	FName AssetPathName;
#endif

	/** Cache for bones removed at a given LOD level, used by the forward kinematic fixup. GameThread only. */
	struct
	{
		const USkinnedAsset* SkinnedAsset = nullptr;
		int32 LODIndex = -1;
		TConstArrayView<FBoneReference> BonesToRemove;
	} BonesToRemoveCache;
};
