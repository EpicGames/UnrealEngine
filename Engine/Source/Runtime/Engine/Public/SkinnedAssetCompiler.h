// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/WeakObjectPtr.h"
#include "Containers/Set.h"
#include "Tickable.h"
#include "Stats/Stats.h"
#include "IAssetCompilingManager.h"

#if WITH_EDITOR

class FAsyncCompilationNotification;
class USkinnedAsset;
class UPrimitiveComponent;
class FQueuedThreadPool;
struct FAssetCompileContext;
enum class EQueuedWorkPriority : uint8;

class FSkinnedAssetCompilingManager : IAssetCompilingManager
{
public:
	struct FFinishCompilationOptions
	{
		/**
		 * In addition to waiting for the specified assets and their dependencies to finish compiling, also wait for
		 * all other meshes currently compiling that depend upon them.
		 * NOTE: This should be set to true if you are about to modify the meshes after the call completes, so that you are
		 * not concurrently editing a mesh that's being referenced by another mesh's async build.
		 */
		bool bIncludeDependentAssets;
		
		FFinishCompilationOptions() : bIncludeDependentAssets(false) {}
	};

	ENGINE_API static FSkinnedAssetCompilingManager& Get();

	/**
	 * Returns true if the feature is currently activated.
	 */
	ENGINE_API bool IsAsyncCompilationEnabled() const;

	/** 
	 * Returns the number of outstanding compilations.
	 */
	ENGINE_API int32 GetNumRemainingJobs() const;

	/** 
	 * Adds skinned assets compiled asynchronously so they are monitored. 
	 */
	ENGINE_API void AddSkinnedAssets(TArrayView<USkinnedAsset* const> InSkinnedAssets);

	/** 
	 * Adds skinned assets (with dependencies that are still compiling) compiled asynchronously so they are monitored.
	 */
	ENGINE_API void AddSkinnedAssetsWithDependencies(TArrayView<USkinnedAsset* const> InSkinnedAssets);

	/** 
	 * Blocks until completion of the requested skinned assets.
	 */
	ENGINE_API void FinishCompilation(TArrayView<USkinnedAsset* const> InSkinnedAssets, const FFinishCompilationOptions& Options = FFinishCompilationOptions());

	/** 
	 * Blocks until completion of all async skinned asset compilation.
	 */
	ENGINE_API void FinishAllCompilation() override;

	/**
	 * Returns if asynchronous compilation is allowed for this skinned asset.
	 */
	ENGINE_API bool IsAsyncCompilationAllowed(USkinnedAsset* InSkinnedAsset) const;

	/**
	 * Returns the priority at which the given skinned asset should be scheduled.
	 */
	ENGINE_API EQueuedWorkPriority GetBasePriority(USkinnedAsset* InSkinnedAsset) const;

	/**
	 * Returns the threadpool where skinned asset compilation should be scheduled.
	 */
	ENGINE_API FQueuedThreadPool* GetThreadPool() const;

	/**
	 * Cancel any pending work and blocks until it is safe to shut down.
	 */
	ENGINE_API void Shutdown() override;

	/** Get the name of the asset type this compiler handles */
	ENGINE_API static FName GetStaticAssetTypeName();

private:
	FSkinnedAssetCompilingManager();

	FName GetAssetTypeName() const override;
	FTextFormat GetAssetNameFormat() const override;
	TArrayView<FName> GetDependentTypeNames() const override;
	int32 GetNumRemainingAssets() const override;
	void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;

	void FinishCompilationForObjects(TArrayView<UObject* const> InObjects) override;

	friend class FAssetCompilingManager;
	
	bool bHasShutdown = false;
	TSet<TWeakObjectPtr<USkinnedAsset>> RegisteredSkinnedAsset;
	TSet<TWeakObjectPtr<USkinnedAsset>> SkinnedAssetsWithPendingDependencies;
	TMap<TWeakObjectPtr<USkinnedAsset>, TSet<TWeakObjectPtr<USkinnedAsset>>> ReverseDependencyLookup;
	TUniquePtr<FAsyncCompilationNotification> Notification;
	void FinishCompilationsForGame();
	void Reschedule();
	void ProcessSkinnedAssets(bool bLimitExecutionTime, int32 MinBatchSize = 1);
	void UpdateCompilationNotification();

	void PostCompilation(USkinnedAsset* SkinnedAsset);
	void PostCompilation(TArrayView<USkinnedAsset* const> InSkinnedAssets);
	void SchedulePendingCompilations();

	void OnPostReachabilityAnalysis();
	FDelegateHandle PostReachabilityAnalysisHandle;
	
	void OnPreGarbageCollect();
	FDelegateHandle PreGarbageCollectHandle;
};

#endif // #if WITH_EDITOR
