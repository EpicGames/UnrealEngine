// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SkinnedMeshComponent.h"
#include "ContentStreaming.h"
#include "Engine/MaterialOverlayHelper.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAsset.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StreamableRenderAsset.h"
#include "Materials/MaterialInterface.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PrimitiveComponentHelper.h"
#include "Rendering/NaniteResources.h"
#include "Rendering/SkeletalMeshRenderData.h"

/** Helper class used to share implementation for different SkinnedMeshComponent types */
class FSkinnedMeshComponentHelper
{
public:
	template <typename T>
	static UMaterialInterface* GetMaterial(const T& InComponent, int32 InMaterialIndex);

	template <typename T>
	static void GetUsedMaterials(const T& InComponent, TArray<UMaterialInterface*>& OutMaterials, bool bInGetDebugMaterials);

	template <typename T>
	static void GetDefaultMaterialSlotsOverlayMaterial(const T& InComponent, TArray<TObjectPtr<UMaterialInterface>>& OutMaterials);

	template <typename T>
	static int32 GetNumLODs(const T& InComponent);

	template <typename T>
	static int32 ComputeMinLOD(const T& InComponent);

	template <typename T>
	static int32 GetValidMinLOD(const T& InComponent, const int32 InMinLODIndex);

	template <typename T>
	static const Nanite::FResources* GetNaniteResources(const T& InComponent);

	template <typename T>
	static bool HasValidNaniteData(const T& InComponent);

	template <typename T>
	static bool ShouldNaniteSkin(const T& InComponent);	
	
	template <typename T>
	static FSkeletalMeshRenderData* GetSkeletalMeshRenderData(const T& InComponent);
};

template<class T>
FSkeletalMeshRenderData* FSkinnedMeshComponentHelper::GetSkeletalMeshRenderData(const T& InComponent)
{
	if (InComponent.GetMeshObject())
	{
		return &InComponent.GetMeshObject()->GetSkeletalMeshRenderData();
	}
	else if (InComponent.GetSkinnedAsset())
	{
		return InComponent.GetSkinnedAsset()->GetResourceForRendering();
	}
	else
	{
		return nullptr;
	}
}

template<class T>
int32 FSkinnedMeshComponentHelper::GetNumLODs(const T& InComponent)
{
	int32 NumLODs = 0;
	FSkeletalMeshRenderData* RenderData = GetSkeletalMeshRenderData(InComponent);
	if (RenderData)
	{
		NumLODs = RenderData->LODRenderData.Num();
	}
	return NumLODs;
}

template<class T>
int32 FSkinnedMeshComponentHelper::ComputeMinLOD(const T& InComponent)
{
	const int32 AssetMinLOD = InComponent.GetSkinnedAsset()->GetMinLodIdx();
	// overriden MinLOD can't be higher than asset MinLOD
	int32 MinLODIndex = InComponent.bOverrideMinLod ? FMath::Max(InComponent.MinLodModel, AssetMinLOD) : AssetMinLOD;
	MinLODIndex = GetValidMinLOD(InComponent, MinLODIndex);
	return MinLODIndex;
}

template<class T>
int32 FSkinnedMeshComponentHelper::GetValidMinLOD(const T& InComponent, const int32 InMinLODIndex)
{
	// Iterate the render data to validate that our min LOD has data that can be used.
	const int32 MaxLODIndex = GetNumLODs(InComponent) - 1;
	const FSkeletalMeshRenderData* RenderData = GetSkeletalMeshRenderData(InComponent);
	const int32 FirstValidLODIndex = RenderData != nullptr ? RenderData->GetFirstValidLODIdx(InMinLODIndex) : INDEX_NONE;

	// Return the first LOD that has render data that can be used.
	/** NOTE: We're logging if the index is not valid in the render data but we still want to return a valid value from 0 to max.
	  * Render data could be invalid if we're still loading/streaming in the asset.
	  */
	return FMath::Clamp<int32>(FirstValidLODIndex, 0, MaxLODIndex);
}

template<class T>
UMaterialInterface* FSkinnedMeshComponentHelper::GetMaterial(const T& InComponent, int32 InMaterialIndex)
{
	if (InComponent.OverrideMaterials.IsValidIndex(InMaterialIndex) && InComponent.OverrideMaterials[InMaterialIndex])
	{
		return InComponent.OverrideMaterials[InMaterialIndex];
	}
	else if (USkinnedAsset* SkinnedAsset = InComponent.GetSkinnedAsset())
	{
		if (!SkinnedAsset->IsCompiling() && SkinnedAsset->GetMaterials().IsValidIndex(InMaterialIndex) && SkinnedAsset->GetMaterials()[InMaterialIndex].MaterialInterface)
		{
			return SkinnedAsset->GetMaterials()[InMaterialIndex].MaterialInterface;
		}
	}

	return nullptr;
}

template<class T>
void FSkinnedMeshComponentHelper::GetUsedMaterials(const T& InComponent, TArray<UMaterialInterface*>& OutMaterials, bool bInGetDebugMaterials)
{
	if (USkinnedAsset* SkinnedAsset = InComponent.GetSkinnedAsset())
	{
		// The max number of materials used is the max of the materials on the skeletal mesh and the materials on the mesh component
		const int32 NumMaterials = FMath::Max(SkinnedAsset->GetMaterials().Num(), InComponent.OverrideMaterials.Num());
		for (int32 MatIdx = 0; MatIdx < NumMaterials; ++MatIdx)
		{
			// GetMaterial will determine the correct material to use for this index.  
			UMaterialInterface* MaterialInterface = InComponent.GetMaterial(MatIdx);
			OutMaterials.Add(MaterialInterface);
		}

		TArray<TObjectPtr<UMaterialInterface>> AssetAndComponentMaterialSlotsOverlayMaterial;
		InComponent.GetMaterialSlotsOverlayMaterial(AssetAndComponentMaterialSlotsOverlayMaterial);
		bool bUseGlobalMeshOverlayMaterial = false;
		FMaterialOverlayHelper::AppendAllOverlayMaterial(AssetAndComponentMaterialSlotsOverlayMaterial, OutMaterials, bUseGlobalMeshOverlayMaterial);
		if (bUseGlobalMeshOverlayMaterial)
		{
			UMaterialInterface* OverlayMaterialInterface = InComponent.GetOverlayMaterial();
			if (OverlayMaterialInterface != nullptr)
			{
				OutMaterials.Add(OverlayMaterialInterface);
			}
		}
	}

	if (bInGetDebugMaterials)
	{
#if WITH_EDITOR
		if (UPhysicsAsset* PhysicsAssetForDebug = InComponent.GetPhysicsAsset())
		{
			PhysicsAssetForDebug->GetUsedMaterials(OutMaterials);
		}
#endif
	}
}

template<class T>
void FSkinnedMeshComponentHelper::GetDefaultMaterialSlotsOverlayMaterial(const T& InComponent, TArray<TObjectPtr<UMaterialInterface>>& OutMaterials)
{
	OutMaterials.Reset();
	if (USkinnedAsset* SkinnedAsset = InComponent.GetSkinnedAsset())
	{
		const TArray<FSkeletalMaterial>& SkeletalMaterials = SkinnedAsset->GetMaterials();
		for (const FSkeletalMaterial& SkeletalMaterial : SkeletalMaterials)
		{
			OutMaterials.Add(SkeletalMaterial.OverlayMaterialInterface);
		}
	}
}

template <typename T>
const Nanite::FResources* FSkinnedMeshComponentHelper::GetNaniteResources(const T& InComponent)
{
	if (InComponent.GetSkinnedAsset() && InComponent.GetSkinnedAsset()->GetResourceForRendering())
	{
		return InComponent.GetSkinnedAsset()->GetResourceForRendering()->NaniteResourcesPtr.Get();
	}

	return nullptr;
}

template <typename T>
bool FSkinnedMeshComponentHelper::HasValidNaniteData(const T& InComponent)
{
	const Nanite::FResources* NaniteResources = GetNaniteResources(InComponent);
	return NaniteResources != nullptr ? NaniteResources->PageStreamingStates.Num() > 0 : false;
}

template <typename T>
bool FSkinnedMeshComponentHelper::ShouldNaniteSkin(const T& InComponent)
{
	if (InComponent.IsDisallowNanite() || InComponent.IsForceDisableNanite())
	{
		return false;
	}
	const EShaderPlatform ShaderPlatform = InComponent.GetScene() ? InComponent.GetScene()->GetShaderPlatform() : GMaxRHIShaderPlatform;
	return USkinnedMeshComponent::ShouldRenderNaniteSkinnedMeshes() && UseNanite(ShaderPlatform) && HasValidNaniteData(InComponent);
}
