// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PrimitiveSceneProxyDesc.h"
#include "ClothingSystemRuntimeTypes.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/SkinnedAsset.h"

#include "Components/ExternalMorphSet.h"

class USkinnedAsset;
class UPhysicsAsset;
class UMaterialInterface;
class FSkeletalMeshObject;
struct FSkelMeshComponentLODInfo;
class UMeshDeformerInstance;
struct FMeshDeformerInstanceSet;

struct FSkinnedMeshSceneProxyDesc : public FPrimitiveSceneProxyDesc
{
	ENGINE_API static FSkeletalMeshObject* CreateMeshObject(const FSkinnedMeshSceneProxyDesc& Desc);
	ENGINE_API static FPrimitiveSceneProxy* CreateSceneProxy(const FSkinnedMeshSceneProxyDesc& Desc, bool bHideSkin);

	UE_DEPRECATED(5.8, "MeshDesc now has MinLODLevel and InClampedLODIndex is unused")
	static FPrimitiveSceneProxy* CreateSceneProxy(const FSkinnedMeshSceneProxyDesc& Desc, bool bHideSkin, int32 InClampedLODIndex)
	{
		FSkinnedMeshSceneProxyDesc CopyDesc = Desc;
		CopyDesc.MinLODLevel = InClampedLODIndex;
		return CreateSceneProxy(CopyDesc, bHideSkin);
	}

	FSkinnedMeshSceneProxyDesc() = default;
	ENGINE_API FSkinnedMeshSceneProxyDesc(const USkinnedMeshComponent* Component);
	ENGINE_API void InitializeFromSkinnedMeshComponent(const USkinnedMeshComponent*);
	ENGINE_API USkinnedAsset* GetSkinnedAsset() const;
	ENGINE_API UPhysicsAsset* GetPhysicsAsset() const;
	ENGINE_API bool ShouldDrawDebugSkeleton() const;
	ENGINE_API const TOptional<FLinearColor>& GetDebugDrawColor() const;
	ENGINE_API void GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const;
	ENGINE_API UMaterialInterface* GetMaterial(int32 MaterialIndex) const;
	ENGINE_API int32 GetBoneIndex( FName BoneName ) const;
	UE_DEPRECATED(5.7, "Please use GetMaterialRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API FMaterialRelevance GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const;
	ENGINE_API FMaterialRelevance GetMaterialRelevance(EShaderPlatform InShaderPlatform) const;
	ENGINE_API float GetOverlayMaterialMaxDrawDistance() const;
	ENGINE_API UMaterialInterface* GetOverlayMaterial() const;
	void GetMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotsOverlayMaterial) const { OutMaterialSlotsOverlayMaterial = MaterialSlotsOverlayMaterial; }

	ENGINE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;

	FVector GetComponentScale() const { return ComponentScale; }
	int32 GetPredictedLODLevel() const { return PredictedLODLevel; }
	int32 GetMinLODLevel() const { return MinLODLevel; }
	float GetMaxDistanceFactor() const { return MaxDistanceFactor; }

	void SetLODInfo(TArrayView<const FSkelMeshComponentLODInfo> InLODInfo) { LODInfo = InLODInfo; }

	bool ShouldCPUSkin() const { return bCPUSkinning; }
	ENGINE_API bool ShouldNaniteSkin() const;
	ENGINE_API bool HasValidNaniteData() const;

	bool IsDisallowNanite() const { return bDisallowNanite; }
	bool IsForceDisableNanite() const { return bForceDisableNanite; }
	int32 GetWorldPositionOffsetDisableDistance() const { return WorldPositionOffsetDisableDistance; }
	bool EvaluateWorldPositionOffset() const { return bEvaluateWorldPositionOffset; }
	ENGINE_API bool IsSkinCacheAllowed(int32 LodIdx) const;
	ENGINE_API UMeshDeformerInstance* GetMeshDeformerInstance() const;

	ENGINE_API UMeshDeformerInstance* GetMeshDeformerInstanceForLOD(int32 LODIndex) const;

	UE_DEPRECATED(5.5, "Use InitializeFromSkinnedMeshComponent instead.")
	void InitializeFrom(const USkinnedMeshComponent* InComponent) { InitializeFromSkinnedMeshComponent(InComponent); }

	TOptional<FLinearColor> DebugDrawColor;
	FMaterialRelevance MaterialRelevance{};
	TArrayView<const FSkelMeshComponentLODInfo> LODInfo;
	
	FSkeletalMeshObject* MeshObject = nullptr;
	FSkeletalMeshObject* PreviousMeshObject = nullptr;
	
	USkinnedAsset* SkinnedAsset = nullptr;
	UPhysicsAsset* PhysicsAsset = nullptr;
	UMaterialInterface* OverlayMaterial = nullptr;
	TArray<TObjectPtr<UMaterialInterface>> MaterialSlotsOverlayMaterial;
	const FMeshDeformerInstanceSet* MeshDeformerInstances = nullptr;
	TArrayView<const TObjectPtr<UMaterialInterface>> OverrideMaterials;
	TArrayView<const ESkinCacheUsage> SkinCacheUsage{};
	int32 PredictedLODLevel = 0;
	int32 MinLODLevel = 0;
	int32 ForceStreamedLOD = INDEX_NONE;
	float MaxDistanceFactor = 1.0f;

	/* SkeletalMesh Archetype Data Begin */
	FVector ComponentScale = FVector::OneVector;
	float StreamingDistanceMultiplier = 1.0f;
	float NanitePixelProgrammableDistance = 0.0f;
	float CapsuleIndirectShadowMinVisibility = 0.0f;
	float OverlayMaterialMaxDrawDistance = 0.0f;
	int32 WorldPositionOffsetDisableDistance = 0;

	uint8 bForceWireframe : 1 = false;
	uint8 bCanHighlightSelectedSections : 1 = false;
	uint8 bRenderStatic : 1 = false;
	uint8 bPerBoneMotionBlur : 1 = false;
	uint8 bCastCapsuleDirectShadow : 1 = false;
	uint8 bCastCapsuleIndirectShadow : 1 = false;
	uint8 bDrawDebugSkeleton : 1 = false;
	uint8 bCPUSkinning : 1 = false;
#if WITH_EDITORONLY_DATA
	uint8 bClothPainting : 1 = false;
#endif
	uint32 bSortTriangles :1 = false;
	uint8 bAllowAlwaysVisible : 1 = false;
	uint8 bEvaluateWorldPositionOffset : 1 = true;
	uint8 bDisallowNanite : 1 = false;
	uint8 bForceDisableNanite : 1 = false;

	/* SkeletalMesh Archetype Data End */

#if WITH_EDITORONLY_DATA
	int32 GetSectionPreview() const { return SectionIndexPreview;  }
	int32 GetMaterialPreview() const { return MaterialIndexPreview; }
	int32 GetSelectedEditorSection() const { return SelectedEditorSection; }
	int32 GetSelectedEditorMaterial() const { return SelectedEditorMaterial; }
private:
	int32 SectionIndexPreview = INDEX_NONE;
	int32 MaterialIndexPreview = INDEX_NONE;
	int32 SelectedEditorSection = INDEX_NONE;
	int32 SelectedEditorMaterial = INDEX_NONE;
#endif // WITH_EDITORONLY_DATA
};

using FExternalMorphSets = TMap<int32, TSharedPtr<FExternalMorphSet>>;

struct FSkinnedMeshSceneProxyDynamicData
{
	ENGINE_API FSkinnedMeshSceneProxyDynamicData(const USkinnedMeshComponent* SkinnedMeshComponent);
	ENGINE_API FSkinnedMeshSceneProxyDynamicData(const USkinnedMeshComponent* SkinnedMeshComponent, const USkinnedMeshComponent* InLeaderPoseComponent);
	ENGINE_API FSkinnedMeshSceneProxyDynamicData();
	ENGINE_API bool IsValidExternalMorphSetLODIndex(uint32 InLODIndex) const;
	ENGINE_API const FExternalMorphSets& GetExternalMorphSets(uint32 InLODIndex) const;
	ENGINE_API int32 GetMeshDeformerMaxLOD() const;
	ENGINE_API UMeshDeformerInstance* GetMeshDeformerInstanceForLOD(int32 LODIndex) const;

	const FName& GetFName() const { return Name; }

	const IClothSimulationDataProvider* GetClothSimulationDataProvider() const { return ClothSimulDataProvider; }
	const TArrayView<const FTransform>& GetComponentSpaceTransforms() const { return ComponentSpaceTransforms; }
	const TArrayView<const FTransform>& GetPreviousComponentTransformsArray() const { return PreviousComponentSpaceTransforms; }
	const TArrayView<const uint8>& GetBoneVisibilityStates() const { return BoneVisibilityStates; }
	const TArrayView<const uint8>& GetPreviousBoneVisibilityStates() const { return PreviousBoneVisibilityStates; }

	FTransform GetComponentTransform() const { return ComponentWorldTransform; }
	const TSharedPtr<FSkelMeshRefPoseOverride>& GetRefPoseOverride() const { return RefPoseOverride; }
	const TArrayView<const int32>& GetLeaderBoneMap() const { return LeaderBoneMap; }
	const TBitArray<>* GetLeaderValidMeshPoseTransforms() const { return LeaderValidMeshPoseTransforms; }

	uint32 GetBoneTransformRevisionNumber() const { return CurrentBoneTransformRevisionNumber; }
	uint32 GetPreviousBoneTransformRevisionNumber() const { return PreviousBoneTransformRevisionNumber; }
	uint32 GetCurrentBoneTransformFrame() const { return CurrentBoneTransformFrame; }
	int32 GetNumLODs() const { return static_cast<uint32>(NumLODs); }
	bool HasLeaderPoseComponent() const { return bHasLeaderPoseComponent; }
	bool HasMeshDeformerInstance() const { return bHasMeshDeformerInstance; }
	bool IsRenderStateRecreating() const { return bRenderStateRecreating; }

	FName Name = NAME_None;
	const IClothSimulationDataProvider* ClothSimulDataProvider = nullptr;
	const FMeshDeformerInstanceSet* MeshDeformerInstances = nullptr;
	TSharedPtr<FSkelMeshRefPoseOverride> RefPoseOverride = nullptr;
	TArrayView<const FExternalMorphSets> ExternalMorphSets{};
	TArrayView<const FTransform> ComponentSpaceTransforms{}; 
	TArrayView<const FTransform> PreviousComponentSpaceTransforms{}; 
	TArrayView<const uint8> BoneVisibilityStates{};
	TArrayView<const uint8> PreviousBoneVisibilityStates{};

	TArrayView<const int32> LeaderBoneMap{};
	const TBitArray<>* LeaderValidMeshPoseTransforms = nullptr;
	TArrayView<const ESkinCacheUsage> SkinCacheUsage{};

	TArrayView<const FMatrix44f> ReferenceToLocalMatrices;
	TArrayView<const FMatrix44f> PreviousReferenceToLocalMatrices;

	FTransform ComponentWorldTransform = FTransform::Identity;
	uint32 CurrentBoneTransformRevisionNumber = INDEX_NONE;
	uint32 PreviousBoneTransformRevisionNumber = INDEX_NONE;
	uint32 CurrentBoneTransformFrame = INDEX_NONE;
	uint16 NumLODs = INDEX_NONE;

	uint8 bHasLeaderPoseComponent : 1 = false;
	uint8 bHasMeshDeformerInstance : 1 = false;
	uint8 bRenderStateRecreating : 1 = false;
	uint8 bDrawInGame : 1 = true;
	uint8 bCastsHiddenShadow : 1 = false;
	uint8 bAffectIndirectLightingWhileHidden : 1 = false;
	uint8 bUseReferenceToLocalMatrices : 1 = false;

	UE_DEPRECATED(5.7, "IsSkinCacheAllowed is deprecated")
	ENGINE_API bool IsSkinCacheAllowed(int32 LodIdx, const USkinnedAsset* InSkinnedAsset) const;
};