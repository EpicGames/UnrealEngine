// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Matrix3x4.h"
#include "PrimitiveSceneProxy.h"
#include "SkeletalRenderPublic.h"

class USkinnedAsset;
class FSkeletalMeshRenderData;
class UTransformProviderData;
class FTransformProviderRenderProxy;
class FSkinningSceneExtension;
class FPrimitiveSceneInfo;
struct FSkinnedMeshSceneProxyDesc;
struct FInstancedSkinnedMeshSceneProxyDesc;

class FSkinningSceneExtensionProxy
{
public:
	ENGINE_API FSkinningSceneExtensionProxy(
		const FSkeletalMeshObject* InMeshObject,
		const USkinnedAsset* InSkinnedAsset,
		bool bAllowScaling,
		bool bIncludeSocketsInBoneMap = false);

	virtual ~FSkinningSceneExtensionProxy() = default;

	inline const USkinnedAsset* GetSkinnedAsset() const
	{
		return SkinnedAsset;
	}

	const FSkeletalMeshObject* GetMeshObject() const
	{
		return MeshObject;
	}

	TConstArrayView<uint16> GetBoneMap() const
	{
		return BoneMap;
	}

	TConstArrayView<uint32> GetBoneHierarchy() const
	{
		return BoneHierarchy;
	}

	TConstArrayView<float> GetBoneObjectSpace() const
	{
		return BoneObjectSpace;
	}

	uint32 GetMaxBoneTransformCount() const
	{
		return MaxBoneTransformCount;
	}

	uint32 GetMaxBoneMapCount() const
	{
		return BoneMap.Num();
	}

	uint32 GetMaxBoneHierarchyCount() const
	{
		return BoneHierarchy.Num();
	}

	uint32 GetMaxBoneObjectSpaceCount() const
	{
		return BoneHierarchy.Num();
	}

	uint32 GetMaxBoneInfluenceCount() const
	{
		return MaxBoneInfluenceCount;
	}

	uint32 GetUniqueAnimationCount() const
	{
		return UniqueAnimationCount;
	}

	uint32 GetLOD() const
	{
		check(LODIndex != InvalidLODIndex);
		return LODIndex;
	}

	bool HasScale() const
	{
		return bHasScale;
	}

	bool HasUnifiedBoneMap() const
	{
		return bHasUnifiedBoneMap;
	}

	bool HasSocketsInBoneMap() const
	{
		return bHasSocketsInBoneMap;
	}

	bool UseSkeletonBatching() const
	{
		return bUseSkeletonBatching;
	}

	bool UseInstancing() const
	{
		return bUseInstancing;
	}

	EBoneTransformStorageMode GetBoneTransformStorageMode() const
	{
		return BoneTransformStorageMode;
	}

	// TODO: TEMP - Move to shared location with GPU
	uint32 GetObjectSpaceFloatCount() const
	{
		return 4 /* quat */ + 3 /* XYZ translation */ + (HasScale() ? 3 : 0 /* XYZ scale */);
	}

	/** Collect bone indices referenced by sockets on the skinned asset. Only raw bones (< GetRawBoneNum) are included. */
	static ENGINE_API void GetSocketBoneIndices(const USkinnedAsset* InSkinnedAsset, TArray<FBoneIndexType>& OutBoneIndices);

	virtual void CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList) {}

	virtual void DestroyRenderThreadResources() {}

	virtual const FGuid& GetTransformProviderId() const;

private:
	bool SetLOD(int32 InLODIndex)
	{
		const bool bSwap = InLODIndex != LODIndex;
		const bool bInit = bSwap && LODIndex == InvalidLODIndex;
		LODIndex = (uint16)InLODIndex;
		if (bSwap && BoneTransformStorageMode == EBoneTransformStorageMode::BoneMap && !bHasUnifiedBoneMap)
		{
			InitForBoneMapStorage();
			return true;
		}
		// Request an update on first use as well.
		return bInit;
	}

protected:
	void InitForDirectStorage(bool bAllowScaling);
	ENGINE_API void InitForBoneMapStorage();

	static const uint16 InvalidLODIndex = TNumericLimits<uint16>::Max();

	const USkinnedAsset* SkinnedAsset = nullptr;
	const FSkeletalMeshObject* MeshObject = nullptr;

	TArray<uint16> BoneMap;
	TArray<uint32> BoneHierarchy;
	TArray<float> BoneObjectSpace;

	uint32 UniqueAnimationCount         = 1u;
	uint16 MaxBoneTransformCount        = 0u;
	uint16 MaxBoneInfluenceCount        = 0u;
	uint16 LODIndex                     = InvalidLODIndex;

	EBoneTransformStorageMode BoneTransformStorageMode = EBoneTransformStorageMode::Direct;

	uint8 bHasScale                     : 1 = false;
	uint8 bHasUnifiedBoneMap            : 1 = false;
	uint8 bHasSocketsInBoneMap          : 1 = false;
	uint8 bUseSkeletonBatching          : 1 = false;
	uint8 bUseInstancing                : 1 = false;

	friend class FSkinningSceneExtension;
};

class FInstancedSkinningSceneExtensionProxyUserData
{
public:
	virtual ~FInstancedSkinningSceneExtensionProxyUserData() = default;
};

class FBoneAttachmentBinding
{
	static const uint32 Unattached   = ~0u;
	static const uint32 MaxSockets   = (1 << 8 ) - 1;
	static const uint32 MaxInstances = (1 << 24) - 1;

public:
	FBoneAttachmentBinding() = default;

	FBoneAttachmentBinding(uint32 InSocketIndex, uint32 InParentInstanceIndex)
	{
		check(InSocketIndex < MaxSockets);
		check(InParentInstanceIndex < MaxInstances);
		Bits.SocketIndex   = InSocketIndex;
		Bits.InstanceIndex = InParentInstanceIndex;
	}

	uint32 GetSocketIndex() const
	{
		check(IsAttached());
		return Bits.SocketIndex;
	}

	uint32 GetParentInstanceIndex() const
	{
		check(IsAttached());
		return Bits.InstanceIndex;
	}

	bool IsAttached() const
	{
		return Bits.Packed != Unattached;
	}

	bool operator==(const FBoneAttachmentBinding& Other) const { return Bits.Packed == Other.Bits.Packed; }
	bool operator!=(const FBoneAttachmentBinding& Other) const { return Bits.Packed != Other.Bits.Packed; }

	friend FArchive& operator<<(FArchive& Ar, FBoneAttachmentBinding& Binding)
	{
		Ar << Binding.Bits.Packed;
		return Ar;
	}

private:
	union
	{
		struct
		{
			uint32 InstanceIndex : 24;
			uint32 SocketIndex   :  8;
		};
		uint32 Packed = Unattached;
	} Bits;
};

/** Fully resolved bone attachment socket data for the render thread. */
struct FBoneAttachmentSocket
{
	FBoneAttachmentSocket()
	{
		RefPoseMatrix.SetIdentity();
	}

	FPrimitiveComponentId ParentComponentId;
	uint16 BoneIndex = INVALID_BONE_INDEX;
	FMatrix3x4 RefPoseMatrix;
	FQuat4f Rotation = FQuat4f::Identity;
	FVector3f Translation = FVector3f::ZeroVector;
};

class FInstancedSkinningSceneExtensionProxy : public FSkinningSceneExtensionProxy
{
public:
	ENGINE_API FInstancedSkinningSceneExtensionProxy(
		const UTransformProviderData* InTransformProvider,
		const FSkeletalMeshObject* InMeshObject,
		const USkinnedAsset* InSkinnedAsset,
		bool bAllowScaling,
		bool bIncludeSocketsInBoneMap);

	ENGINE_API void CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList);
	ENGINE_API void DestroyRenderThreadResources();

	void SetUniqueAnimationCount(uint32 InUniqueAnimationCount)
	{
		UniqueAnimationCount = FMath::Max(InUniqueAnimationCount, 1u);
	}

	ENGINE_API const FGuid& GetTransformProviderId() const override;

	FTransformProviderRenderProxy* GetTransformProviderProxy() const
	{
		return TransformProviderProxy;
	}

	void SetUserData(FInstancedSkinningSceneExtensionProxyUserData* InUserData)
	{
		UserData = InUserData;
	}

	FInstancedSkinningSceneExtensionProxyUserData* GetUserData() const
	{
		return UserData;
	}

	FInstancedSkinningSceneExtensionProxyUserData& GetUserDataChecked() const
	{
		check(UserData);
		return *UserData;
	}

	void SetBoneAttachmentSockets(TArray<FBoneAttachmentSocket>&& InSockets)
	{
		BoneAttachmentSockets = MoveTemp(InSockets);
	}

	void SetBoneAttachmentBindings(TArray<FBoneAttachmentBinding>&& InBindings)
	{
		BoneAttachmentBindings = MoveTemp(InBindings);
	}

	/** Resize the bindings array to match instance count. */
	void ResizeBoneAttachmentBindings(int32 NewNum)
	{
		BoneAttachmentBindings.SetNum(NewNum);
	}

	/** Update a single binding entry. Index must be in range. */
	void SetBoneAttachmentBinding(int32 Index, FBoneAttachmentBinding Binding)
	{
		check(BoneAttachmentBindings.IsValidIndex(Index));
		BoneAttachmentBindings[Index] = Binding;
	}

	TConstArrayView<FBoneAttachmentSocket> GetBoneAttachmentSockets() const
	{
		return BoneAttachmentSockets;
	}

	TConstArrayView<FBoneAttachmentBinding> GetBoneAttachmentBindings() const
	{
		return BoneAttachmentBindings;
	}

	int32 GetNumBoneAttachmentBindings() const
	{
		return BoneAttachmentBindings.Num();
	}

	bool HasBoneAttachments() const
	{
		return BoneAttachmentSockets.Num() > 0;
	}

protected:
	FTransformProviderRenderProxy* TransformProviderProxy = nullptr;
	FInstancedSkinningSceneExtensionProxyUserData* UserData = nullptr;
	FGuid TransformProviderId;
	TArray<FBoneAttachmentSocket> BoneAttachmentSockets;
	TArray<FBoneAttachmentBinding> BoneAttachmentBindings;
};