// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "RendererInterface.h"
#include "Rendering/RenderingCommon.h"
#include "CanvasTypes.h"
#include "Widgets/SLeafWidget.h"
#include "Engine/TextureRenderTarget2D.h"

class FRHICommandListImmediate;
class FSceneViewport;
class FViewportClient;

typedef TSharedPtr<FCanvas, ESPMode::ThreadSafe> FCanvasPtr;

/** Widget wrapper that paints the debug canvas */
class SDebugCanvas : public SLeafWidget
{
	SLATE_BEGIN_ARGS(SDebugCanvas)
	{
		_Visibility = EVisibility::HitTestInvisible;
	}

	SLATE_ATTRIBUTE(FSceneViewport*, SceneViewport)

	SLATE_END_ARGS()

public:
	ENGINE_API SDebugCanvas();

	ENGINE_API void Construct(const FArguments& InArgs);

	/** SWidget interface */
	ENGINE_API virtual bool CustomPrepass(float LayoutScaleMultiplier) override;
	ENGINE_API int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	ENGINE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

	/** Sets the scene viewport that owns the canvas to draw */
	ENGINE_API void SetSceneViewport(FSceneViewport* InSceneViewport);
private:
	/** Viewport used for canvas rendering */
	TAttribute<FSceneViewport*> SceneViewport;
};

/**
 * Custom Slate drawer to render a debug canvas on top of a Slate window
 */
class FDebugCanvasDrawer : public ICustomSlateElement, public FGCObject
{
public:
	FDebugCanvasDrawer();
	~FDebugCanvasDrawer();

	/** @return The debug canvas that the game thread can use */
	FCanvas* GetGameThreadDebugCanvas();

	/**
	 * Sets up the canvas for rendering
	 */
	void BeginRenderingCanvas( const FIntRect& InCanvasRect );

	/**
	 * Creates a new debug canvas and enqueues the previous one for deletion
	 */
	void InitDebugCanvas(FViewportClient* ViewportClient, UWorld* InWorld);

	/** 
	* Releases rendering resources
	*/
	UE_DEPRECATED(5.6, "There are no longer any render thread resources to release. Call ReleaseInternalTexture() on the game thread instead.")
	void ReleaseResources() {}

	/**
	 * Release the internal layer texture (game thread)
	 */
	void ReleaseInternalTexture();

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FDebugCanvasDrawer"); }

private:
	/**
	 * ICustomSlateElement interface 
	 */
	virtual void Draw_RenderThread(FRDGBuilder& GraphBuilder, const FDrawPassInputs& Inputs) override;

	/**
	 * Deletes the rendering thread canvas 
	 */
	void DeleteRenderThreadCanvas();

	/**
	 * Gets the render thread canvas 
	 */
	FCanvasPtr GetRenderThreadCanvas();

	/**
	 * Set the canvas that can be used by the render thread
	 */
	void SetRenderThreadCanvas(const FIntRect& InCanvasRect, FCanvasPtr& Canvas, class FTextureRenderTargetResource* LayerResource);

	/**
	 * Called after a font cache has released its rendering resources
	 */
	void HandleReleaseFontResources(const class FSlateFontCache& InFontCache);

private:
	/** The canvas that can be used by the game thread */
	FCanvasPtr GameThreadCanvas;
	/** The canvas that can be used by the render thread */
	FCanvasPtr RenderThreadCanvas;
	/** Render target that the canvas renders to */
	class FSlateCanvasRenderTarget* RenderTarget;
	/** Rendertarget used in case of self textured canvas */
	TObjectPtr<UTextureRenderTarget2D> LayerTexture;
	/** LayerTexture's resource, for use on the render thread */
	class FTextureRenderTargetResource* LayerResource = nullptr;
	/** HMD layer ID */
	uint32 LayerID;
	/** true if the RenderThreadCanvas rendered elements last frame */
	bool bCanvasRenderedLastFrame;
};
