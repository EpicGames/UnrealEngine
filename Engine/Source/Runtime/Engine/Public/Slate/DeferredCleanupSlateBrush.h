// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderDeferredCleanup.h"
#include "UObject/GCObject.h"
#include "Styling/SlateBrush.h"

class FDeferredCleanupSlateBrush : public ISlateBrushSource, public FDeferredCleanupInterface, public FGCObject
{
public:
	static ENGINE_API TSharedRef<FDeferredCleanupSlateBrush> CreateBrush(const FSlateBrush& Brush);

	static ENGINE_API TSharedRef<FDeferredCleanupSlateBrush> CreateBrush(
		class UTexture* InTexture,
		const FLinearColor& InTint = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f),
		ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile,
		ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor,
		ESlateBrushDrawType::Type InDrawType = ESlateBrushDrawType::Image,
		const FMargin& InMargin = FMargin(0.0f));

	static ENGINE_API TSharedRef<FDeferredCleanupSlateBrush> CreateBrush(
		class UObject* InResource,
		const UE::Slate::FDeprecateVector2DParameter& InImageSize,
		const FLinearColor& InTint = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f),
		ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile,
		ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor,
		ESlateBrushDrawType::Type InDrawType = ESlateBrushDrawType::Image,
		const FMargin& InMargin = FMargin(0.0f));

	virtual const FSlateBrush* GetSlateBrush() const override { return &InternalBrush; }

	// FGCObject
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	ENGINE_API virtual FString GetReferencerName() const override;

	static const FSlateBrush* TrySlateBrush(const TSharedPtr<FDeferredCleanupSlateBrush>& DeferredSlateBrush)
	{
		return DeferredSlateBrush.IsValid() ? DeferredSlateBrush->GetSlateBrush() : nullptr;
	}

private:
	ENGINE_API FDeferredCleanupSlateBrush();

	FDeferredCleanupSlateBrush(const FSlateBrush& Brush)
		: InternalBrush(Brush)
	{
	}

	FSlateBrush InternalBrush;
};
