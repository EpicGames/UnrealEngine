// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Geometry.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SCanvas.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/SlateTypes.h"
#include "UObject/ObjectKey.h"

#include "SGameLayerManager.generated.h"

class FPaintArgs;
class FSceneViewport;
class FSlateWindowElementList;
class SOverlay;
class STooltipPresenter;
class UGameViewportClient;
class ULocalPlayer;
class SDebugCanvas;
class SWindowTitleBarArea;
class SVerticalBox;

/**
 * Allows you to provide a custom layer that multiple sources can contribute to.  Unlike
 * adding widgets directly to the layer manager.  First registering a layer with a name
 * allows multiple widgets to be added.
 */
class IGameLayer : public TSharedFromThis<IGameLayer>
{
public:
	/** Get the layer as a widget. */
	virtual TSharedRef<SWidget> AsWidget() = 0;

public:

	/** Virtual destructor. */
	virtual ~IGameLayer() = default;
};

UENUM()
enum class EWindowTitleBarMode : uint8
{
	Overlay,
	VerticalBox
};

/**
 * Allows widgets to be managed for different users.
 */
class IGameLayerManager
{
public:
	virtual void SetSceneViewport(FSceneViewport* SceneViewport) = 0;

	virtual FGeometry GetViewportWidgetHostGeometry() const = 0;
	virtual FGeometry GetViewportWidgetHostPaintGeometry() const = 0;
	virtual FGeometry GetPlayerWidgetHostGeometry(ULocalPlayer* Player) const = 0;

	virtual void NotifyPlayerAdded(int32 PlayerIndex, ULocalPlayer* AddedPlayer) = 0;
	virtual void NotifyPlayerRemoved(int32 PlayerIndex, ULocalPlayer* RemovedPlayer) = 0;

	virtual void AddWidgetForPlayer(ULocalPlayer* Player, TSharedRef<SWidget> ViewportContent, int32 ZOrder) = 0;
	virtual void RemoveWidgetForPlayer(ULocalPlayer* Player, TSharedRef<SWidget> ViewportContent) = 0;
	virtual void ClearWidgetsForPlayer(ULocalPlayer* Player) = 0;

	virtual TSharedPtr<IGameLayer> FindLayerForPlayer(ULocalPlayer* Player, const FName& LayerName) = 0;
	virtual bool AddLayerForPlayer(ULocalPlayer* Player, const FName& LayerName, TSharedRef<IGameLayer> Layer, int32 ZOrder) = 0;

	virtual void ClearWidgets() = 0;

	virtual void AddGameLayer(TSharedRef<SWidget> ViewportContent, int32 ZOrder) = 0;
	virtual void RemoveGameLayer(TSharedRef<SWidget> ViewportContent) = 0;

	virtual void SetDefaultWindowTitleBarHeight(float Height) = 0;
	virtual void SetWindowTitleBarState(const TSharedPtr<SWidget>& TitleBarContent, EWindowTitleBarMode Mode, bool bTitleBarDragEnabled, bool bWindowButtonsVisible, bool bTitleBarVisible) = 0;
	virtual void RestorePreviousWindowTitleBarState() = 0;
	virtual void SetWindowTitleBarVisibility(bool bIsVisible) = 0;
	virtual TSharedRef<SWidget> AsWidget() = 0;
};

/**
 * 
 */
class SGameLayerManager : public SCompoundWidget, public IGameLayerManager
{
public:

	SLATE_BEGIN_ARGS(SGameLayerManager)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

		/** Slot for this content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_ATTRIBUTE(FSceneViewport*, SceneViewport)

	SLATE_END_ARGS()

	enum class EGameLayerOrder : int32
	{
		Player = 100,
		Viewport = 200,
		TitleBar = 300,
		Tooltip = 400,
		Debug = 500,

		// Add others with an increment of 100 to have enough space for injecting new layers in-between
	};

	ENGINE_API SGameLayerManager();

	ENGINE_API virtual ~SGameLayerManager();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	ENGINE_API void Construct( const FArguments& InArgs );

	// Begin IGameLayerManager
	ENGINE_API virtual void SetSceneViewport(FSceneViewport* InSceneViewport) override;
	ENGINE_API virtual FGeometry GetViewportWidgetHostGeometry() const override;
	ENGINE_API virtual FGeometry GetViewportWidgetHostPaintGeometry() const override;
	ENGINE_API virtual FGeometry GetPlayerWidgetHostGeometry(ULocalPlayer* Player) const override;

	ENGINE_API virtual void NotifyPlayerAdded(int32 PlayerIndex, ULocalPlayer* AddedPlayer) override;
	ENGINE_API virtual void NotifyPlayerRemoved(int32 PlayerIndex, ULocalPlayer* RemovedPlayer) override;

	ENGINE_API virtual void AddWidgetForPlayer(ULocalPlayer* Player, TSharedRef<SWidget> ViewportContent, int32 ZOrder) override;
	ENGINE_API virtual void RemoveWidgetForPlayer(ULocalPlayer* Player, TSharedRef<SWidget> ViewportContent) override;
	ENGINE_API virtual void ClearWidgetsForPlayer(ULocalPlayer* Player) override;

	ENGINE_API virtual TSharedPtr<IGameLayer> FindLayerForPlayer(ULocalPlayer* Player, const FName& LayerName) override;
	ENGINE_API virtual bool AddLayerForPlayer(ULocalPlayer* Player, const FName& LayerName, TSharedRef<IGameLayer> Layer, int32 ZOrder) override;

	ENGINE_API virtual void ClearWidgets() override;

	ENGINE_API virtual void AddGameLayer(TSharedRef<SWidget> ViewportContent, int32 ZOrder) override;
	ENGINE_API virtual void RemoveGameLayer(TSharedRef<SWidget> ViewportContent) override;

	ENGINE_API virtual void SetDefaultWindowTitleBarHeight(float Height);
	ENGINE_API virtual void SetWindowTitleBarState(const TSharedPtr<SWidget>& TitleBarContent, EWindowTitleBarMode Mode, bool bTitleBarDragEnabled, bool bWindowButtonsVisible, bool bTitleBarVisible);
	ENGINE_API virtual void RestorePreviousWindowTitleBarState();
	ENGINE_API virtual void SetWindowTitleBarVisibility(bool bIsVisible);
	ENGINE_API virtual TSharedRef<SWidget> AsWidget() override;
	// End IGameLayerManager

	//~ Begin SWidget overrides
protected:
	virtual bool CustomPrepass(float LayoutScaleMultiplier) override;
	ENGINE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	ENGINE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	ENGINE_API virtual bool OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent) override;
	//~ End SWidget overrides

public:
	/**
	 * Function will instruct internal DPI computations to use a provided reference viewport size instead of the actual viewport size.
	 * After the DPI will be retrieved it will be scaled down with the ratio between the actual viewport size and the provided one.
	 * Check GetGameViewportDPIScale() for more information.
	 */
	ENGINE_API void SetUseFixedDPIValue(const bool bUseFixedDPI, const FIntPoint RefViewportSize = FIntPoint());
	ENGINE_API bool IsUsingFixedDPIValue() const;

private:
	float GetGameViewportDPIScale() const;
	FOptionalSize GetDefaultWindowTitleBarHeight() const;

	//~ Functions for setting the visibility of different canvases containing HUD.
	void ShowPlayerCanvas(bool bIsVisible);
	void ShowDebugCanvas(bool bIsVisible);
	void ShowViewportSlot(bool bIsVisible);

private:

	struct FPlayerLayer : TSharedFromThis<FPlayerLayer>
	{
		TSharedPtr<SOverlay> Widget;
		SCanvas::FSlot* Slot;

		TMap< FName, TSharedPtr<IGameLayer> > Layers;

		FPlayerLayer()
			: Slot(nullptr)
		{
		}
	};

	void UpdateLayout();

	TSharedPtr<FPlayerLayer> FindOrCreatePlayerLayer(ULocalPlayer* LocalPlayer);
	void RemoveMissingPlayerLayers(const TArray<ULocalPlayer*>& GamePlayers);
	void RemovePlayerWidgets(ULocalPlayer* LocalPlayer);
	void AddOrUpdatePlayerLayers(const FGeometry& AllottedGeometry, UGameViewportClient* ViewportClient, const TArray<ULocalPlayer*>& GamePlayers);
	bool GetNormalizeRect(ULocalPlayer* LocalPlayer, FVector2D& OutPosition, FVector2D& OutSize) const;

	void UpdateWindowTitleBar();
	void UpdateWindowTitleBarVisibility();
	void RequestToggleFullscreen();

private:
	FGeometry CachedGeometry;

	TMap<FObjectKey, TSharedPtr<FPlayerLayer>> PlayerLayers;
	TArray<TWeakPtr<SWidget>> GameLayers;

	TAttribute<FSceneViewport*> SceneViewport;
	TSharedPtr<SVerticalBox> WidgetHost;
	TSharedPtr<SCanvas> PlayerCanvas;
	TSharedPtr<SDebugCanvas> DebugCanvas;
	TSharedPtr<SBox> ViewportSlotContainer;
	TSharedPtr<STooltipPresenter> TooltipPresenter;

	TSharedPtr<SWindowTitleBarArea> TitleBarAreaOverlay;
	TSharedPtr<SWindowTitleBarArea> TitleBarAreaVerticalBox;
	TSharedPtr<SBox> WindowTitleBarVerticalBox;
	TSharedPtr<SBox> WindowTitleBarOverlay;
	TSharedPtr<SOverlay> WindowOverlay;

	struct FWindowTitleBarState
	{
		TSharedPtr<SWidget> ContentWidget;
		EWindowTitleBarMode Mode;
		bool bTitleBarDragEnabled;
		bool bWindowButtonsVisible;
		bool bTitleBarVisible;

		FWindowTitleBarState() : ContentWidget(), Mode(EWindowTitleBarMode::Overlay), bTitleBarDragEnabled(false), bWindowButtonsVisible(false), bTitleBarVisible(false) {}
		FWindowTitleBarState(const TSharedPtr<SWidget>& TitleBarContent, EWindowTitleBarMode InMode, bool bInTitleBarDragEnabled, bool bInWindowButtonsVisible, bool bInTitleBarVisible)
			: ContentWidget(TitleBarContent)
			, Mode(InMode)
			, bTitleBarDragEnabled(bInTitleBarDragEnabled)
			, bWindowButtonsVisible(bInWindowButtonsVisible && (PLATFORM_WINDOWS || PLATFORM_LINUX))
			, bTitleBarVisible(bInTitleBarVisible && PLATFORM_DESKTOP)
		{
		}
	};

	FWindowTitleBarState WindowTitleBarState;
	TSharedPtr<SWidget> DefaultTitleBarContentWidget;
	float DefaultWindowTitleBarHeight;
	bool bIsGameUsingBorderlessWindow;
	FIntPoint ScaledDPIViewportReference;
	bool bUseScaledDPI;
	float CachedInverseDPIScale;
};
