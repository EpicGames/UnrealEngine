// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Layout/Geometry.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Input/PopupMethodReply.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"
#include "Widgets/SViewport.h"
#include "Rendering/RenderingCommon.h"
#include "UnrealClient.h"

class FCanvas;
class FDebugCanvasDrawer;
class FSlateRenderer;
class FSlateRenderTargetRHI;
class FSlateWindowElementList;
class FViewportClient;
class SDebugCanvas;
class SWindow;

/** Called in FSceneViewport::ResizeFrame after ResizeViewport*/
DECLARE_DELEGATE_OneParam( FOnSceneViewportResize, FVector2D );

/**
 * A viewport for use with Slate SViewport widgets.
 */
class FSceneViewport : public FViewport, public ISlateViewport
{
public:
	ENGINE_API FSceneViewport(const TSharedPtr<SViewport>& Widget);

	UE_DEPRECATED(5.8, "Use FViewport() + SetViewportClient or static Create(TStrongPtrVariant<FViewportClient>) instead")
	ENGINE_API FSceneViewport(FViewportClient* const Client, const TSharedPtr<SViewport>& Widget);

	// This constructor prevents ambiguity between FViewportClient* and TStrongPtrVariant<FViewportClient> when given a UScriptViewportClient
	UE_DEPRECATED(5.8, "Use FViewport() + SetViewportClient or static Create(TStrongPtrVariant<FViewportClient>) instead")
	ENGINE_API FSceneViewport(UScriptViewportClient* const Client, const TSharedPtr<SViewport>& Widget);

protected:
	template<typename T>
	requires std::is_base_of_v<FSceneViewport, T>
	static TSharedRef<T> Create(const TStrongPtrVariant<FViewportClient>& InClient, const TSharedPtr<SViewport>& Widget)
	{
		TSharedRef<T> Self = MakeShared<T>(Widget);
		if (Widget.IsValid())
		{
			Widget->SetViewportInterface(Self);
		}
		Self->SetViewportClient(InClient);
		return Self;
	}

public:
	static TSharedRef<FSceneViewport> Create(const TStrongPtrVariant<FViewportClient>& InClient, const TSharedPtr<SViewport>& Widget)
	{
		return Create<FSceneViewport>(InClient, Widget);
	}

	ENGINE_API ~FSceneViewport();

	virtual void* GetWindow() override { return NULL; }

	/** FViewport interface */
	virtual void MoveWindow(int32 NewPosX, int32 NewPosY, int32 NewSizeX, int32 NewSizeY) override {}
	ENGINE_API virtual void SetInitialSize(FIntPoint InitialSizeXY) override;
	ENGINE_API virtual bool HasMouseCapture() const override;
	ENGINE_API virtual bool HasFocus() const override;
	ENGINE_API virtual bool IsForegroundWindow() const override;
	ENGINE_API virtual void CaptureMouse( bool bCapture ) override;
	ENGINE_API virtual void LockMouseToViewport( bool bLock ) override;
	ENGINE_API virtual void ShowCursor( bool bVisible ) override;
	ENGINE_API virtual void SetPreCaptureMousePosFromSlateCursor() override;
	virtual bool IsCursorVisible() const override { return bIsCursorVisible; }
	virtual void ShowSoftwareCursor( bool bVisible ) override { bIsSoftwareCursorVisible = bVisible; }
	virtual void SetSoftwareCursorPosition( FVector2D Position ) override { SoftwareCursorPosition = Position; }
	virtual bool IsSoftwareCursorVisible() const override { return bIsSoftwareCursorVisible; }
	virtual FVector2D GetSoftwareCursorPosition() const override { return SoftwareCursorPosition; }
	ENGINE_API virtual FCanvas* GetDebugCanvas() override;
	ENGINE_API virtual float GetDisplayGamma() const override;
	ENGINE_API virtual void EnqueueBeginRenderFrame(const bool bShouldPresent) override;
	ENGINE_API virtual void EnqueueEndRenderFrame(const bool bLockToVsync, const bool bShouldPresent) override;
	ENGINE_API virtual int32 GetCurrentMonitorIndex() const override;
	ENGINE_API virtual bool IsExclusiveFullscreenSupported() const override;
	ENGINE_API virtual void SetViewportClient(const TStrongPtrVariant<FViewportClient>& InViewportClient) override;

	/** Gets the proper RenderTarget based on the current thread*/
	ENGINE_API virtual const FTextureRHIRef& GetRenderTargetTexture() const;

	ENGINE_API virtual void SetRenderTargetTextureRenderThread(FRHITexture* InRenderTargetTexture);

	/**
	 * Captures or uncaptures the joystick
	 *
	 * @param Capture	true if we should capture, false if we should uncapture
	 */
	ENGINE_API virtual bool SetUserFocus(bool bFocus) override;

	/**
	 * Returns the state of the provided key. 
	 *
	 * @param Key	The name of the key to check
	 *
	 * @return true if the key is pressed, false otherwise
	 */
	ENGINE_API virtual bool KeyState(FKey Key) const override;

	/**
	 * @return The current X position of the mouse (in local space, relative to the viewports geometry)                 
	 */
	ENGINE_API virtual int32 GetMouseX() const override;

	/**
	 * @return The current Y position of the mouse (in local space, relative to the viewports geometry)                 
	 */
	ENGINE_API virtual int32 GetMouseY() const override;

	/**
	 * Sets MousePosition to the current mouse position 
	 *
	 * @param MousePosition	Populated with the current mouse position     
	 * @param bLocalPosition Indicates whether the mouse position returned should be in local or absolute space
	 */
	ENGINE_API virtual void GetMousePos( FIntPoint& MousePosition, const bool bLocalPosition = true) override;

	/**
	 * Retrieve current unbounded mouse position. Valid while a mouse button is pressed.
	 * Useful e.g. for drag operations hiding the mouse cursor.
	 */
	ENGINE_API virtual void GetUnboundedMousePos(FIntPoint& OutUnboundedMousePosition, const bool bLocalPosition = true) override;

	/**
	 * Not implemented                   
	 */
	ENGINE_API virtual void SetMouse( int32 X, int32 Y ) override;

	/**
	 * Additional input processing that happens every frame                   
	 */
	ENGINE_API virtual void ProcessInput( float DeltaTime ) override;
	
	ENGINE_API virtual FVector2D VirtualDesktopPixelToViewport(FIntPoint VirtualDesktopPointPx) const override;
	ENGINE_API virtual FIntPoint ViewportToVirtualDesktopPixel(FVector2D ViewportCoordinate) const override;

	/**
	 * Called when the viewport should be invalidated and redrawn                   
	 */
	ENGINE_API virtual void InvalidateDisplay() override;

	/**
	 * Invalidates the viewport's cached hit proxies at the end of the frame.
	 */
	ENGINE_API virtual void DeferInvalidateHitProxy() override;

	/** FViewportFrame interface */
	UE_DEPRECATED(5.8, "FSceneViewport::GetViewport() is deprecated because FViewportFrame is also deprecated. This function call is unnecessary and can be removed.")
	FViewport* GetViewport() { return this; }

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "FSceneViewport::GetViewportFrame() and FViewportFrame are deprecated. Use the scene viewport directly.")
	virtual FViewportFrame* GetViewportFrame() override { return this; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual FSceneViewport* AsSceneViewport() override { return this; }

	/** @return The viewport widget being used */
	TWeakPtr<SViewport> GetViewportWidget() const { return ViewportWidget; }

	/**
	 * Ticks the viewport
	 */
	ENGINE_API virtual void Tick( const FGeometry& AllottedGeometry, double InCurrentTime, float InDeltaTime ) override;

	/**
	 * Performs a resize when in swapping viewports while viewing the play world.
	 *
	 * @param OtherViewport	The previously active viewport
	 */
	ENGINE_API void OnPlayWorldViewportSwapped( const FSceneViewport& OtherViewport );

	/**
	 * Swaps the active stats with another viewports
	 *
	 * @param OtherViewport	The previously active viewport
	 */
	ENGINE_API void SwapStatCommands(const FSceneViewport& OtherViewport);

	/**
	 * Indicate that the viewport should be block for vsync.
	 */
	virtual void SetRequiresVsync(bool bShouldVsync) override { bRequiresVsync = bShouldVsync; }

	/**
	* Indicate that the viewport had a camera cut.
	*/
	virtual void SetCameraCut(bool bInCameraCut) override { bCameraCut = bInCameraCut; }

	/**
	 * Returns true if the viewport should be vsynced.
	 */
	virtual bool RequiresVsync() const override { return bRequiresVsync; }

	void SetMonitorIndexForResize(int32 MonitorIndex) { MonitorIndexForResize = MonitorIndex; }

	/**
	 * Called to resize the actual window where this viewport resides
	 *
	 * @param NewSizeX		The new width of the viewport
	 * @param NewSizeY		The new height of the viewport
	 * @param NewWindowMode	 What window mode should the viewport be resized to
	 */
	ENGINE_API void ResizeFrame(uint32 NewSizeX,uint32 NewSizeY,EWindowMode::Type NewWindowMode);

	/**
	 *	Sets the Viewport resize delegate.
	 */
	void SetOnSceneViewportResizeDel(FOnSceneViewportResize InOnSceneViewportResize) 
	{ 
		OnSceneViewportResizeDel = InOnSceneViewportResize; 
	}

	/** 
	* Sets whether a PIE viewport takes mouse control on startup.
	* @param bGetsMouseControl Takes control if true, or not if false. 
	*/
	void SetPlayInEditorGetsMouseControl(const bool bGetsMouseControl)
	{
		bShouldCaptureMouseOnActivate = bGetsMouseControl;
	}

	void SetPlayInEditorIsSimulate(const bool bIsSimulate)
	{
		bPlayInEditorIsSimulate = bIsSimulate;
	}
	bool GetPlayInEditorIsSimulate() const
	{
		return bPlayInEditorIsSimulate;
	}

	/** Updates the viewport RHI with a new size and fullscreen flag */
	ENGINE_API void UpdateViewportRHI(bool bDestroyed, uint32 NewSizeX, uint32 NewSizeY, EWindowMode::Type NewWindowMode, EPixelFormat PreferredPixelFormat);

	/** ISlateViewport interface */
	ENGINE_API virtual bool IsCameraCut() const override;
	ENGINE_API virtual FSlateShaderResource* GetViewportRenderTargetTexture() const override;
	ENGINE_API virtual TUniquePtr<UE::Slate::FPaintScope> Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, const bool bParentEnabled) override;
	ENGINE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) override;
	ENGINE_API virtual TOptional<TSharedRef<SWidget>> OnMapCursor(const FCursorReply& CursorReply) override;
	ENGINE_API virtual FReply OnMouseButtonDown( const FGeometry& InGeometry, const FPointerEvent& MouseEvent ) override;
	ENGINE_API virtual FReply OnMouseButtonUp( const FGeometry& InGeometry, const FPointerEvent& MouseEvent ) override;
	ENGINE_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	ENGINE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	ENGINE_API virtual FReply OnMouseMove( const FGeometry& InGeometry, const FPointerEvent& MouseEvent ) override;
	ENGINE_API virtual FReply OnMouseWheel( const FGeometry& InGeometry, const FPointerEvent& MouseEvent ) override;
	ENGINE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent ) override;
	ENGINE_API virtual FReply OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	ENGINE_API virtual FReply OnTouchMoved( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	ENGINE_API virtual FReply OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	ENGINE_API virtual FReply OnTouchForceChanged(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent) override;
	ENGINE_API virtual FReply OnTouchFirstMove(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent) override;
	ENGINE_API virtual FReply OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& InGestureEvent) override;
	ENGINE_API virtual FReply OnMotionDetected( const FGeometry& MyGeometry, const FMotionEvent& InMotionEvent ) override;
	ENGINE_API virtual FPopupMethodReply OnQueryPopupMethod() const override;
	ENGINE_API virtual bool HandleNavigation(const uint32 InUserIndex, TSharedPtr<SWidget> InDestination) override;
	ENGINE_API virtual TOptional<bool> OnQueryShowFocus(const EFocusCause InFocusCause) const override;
	ENGINE_API virtual void OnFinishedPointerInput() override;
	ENGINE_API virtual FReply OnKeyDown( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent ) override;
	ENGINE_API virtual FReply OnKeyUp( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent ) override;
	ENGINE_API virtual FReply OnAnalogValueChanged( const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent ) override;
	ENGINE_API virtual FReply OnKeyChar( const FGeometry& InGeometry, const FCharacterEvent& InCharacterEvent ) override;
	ENGINE_API virtual FReply OnFocusReceived( const FFocusEvent& InFocusEvent ) override;
	ENGINE_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	ENGINE_API virtual void OnViewportClosed() override;
	ENGINE_API virtual FReply OnRequestWindowClose() override;
	ENGINE_API virtual TWeakPtr<SWidget> GetWidget() override;
	ENGINE_API virtual FReply OnViewportActivated(const FWindowActivateEvent& InActivateEvent) override;
	ENGINE_API virtual void OnViewportDeactivated(const FWindowActivateEvent& InActivateEvent) override;
	virtual FIntPoint GetSize() const override { return GetSizeXY(); }
	ENGINE_API virtual EDisplayColorGamut GetDisplayColorGamut() const override;
	ENGINE_API virtual EDisplayOutputFormat GetDisplayOutputFormat() const override;
	ENGINE_API virtual bool GetSceneHDREnabled() const override;
	ENGINE_API virtual float GetMinimumLuminanceInNits() const override;
	ENGINE_API virtual float GetMaximumLuminanceInNits() const override;
	ENGINE_API virtual float GetMaximumFullFrameLuminanceInNits() const override;
	ENGINE_API virtual float GetHDRPaperWhiteInNits() const override;
	ENGINE_API virtual const UE::Color::FColorSpace& GetLimitingColorSpace() const override;
	ENGINE_API virtual ESlateViewportDynamicRange GetViewportDynamicRange() const override;

	ENGINE_API void SetViewportSize(uint32 NewSizeX,uint32 NewSizeY);
	ENGINE_API void SetFixedViewportSize(uint32 NewSizeX, uint32 NewSizeY);

	/** Does the viewport has a fixed size */
	ENGINE_API bool HasFixedSize() const;

	ENGINE_API TSharedPtr<SWindow> FindWindow() const;

	/** Should return true, if stereo rendering is allowed in this viewport */
	ENGINE_API virtual bool IsStereoRenderingAllowed() const override;

	/** Returns dimensions of RenderTarget texture. Can be called on a game thread. */
	virtual FIntPoint GetRenderTargetTextureSizeXY() const { return (RTTSize.X != 0) ? RTTSize : GetSizeXY(); }

	/** Returns format for the scene of this viewport. */
	EPixelFormat GetSceneTargetFormat() const override { return SceneTargetFormat; }

	/** Get the cached viewport geometry. */
	const FGeometry& GetCachedGeometry() const { return CachedGeometry; }

	/** Set an optional display gamma to use for this viewport */
	void SetGammaOverride(const float InGammaOverride)
	{
		ViewportGammaOverride = InGammaOverride;
	};

	/** Sets the debug canvas used to display FCanvas on top of this viewport */
	ENGINE_API void SetDebugCanvas(TSharedPtr<SDebugCanvas> InDebugCanvas);

	/** Adds a draw element for the debug canvas.  Called externally by a widget that manages where the debug canvas draws */
	ENGINE_API void PaintDebugCanvas(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

private:
	/**
	 * Called when this viewport is destroyed
	 */
	ENGINE_API void Destroy() override;

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;

	virtual FString GetFriendlyName() const override { return FString("FSlateSceneViewport");}

	/**
	 * Called from Slate when the viewport should be resized
	 *
	 * @param NewSizeX		 The new width of the viewport
	 * @param NewSizeY		 The new height of the viewport
	 * @param NewWindowMode	 What window mode should the viewport be resized to
	 */
	ENGINE_API virtual void ResizeViewport( uint32 NewSizeX,uint32 NewSizeY,EWindowMode::Type NewWindowMode );

	/**
	 * Called from slate when input is finished for this frame, and we should process any accumulated mouse data.
	 */
	ENGINE_API void ProcessAccumulatedPointerInput();

	/**
	 * Updates the cached mouse position from a mouse event
	 *
	 * @param InGeometry	The geometry of the viewport to convert to local space
	 * @param InMouseEvent	The mouse event containing the position of the mouse in absolute space
	 */
	ENGINE_API void UpdateCachedCursorPos( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent, bool bIsMoveEvent = false);

	/**
	 * Updates the cached viewport geometry
	 *
	 * @param InGeometry	The geometry of the viewport to convert to local space
	 * @param InMouseEvent	The mouse event containing the position of the mouse in absolute space
	 */
	ENGINE_API void UpdateCachedGeometry( const FGeometry& InGeometry );

	/**
	 * Updates the KeyStateMap via the modifier keys from a mouse event.
	 * This ensures that the key state is correct after focus changes.
	 *
	 * @param InMouseEvent	The mouse event containing the current state of modifier keys.
	 */
	ENGINE_API void UpdateModifierKeys( const FPointerEvent& InMouseEvent );

	/**
	 * Calls InputKey on the ViewportClient via the modifier keys.
	 * This ensures that the key state is correct just prior to focus change
	 *
	 * @param InKeysState	The key state containing the states of the modifier keys
	 */
	ENGINE_API void ApplyModifierKeys(const FModifierKeysState& InKeysState, const uint64 Timestamp);

	/** Utility function to create an FReply that properly gets Focus and capture based on the settings*/
	ENGINE_API FReply AcquireFocusAndCapture(FIntPoint MousePosition, EFocusCause FocusCause = EFocusCause::SetDirectly);

	/** Utility function to figure out if we are currently a game viewport */
	ENGINE_API bool IsCurrentlyGameViewport();

	/** @return Returns true if we should always render to a separate render target (rather than rendering directly to the
	    viewport backbuffer, taking into account any temporary requirements of head-mounted displays */
	bool UseSeparateRenderTarget() const override
	{
		return bUseSeparateRenderTarget || bForceSeparateRenderTarget;
	}

	ENGINE_API bool IsStereoscopic3D() const override;

	/** @return Returns true if the viewport needs permanent capture. */
	ENGINE_API bool IsInPermanentCapture();

private:
	/** An intermediate reply state that is reset whenever an input event is generated */
	FReply CurrentReplyState;
	/** A mapping of key names to their pressed state */
	TMap<FKey,bool> KeyStateMap;
	/** The last known mouse position in local space, -1, -1 if unknown */
	FIntPoint CachedCursorPos;
	/** 
	 * Mouse position, unbounded: it might be outside the physical display area. 
	 * Valid while a mouse button is pressed - useful e.g. for drag operations hiding the mouse cursor
	 */
	TOptional<FIntPoint> CachedUnboundedCursorPos;
	/** The last known geometry info */
	FGeometry CachedGeometry;
	/** Mouse position before the latest capture */
	FIntPoint PreCaptureCursorPos;
	/**	The current position of the software cursor */
	FVector2D SoftwareCursorPosition;
	/**	Whether the software cursor should be drawn in the viewport */
	bool bIsSoftwareCursorVisible;	
	/** Draws the debug canvas in Slate */
	TSharedPtr<FDebugCanvasDrawer, ESPMode::ThreadSafe> DebugCanvasDrawer;
	/** The Slate viewport widget where this viewport is drawn */
	TWeakPtr<SViewport> ViewportWidget;
	/** Debug canvas widget we invalidate if our FCanvas has draw elements */
	TWeakPtr<SDebugCanvas> DebugCanvas;
	/** The number of input samples in X since input was was last processed */
	int32 NumMouseSamplesX;
	/** The number of input samples in Y since input was was last processed */
	int32 NumMouseSamplesY;
	/** User index supplied by mouse events accumulated into NumMouseSamplesX and NumMouseSamplesY */
	int32 MouseDeltaUserIndex;
	/** The current mouse delta */
	FIntPoint MouseDelta;
	/** true if the cursor is currently visible */
	bool bIsCursorVisible;
	/** true if we had Capture when deactivated */
	bool bShouldCaptureMouseOnActivate;
	/** true if this viewport requires vsync. */
	bool bRequiresVsync;
	/** true if this viewport is in camera cut. */
	bool bCameraCut;
	/** true if this viewport renders to a separate render target.  false to render directly to the windows back buffer */
	bool bUseSeparateRenderTarget;
	/** True if we should force use of a separate render target because the HMD needs it. */
	bool bForceSeparateRenderTarget;
	/** Whether or not we are currently resizing */
	bool bIsResizing;
	/** Do not resize the RenderTarget when resizing */
	bool bForceViewportSize;
	/** The monitor that we want to display the viewport on after ResizeFrame */
	int32 MonitorIndexForResize = -1;
	/** Delegate that is fired off in ResizeFrame after ResizeViewport */
	FOnSceneViewportResize OnSceneViewportResizeDel;
	/** Whether the PIE viewport is currently in simulate in editor mode */
	bool bPlayInEditorIsSimulate;
	/** Whether or not the cursor is hidden when the viewport captures the mouse */
	bool bCursorHiddenDueToCapture;
	/** Whether or not the viewport is in HDR */
	bool bHDRViewport;
	/** Position the cursor was at when we hid it due to capture, so we can put it back afterwards */
	FIntPoint MousePosBeforeHiddenDueToCapture;
	/** Dimensions of RenderTarget texture. */
	FIntPoint RTTSize;
	/** Pixel format of all Buffered RenderTarget textures. */
	EPixelFormat SceneTargetFormat;

	/** Reprojection on some HMD RHI's requires ViewportTargets to be buffered */
	/** The render target used by Slate to draw the viewport.  Can be null if this viewport renders directly to the backbuffer */

	struct FFrameData
	{
		FFrameData() = default;
		FFrameData(FSlateRenderTargetRHI* InSlateHandle)
			: SlateHandle(InSlateHandle)
		{
		}

		FSlateRenderTargetRHI* SlateHandle = nullptr;
		FTextureRHIRef         RenderTarget;
		FTextureRHIRef         ResourceTexture;
	};
	TArray<FFrameData, TInlineAllocator<1>> BufferedFrames;

	FTextureRHIRef RenderTargetTextureRenderThreadRHI;
	FSlateRenderTargetRHI* RenderThreadSlateTexture;

	int32 BufferIndex_GT = 0;
	int32 NextBufferIndex_GT = 0;

	int32 BufferIndex_RT = 0;
	int32 NumBufferedFrames = 0;

	/** Tracks the number of touches currently active on the viewport */
	int32 NumTouches;

	EDisplayColorGamut DisplayColorGamut;
	EDisplayOutputFormat DisplayOutputFormat;
	float MinimumLuminanceInNits;
	float MaximumLuminanceInNits;
	float MaximumFullFrameLuminanceInNits;
	float HDRPaperWhiteInNits;
	UE::Color::FColorSpace LimitingColorSpace;

	/** The optional gamma value to use for this viewport */
	TOptional<float> ViewportGammaOverride;
};
