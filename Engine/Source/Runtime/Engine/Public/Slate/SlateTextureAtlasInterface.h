// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "Engine/Texture.h"
#include "SlateTextureAtlasInterface.generated.h"

/**
 * The data representing a region of a UTexture atlas.
 */
struct FSlateAtlasData
{
public:
	FSlateAtlasData(UTexture* InAtlasTexture, FVector2D InStartUV, FVector2D InSizeUV)
		: AtlasTexture(InAtlasTexture)
		, StartUV(InStartUV)
		, SizeUV(InSizeUV)
	{
	}

public:
	/** The texture pointer for the Atlas */
	UTexture* AtlasTexture;

	/** The region start position in UVs */
	FVector2D StartUV;

	/** The region size in UVs. */
	FVector2D SizeUV;

	/**
	 * Gets the dimensions of the atlas region in pixel coordinates.
	 */
	FVector2D GetSourceDimensions() const
	{
		if ( AtlasTexture )
		{
			return FVector2D(AtlasTexture->GetSurfaceWidth() * SizeUV.X, AtlasTexture->GetSurfaceHeight() * SizeUV.Y);
		}
		else
		{
			return FVector2D(0, 0);
		}
	}
};


/**
 * 
 */
UINTERFACE(meta=( CannotImplementInterfaceInBlueprint ), MinimalAPI)
class USlateTextureAtlasInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/**
 * 
 */
class ISlateTextureAtlasInterface
{
	GENERATED_IINTERFACE_BODY()

public:

	/** Gets the atlas data to use when rendering with slate. */
	virtual FSlateAtlasData GetSlateAtlasData() const = 0;
};
