// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "Textures/SlateTextureData.h"
#include "RenderDeferredCleanup.h"
#include "Textures/SlateShaderResource.h"
#include "TextureResource.h"
#include "Engine/Texture.h"
#include "Textures/SlateUpdatableTexture.h"

/**
 * Encapsulates a Texture2DRHIRef for use by a Slate rendering implementation                   
 */
class FSlateTexture2DRHIRef : public TSlateTexture<FTextureRHIRef>, public FSlateUpdatableTexture, public FDeferredCleanupInterface, public FRenderResource
{
public:
	ENGINE_API FSlateTexture2DRHIRef( FTextureRHIRef InRef, uint32 InWidth, uint32 InHeight );
	ENGINE_API FSlateTexture2DRHIRef( uint32 InWidth, uint32 InHeight, EPixelFormat InPixelFormat, TSharedPtr<FSlateTextureData, ESPMode::ThreadSafe> InTextureData, ETextureCreateFlags InTexCreateFlags = TexCreate_None, bool bCreateEmptyTexture = false );

	ENGINE_API virtual ~FSlateTexture2DRHIRef();

	ENGINE_API virtual void Cleanup() override;

	virtual uint32 GetWidth() const override { return Width; }
	virtual uint32 GetHeight() const override { return Height; }

	/** FRenderResource Interface.  Called when render resources need to be initialized */
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** FRenderResource Interface.  Called when render resources need to be released */
	ENGINE_API virtual void ReleaseRHI() override;

	/**
	 * Resize the texture.  Can only be called on the render thread
	 */
	ENGINE_API void Resize( uint32 Width, uint32 Height );

	/**
	 * @return true if the texture is valid
	 */
	bool IsValid() const { return IsValidRef( ShaderResource ); }

	/** 
	 *  Sets the RHI Ref to use. 
	 */
	ENGINE_API void SetRHIRef( FTextureRHIRef InRenderTargetTexture, uint32 InWidth, uint32 InHeight );

	FTextureRHIRef GetRHIRef() const { return ShaderResource; }

	/**
	 * Sets the bulk data for this texture.  Note: Does not reinitialize the resource,  Can only be used on the render thread
	 *
	 * @param NewTextureData	The new bulk data
	 */
	ENGINE_API void SetTextureData( FSlateTextureDataPtr NewTextureData );
	
	/**
	 * Sets the bulk data for this texture and the format of the rendering resource
	 * Note: Does not reinitialize the resource.  Can only be used on the render thread
	 *
	 * @param NewTextureData	The new texture data
	 * @param InPixelFormat		The format of the texture data
	 * @param InTexCreateFlags	Flags for creating the rendering resource  
	 */
	ENGINE_API void SetTextureData( FSlateTextureDataPtr NewTextureData, EPixelFormat InPixelFormat, ETextureCreateFlags InTexCreateFlags );

	/**
	 * Clears texture data being used.  Can only be accessed on the render thread                   
	 */
	ENGINE_API void ClearTextureData();

	/**
	 * Returns the pixel format of this texture
	 */
	EPixelFormat GetPixelFormat() const { return PixelFormat; }

	// FSlateUpdatableTexture interface
	virtual FSlateShaderResource* GetSlateResource() override {return this;}
	virtual FRenderResource* GetRenderResource() override {return this;}
	ENGINE_API virtual void ResizeTexture( uint32 Width, uint32 Height ) override;
	ENGINE_API virtual void UpdateTexture(const TArray<uint8>& Bytes) override;
	ENGINE_API virtual void UpdateTextureThreadSafe(const TArray<uint8>& Bytes) override;
	ENGINE_API virtual void UpdateTextureThreadSafeRaw(uint32 Width, uint32 Height, const void* Buffer, const FIntRect& Dirty) override;
	ENGINE_API virtual void UpdateTextureThreadSafeWithTextureData(FSlateTextureData* BulkData) override;
	virtual void UpdateTextureThreadSafeWithKeyedTextureHandle(void* TextureHandle, int KeyLockVal, int KeyUnlockVal, const FIntRect& Dirty = FIntRect()) override {}
protected:
	/** Width of this texture */
	uint32 Width;
	/** Height of this texture */
	uint32 Height;
private:
	ENGINE_API void SetTextureData(FRHICommandListImmediate& RHICmdList, const TArray<uint8>& Bytes);

	/** Texture creation flags for if this texture needs to be recreated dynamically */
	ETextureCreateFlags TexCreateFlags;
	/** Data used between ReleaseRHI and InitRHI.  May be null if the data is not used */
	TSharedPtr<FSlateTextureData, ESPMode::ThreadSafe> TextureData;
	/** Pixel format of the texture */
	EPixelFormat PixelFormat;
	/** Whether or not to create an empty texture when this resource is created.  Useful if the texture is being updated elsewhere  */
	bool bCreateEmptyTexture;
};

/**
 * Encapsulates a render target for use by a Slate rendering implementation                   
 */
class FSlateRenderTargetRHI : public TSlateTexture<FTextureRHIRef>, public FRenderResource
{
public:
	FSlateRenderTargetRHI( FTextureRHIRef InRenderTargetTexture, uint32 InWidth, uint32 InHeight )
		: TSlateTexture( InRenderTargetTexture )
		, Width( InWidth )
		, Height( InHeight )
	{
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override {}

	/** 
	 * Releases all dynamic RHI data
	 */
	virtual void ReleaseRHI() override
	{
		ShaderResource.SafeRelease();
	}

	virtual uint32 GetWidth() const override { return Width; }
	virtual uint32 GetHeight() const override { return Height; }

	/** 
	 *  Sets the RHI Ref to use.  Useful for reusing this class for multiple render targets
	 */
	ENGINE_API void SetRHIRef( FTextureRHIRef InRenderTargetTexture, uint32 InWidth, uint32 InHeight );

	FTextureRHIRef GetRHIRef() const { return ShaderResource; }
private:
	/** Width of this texture */
	uint32 Width;
	/** Height of this texture */
	uint32 Height;
};

class FSlateTextureRenderTarget2DResource : public FTextureRenderTargetResource
{
public:
	
	/** Constructor */
	ENGINE_API FSlateTextureRenderTarget2DResource(const FLinearColor& InClearColor, int32 InTargetSizeX, int32 InTargetSizeY, uint8 InFormat, ESamplerFilter InFilter, TextureAddress InAddressX, TextureAddress InAddressY, float InTargetGamma);

	/** Resizes the render target */
	virtual void SetSize(int32 InSizeX,int32 InSizeY);

public:
	// FTextureRenderTargetResource implementation
	virtual void ClampSize(int32 SizeX,int32 SizeY) override;

	// FRenderResource implementation
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	// FRenderTarget interface
	virtual uint32 GetSizeX() const override;
	virtual uint32 GetSizeY() const override;
	virtual FIntPoint GetSizeXY() const override;
	virtual float GetDisplayGamma() const override;

protected:

	// FDeferredUpdateResource implementation
	virtual void UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget=true) override;


private:
	FLinearColor ClearColor;
	int32 TargetSizeX,TargetSizeY;

	uint8 Format;
	ESamplerFilter Filter;
	TextureAddress AddressX;
	TextureAddress AddressY;
	float TargetGamma;
};
