// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/IntRect.h"

// Interface to expose the native backbuffer of a slate SWindow to FSceneViewport
class ISlateViewportProvider
{
protected:
	// Users of this type should never delete it. Lifetime is managed by the Slate RHI renderer.
	virtual ~ISlateViewportProvider() = default;

public:
	// Returns the current RHI back buffer resource the renderer should draw to.
	virtual class FRHITexture* GetBackBufferResource() const = 0;

	// Hooks the platform RHI present call with the given custom present object.
	virtual void SetCustomPresent(class FRHICustomPresent* CustomPresent) = 0;

	// Ability to crop viewport window size on supported platforms
	virtual void SetWindowCropSize(FIntRect& NewCropWindow) = 0;
};
