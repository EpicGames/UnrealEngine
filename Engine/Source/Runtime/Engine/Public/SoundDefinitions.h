// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_keep

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

#include "GraphEditAction.h"
#include "BlueprintUtilities.h"
#include "IAudioExtensionPlugin.h"
#include "Audio.h"
#include "PixelFormat.h"
#include "Components.h"
#include "LocalVertexFactory.h"
#include "SceneTypes.h"
#include "RawIndexBuffer.h"
#include "Math/GenericOctreePublic.h"
#include "Math/GenericOctree.h"
#include "Model.h"
#include "ComponentInstanceDataCache.h"
#include "EngineLogs.h"
#include "EngineDefines.h"
#include "TimerManager.h"
#include "ActiveSound.h"
#include "AudioThread.h"
#include "AudioDeviceManager.h"
#include "AudioDevice.h"

// IWYU pragma: end_keep
