// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/WeakObjectPtr.h"
#include "Containers/Set.h"
#include "IAssetCompilingManager.h"

#if WITH_EDITOR

class FAsyncCompilationNotification;
class USoundWave;
class FQueuedThreadPool;
enum class EQueuedWorkPriority : uint8;

class FSoundWaveCompilingManager : IAssetCompilingManager
{
public:
	ENGINE_API static FSoundWaveCompilingManager& Get();

	/**
	 * Returns true if the feature is currently activated.
	 */
	ENGINE_API bool IsAsyncSoundWaveCompilationEnabled() const;

	/** 
	 * Returns the number of outstanding sound wave compilations.
	 */
	ENGINE_API int32 GetNumRemainingSoundWaves() const;

	/** 
	 * Adds sound waves compiled asynchronously so they are monitored. 
	 */
	ENGINE_API void AddSoundWaves(TArrayView<USoundWave* const> InSoundWaves);

	/** 
	 * Blocks until completion of the requested sound waves.
	 */
	ENGINE_API void FinishCompilation(TArrayView<USoundWave* const> InSoundWaves);

	/** 
	 * Blocks until completion of all async texture compilation.
	 */
	ENGINE_API void FinishAllCompilation();

	/**
	 * Returns if asynchronous compilation is allowed for this sound wave.
	 */
	ENGINE_API bool IsAsyncCompilationAllowed(USoundWave* InSoundWave) const;

	/**
	 * Returns the priority at which the given sound waves should be scheduled.
	 */
	ENGINE_API EQueuedWorkPriority GetBasePriority(USoundWave* InSoundWave) const;

	/**
	 * Returns the threadpool where sound waves compilation should be scheduled.
	 */
	ENGINE_API FQueuedThreadPool* GetThreadPool() const;

	/**
	 * Cancel any pending work and blocks until it is safe to shut down.
	 */
	ENGINE_API void Shutdown();

	/**
	 * Called once per frame, fetches completed tasks and actives them in the world.
	 */
	ENGINE_API void ProcessAsyncTasks(bool bLimitExecutionTime = false);

	/** 
	  * Get the name of the asset type this compiler handles 
	  */
	ENGINE_API static FName GetStaticAssetTypeName();

private:
	friend class FAssetCompilingManager;

	FSoundWaveCompilingManager();

	bool bHasShutdown = false;
	TSet<TWeakObjectPtr<USoundWave>> RegisteredSoundWaves;

	mutable FRWLock Lock;
	FName GetAssetTypeName() const override;
	FTextFormat GetAssetNameFormat() const override;
	TArrayView<FName> GetDependentTypeNames() const override;
	int32 GetNumRemainingAssets() const override;

	/** Handle generic finish compilation */
	void FinishCompilationForObjects(TArrayView<UObject* const> InObjects) override;

	void UpdateCompilationNotification();
	void PostCompilation(TArrayView<USoundWave* const> InCompiledSoundWaves);
	void PostCompilation(USoundWave* SoundWave);
	void ProcessSoundWaves(bool bLimitExecutionTime, int32 MinBatchSize = 1);
	TArray<USoundWave*> GatherPendingSoundWaves();

	/** Notification for the amount of pending sound wave compilations */
	TUniquePtr<FAsyncCompilationNotification> Notification;
};

#endif
