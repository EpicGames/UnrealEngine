// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Allocator for spans from some range that is allowed to grow and shrink to accomodate the allocations.
 * Implementation is biased towards the case where allocations and frees are performed in bulk, if they are interleaved performance will degrade.
 * Prefers allocations at the start of the range, such that the high-watermark can be reduced optimally.
 * Primarily optimized for the use-case where span size == 1, for variable span size allocation performance is sub-optimal as it does not accelerate the search for larger allocs.
 */
class FSpanAllocator
{
public:
	/**
	 * If bInGrowOnly is true the size reported by GetMaxSize() will never decrease except when for when Reset() or Empty() is called.
	 */
	ENGINE_API FSpanAllocator(bool bInGrowOnly = false);

	// Allocate a range.  Returns allocated StartOffset.
	inline int32 Allocate(int32 Num = 1)
	{
		int32 FoundIndex = SearchFreeList(Num, FirstNonEmptySpan);

		if (FoundIndex == INDEX_NONE && !PendingFreeSpans.IsEmpty())
		{
			Consolidate();
			FoundIndex = SearchFreeList(Num, FirstNonEmptySpan);
		}

		CurrentSize += Num;

		// Use an existing free span if one is found
		if (FoundIndex != INDEX_NONE)
		{
			FLinearAllocation FreeSpan = FreeSpans[FoundIndex];

			// Update existing free span with remainder, 
			// note: may become zero, not removing empty spans to avoid moving free list, will be cleaned out when consolidating.
			FreeSpans[FoundIndex] = FLinearAllocation{ FreeSpan.StartOffset + Num, FreeSpan.Num - Num };

			// If this span is now empty && the found index was the first non-empty, we update the first non-empty span index past this.
			if (FreeSpan.Num == Num && FirstNonEmptySpan == FoundIndex)
			{
				FirstNonEmptySpan = FoundIndex + 1;
			}

			return FreeSpan.StartOffset;
		}

		// New allocation
		int32 StartOffset = CurrentMaxSize;
		CurrentMaxSize = CurrentMaxSize + Num;

		PeakMaxSize = FMath::Max(PeakMaxSize, CurrentMaxSize);

		return StartOffset;
	}

	// Free an already allocated range.  
	inline void Free(int32 BaseOffset, int32 Num = 1)
	{
		checkSlow(BaseOffset + Num <= CurrentMaxSize);
		checkSlow(Num <= CurrentSize);
		PendingFreeSpans.Add(FLinearAllocation{ BaseOffset, Num });
		CurrentSize -= Num;
	}

	ENGINE_API void Reset();

	ENGINE_API void Empty();

	inline int32 GetSparselyAllocatedSize() const
	{
		return CurrentSize;
	}

	inline int32 GetMaxSize() const
	{
		return bGrowOnly ? PeakMaxSize : CurrentMaxSize;
	}

	inline int32 GetNumFreeSpans() const
	{
		return FreeSpans.Num() + PendingFreeSpans.Num();
	}

	inline int32 GetNumPendingFreeSpans() const
	{
		return PendingFreeSpans.Num();
	}

	SIZE_T GetAllocatedSize() const
	{
		return FreeSpans.GetAllocatedSize() + PendingFreeSpans.GetAllocatedSize();
	}

#if DO_CHECK
	/**
	 * Loop over all free spans and check if Index is in any of them.
	 * Note: Very costly, only intended for debugging use, and probably best if under a toggle even then.
	 */
	ENGINE_API bool IsFree(int32 Index) const;
#endif // DO_CHECK

	/**
	 * Between these calls to Free just appends the allocation to the free list, rather than trying to merge with existing allocations.
	 * At EndDeferMerges the free list is consolidated by sorting and merging all spans. This amortises the cost of the merge over many calls.
	 */
	ENGINE_API void Consolidate();

private:
	
	struct FLinearAllocation
	{
		int32 StartOffset;
		int32 Num;

		inline bool operator<(const FLinearAllocation& Other) const
		{
			return StartOffset < Other.StartOffset;
		}
	};
	
	// Total of number of items currently allocated
	int32 CurrentSize;
	// Size of the linear range used by the allocator
	int32 CurrentMaxSize;
	// Peak tracked size since last Reset or Empty
	int32 PeakMaxSize;
	// First span in free list that is not empty, used to speed up search for free items
	int32 FirstNonEmptySpan;
	// Ordered free list from low to high
	TArray<FLinearAllocation, TInlineAllocator<10>> FreeSpans;
	// Unordered list of freed items since last consolidate
	TArray<FLinearAllocation, TInlineAllocator<10>> PendingFreeSpans;
	bool bGrowOnly;

	ENGINE_API int32 SearchFreeList(int32 Num, int32 SearchStartIndex = 0);
};