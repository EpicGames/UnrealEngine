// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

class UStreamableSparseVolumeTexture;
class FRDGBuilder;

namespace UE
{
namespace SVT
{

struct FStreamingDebugInfo
{
	struct FSVT
	{
		struct FInstance
		{
			uint32 Key;
			float Frame;
			float FrameRate;
			float RequestedBandwidth;
			float AllocatedBandwidth;
			float RequestedMip;
			float InBudgetMip;
		};

		FName AssetName;
		const float* FrameResidencyPercentages;
		const float* FrameStreamingPercentages;
		const FInstance* Instances;
		int32 NumFrames;
		int32 NumInstances;
	};

	const FSVT* SVTs;
	int32 NumSVTs;
	float RequestedBandwidth;
	float BandwidthLimit;
	float BandwidthScale;
};

enum class EStreamingRequestFlags : uint8
{
	None = 0,
	Blocking = 1u << 0u,		// Use blocking IO requests.
	HasFrameRate = 1u << 1u,	// The passed in FrameRate value can be used to more accurately predict bandwidth requirements.
};
ENUM_CLASS_FLAGS(EStreamingRequestFlags);

// Interface for the SparseVolumeTexture streaming manager
class IStreamingManager
{
public:
	//~ Begin game thread functions.
	virtual void Add_GameThread(UStreamableSparseVolumeTexture* SparseVolumeTexture) = 0;
	virtual void Remove_GameThread(UStreamableSparseVolumeTexture* SparseVolumeTexture) = 0;
	// Request a frame to be streamed in.
	// StreamingInstanceKey can be any arbitrary value that is suitable to keep track of the source of requests for a given SVT. 
	// This key is used internally to associate incoming requests with prior requests issued for the same SVT. A good value to pass here might be a hash of the pointer of the component the SVT is used within.
	// FrameRate is an optional argument which helps to more accurately predict the required bandwidth when using non-blocking requests. EStreamingRequestFlags::HasFrameRate must be set for this.
	// FrameIndex is of float type so that the fractional part can be used to better track the playback speed/direction.
	// This function automatically also requests all higher mip levels and adds prefetch requests for upcoming frames.
	virtual void Request_GameThread(UStreamableSparseVolumeTexture* SparseVolumeTexture, uint32 StreamingInstanceKey, float FrameRate, float FrameIndex, float MipLevel, EStreamingRequestFlags Flags) = 0;
	// Issues a rendering command for updating the streaming manager. This is not normally needed, but is necessary in cases where blocking requests are required
	// and the SVT is used in a rendering command (which is executed before the streaming manager would normally update). A call to this function is not needed when the SVT
	// is used in a regular pass called from FDeferredShadingSceneRenderer::Render().
	virtual void Update_GameThread() = 0;
	//~ End game thread functions.
	
	//~ Begin rendering thread functions.
	virtual void Request(UStreamableSparseVolumeTexture* SparseVolumeTexture, uint32 StreamingInstanceKey, float FrameRate, float FrameIndex, float MipLevel, EStreamingRequestFlags Flags) = 0;
	// Begins updating the streaming manager. If r.SparseVolumeTexture.Streaming.AsyncThread is 1 and bUseAsyncThread is true, most of the updating work is done in another thread.
	virtual void BeginAsyncUpdate(FRDGBuilder& GraphBuilder, bool bUseAsyncThread = true) = 0;
	// Waits for the job started in BeginAsyncUpdate() to complete, issues GPU work and does some cleanup.
	virtual void EndAsyncUpdate(FRDGBuilder& GraphBuilder) = 0;
	// Returns a FStreamingDebugInfo useful for debugging streaming performance. All data is allocated on the passed in GraphBuilder.
	virtual const FStreamingDebugInfo* GetStreamingDebugInfo(FRDGBuilder& GraphBuilder) const = 0;
	//~ End rendering thread functions.

	virtual ~IStreamingManager() = default;
};

ENGINE_API IStreamingManager& GetStreamingManager();

}
}
