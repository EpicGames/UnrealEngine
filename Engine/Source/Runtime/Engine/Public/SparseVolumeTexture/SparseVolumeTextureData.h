// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"

namespace UE
{
namespace SVT
{

struct FTextureDataCreateInfo
{
	FIntVector3 VirtualVolumeAABBMin = FIntVector3(INT32_MAX, INT32_MAX, INT32_MAX);
	FIntVector3 VirtualVolumeAABBMax = FIntVector3(INT32_MIN, INT32_MIN, INT32_MIN);
	TStaticArray<EPixelFormat, 2> AttributesFormats = TStaticArray<EPixelFormat, 2>(InPlace, PF_Unknown); // See UE::SVT::IsSupportedFormat(EPixelFormat Format) for list of supported formats
	TStaticArray<FVector4f, 2> FallbackValues = TStaticArray<FVector4f, 2>(InPlace, FVector4f(0.0f, 0.0f, 0.0f, 0.0f));
};

class ITextureDataProvider
{
public:
	virtual FTextureDataCreateInfo GetCreateInfo() const = 0;
	virtual void IteratePhysicalSource(TFunctionRef<void(const FIntVector3& Coord, int32 AttributesIdx, int32 ComponentIdx, float VoxelValue)> OnVisit) const = 0;
	virtual ~ITextureDataProvider() = default;
};

struct FTextureDataAddressingInfo
{
	FIntVector3 VolumeResolution;
	TextureAddress AddressX;
	TextureAddress AddressY;
	TextureAddress AddressZ;
};

// Holds the data for a SparseVolumeTexture that is stored on disk. It only has a single mip after importing a source asset. The mip chain is built during cooking.
// Tiles are addressed by a flat index; unlike the runtime representation, this one stores all tiles in a 1D array (per mip level) and doesn't have the concept of a 3D physical tile texture.
// The page table itself is 3D though.
struct FTextureData
{
	struct FMipMap
	{
		TArray<uint32> PageTable;
		TArray64<uint8> PhysicalTileDataA;
		TArray64<uint8> PhysicalTileDataB;
		int32 NumPhysicalTiles;
	};

	FHeader Header = {};
	TStaticArray<FVector4f, 2> FallbackValuesQuantized = TStaticArray<FVector4f, 2>(InPlace, FVector4f(0.0f, 0.0f, 0.0f, 0.0f));
	TArray<FMipMap> MipMaps;

	ENGINE_API bool Create(const ITextureDataProvider& DataProvider);
	ENGINE_API bool CreateFromDense(const FTextureDataCreateInfo& CreateInfo, const TArrayView64<uint8>& VoxelDataA, const TArrayView64<uint8>& VoxelDataB);
	ENGINE_API void CreateDefault();
	ENGINE_API FVector4f Load(const FIntVector3& VolumeCoord, int32 MipLevel, int32 AttributesIdx, const FTextureDataAddressingInfo& AddressingInfo) const;
	ENGINE_API bool BuildDerivedData(const FTextureDataAddressingInfo& AddressingInfo, int32 NumMipLevelsGlobal, bool bMoveMip0FromThis, struct FDerivedTextureData& OutDerivedData);

private:

	bool GenerateMipMaps(const FTextureDataAddressingInfo& AddressingInfo, int32 NumMipLevels = -1);
	bool GenerateBorderVoxels(const FTextureDataAddressingInfo& AddressingInfo, int32 MipLevel, const TArray<FIntVector3>& PageCoords);

	uint32 ReadPageTable(const FIntVector3& PageTableCoord, int32 MipLevel) const;
	FVector4f ReadTileDataVoxel(int32 TileIndex, const FIntVector3& TileDataCoord, int32 MipLevel, int32 AttributesIdx) const;
	void WriteTileDataVoxel(int32 TileIndex, const FIntVector3& TileDataCoord, int32 MipLevel, int32 AttributesIdx, const FVector4f& Value, int32 DstComponent = -1);
};

// Represents mip mapped SparseVolumeTexture data ready for compression/serializing. The data is stored as a set of arrays for the page table and a set of arrays for the tile data.
// The page table arrays are essentially struct-of-arrays and store only those elements of the dense page page table which actually point to a tile (non-zero). Each stored page
// consists of a coordinate in the 3D page table, an index to the tile it points to and another index to the parent page, which is the corresponding page in the next higher mip level.
// Deduplicated tiles (of voxels) for all mip levels are stored in a single set of arrays as raw bytes.
struct FDerivedTextureData
{
	// Pages are ordered by mip level. This struct represents the range of pages for a given mip level.
	struct FMipPageRange
	{
		uint32 PageOffset;
		uint32 PageCount;
	};

	FHeader Header = {};
	uint32 NumPhysicalTiles;				// Number of tiles stored in PhysicalTileDataA/PhysicalTileDataB.
	TArray<FMipPageRange> MipPageRanges;	// Page range for each mip level. Indexed by mip level.
	TArray<uint32> PageTableCoords;			// 11|11|10 packed coords of the page within the logical dense 3D page table.
	TArray<uint32> PageTableTileIndices;	// Index of tile in PhysicalTileDataA/PhysicalTileDataB pointed to by this page.
	TArray<uint32> PageTableParentIndices;	// Index of parent page in the next higher mip level.
	TArray64<uint8> PhysicalTileDataA;
	TArray64<uint8> PhysicalTileDataB;

	void Reset();
	void Build(const FTextureData& MippedTextureData);
};

}
}

FArchive& operator<<(FArchive& Ar, UE::SVT::FTextureData& TextureData);
