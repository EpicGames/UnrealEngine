///////////////////////////////////////////////////////////////////////  
//	SpeedTreeWind.h
//
//	*** INTERACTIVE DATA VISUALIZATION (IDV) PROPRIETARY INFORMATION ***
//
//	This software is supplied under the terms of a license agreement or
//	nondisclosure agreement with Interactive Data Visualization and may
//	not be copied or disclosed except in accordance with the terms of
//	that agreement.
//
//      Copyright (c) 2003-2012 IDV, Inc.
//      All Rights Reserved.
//
//		IDV, Inc.
//		Web: http://www.idvinc.com

//  SpeedTree v6.2.2 wind class rewritten for use inside UE with no other dependencies

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "UniformBuffer.h"

///////////////////////////////////////////////////////////////////////  
// class FSpeedTreeWind

class FSpeedTreeWind
{
public:
	enum Constants
	{
		NUM_WIND_POINTS_IN_CURVE = 10,
		NUM_BRANCH_LEVELS = 2,
		NUM_LEAF_GROUPS = 2
	};

	// shader state that are set at compile time
	enum EOptions
	{
		GLOBAL_WIND,
		GLOBAL_PRESERVE_SHAPE,

		BRANCH_SIMPLE_1,
		BRANCH_DIRECTIONAL_1,
		BRANCH_DIRECTIONAL_FROND_1,
		BRANCH_TURBULENCE_1,
		BRANCH_WHIP_1,
		BRANCH_OSC_COMPLEX_1,

		BRANCH_SIMPLE_2,
		BRANCH_DIRECTIONAL_2,
		BRANCH_DIRECTIONAL_FROND_2,
		BRANCH_TURBULENCE_2,
		BRANCH_WHIP_2,
		BRANCH_OSC_COMPLEX_2,

		LEAF_RIPPLE_VERTEX_NORMAL_1,
		LEAF_RIPPLE_COMPUTED_1,
		LEAF_TUMBLE_1,
		LEAF_TWITCH_1,
		LEAF_OCCLUSION_1,

		LEAF_RIPPLE_VERTEX_NORMAL_2,
		LEAF_RIPPLE_COMPUTED_2,
		LEAF_TUMBLE_2,
		LEAF_TWITCH_2,
		LEAF_OCCLUSION_2,

		FROND_RIPPLE_ONE_SIDED,
		FROND_RIPPLE_TWO_SIDED,
		FROND_RIPPLE_ADJUST_LIGHTING,

		ROLLING,

		NUM_WIND_OPTIONS
	};

	// values to be uploaded as shader constants
	enum EShaderValues
	{
		// g_vWindVector
		SH_WIND_DIR_X, SH_WIND_DIR_Y, SH_WIND_DIR_Z, SH_GENERAL_STRENGTH,

		// g_vWindGlobal
		SH_GLOBAL_TIME, SH_GLOBAL_DISTANCE, SH_GLOBAL_HEIGHT, SH_GLOBAL_HEIGHT_EXPONENT,

		// g_vWindBranch
		SH_BRANCH_1_TIME, SH_BRANCH_1_DISTANCE, SH_BRANCH_2_TIME, SH_BRANCH_2_DISTANCE,

		// g_vWindBranchTwitch
		SH_BRANCH_1_TWITCH, SH_BRANCH_1_TWITCH_FREQ_SCALE, SH_BRANCH_2_TWITCH, SH_BRANCH_2_TWITCH_FREQ_SCALE,

		// g_vWindBranchWhip
		SH_BRANCH_1_WHIP, SH_BRANCH_2_WHIP, SH_WIND_PACK0, SH_WIND_PACK1,

		// g_vWindBranchAnchor
		SH_WIND_ANCHOR_X, SH_WIND_ANCHOR_Y, SH_WIND_ANCHOR_Z, SH_WIND_PACK2,

		// g_vWindBranchAdherences
		SH_GLOBAL_DIRECTION_ADHERENCE, SH_BRANCH_1_DIRECTION_ADHERENCE, SH_BRANCH_2_DIRECTION_ADHERENCE, SH_WIND_PACK5,

		// g_vWindTurbulences
		SH_BRANCH_1_TURBULENCE, SH_BRANCH_2_TURBULENCE, SH_WIND_PACK6, SH_WIND_PACK7,

		// g_vWindLeaf1Ripple
		SH_LEAF_1_RIPPLE_TIME, SH_LEAF_1_RIPPLE_DISTANCE, SH_LEAF_1_LEEWARD_SCALAR, SH_WIND_PACK8,

		// g_vWindLeaf1Tumble
		SH_LEAF_1_TUMBLE_TIME, SH_LEAF_1_TUMBLE_FLIP, SH_LEAF_1_TUMBLE_TWIST, SH_LEAF_1_TUMBLE_DIRECTION_ADHERENCE,

		// g_vWindLeaf1Twitch
		SH_LEAF_1_TWITCH_THROW, SH_LEAF_1_TWITCH_SHARPNESS, SH_LEAF_1_TWITCH_TIME, SH_WIND_PACK9,

		// g_vWindLeaf2Ripple
		SH_LEAF_2_RIPPLE_TIME, SH_LEAF_2_RIPPLE_DISTANCE, SH_LEAF_2_LEEWARD_SCALAR, SH_WIND_PACK10,

		// g_vWindLeaf2Tumble
		SH_LEAF_2_TUMBLE_TIME, SH_LEAF_2_TUMBLE_FLIP, SH_LEAF_2_TUMBLE_TWIST, SH_LEAF_2_TUMBLE_DIRECTION_ADHERENCE,

		// g_vWindLeaf2Twitch
		SH_LEAF_2_TWITCH_THROW, SH_LEAF_2_TWITCH_SHARPNESS, SH_LEAF_2_TWITCH_TIME, SH_WIND_PACK11,
				
		// g_vWindFrondRipple
		SH_FROND_RIPPLE_TIME, SH_FROND_RIPPLE_DISTANCE, SH_FROND_RIPPLE_TILE, SH_FROND_RIPPLE_LIGHTING_SCALAR,

		// g_vWindRollingBranch
		SH_ROLLING_BRANCH_FIELD_MIN, SH_ROLLING_BRANCH_LIGHTING_ADJUST, SH_ROLLING_BRANCH_VERTICAL_OFFSET, SH_WIND_PACK12,

		// g_vWindRollingLeafAndDir
		SH_ROLLING_LEAF_RIPPLE_MIN, SH_ROLLING_LEAF_TUMBLE_MIN, SH_ROLLING_X, SH_ROLLING_Y,

		// g_vWindRollingNoise
		SH_ROLLING_NOISE_PERIOD, SH_ROLLING_NOISE_SIZE, SH_ROLLING_NOISE_TURBULENCE, SH_ROLLING_NOISE_TWIST,

		// total values, including packing
		NUM_SHADER_VALUES
	};

	// wind simulation components that oscillate
	enum EOscillationComponents
	{
		OSC_GLOBAL, 
		OSC_BRANCH_1, 
		OSC_BRANCH_2, 
		OSC_LEAF_1_RIPPLE, 
		OSC_LEAF_1_TUMBLE, 
		OSC_LEAF_1_TWITCH, 
		OSC_LEAF_2_RIPPLE, 
		OSC_LEAF_2_TUMBLE, 
		OSC_LEAF_2_TWITCH, 
		OSC_FROND_RIPPLE, 
		NUM_OSC_COMPONENTS
	};

	struct SBranchWindLevel
	{
		ENGINE_API	SBranchWindLevel();

		float		m_afDistance[NUM_WIND_POINTS_IN_CURVE];
		float		m_afDirectionAdherence[NUM_WIND_POINTS_IN_CURVE];
		float		m_afWhip[NUM_WIND_POINTS_IN_CURVE];
		float		m_fTurbulence;
		float		m_fTwitch;
		float		m_fTwitchFreqScale;
	};

	struct SWindGroup
	{
		ENGINE_API	SWindGroup();

		float		m_afRippleDistance[NUM_WIND_POINTS_IN_CURVE];
		float		m_afTumbleFlip[NUM_WIND_POINTS_IN_CURVE];
		float		m_afTumbleTwist[NUM_WIND_POINTS_IN_CURVE];
		float		m_afTumbleDirectionAdherence[NUM_WIND_POINTS_IN_CURVE];
		float		m_afTwitchThrow[NUM_WIND_POINTS_IN_CURVE];
		float		m_fTwitchSharpness;
		float		m_fRollMaxScale;
		float		m_fRollMinScale;
		float		m_fRollSpeed;
		float		m_fRollSeparation;
		float		m_fLeewardScalar;
	};

	struct SParams
	{
		ENGINE_API			SParams();

		// settings
		float				m_fStrengthResponse;
		float				m_fDirectionResponse;

		float				m_fAnchorOffset;
		float				m_fAnchorDistanceScale;

		// oscillation components
		float				m_afFrequencies[NUM_OSC_COMPONENTS][NUM_WIND_POINTS_IN_CURVE];

		// global motion
		float				m_fGlobalHeight;
		float				m_fGlobalHeightExponent;
		float				m_afGlobalDistance[NUM_WIND_POINTS_IN_CURVE];
		float				m_afGlobalDirectionAdherence[NUM_WIND_POINTS_IN_CURVE];

		// branch motion
		SBranchWindLevel	m_asBranch[NUM_BRANCH_LEVELS];

		// leaf motion
		SWindGroup			m_asLeaf[NUM_LEAF_GROUPS];

		// frond ripple
		float				m_afFrondRippleDistance[NUM_WIND_POINTS_IN_CURVE];
		float				m_fFrondRippleTile;
		float				m_fFrondRippleLightingScalar;

		// rolling
		float				m_fRollingNoiseSize;
		float				m_fRollingNoiseTwist;
		float				m_fRollingNoiseTurbulence;
		float				m_fRollingNoisePeriod;
		float				m_fRollingNoiseSpeed;

		float				m_fRollingBranchFieldMin;
		float				m_fRollingBranchLightingAdjust;
		float				m_fRollingBranchVerticalOffset;
		float				m_fRollingLeafRippleMin;
		float				m_fRollingLeafTumbleMin;

		// gusting
		float				m_fGustFrequency;
		float				m_fGustStrengthMin;
		float				m_fGustStrengthMax;
		float				m_fGustDurationMin;
		float				m_fGustDurationMax;
		float				m_fGustRiseScalar;
		float				m_fGustFallScalar;
	};

	ENGINE_API									FSpeedTreeWind();

	// settings
	ENGINE_API	void							SetParams(const FSpeedTreeWind::SParams& sParams);	// this should be called infrequently and never when trees that use it are visible
	ENGINE_API	const FSpeedTreeWind::SParams&	GetParams(void) const;
	ENGINE_API	void							SetStrength(float fStrength);						// use this function to set a new desired strength (it will reach that strength smoothly)
	ENGINE_API	void							SetDirection(const FVector& vDir);					// use this function to set a new desired direction (it will reach that direction smoothly)

	//instantly set gust min/max.   Trees will pop if visible.  Don't call during gameplay while trees are visible.
	ENGINE_API	void							SetGustMin(float InGustMin);
	ENGINE_API	void							SetGustMax(float InGustMax);

	ENGINE_API	void							SetInitDirection(const FVector& vDir);				// use this function to set a starting direction, once
	ENGINE_API	void							EnableGusting(bool bEnabled);
	ENGINE_API	void							SetGustFrequency(float fGustFreq);
	ENGINE_API	void							Scale(float fScalar);

	// tree-specific values
	ENGINE_API	void							SetTreeValues(const FVector& vBranchAnchor, float fMaxBranchLength);
	ENGINE_API	const float*					GetBranchAnchor(void) const;
	ENGINE_API	float							GetMaxBranchLength(void) const;

	// shader options
	ENGINE_API	void							SetOption(EOptions eOption, bool bState);
	ENGINE_API	bool							IsOptionEnabled(EOptions eOption) const;
		
	// animation
	ENGINE_API	void							Advance(bool bEnabled, double fTime);				// called every frame to 'tick' the wind
	ENGINE_API	const float*					GetShaderTable(void) const;

	friend		FArchive&						operator<<(FArchive& Ar, FSpeedTreeWind& Wind);

	void							SetNeedsReload(bool bReload = true) { m_bNeedsReload = bReload; }
	bool							NeedsReload(void) { return m_bNeedsReload; }
	

protected:
				void							Gust(double fTime);
				float							RandomFloat(float fMin, float fMax) const;
				float							LinearSigmoid(float fInput, float fLinearness);
				float							Interpolate(float fA, float fB, float fAmt);
				void							Normalize(float* pVector);
				void							ComputeWindAnchor(float* pPos);

protected:
					SParams		m_sParams;

					float		m_fStrength;
					float		m_afDirection[3];

					double		m_fLastTime;
					double		m_fElapsedTime;

					bool		m_bGustingEnabled;
					float		m_fGust;
					double		m_fGustTarget;
					double		m_fGustRiseTarget;
					double		m_fGustFallTarget;
					double		m_fGustStart;
					double		m_fGustAtStart;
					double		m_fGustFallStart;

					float		m_fStrengthTarget;
					double		m_fStrengthChangeStartTime;
					double		m_fStrengthChangeEndTime;
					float		m_fStrengthAtStart;

					float		m_afDirectionTarget[3];
					float		m_afDirectionMidTarget[3];
					double		m_fDirectionChangeStartTime;
					double		m_fDirectionChangeEndTime;
					float		m_afDirectionAtStart[3];

					float		m_afRollingOffset[2];

					float		m_fCombinedStrength;

					float		m_afOscillationTimes[NUM_OSC_COMPONENTS];

					bool		m_abOptions[NUM_WIND_OPTIONS];

					float		m_afBranchWindAnchor[3];
					float		m_fMaxBranchLevel1Length;

					bool		m_bNeedsReload;

					// Includes Previous frame's values after current
	MS_ALIGN(16)	float		m_afShaderTable[NUM_SHADER_VALUES * 2] GCC_ALIGN(16);
};


/**
 * Uniform buffer setup for SpeedTrees.
 */

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSpeedTreeUniformParameters, ENGINE_API)
	SHADER_PARAMETER(FVector4f, WindVector)
	SHADER_PARAMETER(FVector4f, WindGlobal)
	SHADER_PARAMETER(FVector4f, WindBranch)
	SHADER_PARAMETER(FVector4f, WindBranchTwitch)
	SHADER_PARAMETER(FVector4f, WindBranchWhip)
	SHADER_PARAMETER(FVector4f, WindBranchAnchor)
	SHADER_PARAMETER(FVector4f, WindBranchAdherences)
	SHADER_PARAMETER(FVector4f, WindTurbulences)
	SHADER_PARAMETER(FVector4f, WindLeaf1Ripple)
	SHADER_PARAMETER(FVector4f, WindLeaf1Tumble)
	SHADER_PARAMETER(FVector4f, WindLeaf1Twitch)
	SHADER_PARAMETER(FVector4f, WindLeaf2Ripple)
	SHADER_PARAMETER(FVector4f, WindLeaf2Tumble)
	SHADER_PARAMETER(FVector4f, WindLeaf2Twitch)
	SHADER_PARAMETER(FVector4f, WindFrondRipple)
	SHADER_PARAMETER(FVector4f, WindRollingBranch)
	SHADER_PARAMETER(FVector4f, WindRollingLeafAndDirection)
	SHADER_PARAMETER(FVector4f, WindRollingNoise)
	SHADER_PARAMETER(FVector4f, WindAnimation)
	// Straight copy of the previous members for last frame's values
	SHADER_PARAMETER(FVector4f, PrevWindVector)
	SHADER_PARAMETER(FVector4f, PrevWindGlobal)
	SHADER_PARAMETER(FVector4f, PrevWindBranch)
	SHADER_PARAMETER(FVector4f, PrevWindBranchTwitch)
	SHADER_PARAMETER(FVector4f, PrevWindBranchWhip)
	SHADER_PARAMETER(FVector4f, PrevWindBranchAnchor)
	SHADER_PARAMETER(FVector4f, PrevWindBranchAdherences)
	SHADER_PARAMETER(FVector4f, PrevWindTurbulences)
	SHADER_PARAMETER(FVector4f, PrevWindLeaf1Ripple)
	SHADER_PARAMETER(FVector4f, PrevWindLeaf1Tumble)
	SHADER_PARAMETER(FVector4f, PrevWindLeaf1Twitch)
	SHADER_PARAMETER(FVector4f, PrevWindLeaf2Ripple)
	SHADER_PARAMETER(FVector4f, PrevWindLeaf2Tumble)
	SHADER_PARAMETER(FVector4f, PrevWindLeaf2Twitch)
	SHADER_PARAMETER(FVector4f, PrevWindFrondRipple)
	SHADER_PARAMETER(FVector4f, PrevWindRollingBranch)
	SHADER_PARAMETER(FVector4f, PrevWindRollingLeafAndDirection)
	SHADER_PARAMETER(FVector4f, PrevWindRollingNoise)
	SHADER_PARAMETER(FVector4f, PrevWindAnimation)
END_GLOBAL_SHADER_PARAMETER_STRUCT()
