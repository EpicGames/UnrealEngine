// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderingThread.h"
#include "ShaderParameters.h"
#include "VertexFactory.h"
#include "LocalVertexFactory.h"
#include "PrimitiveViewRelevance.h"
#include "Components/SplineMeshComponent.h"
#include "StaticMeshResources.h"
#include "StaticMeshSceneProxy.h"
#include "SplineMeshShaderParams.h"
#include "NaniteSceneProxy.h"
#include "InstanceDataSceneProxy.h"
#include "SplineMeshSceneProxyDesc.h"

struct FSplineMeshSceneProxyDesc;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSplineMeshVFPrimitiveShaderParameters, ENGINE_API)
	SHADER_PARAMETER_ARRAY(FVector4f, SplineParams, [SPLINE_MESH_PARAMS_FLOAT4_SIZE])
END_GLOBAL_SHADER_PARAMETER_STRUCT()

//////////////////////////////////////////////////////////////////////////
// SplineMeshVertexFactory

/** A vertex factory for spline-deformed static meshes */
struct FSplineMeshVertexFactory : public FLocalVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE(FSplineMeshVertexFactory);
public:
	FSplineMeshVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FLocalVertexFactory(InFeatureLevel, "FSplineMeshVertexFactory")
	{
	}

	/** Should we cache the material's shadertype on this platform with this vertex factory? */
	static bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/** Modify compile environment to enable spline deformation */
	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/** Get vertex elements used when during PSO precaching materials using this vertex factory type */
	static void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);

	/** Copy the data from another vertex factory */
	void Copy(const FSplineMeshVertexFactory& Other)
	{
		FSplineMeshVertexFactory* VertexFactory = this;
		const FDataType* DataCopy = &Other.Data;
		ENQUEUE_RENDER_COMMAND(FSplineMeshVertexFactoryCopyData)(
			[VertexFactory, DataCopy](FRHICommandListImmediate& RHICmdList)
			{
				VertexFactory->Data = *DataCopy;
			});
		BeginUpdateResourceRHI(this);
	}
};

//////////////////////////////////////////////////////////////////////////
// FSplineMeshVertexFactoryShaderParameters

/** Factory specific params */
class FSplineMeshVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FSplineMeshVertexFactoryShaderParameters, NonVirtual);
public:
	void GetElementShaderBindings(
		const class FSceneInterface* Scene,
		const FSceneView* View,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams
		) const;

private:
};

struct FSplineMeshSceneInstanceDataBuffers : public FSingleInstanceDataBuffers
{
	ENGINE_API void Setup(const FSplineMeshShaderParams& InSplineMeshShaderParams);
	ENGINE_API bool Update(const FSplineMeshShaderParams& InSplineMeshShaderParams);
};

//////////////////////////////////////////////////////////////////////////
// SplineMeshSceneProxy

/**
 * This interface ties common functionality to the two different spline mesh scene proxies without duplicating code or
 * introducing diamond inheritance.
 */
class FSplineMeshSceneProxyCommon
{
public:
	FSplineMeshSceneProxyCommon()
	{
		FMemory::Memzero(&SplineParams, sizeof(SplineParams));
		SplineParams.TextureCoord = FUintVector2(INDEX_NONE, INDEX_NONE);
	}
	virtual ~FSplineMeshSceneProxyCommon() = default;

	const FSplineMeshShaderParams& GetSplineMeshParams() const { return SplineParams; }

	ENGINE_API void SetSplineTextureCoord_RenderThread(uint32 SplineIndex, FUintVector2 TexCoord);
	
	ENGINE_API void UpdateSplineMeshParams_RenderThread(const FSplineMeshShaderParams& Params);

protected:
	void RepackSplineMeshParamsInternal();

	bool OnMeshCardsTransformChanged(const FVector& LocalToWorldScale);

	FMatrix GetMeshCardsToWorld(const FMatrix& LocalToWorld) const;

protected:
	/** Parameters that define the spline, used to deform mesh */
	FSplineMeshShaderParams SplineParams;

	FSplineMeshSceneInstanceDataBuffers SplineMeshInstanceData;

	FQuat4f MeshCardsToLocalRotation = FQuat4f::Identity;
	FVector3f MeshCardsToLocalTranslation = FVector3f::ZeroVector;
	FVector3f CachedLocalToWorldScale = FVector3f::OneVector;
	bool bCardsInWorldScale = false;

	bool bUpdateRayTracingGeometry = true;
	FRWBuffer RayTracingDynamicVertexBuffer;

	TUniformBufferRef<FSplineMeshVFPrimitiveShaderParameters> SplineMeshVFPrimitiveShaderParametersUB;
private:
	/** implemented by derived to provide access to proxy */
	virtual FPrimitiveSceneProxy& GetProxy() = 0;
};

/** Scene proxy for SplineMesh instance */
class FSplineMeshSceneProxy final : public FStaticMeshSceneProxy, public FSplineMeshSceneProxyCommon
{
private:
	virtual FPrimitiveSceneProxy& GetProxy() override { return *this; }
public:
	FSplineMeshSceneProxy(USplineMeshComponent* InComponent);

	ENGINE_API FSplineMeshSceneProxy(const FStaticMeshSceneProxyDesc& InMeshDesc, const FSplineMeshSceneProxyDesc& InSplineDesc);
	// FPrimitiveSceneProxy interface
	virtual SIZE_T GetTypeHash() const override;
	virtual bool GetShadowMeshElement(int32 LODIndex, int32 BatchIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch, bool bDitheredLODTransition) const override;
	virtual bool GetMeshElement(int32 LODIndex, int32 BatchIndex, int32 ElementIndex, uint8 InDepthPriorityGroup, bool bUseSelectionOutline, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const override;
	virtual bool GetWireframeMeshElement(int32 LODIndex, int32 BatchIndex, const FMaterialRenderProxy* WireframeRenderProxy, uint8 InDepthPriorityGroup, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const override;
	virtual bool GetCollisionMeshElement(int32 LODIndex, int32 BatchIndex, int32 ElementIndex, uint8 InDepthPriorityGroup, const FMaterialRenderProxy* RenderProxy, FMeshBatch& OutMeshBatch) const override;
	virtual bool GetRayTracingMeshElement(int32 LODIndex, int32 BatchIndex, int32 SectionIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch) const;
#if RHI_RAYTRACING
	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override;
#endif // RHI_RAYTRACING
	virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override;

private:
	struct FLODResources
	{
		/** Pointer to vertex factory object */
		FSplineMeshVertexFactory* VertexFactory;

		FLODResources(FSplineMeshVertexFactory* InVertexFactory) :
			VertexFactory(InVertexFactory)
		{
		}
	};

	void SetupMeshBatchForSpline(int32 InLODIndex, FMeshBatch& OutMeshBatch) const;
	void SetupRayTracingMeshBatchForSpline(int32 InLODIndex, FMeshBatch& OutMeshBatch) const;

private:
	TArray<FLODResources> LODResources;
};

/** Scene proxy for SplineMesh instance for Nanite */
class FNaniteSplineMeshSceneProxy final : public Nanite::FSceneProxy, public FSplineMeshSceneProxyCommon
{
private:
	virtual FPrimitiveSceneProxy& GetProxy() override { return *this; }
public:
	FNaniteSplineMeshSceneProxy(const Nanite::FMaterialAudit& MaterialAudit, USplineMeshComponent* InComponent);
	ENGINE_API FNaniteSplineMeshSceneProxy(const Nanite::FMaterialAudit& MaterialAudit, const FStaticMeshSceneProxyDesc& InMeshDesc, const FSplineMeshSceneProxyDesc& InSplineDesc);

	// FPrimitiveSceneProxy interface
	virtual SIZE_T GetTypeHash() const override;
	virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override;
	virtual FMatrix GetMeshCardsToWorld(int32 InstanceIndex) const override;
#if RHI_RAYTRACING
	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override;
	virtual void SetupFallbackRayTracingMaterials(int32 LODIndex, TArray<FMeshBatch>& OutMaterials) const override;
#endif
};

/** Helper to update the parameters of the specified spline mesh scene proxy */
ENGINE_API void UpdateSplineMeshParams_RenderThread(FPrimitiveSceneProxy* SceneProxy, const FSplineMeshShaderParams& Params);



