// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "StaticMeshSceneProxyDesc.h"
#include "Components/SplineMeshComponent.h"

class USplineMeshComponent;
class FStaticMeshRenderData;

struct FSplineMeshSceneProxyDesc
{
	FSplineMeshSceneProxyDesc() = default;
	ENGINE_API FSplineMeshSceneProxyDesc(const USplineMeshComponent* InComponent);
	
	void InitializeFrom(const USplineMeshComponent* InComponent);

	void InitializeFrom(const FSplineMeshShaderParams& ShaderParams, const UStaticMesh* StaticMesh, const FStaticMeshRenderData* RenderData);
	
	FSplineMeshParams SplineParams{};
	FVector SplineUpDir = FVector::UpVector;
	float SplineBoundaryMin = 0.0f;
	float SplineBoundaryMax = 0.0f;
	TEnumAsByte<ESplineMeshAxis::Type> ForwardAxis = ESplineMeshAxis::X;
	bool bSmoothInterpRollScale = false;
	bool bNaniteEnabled = false;
	FBoxSphereBounds SourceMeshBounds { ForceInit };

	FSplineMeshParams GetSplineParams() const { return SplineParams; }

	FSplineMeshShaderParams CalculateShaderParams() const;
	float ComputeRatioAlongSpline(float DistanceAlong) const;
	void ComputeVisualMeshSplineTRange(float& MinT, float& MaxT) const;
	ENGINE_API FBox ComputeDistortedBounds(const FTransform& InLocalToWorld, const FBoxSphereBounds& InMeshBounds, const FBoxSphereBounds* InBoundsToDistort = nullptr) const;
	FTransform CalcSliceTransform(const float DistanceAlong) const;
	FTransform CalcSliceTransformAtSplineOffset(const float Alpha, const float MinT, const float MaxT) const;
	
	// Lumen mesh cards bounds are built in mesh space so we need to compute a new one after the mesh is distorted.
	// The bounds is an AABB. To get a tighter bounds, we build a new coordinate frame instead of using the component space frame.
	// @return Whether the resulting box and transform already have local to world scale applied
	bool ComputeDistortedBoundsForLumenCardCapture(const FVector& LocalToWorldScale, FBox& OutMeshCardsBounds, FTransform& OutMeshCardsToLocal) const;

	static void InitVertexFactory(UStaticMesh* Mesh, const ERHIFeatureLevel::Type FeatureLevel, int32 InLODIndex, FColorVertexBuffer* InOverrideColorVertexBuffer);
	static void InitRayTracingProxyVertexFactory(UStaticMesh* Mesh, const ERHIFeatureLevel::Type FeatureLevel, int32 InLODIndex, FColorVertexBuffer* InOverrideColorVertexBuffer);
};