// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PointLightSceneProxy.h"
#include "SceneInterface.h"
#include "SceneView.h"

class USpotLightComponent;
struct FSpotLightSceneProxyDesc;

/**
 * The scene info for a spot light.
 */
class FSpotLightSceneProxy : public FPointLightSceneProxy
{
public:

	/** Initialization constructors. */
	FSpotLightSceneProxy(const USpotLightComponent* Component);
	ENGINE_API FSpotLightSceneProxy(const FSpotLightSceneProxyDesc& LightDesc);

	//~Begin FLightSceneProxy Interface
	virtual bool AffectsBounds(const FBoxSphereBounds& Bounds) const override;
	virtual bool GetWholeSceneProjectedShadowInitializer(const FSceneViewFamily& ViewFamily, TArray<FWholeSceneProjectedShadowInitializer, TInlineAllocator<6> >& OutInitializers) const override;
	virtual float GetOuterConeAngle() const override;
	virtual FSphere GetBoundingSphere() const override;
	virtual float GetEffectiveScreenRadius(const FViewMatrices& ShadowViewMatrices, const FIntPoint& CameraViewRectSize) const override;
	virtual void GetLightShaderParameters(FLightRenderParameters& LightParameters, uint32 Flags = 0) const override;
	//~End FLightSceneProxy Interface

protected:

	/** Outer cone angle in radians, clamped to a valid range. */
	float OuterConeAngle;

	/** Cosine of the spot light's inner cone angle. */
	float CosInnerCone;

	/** Cosine of the spot light's outer cone angle. */
	float CosOuterCone;

	/** 1 / (CosInnerCone - CosOuterCone) */
	float InvCosConeDifference;

	/** Sine of the spot light's outer cone angle. */
	float SinOuterCone;

	/** 1 / Tangent of the spot light's outer cone angle. */
	float InvTanOuterCone;
};