// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PointLightSceneProxyDesc.h"

struct FSpotLightSceneProxyDesc : public FPointLightSceneProxyDesc
{
	typedef FPointLightSceneProxyDesc Super;

	FSpotLightSceneProxyDesc() = default;
	virtual ~FSpotLightSceneProxyDesc() = default;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** Inner cone angle (degree) */
	float InnerConeAngle = 0.0f;

	/** Outer cone angle (degree) */
	float OuterConeAngle = 0.0f;
};