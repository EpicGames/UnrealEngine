// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"
#include "ExponentialHeightFogStateStreamHandle.h"
#include "Rendering/ExponentialHeightFogData.h"
#include "StateStreamDefinitions.h"
#include "TransformStateStreamMath.h"
#include "ExponentialHeightFogStateStream.generated.h"

class UTextureCube;

inline bool StateStreamEquals(const FExponentialHeightFogData& A, const FExponentialHeightFogData& B);
inline void StateStreamInterpolate(FStateStreamInterpolateContext& Context, FExponentialHeightFogData& Out, const FExponentialHeightFogData& From, const FExponentialHeightFogData& To);

////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(StateStreamStaticState)
struct FExponentialHeightFogStaticState
{
	GENERATED_USTRUCT_BODY()
};


////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT(StateStreamDynamicState)
struct FExponentialHeightFogDynamicState
{
	GENERATED_USTRUCT_BODY()

// private: // Should be private but complicates some of the code that needs support both this path and UComponent path

	UPROPERTY()
	float FogDensity = 0.0f;
	UPROPERTY()
	float FogHeightFalloff = 0.0f;
	UPROPERTY()
	FExponentialHeightFogData SecondFogData;
	UPROPERTY()
	FLinearColor FogInscatteringLuminance = FLinearColor::White;
	UPROPERTY()
	FLinearColor SkyAtmosphereAmbientContributionColorScale = FLinearColor::White;
	UPROPERTY()
	TObjectPtr<UTextureCube> InscatteringColorCubemap;
	UPROPERTY()
	float SkyLightCaptureAffectsHeightFogRoughness = 0.0f;
	UPROPERTY()
	float SkyLightCaptureAffectsHeightFogStrength = 0.0f;
	UPROPERTY()
	float InscatteringColorCubemapAngle = 0.0f;
	UPROPERTY()
	FLinearColor InscatteringTextureTint = FLinearColor::White;
	UPROPERTY()
	float FullyDirectionalInscatteringColorDistance = 0.0f;
	UPROPERTY()
	float NonDirectionalInscatteringColorDistance = 0.0f;
	UPROPERTY()
	float DirectionalInscatteringExponent = 0.0f;
	UPROPERTY()
	float DirectionalInscatteringStartDistance = 0.0f;
	UPROPERTY()
	FLinearColor DirectionalInscatteringLuminance = FLinearColor::White;
	UPROPERTY()
	float FogMaxOpacity = 0.0f;
	UPROPERTY()
	float StartDistance = 0.0f;
	UPROPERTY()
	float EndDistance = 0.0f;
	UPROPERTY()
	float FogCutoffDistance = 0.0f;
	UPROPERTY()
	bool bEnableVolumetricFog = false;
	UPROPERTY()
	float VolumetricFogScatteringDistribution = 0.0f;
	UPROPERTY()
	FColor VolumetricFogAlbedo = FColor::White;
	UPROPERTY()
	FLinearColor VolumetricFogEmissive = FLinearColor::White;
	UPROPERTY()
	float VolumetricFogExtinctionScale = 0.0f;
	UPROPERTY()
	float VolumetricFogDistance = 0.0f;
	UPROPERTY()
	float VolumetricFogStartDistance = 0.0f;
	UPROPERTY()
	float VolumetricFogNearFadeInDistance = 0.0f;
	UPROPERTY()
	float VolumetricFogStaticLightingScatteringIntensity = 0.0f;
	UPROPERTY()
	bool bOverrideLightColorsWithFogInscatteringColors = false;
	UPROPERTY()
	bool bEnableFSSS = false;
	UPROPERTY()
	float FSSSSceneColorScatteringAmountScale = 1.0f;
	UPROPERTY()
	float FSSSSceneColorScatteringAmountPower = 1.0f;
	UPROPERTY()
	float FSSSSpreadScale = 0.1f;
	UPROPERTY()
	float FSSSBlurControl = 0.5f;
	UPROPERTY()
	uint8 bHoldout : 1 = false;
	UPROPERTY()
	uint8 bRenderInMainPass : 1 = false;
	UPROPERTY()
	uint8 bVisibleInReflectionCaptures : 1 = false;
	UPROPERTY()
	uint8 bVisibleInRealTimeSkyCaptures : 1 = false;
	UPROPERTY()
	float Height = 0.0f;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

inline constexpr uint32 ExponentialHeightFogStateStreamId = 7;


////////////////////////////////////////////////////////////////////////////////////////////////////

class IExponentialHeightFogStateStream
{
public:
	DECLARE_STATESTREAM(ExponentialHeightFog)
	virtual FExponentialHeightFogHandle Game_CreateInstance(const FExponentialHeightFogStaticState& Ss, const FExponentialHeightFogDynamicState& Ds, uint32 LaneId = DefaultLaneId) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

inline bool StateStreamEquals(const FExponentialHeightFogData& A, const FExponentialHeightFogData& B)
{
	return A.FogDensity == B.FogDensity && A.FogHeightFalloff == B.FogHeightFalloff && A.FogHeightOffset == B.FogHeightOffset;
}

inline void StateStreamInterpolate(FStateStreamInterpolateContext& Context, FExponentialHeightFogData& Out, const FExponentialHeightFogData& From, const FExponentialHeightFogData& To)
{
	StateStreamInterpolate(Context, Out.FogDensity, From.FogDensity, To.FogDensity);
	StateStreamInterpolate(Context, Out.FogHeightFalloff, From.FogHeightFalloff, To.FogHeightFalloff);
	StateStreamInterpolate(Context, Out.FogHeightOffset, From.FogHeightOffset, To.FogHeightOffset);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
