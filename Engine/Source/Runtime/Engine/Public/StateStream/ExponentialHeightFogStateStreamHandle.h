// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateStreamHandle.h"
#include "ExponentialHeightFogStateStreamHandle.generated.h"

struct FExponentialHeightFogDynamicState;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Handle for mesh instance

USTRUCT(StateStreamHandle)
struct FExponentialHeightFogHandle : public FStateStreamHandle
{
	GENERATED_USTRUCT_BODY()
};

////////////////////////////////////////////////////////////////////////////////////////////////////
