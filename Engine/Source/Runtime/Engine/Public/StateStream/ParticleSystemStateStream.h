// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ParticleSystemStateStreamHandle.h"
#include "StateStreamDefinitions.h"
#include "TransformStateStreamHandle.h"
#include "ParticleSystemStateStream.generated.h"

class UFXSystemAsset;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Static state for mesh instance. Can only be set upon creation

USTRUCT(StateStreamStaticState)
struct FParticleSystemStaticState
{
	GENERATED_USTRUCT_BODY()
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Dynamic state for mesh instance. Can be updated inside ticks

USTRUCT(StateStreamDynamicState)
struct FParticleSystemDynamicState
{
	GENERATED_USTRUCT_BODY()

private:

	UPROPERTY()
	FTransformHandle Transform;

	UPROPERTY()
	TObjectPtr<UFXSystemAsset> SystemAsset;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Mesh state stream id used for registering dependencies and find statestream

inline constexpr uint32 ParticleSystemStateStreamId = 4;


////////////////////////////////////////////////////////////////////////////////////////////////////
// Interface for creating mesh instances

class IParticleSystemStateStream
{
public:
	DECLARE_STATESTREAM(ParticleSystem)
	virtual FParticleSystemHandle Game_CreateInstance(const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds, uint32 LaneId = DefaultLaneId) = 0;

	virtual void SetOtherBackend(IParticleSystemStateStream* Other) {}
};
