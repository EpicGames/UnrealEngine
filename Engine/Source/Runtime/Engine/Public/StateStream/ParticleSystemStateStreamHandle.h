// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateStreamHandle.h"
#include "ParticleSystemStateStreamHandle.generated.h"

struct FParticleSystemDynamicState;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Handle for mesh instance

USTRUCT(StateStreamHandle)
struct FParticleSystemHandle : public FStateStreamHandle
{
	GENERATED_USTRUCT_BODY()
};

////////////////////////////////////////////////////////////////////////////////////////////////////
