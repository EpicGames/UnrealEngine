// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialRelevance.h"
#include "SkinnedMeshStateStreamHandle.h"
#include "StateStreamDefinitions.h"
#include "TransformStateStreamHandle.h"
#include "SkinnedMeshStateStream.generated.h"

class USkinnedAsset;
class UMaterialInterface;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Skinned state for mesh instance. Can only be set upon creation

USTRUCT(StateStreamStaticState)
struct FSkinnedMeshStaticState
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	uint64 MaterialRelevance = 0;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Dynamic state for mesh instance. Can be updated inside ticks

USTRUCT(StateStreamDynamicState)
struct FSkinnedMeshDynamicState
{
	GENERATED_USTRUCT_BODY()

private: // Use accessors (SetX/GetX) instead of properties directly

	UPROPERTY()
	FTransformHandle Transform;

	UPROPERTY()
	TObjectPtr<USkinnedAsset> SkinnedAsset;

	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Mesh state stream id used for registering dependencies and find statestream

inline constexpr uint32 SkinnedMeshStateStreamId = 3;


////////////////////////////////////////////////////////////////////////////////////////////////////
// Interface for creating mesh instances

class ISkinnedMeshStateStream
{
public:
	DECLARE_STATESTREAM(SkinnedMesh)
	virtual FSkinnedMeshHandle Game_CreateInstance(const FSkinnedMeshStaticState& Ss, const FSkinnedMeshDynamicState& Ds, uint32 LaneId = DefaultLaneId) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
