// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateStreamHandle.h"
#include "SkinnedMeshStateStreamHandle.generated.h"

struct FSkinnedMeshDynamicState;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Handle for mesh instance

USTRUCT(StateStreamHandle)
struct FSkinnedMeshHandle : public FStateStreamHandle
{
	GENERATED_USTRUCT_BODY()
};

////////////////////////////////////////////////////////////////////////////////////////////////////
