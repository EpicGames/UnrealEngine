// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"
#include "SkyAtmosphereStateStreamHandle.h"
#include "StateStreamDefinitions.h"
#include "TransformStateStreamHandle.h"
#include "TransformStateStreamMath.h"
#include "SkyAtmosphereStateStream.generated.h"

enum class ESkyAtmosphereTransformMode : uint8;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Static state for mesh instance. Can only be set upon creation

USTRUCT(StateStreamStaticState)
struct FSkyAtmosphereStaticState
{
	GENERATED_USTRUCT_BODY()
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Dynamic state for mesh instance. Can be updated inside ticks

USTRUCT()
struct FOverrideAtmosphericLight
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	uint8 EnabledMask;

	UPROPERTY()
	FVector Direction[NUM_ATMOSPHERE_LIGHTS];

	FOverrideAtmosphericLight() { memset(this, 0, sizeof(FOverrideAtmosphericLight)); }
	bool operator==(const FOverrideAtmosphericLight& O) const { return memcmp(this, &O, sizeof(FOverrideAtmosphericLight)) == 0; }
};

USTRUCT(StateStreamDynamicState)
struct FSkyAtmosphereDynamicState
{
	GENERATED_USTRUCT_BODY()

// private: // Should be private but complicates some of the code that needs support both this path and UComponent path

	UPROPERTY()
	ESkyAtmosphereTransformMode TransformMode = ESkyAtmosphereTransformMode(0);
	UPROPERTY()
	float BottomRadius = 0.0f;
	UPROPERTY()
	FColor GroundAlbedo = FColor::White;
	UPROPERTY()
	float AtmosphereHeight = 0.0f;
	UPROPERTY()
	float MultiScatteringFactor = 0.0f;
	UPROPERTY()
	float TraceSampleCountScale = 0.0f;
	UPROPERTY()
	float RayleighScatteringScale = 0.0f;
	UPROPERTY()
	FLinearColor RayleighScattering = FLinearColor::White;
	UPROPERTY()
	float RayleighExponentialDistribution = 0.0f;
	UPROPERTY()
	float MieScatteringScale = 0.0f;
	UPROPERTY()
	FLinearColor MieScattering = FLinearColor::White;
	UPROPERTY()
	float MieAbsorptionScale = 0.0f;
	UPROPERTY()
	FLinearColor MieAbsorption = FLinearColor::White;
	UPROPERTY()
	float MieAnisotropy = 0.0f;
	UPROPERTY()
	float MieExponentialDistribution = 0.0f;
	UPROPERTY()
	float OtherAbsorptionScale = 0.0f;
	UPROPERTY()
	FLinearColor OtherAbsorption = FLinearColor::White;
	UPROPERTY()
	float OtherTentDistributionTipAltitude = 0.0f;
	UPROPERTY()
	float OtherTentDistributionTipValue = 0.0f;
	UPROPERTY()
	float OtherTentDistributionWidth = 1.0f;
	UPROPERTY()
	FLinearColor SkyLuminanceFactor = FLinearColor::White;
	UPROPERTY()
	FLinearColor SkyAndAerialPerspectiveLuminanceFactor = FLinearColor::White;
	UPROPERTY()
	float AerialPespectiveViewDistanceScale = 0.0f;
	UPROPERTY()
	float HeightFogContribution = 0.0f;
	UPROPERTY()
	float TransmittanceMinLightElevationAngle = 0.0f;
	UPROPERTY()
	float AerialPerspectiveStartDepth = 0.0f;
	UPROPERTY()
	uint8 bHoldout : 1 = false;
	UPROPERTY()
	uint8 bRenderInMainPass : 1 = false;
	UPROPERTY()
	uint8 bBuilt : 1 = false;

	UPROPERTY()
	FTransform ComponentTransform;

	UPROPERTY()
	FOverrideAtmosphericLight OverrideAtmosphericLight;

	//UPROPERTY()
	//bool OverrideAtmosphericLight[NUM_ATMOSPHERE_LIGHTS];

	//UPROPERTY()
	//FVector OverrideAtmosphericLightDirection[NUM_ATMOSPHERE_LIGHTS];
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Mesh state stream id used for registering dependencies and find statestream

inline constexpr uint32 SkyAtmosphereStateStreamId = 6;


////////////////////////////////////////////////////////////////////////////////////////////////////
// Interface for creating mesh instances

class ISkyAtmosphereStateStream
{
public:
	DECLARE_STATESTREAM(SkyAtmosphere)
	virtual FSkyAtmosphereHandle Game_CreateInstance(const FSkyAtmosphereStaticState& Ss, const FSkyAtmosphereDynamicState& Ds, uint32 LaneId = DefaultLaneId) = 0;
};
