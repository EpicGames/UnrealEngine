// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateStreamHandle.h"
#include "SkyAtmosphereStateStreamHandle.generated.h"

struct FSkyAtmosphereDynamicState;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Handle for mesh instance

USTRUCT(StateStreamHandle)
struct FSkyAtmosphereHandle : public FStateStreamHandle
{
	GENERATED_USTRUCT_BODY()
};

////////////////////////////////////////////////////////////////////////////////////////////////////
