// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StaticMeshStateStreamHandle.h"
#include "StateStreamDefinitions.h"
#include "TransformStateStreamHandle.h"
#include "StaticMeshStateStream.generated.h"

class UStaticMesh;
class UMaterialInterface;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Static state for mesh instance. Can only be set upon creation

USTRUCT(StateStreamStaticState)
struct FStaticMeshStaticState
{
	GENERATED_USTRUCT_BODY()
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Dynamic state for mesh instance. Can be updated inside ticks

USTRUCT(StateStreamDynamicState)
struct FStaticMeshDynamicState
{
	GENERATED_USTRUCT_BODY()

private: // Use accessors (SetX/GetX) instead of properties directly

	UPROPERTY()
	FTransformHandle Transform;

	UPROPERTY()
	TObjectPtr<UStaticMesh> Mesh;

	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;

	UPROPERTY()
	TArray<uint32> Owners;

	UPROPERTY()
	uint32 bOnlyOwnerSee : 1 = false;

	UPROPERTY()
	uint32 bOwnerNoSee : 1 = false;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Mesh state stream id used for registering dependencies and find statestream

inline constexpr uint32 StaticMeshStateStreamId = 2;


////////////////////////////////////////////////////////////////////////////////////////////////////
// Interface for creating mesh instances

class IStaticMeshStateStream
{
public:
	DECLARE_STATESTREAM(StaticMesh)
	virtual FStaticMeshHandle Game_CreateInstance(const FStaticMeshStaticState& Ss, const FStaticMeshDynamicState& Ds, uint32 LaneId = DefaultLaneId) = 0;
};
