// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticLighting.h: Static lighting definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/RefCounting.h"
#include "Components.h"

class FStaticLightingMapping;
class FStaticLightingBuildContext;
class FStaticLightingTextureMapping;
class ULevel;
class ULightComponent;
class UPrimitiveComponent;

/** The vertex data used to build static lighting. */
struct FStaticLightingVertex
{
	FVector WorldPosition;
	FVector WorldTangentX;
	FVector WorldTangentY;
	FVector WorldTangentZ;
	FVector2D TextureCoordinates[MAX_STATIC_TEXCOORDS];

	// Operators used for linear combinations of static lighting vertices.

	friend FStaticLightingVertex operator+(const FStaticLightingVertex& A,const FStaticLightingVertex& B)
	{
		FStaticLightingVertex Result;
		Result.WorldPosition =	A.WorldPosition + B.WorldPosition;
		Result.WorldTangentX =	A.WorldTangentX + B.WorldTangentX;
		Result.WorldTangentY =	A.WorldTangentY + B.WorldTangentY;
		Result.WorldTangentZ =	A.WorldTangentZ + B.WorldTangentZ;
		for(int32 CoordinateIndex = 0;CoordinateIndex < MAX_STATIC_TEXCOORDS;CoordinateIndex++)
		{
			Result.TextureCoordinates[CoordinateIndex] = A.TextureCoordinates[CoordinateIndex] + B.TextureCoordinates[CoordinateIndex];
		}
		return Result;
	}

	friend FStaticLightingVertex operator-(const FStaticLightingVertex& A,const FStaticLightingVertex& B)
	{
		FStaticLightingVertex Result;
		Result.WorldPosition =	A.WorldPosition - B.WorldPosition;
		Result.WorldTangentX =	A.WorldTangentX - B.WorldTangentX;
		Result.WorldTangentY =	A.WorldTangentY - B.WorldTangentY;
		Result.WorldTangentZ =	A.WorldTangentZ - B.WorldTangentZ;
		for(int32 CoordinateIndex = 0;CoordinateIndex < MAX_STATIC_TEXCOORDS;CoordinateIndex++)
		{
			Result.TextureCoordinates[CoordinateIndex] = A.TextureCoordinates[CoordinateIndex] - B.TextureCoordinates[CoordinateIndex];
		}
		return Result;
	}

	friend FStaticLightingVertex operator*(const FStaticLightingVertex& A,float B)
	{
		FStaticLightingVertex Result;
		Result.WorldPosition =	A.WorldPosition * B;
		Result.WorldTangentX =	A.WorldTangentX * B;
		Result.WorldTangentY =	A.WorldTangentY * B;
		Result.WorldTangentZ =	A.WorldTangentZ * B;
		for(int32 CoordinateIndex = 0;CoordinateIndex < MAX_STATIC_TEXCOORDS;CoordinateIndex++)
		{
			Result.TextureCoordinates[CoordinateIndex] = A.TextureCoordinates[CoordinateIndex] * B;
		}
		return Result;
	}

	friend FStaticLightingVertex operator/(const FStaticLightingVertex& A,float B)
	{
		const float InvB = 1.0f / B;

		FStaticLightingVertex Result;
		Result.WorldPosition =	A.WorldPosition * InvB;
		Result.WorldTangentX =	A.WorldTangentX * InvB;
		Result.WorldTangentY =	A.WorldTangentY * InvB;
		Result.WorldTangentZ =	A.WorldTangentZ * InvB;
		for(int32 CoordinateIndex = 0;CoordinateIndex < MAX_STATIC_TEXCOORDS;CoordinateIndex++)
		{
			Result.TextureCoordinates[CoordinateIndex] = A.TextureCoordinates[CoordinateIndex] * InvB;
		}
		return Result;
	}
};

/** The result of an intersection between a light ray and the scene. */
class FLightRayIntersection
{
public:

	/** true if the light ray intersected scene geometry. */
	uint32 bIntersects : 1;

	/** The differential geometry which the light ray intersected with. */
	FStaticLightingVertex IntersectionVertex;

	/** Initialization constructor. */
	FLightRayIntersection(bool bInIntersects,const FStaticLightingVertex& InIntersectionVertex):
		bIntersects(bInIntersects),
		IntersectionVertex(InIntersectionVertex)
	{}

	/** No intersection constructor. */
	static FLightRayIntersection None() { return FLightRayIntersection(false,FStaticLightingVertex()); }
};

/** A mesh which is used for computing static lighting. */
class FStaticLightingMesh : public virtual FRefCountedObject
{
public:

	/** The number of triangles in the mesh that will be used for visibility tests. */
	const int32 NumTriangles;

	/** The number of shading triangles in the mesh. */
	const int32 NumShadingTriangles;

	/** The number of vertices in the mesh that will be used for visibility tests. */
	const int32 NumVertices;

	/** The number of shading vertices in the mesh. */
	const int32 NumShadingVertices;

	/** The texture coordinate index which is used to parametrize materials. */
	const int32 TextureCoordinateIndex;

	/** Used for precomputed visibility */
	TArray<int32> VisibilityIds;

	/** Whether the mesh casts a shadow. */
	const uint32 bCastShadow : 1;

	/** Whether the mesh uses a two-sided material. */
	const uint32 bTwoSidedMaterial : 1;

	/** The lights which affect the mesh's primitive. */
	const TArray<ULightComponent*> RelevantLights;

	/** The guids of the lights which affect the mesh's primitive. */
	TArray<FGuid> RelevantLightsGuid;
	
	/** The primitive component this mesh was created by. */
	const UPrimitiveComponent* const Component;

	/** The guid identifying the component associated with this mesh */
	FGuid ComponentGuid;

	/** The bounding box of the mesh. */
	FBox BoundingBox;

	/** Unique ID for tracking this lighting mesh during distributed lighting */
	FGuid Guid;

	/** Cached guid for the source mesh */
	FGuid SourceMeshGuid;

	/** Other FStaticLightingMesh's that should be considered the same mesh object (just different LOD), and should not shadow this LOD. */
	TArray<TRefCountPtr<FStaticLightingMesh> > OtherMeshLODs;

	uint32 HLODTreeIndex;
	uint32 HLODChildStartIndex;
	uint32 HLODChildEndIndex;

	/** Initialization constructor. */
	ENGINE_API FStaticLightingMesh(
		int32 InNumTriangles,
		int32 InNumShadingTriangles,
		int32 InNumVertices,
		int32 InNumShadingVertices,
		int32 InTextureCoordinateIndex,
		bool bInCastShadow,
		bool bInTwoSidedMaterial,
		const TArray<ULightComponent*>& InRelevantLights,
		const UPrimitiveComponent* const InComponent,
		const FBox& InBoundingBox,
		const FGuid& InSourceMeshGuid,
		const FGuid& InComponentGuid
		);

	ENGINE_API FStaticLightingMesh();

	ENGINE_API virtual void Serialize(FArchive& Ar);

	/** Virtual destructor. */
	virtual ~FStaticLightingMesh() = default;

	/**
	 * Accesses a triangle.
	 * @param TriangleIndex - The triangle to access.
	 * @param OutV0 - Upon return, should contain the first vertex of the triangle.
	 * @param OutV1 - Upon return, should contain the second vertex of the triangle.
	 * @param OutV2 - Upon return, should contain the third vertex of the triangle.
	 */
	virtual void GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2) const = 0;

	/**
	 * Accesses a triangle for shading.
	 * @param TriangleIndex - The triangle to access, valid range is [0, NumShadingTriangles).
	 * @param OutV0 - Upon return, should contain the first vertex of the triangle.
	 * @param OutV1 - Upon return, should contain the second vertex of the triangle.
	 * @param OutV2 - Upon return, should contain the third vertex of the triangle.
	 * @param ElementIndex - Indicates the element index of the triangle.
	 */
	virtual void GetShadingTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2) const
	{
		checkSlow(NumTriangles == NumShadingTriangles);
		// By default the geometry used for shading is the same as the geometry used for visibility testing.
		GetTriangle(TriangleIndex, OutV0, OutV1, OutV2);
	}

	/**
	 * Accesses a triangle's vertex indices.
	 * @param TriangleIndex - The triangle to access.
	 * @param OutI0 - Upon return, should contain the first vertex index of the triangle.
	 * @param OutI1 - Upon return, should contain the second vertex index of the triangle.
	 * @param OutI2 - Upon return, should contain the third vertex index of the triangle.
	 */
	virtual void GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const = 0;

	/**
	 * Accesses a triangle's vertex indices for shading.
	 * @param TriangleIndex - The triangle to access, valid range is [0, NumShadingTriangles).
	 * @param OutI0 - Upon return, should contain the first vertex index of the triangle.
	 * @param OutI1 - Upon return, should contain the second vertex index of the triangle.
	 * @param OutI2 - Upon return, should contain the third vertex index of the triangle.
	 */
	virtual void GetShadingTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const
	{ 
		checkSlow(NumTriangles == NumShadingTriangles);
		// By default the geometry used for shading is the same as the geometry used for visibility testing.
		GetTriangleIndices(TriangleIndex, OutI0, OutI1, OutI2);
	}

	/**
	 * Determines whether the mesh should cast a shadow from a specific light on a specific mapping.
	 * This doesn't determine if the mesh actually shadows the receiver, just whether it should be allowed to.
	 * @param Light - The light source.
	 * @param Receiver - The mapping which is receiving the light.
	 * @return true if the mesh should shadow the receiver from the light.
	 */
	virtual bool ShouldCastShadow(ULightComponent* Light,const FStaticLightingMapping* Receiver) const;

	/** @return		true if the specified triangle casts a shadow. */
	virtual bool IsTriangleCastingShadow(uint32 TriangleIndex) const
	{
		return true;
	}

	/** @return		true if the mesh wants to control shadow casting per element rather than per mesh. */
	virtual bool IsControllingShadowPerElement() const
	{
		return false;
	}

	/**
	 * Checks whether ShouldCastShadow will return true always.
	 * @return true if ShouldCastShadow will return true always. 
	 */
	virtual bool IsUniformShadowCaster() const
	{
		return bCastShadow;
	}

	/**
	 * Checks if a line segment intersects the mesh.
	 * @param Start - The start point of the line segment.
	 * @param End - The end point of the line segment.
	 * @param bFindNearestIntersection - Whether the nearest intersection is needed, or any intersection.
	 * @return The intersection of the light-ray with the mesh.
	 */
	virtual FLightRayIntersection IntersectLightRay(const FVector& Start,const FVector& End,bool bFindNearestIntersection) const = 0;

#if WITH_EDITOR
	/** 
	 * Export static lighting mesh instance data to an exporter 
	 * @param Exporter - export interface to process static lighting data
	 **/
	virtual void ExportMeshInstance(class FLightmassExporter* Exporter) const {}
#endif	//#if WITH_EDITOR

	/**
	 * Returns the Guid used for static lighting.
	 * @return FGuid that identifies the mapping
	 **/
	virtual const FGuid&	GetLightingGuid() const
	{
		return Guid;
	}

	/** 
	* Returns true if this mesh is an instanced mesh.  
	**/
	virtual bool IsInstancedMesh() const = 0;
};

/** A mapping between world-space surfaces and a static lighting cache. */
class FStaticLightingMapping : public virtual FRefCountedObject
{
public:

	/** The mesh associated with the mapping. */
	class FStaticLightingMesh* Mesh;

	/** The object which owns the mapping. */
	UObject* Owner;

	/** true if the mapping should be processed by Lightmass. */
	uint32 bProcessMapping : 1;

	/** true if the mapping should be deferred until a finalizing pass. */
	uint32 bIsDeferred : 1;

	/** Initialization constructor. */
	FStaticLightingMapping(FStaticLightingMesh* InMesh,UObject* InOwner):
		Mesh(InMesh),
		Owner(InOwner),
		bProcessMapping(false),
		bIsDeferred(false)
	{}

	/** Virtual destructor. */
	virtual ~FStaticLightingMapping() = default;

	/** @return If the mapping is a texture mapping, returns a pointer to this mapping as a texture mapping.  Otherwise, returns NULL. */
	virtual FStaticLightingTextureMapping* GetTextureMapping()
	{
		return NULL;
	}

	/** @return true if the mapping is a texture mapping.  Otherwise, returns false. */
	virtual bool IsTextureMapping() const
	{
		return false;
	}

#if WITH_EDITOR
	virtual bool DebugThisMapping() const = 0;

	/** 
	 * Export static lighting mapping instance data to an exporter 
	 * @param Exporter - export interface to process static lighting data
	 */
	virtual void ExportMapping(class FLightmassExporter* Exporter) {};
#endif	//WITH_EDITOR

	/**
	 * Returns the Guid used for static lighting.
	 * @return FGuid that identifies the mapping
	 */
	virtual const FGuid& GetLightingGuid() const
	{
		return Mesh->Guid;
	}

	virtual FString GetDescription() const
	{
		return FString("Mapping");
	}

	virtual int32 GetTexelCount() const
	{
		return 0;
	}

	bool IsDeferred() const
	{
		return !!bIsDeferred;
	}

	FStaticLightingMapping(const FArchive&) : Mesh(nullptr), Owner(nullptr), bProcessMapping(false), bIsDeferred(false) { }

#if WITH_EDITOR

	ENGINE_API virtual void Serialize(FArchive& Ar);
	
	/**
	 *	@return	UOject*		The object that is mapped by this mapping
	 */
	virtual UObject* GetMappedObject() const
	{
		return Owner;
	}
#endif	//WITH_EDITOR
};

/**
 * Determines whether the mesh should cast a shadow from a specific light on a specific mapping.
 * This doesn't determine if the mesh actually shadows the receiver, just whether it should be allowed to.
 * @param Light - The light source.
 * @param Receiver - The mapping which is receiving the light.
 * @return true if the mesh should shadow the receiver from the light.
 */
inline bool FStaticLightingMesh::ShouldCastShadow(ULightComponent* Light,const FStaticLightingMapping* Receiver) const
{
	// If this is a shadow casting mesh, then it is allowed to cast a shadow on the receiver from this light.
	return bCastShadow;
}

/** A mapping between world-space surfaces and static lighting cache textures. */
class FStaticLightingTextureMapping : public FStaticLightingMapping
{
public:

	/** The width of the static lighting textures used by the mapping. */
	int32 SizeX;

	/** The height of the static lighting textures used by the mapping. */
	int32 SizeY;

	/** The lightmap texture coordinate index which is used for the mapping. */
	int32 LightmapTextureCoordinateIndex;

	/** Whether to apply a bilinear filter to the sample or not. */
	bool bBilinearFilter;

	/** Initialization constructor. */
	ENGINE_API FStaticLightingTextureMapping(FStaticLightingMesh* InMesh, UObject* InOwner, int32 InSizeX, int32 InSizeY, int32 InLightmapTextureCoordinateIndex, bool bInBilinearFilter = true);

	/**
	 * Called when the static lighting has been computed to apply it to the mapping's owner.
	 * This function is responsible for deleting ShadowMapData and QuantizedData.
	 * @param LightMapData - The light-map data which has been computed for the mapping.
	 */
	virtual void Apply(struct FQuantizedLightmapData* QuantizedData, const TMap<ULightComponent*,class FShadowMapData2D*>& ShadowMapData, const FStaticLightingBuildContext* LightingContext) = 0;

	// FStaticLightingMapping interface.
	virtual FStaticLightingTextureMapping* GetTextureMapping()
	{
		return this;
	}

	/** @return true if the mapping is a texture mapping.  Otherwise, returns false. */
	virtual bool IsTextureMapping() const
	{
		return true;
	}

	/** Whether or not this mapping should be processed or imported */
	virtual bool IsValidMapping() const {return true;}

#if WITH_EDITOR
	UNREALED_API virtual bool DebugThisMapping() const;
	ENGINE_API virtual void Serialize(FArchive& Ar);
#endif	//WITH_EDITOR

	virtual FString GetDescription() const
	{
		return FString("TextureMapping");
	}

	virtual int32 GetTexelCount() const
	{
		return (SizeX * SizeY);
	}

	/** Default constructor used by serializer */
	FStaticLightingTextureMapping(const FArchive& Ar) : FStaticLightingMapping(Ar) { }
};

/** 
 * Represents an object which will use the global volumetric lightmap.
 * Hack: currently represented as a texture mapping for Lightmass GI solver surface caching
 */
class FStaticLightingGlobalVolumeMapping : public FStaticLightingTextureMapping
{
public:

	/** Initialization constructor. */
	ENGINE_API FStaticLightingGlobalVolumeMapping(FStaticLightingMesh* InMesh, UObject* InOwner, int32 InSizeX, int32 InSizeY, int32 InLightmapTextureCoordinateIndex);

	virtual void Apply(struct FQuantizedLightmapData* QuantizedData, const TMap<ULightComponent*,class FShadowMapData2D*>& ShadowMapData, const FStaticLightingBuildContext* LightingContext) override
	{
		// Should never be processed
		check(false);
	}

#if WITH_EDITOR
	virtual bool DebugThisMapping() const
	{
		return false;
	}

	/** 
	 * Export static lighting mapping instance data to an exporter 
	 * @param Exporter - export interface to process static lighting data
	 */
	UNREALED_API virtual void ExportMapping(class FLightmassExporter* Exporter) override;
#endif	//WITH_EDITOR

	/** Whether or not this mapping should be processed or imported */
	virtual bool IsValidMapping() const {return true;}

	virtual FString GetDescription() const
	{
		return FString("VolumeMapping");
	}
};

/** The info about an actor component which the static lighting system needs. */
struct FStaticLightingPrimitiveInfo
{
	FStaticLightingPrimitiveInfo() :
		VisibilityId(INDEX_NONE)
	{}
	int32 VisibilityId;
	/** The primitive's meshes. */
	TArray< TRefCountPtr<FStaticLightingMesh> > Meshes;

	/** The primitive's static lighting mappings. */
	TArray< TRefCountPtr<FStaticLightingMapping> > Mappings;
};



