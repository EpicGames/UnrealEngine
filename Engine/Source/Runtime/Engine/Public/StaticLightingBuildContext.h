// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "Misc/Guid.h"

class UMapBuildDataRegistry;
class ULevel;
class UWorld;
class AActor;
struct FVolumetricLightMapGridDesc;
class FPrecomputedVolumetricLightmapData;
struct FStaticLightingDescriptors;

#if WITH_EDITOR
class FStaticLightingBuildContext
{	
	mutable UMapBuildDataRegistry* MapBuildDataRegistry;
	
	TMap<FGuid, const TWeakObjectPtr<ULevel>> LevelGuids;
	
	FBox ImportanceBounds;
	FIntVector LocalToGlobalIndirectionOffset;
	FVolumetricLightMapGridDesc* VolumetricLightMapGridDesc;

	
public:

	FStaticLightingDescriptors* Descriptors;
	UWorld*						World;
	ULevel*						LightingScenario;

	ENGINE_API FStaticLightingBuildContext(UWorld* InWorld, ULevel* InLightingScenario);
	ENGINE_API FStaticLightingBuildContext(FStaticLightingBuildContext&& InFrom);
	ENGINE_API ~FStaticLightingBuildContext();
	
	ENGINE_API void SetImportanceBounds(const FBox& Bounds);

	ENGINE_API ULevel* GetLightingStorageLevel(ULevel* Level) const;

	ENGINE_API bool ShouldIncludeActor(AActor* Actor) const;
	ENGINE_API bool ShouldIncludeLevel(ULevel* Level) const;

	ENGINE_API UMapBuildDataRegistry* GetOrCreateGlobalRegistry() const;
	
	ENGINE_API UMapBuildDataRegistry* GetOrCreateRegistryForLevelGuid(const FGuid& Guid) const;
	ENGINE_API FGuid GetLevelGuidForActor(AActor* Actor) const;
	ENGINE_API FGuid GetPersistentLevelGuid() const;
	ENGINE_API FGuid GetLevelGuidForLevel(ULevel* Level) const;
	ENGINE_API FGuid GetLevelGuidForVLMBrick(const FIntVector& BrickCoordinates) const;
	ENGINE_API const TWeakObjectPtr<ULevel> GetLevelForGuid(const FGuid& Guid) const;
	ENGINE_API FGuid GetLevelBuildDataID(const FGuid& LevelGuid) const;

	ENGINE_API FPrecomputedVolumetricLightmapData& GetOrCreateLevelPrecomputedVolumetricLightmapBuildData(const FGuid& LevelId) const;

	FVolumetricLightMapGridDesc* GetVolumetricLightMapGridDesc() const { return VolumetricLightMapGridDesc; }
	void ReleaseVolumetricLightMapGridDesc() { VolumetricLightMapGridDesc = nullptr; }

	ENGINE_API UMapBuildDataRegistry* GetRegistryForLevel(ULevel* Level) const;
	ENGINE_API UMapBuildDataRegistry* GetOrCreateRegistryForActor(AActor* Actor) const;
	ENGINE_API UMapBuildDataRegistry* GetOrCreateRegistryForLevel(ULevel* Level) const;	
};
#endif // WITH_EDITOR
