// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/WeakObjectPtr.h"
#include "Containers/Set.h"
#include "IAssetCompilingManager.h"

#if WITH_EDITOR

class FAsyncCompilationNotification;
class UStaticMesh;
class UPrimitiveComponent;
class UStaticMeshComponent;
class FQueuedThreadPool;
struct FAssetCompileContext;
enum class EQueuedWorkPriority : uint8;

class FStaticMeshCompilingManager : IAssetCompilingManager
{
public:
	struct FFinishCompilationOptions
	{
		/**
		 * In addition to waiting for the specified static meshes and their dependencies to finish compiling, also wait for
		 * all other meshes currently compiling that depend upon them.
		 * NOTE: This should be set to true if you are about to modify the meshes after the call completes, so that you are
		 * not concurrently editing a mesh that's being referenced by another mesh's async build.
		 */
		bool bIncludeDependentMeshes;

		FFinishCompilationOptions() : bIncludeDependentMeshes(false) {}
	};	

	ENGINE_API static FStaticMeshCompilingManager& Get();

	/**
	 * Returns true if the feature is currently activated.
	 */
	ENGINE_API bool IsAsyncStaticMeshCompilationEnabled() const;
	
	/**
	 * Returns true if the cancelation feature is currently activated.
	 */
	ENGINE_API bool IsAsyncCompilationCancelable() const;

	/** 
	 * Returns the number of outstanding texture compilations.
	 */
	ENGINE_API int32 GetNumRemainingMeshes() const;

	/** 
	 * Adds static meshes compiled asynchronously so they are monitored.
	 */
	ENGINE_API void AddStaticMeshes(TArrayView<UStaticMesh* const> InStaticMeshes);

	/** 
	 * Adds static meshes (with dependencies that are still compiling) compiled asynchronously so they are monitored.
	 */
	ENGINE_API void AddStaticMeshesWithDependencies(TArrayView<UStaticMesh* const> InStaticMeshes);

	/** 
	 * Blocks until completion of the requested static meshes.
	 */
	ENGINE_API void FinishCompilation(TArrayView<UStaticMesh* const> InStaticMeshes, const FFinishCompilationOptions& Options = FFinishCompilationOptions());

	/** 
	 * Blocks until completion of all async static mesh compilation.
	 */
	ENGINE_API void FinishAllCompilation() override;

	/**
	 * Returns if asynchronous compilation is allowed for this static mesh.
	 */
	ENGINE_API bool IsAsyncCompilationAllowed(UStaticMesh* InStaticMesh) const;

	/**
	 * Returns the priority at which the given static mesh should be scheduled.
	 */
	ENGINE_API EQueuedWorkPriority GetBasePriority(UStaticMesh* InStaticMesh) const;

	/**
	 * Returns the threadpool where static mesh compilation should be scheduled.
	 */
	ENGINE_API FQueuedThreadPool* GetThreadPool() const;

	/**
	 * Cancel any pending work and blocks until it is safe to shut down.
	 */
	ENGINE_API void Shutdown() override;

	/** Get the name of the asset type this compiler handles */
	ENGINE_API static FName GetStaticAssetTypeName();

private:
	friend class FAssetCompilingManager;

	FStaticMeshCompilingManager();

	FName GetAssetTypeName() const override;
	FTextFormat GetAssetNameFormat() const override;
	TArrayView<FName> GetDependentTypeNames() const override;
	int32 GetNumRemainingAssets() const override;

	/** Called once per frame, fetches completed tasks and applies them to the scene. */
	void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;

	/** Handle generic finish compilation */
	void FinishCompilationForObjects(TArrayView<UObject* const> InObjects) override;

	/** Mark compilation of the provided static meshes as canceled. */
	void MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects) override;

	bool bHasShutdown = false;
	TSet<TWeakObjectPtr<UStaticMesh>> RegisteredStaticMesh;
	TSet<TWeakObjectPtr<UStaticMesh>> StaticMeshesWithPendingDependencies;
	TMap<TWeakObjectPtr<UStaticMesh>, TSet<TWeakObjectPtr<UStaticMesh>>> ReverseDependencyLookup;
	TUniquePtr<FAsyncCompilationNotification> Notification;

	void FinishCompilationsForGame();
	void Reschedule();
	void ProcessStaticMeshes(bool bLimitExecutionTime, int32 MinBatchSize = 1);
	void UpdateCompilationNotification();

	void PostCompilation(TArrayView<UStaticMesh* const> InStaticMeshes);
	void PostCompilation(UStaticMesh* StaticMesh);
	void SchedulePendingCompilations();

	void OnPostReachabilityAnalysis();
	FDelegateHandle PostReachabilityAnalysisHandle;
};

#endif // #if WITH_EDITOR
