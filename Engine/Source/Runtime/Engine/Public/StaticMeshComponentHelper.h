// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rendering/NaniteResourcesHelper.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "AI/NavigationSystemHelpers.h"
#include "MeshComponentHelper.h"
#include "StaticMeshResources.h"
#include "StaticMeshSceneProxy.h"
#include "NaniteVertexFactory.h"
#include "VertexFactory.h"
#include "PSOPrecacheFwd.h"
#include "PSOPrecache.h"
#include "RenderUtils.h"
#include "Engine/MaterialOverlayHelper.h"
#include "Engine/World.h"
#include "SceneInterface.h"

template<typename T>
concept HasOverridesMaterialsMember = requires(T t)
{
	t.OverrideMaterials;
};

/** Helper class used to share implementation for different StaticMeshComponent types */
class FStaticMeshComponentHelper
{
	using GetPSOVertexElementsFn = TFunctionRef<void(const FStaticMeshLODResources& LODRenderData, int32 LODIndex, bool bSupportsManualVertexFetch, FVertexDeclarationElementList& Elements)>;

public:
	enum class ESceneProxyCreationError
	{
		None,
		WaitingPSOs,
		MeshCompiling,
		InvalidMesh
	};

public:
	template<class T>
	static void GetUsedRayTracingOnlyMaterials(const T& Component, TArray<UMaterialInterface*>& OutMaterials);

	template<class T>
	static UMaterialInterface* GetMaterial(const T& Component, int32 MaterialIndex, bool bDoingNaniteMaterialAudit = false, bool bIgnoreNaniteOverrideMaterials = false);

	template<class T>
	static void GetUsedMaterials(const T& Component, TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials);

	template<class T>
	static void CollectPSOPrecacheDataImpl(const T& Component, const FVertexFactoryType* VFType, const FPSOPrecacheParams& BasePrecachePSOParams, GetPSOVertexElementsFn GetVertexElements, FMaterialInterfacePSOPrecacheParamsList& OutParams);

	template<class T>
	static void CollectPSOPrecacheData(const T& Component, const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams);

	template<class T>
	static bool IsNavigationRelevant(const T& Component);

	template<class T>
	static FBox GetNavigationBounds(const T& Component);

	template<class T>
	static void GetNavigationData(const T& Component, FNavigationRelevantData& Data);

	template<class T>
	static bool DoCustomNavigableGeometryExport(const T& Component, FNavigableGeometryExport& GeomExport);

	template<class T, bool bAssumeRenderDataIsReady = false>
	static FPrimitiveSceneProxy* CreateSceneProxy(T& Component, FStaticMeshComponentHelper::ESceneProxyCreationError* OutError = nullptr);

	template<class T>
	static TConstArrayView<TObjectPtr<UMaterialInterface>> GetOverrideMaterials(const T& Component)
	{
		if constexpr (HasOverridesMaterialsMember<std::remove_reference_t<T>>)
		{
			return Component.OverrideMaterials;
		}
		else
		{
			return Component.GetOverrideMaterials();
		}
	}
};

template<typename T>
void FStaticMeshComponentHelper::GetUsedRayTracingOnlyMaterials(const T& Component, TArray<UMaterialInterface*>& OutMaterials)
{
	TArray<UMaterialInterface*> RayTracingMaterials;

	const auto& OverrideMaterials = GetOverrideMaterials(Component);

	if (UStaticMesh* StaticMesh = Component.GetStaticMesh())
	{
		StaticMesh->GetUsedMaterials(RayTracingMaterials, [&Component, &OverrideMaterials](int32 MaterialIndex)
		{
			UMaterialInterface* OutMaterial = nullptr;

			// If we have a base materials array, use that
			if (OverrideMaterials.IsValidIndex(MaterialIndex) && OverrideMaterials[MaterialIndex])
			{
				OutMaterial = OverrideMaterials[MaterialIndex];
			}
			// Otherwise get from static mesh
			else if (Component.GetStaticMesh())
			{
				OutMaterial = Component.GetStaticMesh()->GetMaterial(MaterialIndex);
			}

			if (OutMaterial)
			{
				//@note FH: temporary preemptive PostLoad until zenloader load ordering improvements
				OutMaterial->ConditionalPostLoad();

				if (UMaterialInterface* NaniteOverride = OutMaterial->GetNaniteOverride())
				{
					return OutMaterial;
				}
			}

			return (UMaterialInterface*)nullptr;
		});
	}

	if (!RayTracingMaterials.IsEmpty())
	{
		RayTracingMaterials.RemoveAll([](UMaterialInterface* Material) { return Material == nullptr; });
		OutMaterials.Append(RayTracingMaterials);
	}
}

template<class T>
UMaterialInterface* FStaticMeshComponentHelper::GetMaterial(const T& Component, int32 MaterialIndex, bool bDoingNaniteMaterialAudit, bool bIgnoreNaniteOverrideMaterials)
{
	UMaterialInterface* OutMaterial = nullptr;

	const auto& OverrideMaterials = GetOverrideMaterials(Component);

	// If we have a base materials array, use that
	if (OverrideMaterials.IsValidIndex(MaterialIndex) && OverrideMaterials[MaterialIndex])
	{
		OutMaterial = OverrideMaterials[MaterialIndex];
	}
	// Otherwise get from static mesh
	else if (Component.GetStaticMesh())
	{
		OutMaterial = Component.GetStaticMesh()->GetMaterial(MaterialIndex);
	}

	// If we have a nanite override, use that
	if (OutMaterial)
	{
		//@note FH: temporary preemptive PostLoad until zenloader load ordering improvements
		OutMaterial->ConditionalPostLoad();

		if (!bIgnoreNaniteOverrideMaterials && Component.UseNaniteOverrideMaterials(bDoingNaniteMaterialAudit))
		{
			UMaterialInterface* NaniteOverride = OutMaterial->GetNaniteOverride();
			OutMaterial = NaniteOverride != nullptr ? NaniteOverride : OutMaterial;
		}
	}

	return OutMaterial;
}

template<class T>
void FStaticMeshComponentHelper::GetUsedMaterials(const T& Component, TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials)
{
	if (Component.GetStaticMesh())
	{
		Component.GetStaticMesh()->GetUsedMaterials(OutMaterials, [&Component](int32 Index) { return Component.GetMaterial(Index); });

		// For ray tracing if the mesh is using nanite override materials we need to include fallback materials as well
		if (IsRayTracingAllowed() && Component.GetStaticMesh()->bSupportRayTracing && Component.UseNaniteOverrideMaterials(false))
		{
			GetUsedRayTracingOnlyMaterials(Component, OutMaterials);
		}

		if (OutMaterials.Num() > 0)
		{
			TArray<TObjectPtr<UMaterialInterface>> AssetAndComponentMaterialSlotsOverlayMaterial;
			Component.GetMaterialSlotsOverlayMaterial(AssetAndComponentMaterialSlotsOverlayMaterial);
			bool bUseGlobalMeshOverlayMaterial = false;
			FMaterialOverlayHelper::AppendAllOverlayMaterial(AssetAndComponentMaterialSlotsOverlayMaterial, OutMaterials, bUseGlobalMeshOverlayMaterial);

			if (bUseGlobalMeshOverlayMaterial)
			{
				UMaterialInterface* OverlayMaterialInterface = Component.GetOverlayMaterial();
				if (OverlayMaterialInterface != nullptr)
				{
					OutMaterials.Add(OverlayMaterialInterface);
				}
			}
		}
	}
}

template<class T>
void FStaticMeshComponentHelper::CollectPSOPrecacheDataImpl(const T& Component, const FVertexFactoryType* VFType, const FPSOPrecacheParams& BasePrecachePSOParams, FStaticMeshComponentHelper::GetPSOVertexElementsFn GetVertexElements, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	check(Component.GetStaticMesh() != nullptr && Component.GetStaticMesh()->GetRenderData() != nullptr);

	bool bSupportsManualVertexFetch = VFType->SupportsManualVertexFetch(GMaxRHIFeatureLevel);
	bool bAnySectionCastsShadows = false;
	int32 MeshMinLOD = Component.GetStaticMesh()->GetMinLODIdx();

	FPSOPrecacheVertexFactoryDataPerMaterialIndexList VFTypesPerMaterialIndex;
	FStaticMeshLODResourcesArray& LODResources = Component.GetStaticMesh()->GetRenderData()->LODResources;
	for (int32 LODIndex = MeshMinLOD; LODIndex < LODResources.Num(); ++LODIndex)
	{
		FStaticMeshLODResources& LODRenderData = LODResources[LODIndex];
		FVertexDeclarationElementList VertexElements;
		if (!bSupportsManualVertexFetch)
		{
			GetVertexElements(LODRenderData, LODIndex, bSupportsManualVertexFetch, VertexElements);
		}

		for (FStaticMeshSection& RenderSection : LODRenderData.Sections)
		{
			bAnySectionCastsShadows |= RenderSection.bCastShadow;

			int16 MaterialIndex = (int16)RenderSection.MaterialIndex;
			FPSOPrecacheVertexFactoryDataPerMaterialIndex* VFsPerMaterial = VFTypesPerMaterialIndex.FindByPredicate(
				[MaterialIndex](const FPSOPrecacheVertexFactoryDataPerMaterialIndex& Other) { return Other.MaterialIndex == MaterialIndex; });
			if (VFsPerMaterial == nullptr)
			{
				VFsPerMaterial = &VFTypesPerMaterialIndex.AddDefaulted_GetRef();
				VFsPerMaterial->MaterialIndex = (int16)RenderSection.MaterialIndex;
			}

			if (bSupportsManualVertexFetch)
			{
				VFsPerMaterial->VertexFactoryDataList.AddUnique(FPSOPrecacheVertexFactoryData(VFType));
			}
			else
			{
				VFsPerMaterial->VertexFactoryDataList.AddUnique(FPSOPrecacheVertexFactoryData(VFType, VertexElements));
			}
		}
	}

	bool bIsLocalToWorldDeterminantNegative = Component.GetRenderMatrix().Determinant() < 0;

	FPSOPrecacheParams PrecachePSOParams = BasePrecachePSOParams;
	PrecachePSOParams.bCastShadow = bAnySectionCastsShadows;
	PrecachePSOParams.bReverseCulling = PrecachePSOParams.bReverseCulling || Component.IsReverseCulling() != bIsLocalToWorldDeterminantNegative;
	PrecachePSOParams.bForceLODModel = Component.GetForcedLodModel() > 0;
	PrecachePSOParams.bHasWPODistance = Component.GetWorldPositionOffsetDisableDistance() > 0;
	PrecachePSOParams.bHasPixelDistance = Component.GetNanitePixelProgrammableDistance() > 0;
	PrecachePSOParams.bRequireRTDynamicGeometryUpdate = Component.EvaluateWorldPositionOffsetInRayTracing();

	for (FPSOPrecacheVertexFactoryDataPerMaterialIndex& VFsPerMaterial : VFTypesPerMaterialIndex)
	{
		UMaterialInterface* MaterialInterface = Component.GetMaterial(VFsPerMaterial.MaterialIndex);
		if (MaterialInterface == nullptr)
		{
			MaterialInterface = UMaterial::GetDefaultMaterial(MD_Surface);
		}

		FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
		ComponentParams.MaterialInterface = MaterialInterface;
		ComponentParams.VertexFactoryDataList = VFsPerMaterial.VertexFactoryDataList;
		ComponentParams.PSOPrecacheParams = PrecachePSOParams;
	}

	//Add all overlay materials
	if (VFTypesPerMaterialIndex.Num() != 0)
	{
		//Add all per section overlay material and add the global mesh only if not all sections has an overlay override
		TArray<UMaterialInterface*> OverlayMaterials;
		
		TArray<TObjectPtr<UMaterialInterface>> AssetAndComponentMaterialSlotsOverlayMaterial;
		Component.GetMaterialSlotsOverlayMaterial(AssetAndComponentMaterialSlotsOverlayMaterial);
		bool bUseGlobalMeshOverlayMaterial = false;
		FMaterialOverlayHelper::AppendAllOverlayMaterial(AssetAndComponentMaterialSlotsOverlayMaterial, OverlayMaterials, bUseGlobalMeshOverlayMaterial);
		if (bUseGlobalMeshOverlayMaterial)
		{
			UMaterialInterface* OverlayMaterialInterface = Component.GetOverlayMaterial();
			if (OverlayMaterialInterface)
			{
				OverlayMaterials.Add(OverlayMaterialInterface);
			}
		}
		for (UMaterialInterface* OverlayMaterialToPreCache : OverlayMaterials)
		{
			if (OverlayMaterialToPreCache)
			{
				// Overlay is rendered with the same set of VFs
				FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
				ComponentParams.MaterialInterface = OverlayMaterialToPreCache;
				ComponentParams.VertexFactoryDataList = VFTypesPerMaterialIndex[0].VertexFactoryDataList;
				ComponentParams.PSOPrecacheParams = PrecachePSOParams;
				ComponentParams.PSOPrecacheParams.bCastShadow = false;
			}
		}
	}
}

template<class T>
void FStaticMeshComponentHelper::CollectPSOPrecacheData(const T& Component, const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	if (Component.GetStaticMesh() == nullptr || Component.GetStaticMesh()->GetRenderData() == nullptr)
	{
		return;
	}

	int32 LightMapCoordinateIndex = Component.GetStaticMesh()->GetLightMapCoordinateIndex();

	auto SMC_GetElements = [LightMapCoordinateIndex, &LODData = Component.LODData](const FStaticMeshLODResources& LODRenderData, int32 LODIndex, bool bSupportsManualVertexFetch, FVertexDeclarationElementList& Elements)
	{
		int32 NumTexCoords = (int32)LODRenderData.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
		int32 LODLightMapCoordinateIndex = LightMapCoordinateIndex < NumTexCoords ? LightMapCoordinateIndex : NumTexCoords - 1;
		bool bOverrideColorVertexBuffer = LODIndex < LODData.Num() && LODData[LODIndex].OverrideVertexColors != nullptr;
		FLocalVertexFactory::FDataType Data;
		LODRenderData.VertexBuffers.InitComponentVF(nullptr /*VertexFactory*/, LODLightMapCoordinateIndex, bOverrideColorVertexBuffer, Data);
		FLocalVertexFactory::GetVertexElements(GMaxRHIFeatureLevel, EVertexInputStreamType::Default, bSupportsManualVertexFetch, Data, Elements);
	};

	Nanite::FMaterialAudit NaniteMaterials{};
	if (Nanite::FNaniteResourcesHelper::ShouldCreateNaniteProxy(Component, &NaniteMaterials))
	{
		FStaticMeshComponentHelper::CollectPSOPrecacheDataImpl(Component, &FNaniteVertexFactory::StaticType, BasePrecachePSOParams, SMC_GetElements, OutParams);

		// If bRequireRTDynamicGeometryUpdate is set then add another entry with local vertex factory because nanite can't be used for dynamic RT geometry updates
		FMaterialInterfacePSOPrecacheParamsList RTDynamicGeometryUpdateParams;
		for (FMaterialInterfacePSOPrecacheParams& Params : OutParams)
		{
			if (Params.PSOPrecacheParams.bRequireRTDynamicGeometryUpdate)
			{
				// Add new entry with local vertex factory and disabling rendering to all passes to reduce number of PSOs to compile
				FMaterialInterfacePSOPrecacheParams NewEntry = Params;
				NewEntry.VertexFactoryDataList.Empty(1);
				NewEntry.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType));
				NewEntry.PSOPrecacheParams.bRenderInMainPass = false;
				NewEntry.PSOPrecacheParams.bRenderInDepthPass = false;
				NewEntry.PSOPrecacheParams.bCastShadow = false;
				NewEntry.PSOPrecacheParams.bRenderCustomDepth = false;
				RTDynamicGeometryUpdateParams.Add(NewEntry);

				// Nanite VF entry doesn't require RT dynamic geometry update anymore
				Params.PSOPrecacheParams.bRequireRTDynamicGeometryUpdate = false;
			}
		}
		OutParams.Append(RTDynamicGeometryUpdateParams);
	}
	else
	{
		FStaticMeshComponentHelper::CollectPSOPrecacheDataImpl(Component, &FLocalVertexFactory::StaticType, BasePrecachePSOParams, SMC_GetElements, OutParams);
	}
}

template<class T>
bool FStaticMeshComponentHelper::IsNavigationRelevant(const T& Component)
{
	if (const UStaticMesh* Mesh = Component.GetStaticMesh())
	{
		// Pending compilation, update to the navigation system will be done once compilation finishes.
		return !Mesh->IsCompiling() && Mesh->IsNavigationRelevant() && Component.Super::IsNavigationRelevant();
	}
	return false;
}

template<class T>
FBox FStaticMeshComponentHelper::GetNavigationBounds(const T& Component)
{
	if (const UStaticMesh* Mesh = Component.GetStaticMesh())
	{
#if WITH_EDITOR
		// @see GetNavigationData
		if (!Mesh->IsCompiling())
#endif // WITH_EDITOR
		{
			return Mesh->GetNavigationBounds(Component.GetComponentTransform());
		}
	}

	return Component.Super::GetNavigationBounds();
}

template<class T>
void FStaticMeshComponentHelper::GetNavigationData(const T& Component, FNavigationRelevantData& Data)
{
	Component.Super::GetNavigationData(Data);

	const FVector Scale3D = Component.GetComponentTransform().GetScale3D();
	if (!Scale3D.IsZero())
	{
		if (const UStaticMesh* Mesh = Component.GetStaticMesh())
		{
#if WITH_EDITOR
			// In Editor it's possible that compilation of a StaticMesh gets triggered on a newly registered component for
			// which a pending update is queued for the Navigation system.
			// Then GetNavigationData is called when the pending update is processed but we don't consider the current component
			// relevant to navigation until associated mesh is compiled.
			// On mesh post compilation the component will reregister with the right mesh.
			if (!Mesh->IsCompiling())
#endif // WITH_EDITOR
			{
				if (UNavCollisionBase* NavCollision = Mesh->GetNavCollision())
				{
					// Call GetNavigationModifier for obstacles or when surface area class is used.
					if (Component.ShouldExportAsObstacle(*NavCollision) || NavCollision->HasSurfaceAreaClass())
					{
						NavCollision->GetNavigationModifier(Data.Modifiers, Component.GetComponentTransform());
					}
				}
			}
		}
	}
}

/** Helper to get information from a UObject if available */
namespace UObjectHelper
{
	template<typename T, typename = void>
	struct THasGetFullName : std::false_type {};

	template<typename T>
	struct THasGetFullName<T, std::void_t<decltype(std::declval<T>().GetFullName())>> : std::true_type {};

	template<typename T>
	FString GetFullNameIfAvailable(T& Object)
	{
		if constexpr (THasGetFullName<T>::value)
		{
			return Object.GetFullName();
		}
		else
		{
			return FString("None");
		}
	}
}

template<class T>
bool FStaticMeshComponentHelper::DoCustomNavigableGeometryExport(const T& Component, FNavigableGeometryExport& GeomExport)
{
	const FVector Scale3D = Component.GetComponentTransform().GetScale3D();

	if (!Scale3D.IsZero())
	{
		if (const UStaticMesh* Mesh = Component.GetStaticMesh())
		{
			if (ensureMsgf(!Mesh->IsCompiling(), TEXT("Component %s is not considered relevant to navigation until associated mesh is compiled."), *UObjectHelper::GetFullNameIfAvailable(Component)))
			{
				if (const UNavCollisionBase* NavCollision = Mesh->GetNavCollision())
				{
					if (Component.ShouldExportAsObstacle(*NavCollision))
					{
						// skip default export
						return false;
					}

					const bool bHasData = NavCollision->ExportGeometry(Component.GetComponentTransform(), GeomExport);
					if (bHasData)
					{
						// skip default export
						return false;
					}
				}
			}
		}
	}

	return true;
}

template<class T, bool bAssumeRenderDataIsReady>
FPrimitiveSceneProxy* FStaticMeshComponentHelper::CreateSceneProxy(T& Component, FStaticMeshComponentHelper::ESceneProxyCreationError* OutError)
{
	UStaticMesh* StaticMesh = Component.GetStaticMesh();

	auto SetError = [OutError](FStaticMeshComponentHelper::ESceneProxyCreationError InError)
	{
		if (OutError)
		{
			*OutError = InError;
		}
	};

	if constexpr (!bAssumeRenderDataIsReady)
	{
		if (StaticMesh == nullptr)
		{
			UE_LOGF(LogStaticMesh, Verbose, "Skipping CreateSceneProxy for StaticMeshComponent %ls (StaticMesh is null)", *UObjectHelper::GetFullNameIfAvailable(Component));
			SetError(ESceneProxyCreationError::InvalidMesh);
			return nullptr;
		}

		// Prevent accessing the RenderData during async compilation. The RenderState will be recreated when compilation finishes.
		if (StaticMesh->IsCompiling())
		{
			UE_LOGF(LogStaticMesh, Verbose, "Skipping CreateSceneProxy for StaticMeshComponent %ls (StaticMesh is not ready)", *UObjectHelper::GetFullNameIfAvailable(Component));
			SetError(ESceneProxyCreationError::MeshCompiling);
			return nullptr;
		}

		if (StaticMesh->GetRenderData() == nullptr)
		{
			UE_LOGF(LogStaticMesh, Verbose, "Skipping CreateSceneProxy for StaticMeshComponent %ls (RenderData is null)", *UObjectHelper::GetFullNameIfAvailable(Component));
			SetError(ESceneProxyCreationError::InvalidMesh);
			return nullptr;
		}

		if (!StaticMesh->GetRenderData()->IsInitialized())
		{
			UE_LOGF(LogStaticMesh, Verbose, "Skipping CreateSceneProxy for StaticMeshComponent %ls (RenderData is not initialized)", *UObjectHelper::GetFullNameIfAvailable(Component));
			SetError(ESceneProxyCreationError::InvalidMesh);
			return nullptr;
		}
	}
	else
	{
		check(StaticMesh);
		check(!StaticMesh->IsCompiling());
		check(StaticMesh->GetRenderData());
		check(StaticMesh->GetRenderData()->IsInitialized());		
	}

	EPSOPrecachePriority PSOPrecachePriority = GetStaticMeshComponentBoostPSOPrecachePriority();
	if (Component.CheckPSOPrecachingAndBoostPriority(PSOPrecachePriority) && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogStaticMesh, Verbose, "Skipping CreateSceneProxy for StaticMeshComponent %ls (Static mesh component PSOs are still compiling)", *UObjectHelper::GetFullNameIfAvailable(Component));
		SetError(ESceneProxyCreationError::WaitingPSOs);
		return nullptr;
	}

	const bool bIsMaskingAllowed = Nanite::IsMaskingAllowed(Component.GetWorld(), Component.GetForceNaniteForMasked());

	Nanite::FMaterialAudit NaniteMaterials{};

	// Is Nanite supported, and is there built Nanite data for this static mesh?
	const bool bUseNanite = Component.ShouldCreateNaniteProxy(&NaniteMaterials);

	if (bUseNanite)
	{
		// Nanite is fully supported
		return Component.CreateStaticMeshSceneProxy(NaniteMaterials, true);
	}

	// If we didn't get a proxy, but Nanite was enabled on the asset when it was built, evaluate proxy creation
	if (Component.HasValidNaniteData())
	{
		if (NaniteMaterials.IsValid(bIsMaskingAllowed))
		{
			const bool bAllowProxyRender = Nanite::GetProxyRenderMode() == Nanite::EProxyRenderMode::Allow
#if WITH_EDITORONLY_DATA
				// Check for specific case of static mesh editor "proxy toggle"
				|| (Component.IsDisplayNaniteFallbackMesh() && Nanite::GetProxyRenderMode() == Nanite::EProxyRenderMode::AllowForDebugging)
#endif
				;

			if (!bAllowProxyRender) // Never render proxies
			{
				// We don't want to fall back to Nanite proxy rendering, so just make the mesh invisible instead.
				return nullptr;
			}
		}

		// Fall back to rendering Nanite proxy meshes with traditional static mesh scene proxies

		FSceneInterface* Scene = Component.GetScene();
		const EShaderPlatform ShaderPlatform = Scene ? Scene->GetShaderPlatform() : GMaxRHIShaderPlatform;

		// TODO: handle Nanite representation being overriden using OnGetNaniteResources
		// for now need to check UStaticMesh::HasValidNaniteData() directly here
		const bool bFallbackGenerated = !StaticMesh->HasValidNaniteData() || StaticMesh->HasNaniteFallbackMesh(ShaderPlatform);

		if (!bFallbackGenerated)
		{
			// TODO: automatically enable fallback on the static mesh asset?

			UE_LOGF(LogStaticMesh, Warning, "Unable to create a proxy for StaticMeshComponent [%ls] because it doesn't have a fallback mesh.", *UObjectHelper::GetFullNameIfAvailable(Component));
			SetError(ESceneProxyCreationError::InvalidMesh);
			return nullptr;
		}
	}

	// Validate the LOD resources here
	const FStaticMeshLODResourcesArray& LODResources = StaticMesh->GetRenderData()->LODResources;
	const int32 SMCurrentMinLOD = StaticMesh->GetMinLODIdx();
	const int32 EffectiveMinLOD = Component.GetOverrideMinLOD() ? FMath::Max(Component.GetMinLOD(), SMCurrentMinLOD) : SMCurrentMinLOD;
	if (LODResources.Num() == 0 || LODResources[FMath::Clamp<int32>(EffectiveMinLOD, 0, LODResources.Num() - 1)].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() == 0)
	{
		UE_LOGF(LogStaticMesh, Verbose, "Skipping CreateSceneProxy for StaticMeshComponent %ls (LOD problems)", *UObjectHelper::GetFullNameIfAvailable(Component));
		SetError(ESceneProxyCreationError::InvalidMesh);
		return nullptr;
	}

	return Component.CreateStaticMeshSceneProxy(NaniteMaterials, false);
}