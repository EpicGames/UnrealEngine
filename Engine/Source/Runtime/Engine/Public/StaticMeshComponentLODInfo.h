// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RawIndexBuffer.h"

#include "StaticMeshComponentLODInfo.generated.h"

class FColorVertexBuffer;
class FMeshMapBuildData;

USTRUCT()
struct FStaticMeshComponentLODInfo
{
	GENERATED_USTRUCT_BODY()

	/** Uniquely identifies this LOD's built map data. */
	FGuid OriginalMapBuildDataId;

	/** Uniquely identifies this LOD's built map data, a combination of OriginalMapBuildDataID and the ActorInstanceID **/
	FGuid MapBuildDataId;

	/** Used during deserialization to temporarily store legacy lightmap data. */
	FMeshMapBuildData* LegacyMapBuildData;

	/** 
	 * Transient override lightmap data, used by landscape grass.
	 * Be sure to add your component to UMapBuildDataRegistry::CleanupTransientOverrideMapBuildData() for proper cleanup
	 * so that you don't get stale rendering resource references if the underlying MapBuildData is gone (lighting scenario changes, new static lighting build, etc.)
	 */
	TUniquePtr<FMeshMapBuildData> OverrideMapBuildData;

	/** Vertex data cached at the time this LOD was painted, if any */
	TArray<struct FPaintedVertex> PaintedVertices;

	/** Vertex colors to use for this mesh LOD */
	FColorVertexBuffer* OverrideVertexColors;

	/** 
	 * Owner of this FStaticMeshComponentLODInfo 
	 * Warning, can be NULL for a component created via SpawnActor off of a blueprint default (LODData will be created without a call to SetLODDataCount).
	 */
	class UStaticMeshComponent* OwningComponent;

	/** Default constructor */
	FStaticMeshComponentLODInfo();
	FStaticMeshComponentLODInfo(UStaticMeshComponent* InOwningComponent);
	/** Destructor */
	ENGINE_API ~FStaticMeshComponentLODInfo();

	/** Delete existing resources */
	void CleanUp();

	/** 
	 * Ensure this LODInfo has a valid OriginalMapBuildDataId GUID.
	 * @param LodIndex Index of the LOD this LODInfo represents.
	 * @return true if a new GUID was created, false otherwise.
	 */
	ENGINE_API bool CreateMapBuildDataId(int32 LodIndex);

	/**
	* Enqueues a rendering command to release the vertex colors.
	* The game thread must block until the rendering thread has processed the command before deleting OverrideVertexColors.
	*/
	ENGINE_API void BeginReleaseOverrideVertexColors();

	ENGINE_API void ReleaseOverrideVertexColorsAndBlock();

	void ReleaseResources();

	/** Methods for importing and exporting the painted vertex array to text */
	void ExportText(FString& ValueStr);
	void ImportText(const TCHAR** SourceText);

	/** Serializer. */
	friend FArchive& operator<<(FArchive& Ar,FStaticMeshComponentLODInfo& I);

#if WITH_EDITOR
	bool bMapBuildDataChanged = false;
#endif

private:
	/** Purposely hidden */
	FStaticMeshComponentLODInfo &operator=( const FStaticMeshComponentLODInfo &rhs ) { check(0); return *this; }
};

template<>
struct TStructOpsTypeTraits<FStaticMeshComponentLODInfo> : public TStructOpsTypeTraitsBase2<FStaticMeshComponentLODInfo>
{
	enum
	{
		WithCopy = false
	};
};
