// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMeshLight.h: Static mesh lighting code.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "StaticLighting.h"
#include "RawIndexBuffer.h"

class FShadowMapData2D;
class ULevel;
class ULightComponent;
class UStaticMesh;
class UStaticMeshComponent;
struct FQuantizedLightmapData;
struct FStaticMeshLODResources;

/** Represents the triangles of one LOD of a static mesh primitive to the static lighting system. */
class FStaticMeshStaticLightingMesh : public FStaticLightingMesh
{
public:

	/** The meshes representing other LODs of this primitive. */
	TArray<FStaticLightingMesh*> OtherLODs;

	/** Initialization constructor. */
	FStaticMeshStaticLightingMesh(const UStaticMeshComponent* InPrimitive,int32 InLODIndex,const TArray<ULightComponent*>& InRelevantLights);

	FStaticMeshStaticLightingMesh();

	// FStaticLightingMesh interface.

	virtual void GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2) const;

	virtual void GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const;

	virtual bool ShouldCastShadow(ULightComponent* Light,const FStaticLightingMapping* Receiver) const;

	/** @return		true if the specified triangle casts a shadow. */
	virtual bool IsTriangleCastingShadow(uint32 TriangleIndex) const;

	/** @return		true if the mesh wants to control shadow casting per element rather than per mesh. */
	virtual bool IsControllingShadowPerElement() const;

	virtual bool IsUniformShadowCaster() const;

	virtual FLightRayIntersection IntersectLightRay(const FVector& Start,const FVector& End,bool bFindNearestIntersection) const;

	virtual bool IsInstancedMesh() const override { return false; }

#if WITH_EDITOR
	/** 
	* Export static lighting mesh instance data to an exporter 
	* @param Exporter - export interface to process static lighting data
	**/
	UNREALED_API virtual void ExportMeshInstance(class FLightmassExporter* Exporter) const;

	virtual const struct FSplineMeshParams* GetSplineParameters() const { return NULL; }

#endif	//WITH_EDITOR

protected:

	/** The LOD this mesh represents. */
	const int32 LODIndex;

	/** 
	 * Sets the local to world matrix for this mesh, will also update LocalToWorldInverseTranspose
	 *
	 * @param InLocalToWorld Local to world matrix to apply
	 */
	void SetLocalToWorld(const FMatrix& InLocalToWorld);

private:

	/** The static mesh this mesh represents. */
	const UStaticMesh* StaticMesh;

	/** The primitive this mesh represents. */
	const UStaticMeshComponent* const Primitive;

	/** The resources for this LOD. */
	FStaticMeshLODResources* LODRenderData;

	/** A view in to the index buffer for this LOD. */
	FIndexArrayView LODIndexBuffer;

	/** Cached local to world matrix to transform all the verts by */
	FMatrix LocalToWorld;
	
	/** The inverse transpose of the primitive's local to world transform. */
	FMatrix LocalToWorldInverseTranspose;

	/** Cached determinant for the local to world */
	float LocalToWorldDeterminant;

	/** true if the primitive has a transform which reverses the winding of its triangles. */
	const uint32 bReverseWinding : 1;
	
	friend class FLightmassExporter;
};

/** Represents a static mesh primitive with texture mapped static lighting. */
class FStaticMeshStaticLightingTextureMapping : public FStaticLightingTextureMapping
{
public:

	/** Initialization constructor. */
	FStaticMeshStaticLightingTextureMapping(UStaticMeshComponent* InPrimitive,int32 InLODIndex,FStaticLightingMesh* InMesh,int32 InSizeX,int32 InSizeY,int32 InTextureCoordinateIndex,bool bPerformFullQualityRebuild);

#if WITH_EDITOR		

	// For serialization
	FStaticMeshStaticLightingTextureMapping(const FArchive& Ar ) : FStaticLightingTextureMapping(Ar) { }

	// FStaticLightingTextureMapping interface
	ENGINE_API virtual void Apply(FQuantizedLightmapData* QuantizedData, const TMap<ULightComponent*,FShadowMapData2D*>& ShadowMapData, const FStaticLightingBuildContext* LightingContext) override;

	/** 
	 * Export static lighting mapping instance data to an exporter 
	 * @param Exporter - export interface to process static lighting data
	 */
	UNREALED_API virtual void ExportMapping(class FLightmassExporter* Exporter) override;

	ENGINE_API virtual void Serialize(FArchive& Ar);
#endif	//WITH_EDITOR

	virtual FString GetDescription() const override
	{
		return FString("SMTextureMapping");
	}

	/** Whether or not this mapping should be processed or imported */
	virtual bool IsValidMapping() const override {return Primitive.IsValid();}

	int32 GetLODIndex() { return LODIndex; }

protected:

	/** The primitive this mapping represents. */
	TWeakObjectPtr<UStaticMeshComponent> Primitive;

	/** The LOD this mapping represents. */
	int32 LODIndex;
};

