// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMesh.h: Static mesh class definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "Containers/Set.h"
#include "Misc/Guid.h"
#include "Engine/EngineTypes.h"
#include "UObject/UObjectIterator.h"
#include "Materials/MaterialInterface.h"
#include "RenderResource.h"
#include "PackedNormal.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "RawIndexBuffer.h"
#include "Components.h"
#include "LocalVertexFactory.h"
#include "PrimitiveViewRelevance.h"
#include "UObject/UObjectHash.h"
#include "MeshBatch.h"
#include "MeshReductionSettings.h"
#include "SceneManagement.h"
#include "Components/StaticMeshComponent.h"
#include "BodySetupEnums.h"
#include "Materials/MaterialInterface.h"
#include "Rendering/ColorVertexBuffer.h"
#include "Rendering/StaticMeshVertexBuffer.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/StaticMeshVertexDataInterface.h"
#include "Rendering/NaniteInterface.h"
#include "Rendering/RayTracingStreamableAsset.h"
#include "RenderTransform.h"
#include "Templates/UniquePtr.h"
#include "Serialization/BulkData.h"
#include "WeightedRandomSampler.h"
#include "UObject/PerPlatformProperties.h"
#include "RayTracingGeometry.h"
#if WITH_EDITORONLY_DATA
#include "Interface_CollisionDataProviderCore.h"
#endif

class FDistanceFieldVolumeData;
class UBodySetup;
class USimpleConstructionScript;

/** The maximum number of static mesh LODs allowed. */
#define MAX_STATIC_MESH_LODS 8

struct FStaticMaterial;

/**
 * The LOD settings to use for a group of static meshes.
 */
class FStaticMeshLODGroup
{
public:
	/** Default values. */
	FStaticMeshLODGroup()
		: DefaultNumLODs(1)
		, DefaultMaxNumStreamedLODs(0)
		, DefaultMaxNumOptionalLODs(0)
		, DefaultLightMapResolution(64)
		, BasePercentTrianglesMult(1.0f)
		, BasePercentVerticesMult(1.0f)
		, bSupportLODStreaming(false)
		, DisplayName( NSLOCTEXT( "UnrealEd", "None", "None" ) )
	{
		FMemory::Memzero(SettingsBias);
		SettingsBias.PercentTriangles = 1.0f;
		SettingsBias.PercentVertices = 1.0f;
	}

	/** Returns the default number of LODs to build. */
	int32 GetDefaultNumLODs() const
	{
		return DefaultNumLODs;
	}

	/** Returns the default maximum of streamed LODs */
	int32 GetDefaultMaxNumStreamedLODs() const
	{
		return DefaultMaxNumStreamedLODs;
	}

	/** Returns the default maximum of optional LODs */
	int32 GetDefaultMaxNumOptionalLODs() const
	{
		return DefaultMaxNumOptionalLODs;
	}

	/** Returns the default lightmap resolution. */
	int32 GetDefaultLightMapResolution() const
	{
		return DefaultLightMapResolution;
	}

	/** Returns whether this LOD group supports LOD streaming. */
	bool IsLODStreamingSupported() const
	{
		return bSupportLODStreaming;
	}

	/** Returns default reduction settings for the specified LOD. */
	FMeshReductionSettings GetDefaultSettings(int32 LODIndex) const
	{
		check(LODIndex >= 0 && LODIndex < MAX_STATIC_MESH_LODS);
		return DefaultSettings[LODIndex];
	}

	/** Applies global settings tweaks for the specified LOD. */
	ENGINE_API FMeshReductionSettings GetSettings(const FMeshReductionSettings& InSettings, int32 LODIndex) const;

private:
	/** FStaticMeshLODSettings initializes group entries. */
	friend class FStaticMeshLODSettings;
	/** The default number of LODs to build. */
	int32 DefaultNumLODs;
	/** Maximum number of streamed LODs */
	int32 DefaultMaxNumStreamedLODs;
	/** Maximum number of optional LODs (currently, need to be either 0 or > max number of LODs below MinLOD) */
	int32 DefaultMaxNumOptionalLODs;
	/** Default lightmap resolution. */
	int32 DefaultLightMapResolution;
	/** An additional reduction of base meshes in this group. */
	float BasePercentTrianglesMult;
	/** An additional reduction of base meshes in this group. */
	float BasePercentVerticesMult;
	/** Whether static meshes in this LOD group can be streamed. */
	bool bSupportLODStreaming;
	/** Display name. */
	FText DisplayName;
	/** Default reduction settings for meshes in this group. */
	FMeshReductionSettings DefaultSettings[MAX_STATIC_MESH_LODS];
	/** Biases applied to reduction settings. */
	FMeshReductionSettings SettingsBias;
};

/**
 * Per-group LOD settings for static meshes.
 */
class FStaticMeshLODSettings
{
public:

	/**
	 * Initializes LOD settings by reading them from the passed in config file section.
	 * @param IniFile Preloaded ini file object to load from
	 */
	ENGINE_API void Initialize(const ITargetPlatform* TargetPlatform);
	ENGINE_API void Initialize(const class ITargetPlatformSettings* TargetPlatform);

	/** Retrieve the settings for the specified LOD group. */
	const FStaticMeshLODGroup& GetLODGroup(FName LODGroup) const
	{
		const FStaticMeshLODGroup* Group = Groups.Find(LODGroup);
		if (Group == NULL)
		{
			Group = Groups.Find(NAME_None);
		}
		check(Group);
		return *Group;
	}

	int32 GetLODGroupIdx(FName GroupName) const
	{
		const int32* IdxPtr = GroupName2Index.Find(GroupName);
		return IdxPtr ? *IdxPtr : INDEX_NONE;
	}

	/** Retrieve the names of all defined LOD groups. */
	void GetLODGroupNames(TArray<FName>& OutNames) const;

	/** Retrieves the localized display names of all LOD groups. */
	void GetLODGroupDisplayNames(TArray<FText>& OutDisplayNames) const;

private:
	/** Reads an entry from the INI to initialize settings for an LOD group. */
	void ReadEntry(FStaticMeshLODGroup& Group, FString Entry);
	/** Per-group settings. */
	TMap<FName,FStaticMeshLODGroup> Groups;
	/** For fast index lookup. Must not change after initialization */
	TMap<FName, int32> GroupName2Index;
};


/**
 * A set of static mesh triangles which are rendered with the same material.
 */
struct FStaticMeshSection
{
	/** The index of the material with which to render this section. */
	int32 MaterialIndex;

	/** Range of vertices and indices used when rendering this section. */
	uint32 FirstIndex;
	uint32 NumTriangles;
	uint32 MinVertexIndex;
	uint32 MaxVertexIndex;

	/** If true, collision is enabled for this section. */
	bool bEnableCollision;

	/** If true, this section will cast a shadow. */
	bool bCastShadow;
	/** If true, this section will be visible in ray tracing effects. */
	bool bVisibleInRayTracing;
	/** If true, this section will affect lighting methods that use Distance Fields. */
	bool bAffectDistanceFieldLighting;
	/** If true, this section will be considered opaque in ray tracing effects. */
	bool bForceOpaque;
#if WITH_EDITORONLY_DATA
	/** The UV channel density in LocalSpaceUnit / UV Unit. */
	float UVDensities[MAX_STATIC_TEXCOORDS];

	/** The weights to apply to the UV density, based on the area. */
	float Weights[MAX_STATIC_TEXCOORDS];
#endif

	/** Constructor. */
	FStaticMeshSection()
		: MaterialIndex(0)
		, FirstIndex(0)
		, NumTriangles(0)
		, MinVertexIndex(0)
		, MaxVertexIndex(0)
		, bEnableCollision(false)
		, bCastShadow(true)
		, bVisibleInRayTracing(true)
		, bAffectDistanceFieldLighting(true)
		, bForceOpaque(false)
	{
#if WITH_EDITORONLY_DATA
		FMemory::Memzero(UVDensities);
		FMemory::Memzero(Weights);
#endif
	}

	/** Serializer. */
	ENGINE_API friend FArchive& operator<<(FArchive& Ar,FStaticMeshSection& Section);
};


struct FStaticMeshLODResources;

/** Creates distribution for uniformly sampling a mesh section. */
struct FStaticMeshSectionAreaWeightedTriangleSampler : FWeightedRandomSampler
{
	ENGINE_API FStaticMeshSectionAreaWeightedTriangleSampler();
	ENGINE_API void Init(FStaticMeshLODResources* InOwner, int32 InSectionIdx);
	ENGINE_API virtual float GetWeights(TArray<float>& OutWeights) override;

protected:

	FStaticMeshLODResources* Owner;
	int32 SectionIdx;
};

struct FStaticMeshAreaWeightedSectionSampler : FWeightedRandomSampler
{
	ENGINE_API FStaticMeshAreaWeightedSectionSampler();
	ENGINE_API void Init(const FStaticMeshLODResources* InOwner);

protected:

	ENGINE_API virtual float GetWeights(TArray<float>& OutWeights)override;

	TRefCountPtr<const FStaticMeshLODResources> Owner;
};

typedef TArray<FStaticMeshSectionAreaWeightedTriangleSampler, FMemoryImageAllocator> FStaticMeshSectionAreaWeightedTriangleSamplerArray;

/** Represents GPU resource needed for area weighted uniform sampling of a mesh surface. */
class FStaticMeshSectionAreaWeightedTriangleSamplerBuffer : public FRenderResource
{
public:

	ENGINE_API FStaticMeshSectionAreaWeightedTriangleSamplerBuffer();
	ENGINE_API ~FStaticMeshSectionAreaWeightedTriangleSamplerBuffer();

	void Init(FStaticMeshSectionAreaWeightedTriangleSamplerArray* SamplerToUpload) { Samplers = SamplerToUpload; }

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;
	virtual FString GetFriendlyName() const override { return TEXT("FStaticMeshSectionAreaWeightedTriangleSamplerBuffer"); }

	const FBufferRHIRef& GetBufferRHI() const { return BufferSectionTriangleRHI; }
	const FShaderResourceViewRHIRef& GetBufferSRV() const { return BufferSectionTriangleSRV; }

private:
	struct SectionTriangleInfo
	{
		float  Prob;
		uint32 Alias;
	};

	FBufferRHIRef BufferSectionTriangleRHI = nullptr;
	FShaderResourceViewRHIRef BufferSectionTriangleSRV = nullptr;

	FStaticMeshSectionAreaWeightedTriangleSamplerArray* Samplers = nullptr;
};


struct FDynamicMeshVertex;
struct FModelVertex;

struct FStaticMeshVertexBuffers
{
	/** The buffer containing vertex data. */
	FStaticMeshVertexBuffer StaticMeshVertexBuffer;

	/** The buffer containing the position vertex data. */
	FPositionVertexBuffer PositionVertexBuffer;

	/** The buffer containing the vertex color data. */
	FColorVertexBuffer ColorVertexBuffer;

	void inline InitWithDummyData(FRHICommandListBase& RHICmdList, FLocalVertexFactory* VertexFactory, uint32 NumVertices, uint32 NumTexCoords = 1, uint32 LightMapIndex = 0)
	{
		InitWithDummyData(&RHICmdList, nullptr, VertexFactory, NumVertices, NumTexCoords, LightMapIndex);
	}

	void inline InitWithDummyData(FRenderCommandPipe* RenderCommandPipe, FLocalVertexFactory* VertexFactory, uint32 NumVertices, uint32 NumTexCoords = 1, uint32 LightMapIndex = 0)
	{
		InitWithDummyData(nullptr, RenderCommandPipe, VertexFactory, NumVertices, NumTexCoords, LightMapIndex);
	}

	void inline InitWithDummyData(FLocalVertexFactory* VertexFactory, uint32 NumVertices, uint32 NumTexCoords = 1, uint32 LightMapIndex = 0)
	{
		InitWithDummyData(nullptr, nullptr, VertexFactory, NumVertices, NumTexCoords, LightMapIndex);
	}

	inline void InitFromDynamicVertex(FRHICommandListBase& RHICmdList, FLocalVertexFactory* VertexFactory, TArray<FDynamicMeshVertex>& Vertices, uint32 NumTexCoords = 1, uint32 LightMapIndex = 0)
	{
		InitFromDynamicVertex(&RHICmdList, nullptr, VertexFactory, Vertices, NumTexCoords, LightMapIndex);
	}

	inline void InitFromDynamicVertex(FRenderCommandPipe* RenderCommandPipe, FLocalVertexFactory* VertexFactory, TArray<FDynamicMeshVertex>& Vertices, uint32 NumTexCoords = 1, uint32 LightMapIndex = 0)
	{
		InitFromDynamicVertex(nullptr, RenderCommandPipe, VertexFactory, Vertices, NumTexCoords, LightMapIndex);
	}

	inline void InitFromDynamicVertex(FLocalVertexFactory* VertexFactory, TArray<FDynamicMeshVertex>& Vertices, uint32 NumTexCoords = 1, uint32 LightMapIndex = 0)
	{
		InitFromDynamicVertex(nullptr, nullptr, VertexFactory, Vertices, NumTexCoords, LightMapIndex);
	}

	void ENGINE_API InitModelBuffers(TArray<FModelVertex>& Vertices);
	void ENGINE_API InitComponentVF(FLocalVertexFactory* VertexFactory, int32 LightMapCoordinateIndex, bool bOverrideColorVertexBuffer, FLocalVertexFactory::FDataType& OutDat) const;

	inline void InitModelVF(FRHICommandListBase& RHICmdList, FLocalVertexFactory* VertexFactory)
	{
		InitModelVF(&RHICmdList, nullptr, VertexFactory);
	}

	inline void InitModelVF(FRenderCommandPipe* RenderCommandPipe, FLocalVertexFactory* VertexFactory)
	{
		InitModelVF(nullptr, RenderCommandPipe, VertexFactory);
	}

	inline void InitModelVF(FLocalVertexFactory* VertexFactory)
	{
		InitModelVF(nullptr, nullptr, VertexFactory);
	}

	void SetOwnerName(FName OwnerName)
	{
#if RHI_ENABLE_RESOURCE_INFO
		PositionVertexBuffer.SetOwnerName(OwnerName);
		StaticMeshVertexBuffer.SetOwnerName(OwnerName);
		ColorVertexBuffer.SetOwnerName(OwnerName);
#endif
	}

private:
	void ENGINE_API InitWithDummyData(FRHICommandListBase* RHICmdList, FRenderCommandPipe* RenderCommandPipe, FLocalVertexFactory* VertexFactory, uint32 NumVertices, uint32 NumTexCoords, uint32 LightMapIndex);
	void ENGINE_API InitFromDynamicVertex(FRHICommandListBase* RHICmdList, FRenderCommandPipe* RenderCommandPipe, FLocalVertexFactory* VertexFactory, TArray<FDynamicMeshVertex>& Vertices, uint32 NumTexCoords, uint32 LightMapIndex);
	void ENGINE_API InitModelVF(FRHICommandListBase* RHICmdList, FRenderCommandPipe* RenderCommandPipe, FLocalVertexFactory* VertexFactory);
};

struct FAdditionalStaticMeshIndexBuffers
{
	/** Reversed index buffer, used to prevent changing culling state between drawcalls. */
	FRawStaticIndexBuffer ReversedIndexBuffer;
	/** Reversed depth only index buffer, used to prevent changing culling state between drawcalls. */
	FRawStaticIndexBuffer ReversedDepthOnlyIndexBuffer;
	/** Index buffer resource for rendering wireframe mode. */
	FRawStaticIndexBuffer WireframeIndexBuffer;
};

class FStaticMeshSectionArray : public TArray<FStaticMeshSection, TInlineAllocator<1>>
{
	using Super = TArray<FStaticMeshSection, TInlineAllocator<1>>;
public:
	using Super::Super;
};

template <>
struct TIsZeroConstructType<FStaticMeshSectionArray> : TIsZeroConstructType<TArray<FStaticMeshSection, TInlineAllocator<1>>>
{
};
template <>
struct TIsContiguousContainer<FStaticMeshSectionArray> : TIsContiguousContainer<TArray<FStaticMeshSection, TInlineAllocator<1>>>
{
};
//using FStaticMeshSectionArray = TArray<FStaticMeshSection, TInlineAllocator<1>>;

/** 
 * Rendering resources needed to render an individual static mesh LOD.
 * This structure is ref counted to allow the LOD streamer to evaluate the number of readers to it (readers that could access the CPU data).
 * Because the stream out clears the CPU readable data, CPU code that samples it must ensure to only reference LODs above CurrentFirstLODIdx.
 */
struct FStaticMeshLODResources : public UE::Private::FQueryableRefCountedObject
{
public:

	/** Sections for this LOD. */
	FStaticMeshSectionArray Sections;

	/** Distance field data associated with this mesh, null if not present.  */
	class FDistanceFieldVolumeData* DistanceFieldData = nullptr; 

	/** Card Representation data associated with this mesh, null if not present.  */
	class FCardRepresentationData* CardRepresentationData;

	/** Geometry for ray tracing. */
	FRayTracingGeometry* RayTracingGeometry = nullptr;

	/** The bounds of the source mesh data for this LOD */
	FBoxSphereBounds SourceMeshBounds { ForceInit };

	/** The maximum distance by which this LOD deviates from the base from which it was generated. */
	float MaxDeviation;

	/** True if the depth only index buffers contained data at init. Needed as it will not be available to the CPU afterwards. */
	uint32 bHasDepthOnlyIndices : 1;

	/** True if the reversed index buffers contained data at init. Needed as it will not be available to the CPU afterwards. */
	uint32 bHasReversedIndices : 1;

	/** True if the reversed index buffers contained data at init. Needed as it will not be available to the CPU afterwards. */
	uint32 bHasReversedDepthOnlyIndices : 1;

	uint32 bHasColorVertexData : 1;

	uint32 bHasWireframeIndices : 1;

	/** True if vertex and index data are serialized inline */
	uint32 bBuffersInlined : 1;

	/** True if this LOD is optional. That is, vertex and index data may not be available */
	uint32 bIsOptionalLOD : 1;

	uint32 DepthOnlyNumTriangles;

	/** Sum of all vertex and index buffer sizes. Calculated in SerializeBuffers */
	uint32 BuffersSize;

	FByteBulkData StreamingBulkData;

#if STATS
	uint32 StaticMeshIndexMemory;
#endif

#if WITH_EDITOR
	FByteBulkData BulkData;

	FString DerivedDataKey;

	/** Map of wedge index to vertex index. Each LOD need one*/
	TArray<int32> WedgeMap;
#endif

	FStaticMeshVertexBuffers VertexBuffers;

	/** Index buffer resource for rendering. */
	FRawStaticIndexBuffer IndexBuffer;

	/** Index buffer resource for rendering in depth only passes. */
	FRawStaticIndexBuffer DepthOnlyIndexBuffer;

	FAdditionalStaticMeshIndexBuffers* AdditionalIndexBuffers = nullptr;

	/**	Allows uniform random selection of mesh sections based on their area. */
	FStaticMeshAreaWeightedSectionSampler AreaWeightedSampler;
	/**	Allows uniform random selection of triangles on each mesh section based on triangle area. */
	FStaticMeshSectionAreaWeightedTriangleSamplerArray AreaWeightedSectionSamplers;
	/** Allows uniform random selection of triangles on GPU. It is not cooked and serialised but created at runtime from AreaWeightedSectionSamplers when it is available and static mesh bSupportGpuUniformlyDistributedSampling=true*/
	FStaticMeshSectionAreaWeightedTriangleSamplerBuffer AreaWeightedSectionSamplersBuffer;
	
	/** Default constructor. Add a reference if not stored with a TRefCountPtr */
	ENGINE_API FStaticMeshLODResources(bool bAddRef = true);

	ENGINE_API ~FStaticMeshLODResources();

	template <typename TBatcher>
	void ReleaseRHIForStreaming(TBatcher& Batcher)
	{
		VertexBuffers.StaticMeshVertexBuffer.ReleaseRHIForStreaming(Batcher);
		VertexBuffers.PositionVertexBuffer.ReleaseRHIForStreaming(Batcher);
		VertexBuffers.ColorVertexBuffer.ReleaseRHIForStreaming(Batcher);

		IndexBuffer.ReleaseRHIForStreaming(Batcher);
		DepthOnlyIndexBuffer.ReleaseRHIForStreaming(Batcher);

		if (AdditionalIndexBuffers)
		{
			AdditionalIndexBuffers->ReversedIndexBuffer.ReleaseRHIForStreaming(Batcher);
			AdditionalIndexBuffers->ReversedDepthOnlyIndexBuffer.ReleaseRHIForStreaming(Batcher);
			AdditionalIndexBuffers->WireframeIndexBuffer.ReleaseRHIForStreaming(Batcher);
		}
	}

	/** Initializes all rendering resources. */
	void InitResources(UStaticMesh* Parent, int32 LODIndex);

	/** Releases all rendering resources. */
	void ReleaseResources();

	/** Serialize. */
	void Serialize(FArchive& Ar, UObject* Owner, int32 Idx);

	ENGINE_API void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const;

#if RHI_RAYTRACING
	void SetupRayTracingGeometryInitializer(FRayTracingGeometryInitializer& Initializer, const FDebugName& DebugName, const FName& OwnerName) const;
#endif // RHI_RAYTRACING

	/** Get the estimated memory overhead of buffers marked as NeedsCPUAccess. */
	SIZE_T GetCPUAccessMemoryOverhead() const;

	/** Return the triangle count of this LOD. */
	ENGINE_API int32 GetNumTriangles() const;

	/** Return the number of vertices in this LOD. */
	ENGINE_API int32 GetNumVertices() const;

	ENGINE_API int32 GetNumTexCoords() const;

private:
	enum EClassDataStripFlag : uint8
	{
		CDSF_AdjacencyData_DEPRECATED = 1,
		CDSF_MinLodData = 2,
		CDSF_ReversedIndexBuffer = 4,
		CDSF_RayTracingResources = 8
	};

	/**
	 * Due to discard on load, size of an static mesh LOD is not known at cook time and
	 * this struct is used to keep track of all the information needed to compute LOD size
	 * at load time
	 */
	struct FStaticMeshBuffersSize
	{
		uint32 SerializedBuffersSize = 0;
		uint32 DepthOnlyIBSize       = 0;
		uint32 ReversedIBsSize       = 0;

		void Clear()
		{
			SerializedBuffersSize = 0;
			DepthOnlyIBSize = 0;
			ReversedIBsSize = 0;
		}

		uint32 CalcBuffersSize() const;

		friend FArchive& operator<<(FArchive& Ar, FStaticMeshBuffersSize& Info)
		{
			Ar << Info.SerializedBuffersSize;
			Ar << Info.DepthOnlyIBSize;
			Ar << Info.ReversedIBsSize;
			return Ar;
		}
	};

	static int32 GetPlatformMinLODIdx(const ITargetPlatform* TargetPlatform, UStaticMesh* StaticMesh);

	static uint8 GenerateClassStripFlags(FArchive& Ar, UStaticMesh* OwnerStaticMesh, int32 Index);

	static bool IsLODCookedOut(const ITargetPlatform* TargetPlatform, UStaticMesh* StaticMesh, bool bIsBelowMinLOD);

	static bool IsLODInlined(const ITargetPlatform* TargetPlatform, UStaticMesh* StaticMesh, int32 LODIdx, bool bIsBelowMinLOD);

	static int32 GetNumOptionalLODsAllowed(const ITargetPlatform* TargetPlatform, UStaticMesh* StaticMesh);

	/** Compute the size of VertexBuffers and add the result to OutSize */
	static void AccumVertexBuffersSize(const FStaticMeshVertexBuffers& VertexBuffers, uint32& OutSize);

	/** Compute the size of IndexBuffer and add the result to OutSize */
	static void AccumIndexBufferSize(const FRawStaticIndexBuffer& IndexBuffer, uint32& OutSize);

	/** Compute the size of RayTracingGeometry and add the result to OutSize */
	static void AccumRayTracingGeometrySize(const FRayTracingGeometry& RayTracingGeometry, uint32& OutSize);

	/**
	 * Serialize vertex and index buffer data for this LOD
	 * OutBuffersSize - Size of all serialized data in bytes
	 */
	void SerializeBuffers(FArchive& Ar, UStaticMesh* OwnerStaticMesh, uint8 InStripFlags, FStaticMeshBuffersSize& OutBuffersSize);

	/**
	 * Serialize availability information such as bHasDepthOnlyIndices and size of buffers so it
	 * can be retrieved without loading in actual vertex or index data
	 */
	void SerializeAvailabilityInfo(FArchive& Ar);

	void ClearAvailabilityInfo();

	template <bool bIncrement>
	void UpdateIndexMemoryStats();

	template <bool bIncrement>
	void UpdateVertexMemoryStats() const;

	void IncrementMemoryStats();

	void DecrementMemoryStats();

	/** Discard loaded vertex and index data. Used when a streaming request is cancelled */
	void DiscardCPUData();

	friend class FStaticMeshRenderData;
	friend class FStaticMeshStreamIn;
	friend class FStaticMeshStreamIn_IO;
	friend class FStaticMeshStreamOut;
};

struct FStaticMeshVertexFactories
{
	FStaticMeshVertexFactories(ERHIFeatureLevel::Type InFeatureLevel)
		: VertexFactory(InFeatureLevel, "FStaticMeshVertexFactories")
		, VertexFactoryOverrideColorVertexBuffer(InFeatureLevel, "FStaticMeshVertexFactories_Override")
		, SplineVertexFactory(nullptr)
		, SplineVertexFactoryOverrideColorVertexBuffer(nullptr)
	{
		// FLocalVertexFactory::InitRHI requires valid current feature level to setup streams properly
		check(InFeatureLevel < ERHIFeatureLevel::Num);
	}

	ENGINE_API ~FStaticMeshVertexFactories();

	/** The vertex factory used when rendering this mesh. */
	FLocalVertexFactory VertexFactory;

	/** The vertex factory used when rendering this mesh with vertex colors. This is lazy init.*/
	FLocalVertexFactory VertexFactoryOverrideColorVertexBuffer;

	struct FSplineMeshVertexFactory* SplineVertexFactory;

	struct FSplineMeshVertexFactory* SplineVertexFactoryOverrideColorVertexBuffer;

	/**
	* Initializes a vertex factory for rendering this static mesh
	*
	* @param	InOutVertexFactory				The vertex factory to configure
	* @param	InParentMesh					Parent static mesh
	* @param	bInOverrideColorVertexBuffer	If true, make a vertex factory ready for per-instance colors
	*/
	ENGINE_API void InitVertexFactory(const FStaticMeshVertexBuffers& VertexBuffers, FLocalVertexFactory& InOutVertexFactory, uint32 LODIndex, const UStaticMesh* InParentMesh, bool bInOverrideColorVertexBuffer);

	UE_DEPRECATED(5.5, "Provide FStaticMeshVertexBuffers instead of FStaticMeshLODResources")
	void InitVertexFactory(const FStaticMeshLODResources& LodResources, FLocalVertexFactory& InOutVertexFactory, uint32 LODIndex, const UStaticMesh* InParentMesh, bool bInOverrideColorVertexBuffer)
	{
		InitVertexFactory(LodResources.VertexBuffers, InOutVertexFactory, LODIndex, InParentMesh, bInOverrideColorVertexBuffer);
	}

	/** Initializes all rendering resources. */
	ENGINE_API void InitResources(const FStaticMeshVertexBuffers& VertexBuffers, uint32 LODIndex, const UStaticMesh* Parent);

	UE_DEPRECATED(5.5, "Provide FStaticMeshVertexBuffers instead of FStaticMeshLODResources")
	void InitResources(const FStaticMeshLODResources& LodResources, uint32 LODIndex, const UStaticMesh* Parent)
	{
		InitResources(LodResources.VertexBuffers, LODIndex, Parent);
	}

	/** Releases all rendering resources. */
	ENGINE_API void ReleaseResources();
};

using FStaticMeshLODResourcesArray = TIndirectArray<FStaticMeshLODResources>;
using FStaticMeshVertexFactoriesArray = TArray<FStaticMeshVertexFactories>;

struct FStaticMeshRayTracingProxyLOD : public FRayTracingStreamableAsset
{
	FRayTracingGeometry* RayTracingGeometry = nullptr;

	FStaticMeshSectionArray* Sections = nullptr;

	FStaticMeshVertexBuffers* VertexBuffers = nullptr;
	FRawStaticIndexBuffer* IndexBuffer = nullptr;

	FByteBulkData StreamableData;
	uint32 BuffersSize = 0;
	uint32 OfflineBVHOffset = 0;
	uint32 OfflineBVHSize = 0;

	bool bOwnsRayTracingGeometry : 1 = true;
	bool bOwnsBuffers : 1 = true;
	bool bBuffersInlined : 1 = false;

	bool bRayTracingGeometryStreamedIn : 1 = false;
	bool bBuffersStreamedIn : 1 = false;

	ENGINE_API virtual ~FStaticMeshRayTracingProxyLOD();

	void InitResources(UStaticMesh* Owner, int32 LODIndex);
	void ReleaseResources();

	void Serialize(FArchive& Ar, UObject* Owner, int32 Index);

	void SerializeBuffers(FArchive& Ar, UStaticMesh* OwnerStaticMesh, uint8 InStripFlags);
	void SerializeRayTracingGeometry(FArchive& Ar, UStaticMesh* OwnerStaticMesh, uint8 InStripFlags);
	void SerializeMetaData(FArchive& Ar);

	/** Discard loaded data. Used when a streaming request is cancelled */
	void DiscardCPUData();

#if RHI_RAYTRACING
	void SetupRayTracingGeometryInitializer(FRayTracingGeometryInitializer& Initializer, const FDebugName& DebugName, const FName& OwnerName) const;
#endif // RHI_RAYTRACING

	// FRayTracingStreamableAsset
	ENGINE_API virtual uint32 GetRequestSize() const override;
	ENGINE_API virtual uint32 GetRequestSizeBVH() const override;
	ENGINE_API virtual uint32 GetRequestSizeBuffers() const override;
	ENGINE_API virtual bool AreBuffersStreamedIn() const override;
	ENGINE_API virtual bool IsBVHStreamedIn() const override;
	ENGINE_API virtual void IssueRequest(FBulkDataBatchRequest& Request, FIoBuffer& RequestBuffer, EAsyncIOPriorityAndFlags Priority, bool bBuffersOnly = false) override;
	ENGINE_API virtual void InitWithStreamedData(FRHICommandListBase& RHICmdList, FMemoryView StreamedData, bool bBuffersOnly = false) override;
	ENGINE_API virtual void ReleaseForStreaming(FRHIResourceReplaceBatcher& Batcher) override;
	ENGINE_API virtual void ReleaseBVHForStreaming(FRHIResourceReplaceBatcher& Batcher) override;
	ENGINE_API virtual void ReleaseBuffersForStreaming(FRHIResourceReplaceBatcher& Batcher) override;

	UE_DEPRECATED(5.8, "Please specify EAsyncIOPriorityAndFlags")
	void IssueRequest(FBulkDataBatchRequest& Request, FIoBuffer& RequestBuffer, bool bBuffersOnly = false) override
	{
		IssueRequest(Request, RequestBuffer, AIOP_Low, bBuffersOnly);
	}
};

using FStaticMeshRayTracingProxyLODArray = TIndirectArray<FStaticMeshRayTracingProxyLOD>;

struct FStaticMeshRayTracingProxy
{
	FStaticMeshRayTracingProxyLODArray LODs;
	FStaticMeshVertexFactoriesArray* LODVertexFactories = nullptr;

	bool bUsingRenderingLODs = false;

#if WITH_EDITORONLY_DATA
	int32 PreviewLODLevel = INDEX_NONE;
#endif

	~FStaticMeshRayTracingProxy();

	void InitResources(UStaticMesh* Owner);
	void ReleaseResources();

	void Serialize(FArchive& Ar, UObject* Owner, FStaticMeshRenderData* RenderData, bool bCooked);
};

/**
 * FStaticMeshRenderData - All data needed to render a static mesh.
 */
class FStaticMeshRenderData
{
public:

	/** Default constructor. */
	ENGINE_API FStaticMeshRenderData();
	ENGINE_API ~FStaticMeshRenderData();

	/**
	 * Per-LOD resources. For compatibility reasons, the FStaticMeshLODResources array are not referenced through TRefCountPtr.
	 * The LODResource still has a ref count of at least 1, see FStaticMeshLODResources() constructor.
	 */
	FStaticMeshLODResourcesArray LODResources;
	FStaticMeshVertexFactoriesArray LODVertexFactories;

	/** Screen size to switch LODs */
	FPerPlatformFloat ScreenSize[MAX_STATIC_MESH_LODS];

	TPimplPtr<Nanite::FResources> NaniteResourcesPtr;

	/** Ray tracing representation of this mesh, null if not present.  */
	FStaticMeshRayTracingProxy* RayTracingProxy = nullptr;

	/** Bounds of the renderable mesh. */
	FBoxSphereBounds Bounds;

#if RHI_RAYTRACING
	RayTracing::FGeometryGroupHandle RayTracingGeometryGroupHandle;
#endif

	bool IsInitialized() const
	{
		return bIsInitialized;
	}

	ENGINE_API bool HasValidNaniteData() const;

	ENGINE_API bool HasNaniteFallbackMesh(EShaderPlatform ShaderPlatform) const;

	/** True if LODs share static lighting data. */
	bool bLODsShareStaticLighting;

	bool bHasNaniteFallbackMesh;

	/** True if rhi resources are initialized */
	bool bReadyForStreaming;

	uint8 NumInlinedLODs;

	uint8 CurrentFirstLODIdx;

	uint8 LODBiasModifier;

#if WITH_EDITORONLY_DATA

	/** The derived data key associated with this render data. */
	FString DerivedDataKey;

	/**
	 * The platform(s) that are using using this RenderData. There may be more than one if multiple
	 * platforms share the same key.
	 */
	TSet<const ITargetPlatform*> SupportedPlatforms;
	/** The number of times Cache has been called. Used to log errors when we are calling it repeatedly. */
	int32 NumCacheAttempts = 0;

	/** Map of material index -> original material index at import time. */
	TArray<int32> MaterialIndexToImportIndex;

	/** UV data used for streaming accuracy debug view modes. In sync for rendering thread */
	TArray<FMeshUVChannelInfo> UVChannelDataPerMaterial;

	/** The next cached derived data in the list. */
	TUniquePtr<class FStaticMeshRenderData> NextCachedRenderData;

	/**
	 * Canned FTriMeshCollisionData for static meshes cooked for CookedCooker platform (see TCookedCookerTargetPlatform).
	 * It is needed because the "cooked cooker" can be cooking a new spline mesh that is deforming a cooked static mesh, so it will request the SM's collision data.
	 */
	TUniquePtr<FTriMeshCollisionData> CollisionDataForCookedCooker;

	/** Estimate of total compressed size of all rendering data, including Nanite data. */
	uint64 EstimatedCompressedSize = 0;

	/** Estimate of total compressed size of Nanite data. Includes streaming and non-streaming data. */
	uint64 EstimatedNaniteTotalCompressedSize = 0;

	/** Estimate of compressed size of Nanite streaming data. */
	uint64 EstimatedNaniteStreamingCompressedSize = 0;

	/** Derived from NaniteSettings to be used in HasNaniteFallbackMesh(...) in builds with editor-only data. */
	bool bForceGenerateNaniteFallbackMesh = false;

	void SyncUVChannelData(const TArray<FStaticMaterial>& ObjectData);

	/**
	 * Cache derived renderable data for the static mesh with the provided
	 * level of detail settings.
	 */
	ENGINE_API void Cache(const ITargetPlatform* TargetPlatform, UStaticMesh* Owner, const FStaticMeshLODSettings& LODSettings);
	/**
	 * Copies data from an earlier FStaticMeshRenderData for the same platform which should persist across
	 * retries; the persistent data is used to track repeated failures and error exit.
	 */
	ENGINE_API void CopyCacheRetryData(FStaticMeshRenderData& Other);
#endif // #if WITH_EDITORONLY_DATA

	/** Count the number of LODs that are inlined and not streamable. Starting from the last LOD and stopping at the first non inlined LOD. */
	int32 GetNumNonStreamingLODs() const;
	/** Count the number of LODs that not optional and guarantied to be installed. Starting from the last LOD and stopping at the first optional LOD. */
	int32 GetNumNonOptionalLODs() const;

	/** Serialization. */
	void Serialize(FArchive& Ar, UStaticMesh* Owner, bool bCooked);

	/** Serialize mesh build data which is inlined. */
	void SerializeInlineDataRepresentations(FArchive& Ar, UStaticMesh* Owner);

	/** Initialize the render resources. */
	void InitResources(ERHIFeatureLevel::Type InFeatureLevel, UStaticMesh* Owner);

	/** Releases the render resources. */
	ENGINE_API void ReleaseResources();

	/** Compute the size of this resource. */
	ENGINE_API void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const;

	/** Get the estimated memory overhead of buffers marked as NeedsCPUAccess. */
	ENGINE_API SIZE_T GetCPUAccessMemoryOverhead() const;

	/** Allocate LOD resources. */
	ENGINE_API void AllocateLODResources(int32 NumLODs);

	/** Update LOD-SECTION uv densities. */
	void ComputeUVDensities(UStaticMesh* Mesh);

	void BuildAreaWeighedSamplingData();

	ENGINE_API void InitializeRayTracingRepresentationFromRenderingLODs();

#if WITH_EDITOR
	/** Resolve all per-section settings. */
	ENGINE_API void ResolveSectionInfo(UStaticMesh* Owner);
#endif // #if WITH_EDITORONLY_DATA

	/** Return first valid LOD index starting at MinLODIdx. */
	ENGINE_API int32 GetFirstValidLODIdx(int32 MinLODIdx) const;

	/** Return the current first LODIdx that can be used. */
	inline int32 GetCurrentFirstLODIdx(int32 MinLODIdx) const
	{
		return GetFirstValidLODIdx(FMath::Max<int32>(CurrentFirstLODIdx, MinLODIdx));
	}

	/** 
	 * Return the current first LOD that can be used for rendering starting at MinLODIdx.
	 * This takes into account the streaming status from CurrentFirstLODIdx, 
	 * and MinLODIdx is expected to be UStaticMesh::MinLOD, which is platform specific.
	 */
	inline const FStaticMeshLODResources* GetCurrentFirstLOD(int32 MinLODIdx) const
	{
		const int32 LODIdx = GetCurrentFirstLODIdx(MinLODIdx);
		return LODResources.IsValidIndex(LODIdx) ? &LODResources[LODIdx] : nullptr;
	}

private:
	void FreeLODResources();

#if WITH_EDITORONLY_DATA
	/** Allow the editor to explicitly update section information. */
	friend class FLevelOfDetailSettingsLayout;
#endif // #if WITH_EDITORONLY_DATA
	bool bIsInitialized = false;
};

/**
 * FStaticMeshComponentRecreateRenderStateContext - Destroys render state for all StaticMeshComponents using a given StaticMesh and 
 * recreates them when it goes out of scope. Used to ensure stale rendering data isn't kept around in the components when importing
 * over or rebuilding an existing static mesh.
 */
class FStaticMeshComponentRecreateRenderStateContext
{
public:

	/** Initialization constructor. */
	ENGINE_API FStaticMeshComponentRecreateRenderStateContext(UStaticMesh* InStaticMesh, bool InUnbuildLighting = true, bool InRefreshBounds = false);

	/** Initialization constructor. */
	ENGINE_API FStaticMeshComponentRecreateRenderStateContext(const TArray<UStaticMesh*>& InStaticMeshes, bool InUnbuildLighting = true, bool InRefreshBounds = false);

	/**
	 * Get all static mesh components that are using the provided static mesh.
	 * @param StaticMesh	The static mesh from which you want to obtain a list of components.
	 * @return An reference to an array of static mesh components that are using this mesh.
	 * @note Will only work using the static meshes provided at construction.
	 */
	ENGINE_API const TArray<UStaticMeshComponent*>& GetComponentsUsingMesh(UStaticMesh* StaticMesh) const;

	/** Destructor: recreates render state for all components that had their render states destroyed in the constructor. */
	ENGINE_API ~FStaticMeshComponentRecreateRenderStateContext();

private:
	struct FData
	{
		TArray<UStaticMeshComponent*> UStaticMeshComponents;
		TArray<IStaticMeshComponent*> IStaticMeshComponents;
	};
	TMap<void*, FData> StaticMeshComponents;
	bool bUnbuildLighting;
	bool bRefreshBounds;
};

/**
 * FStaticMeshComponentBulkReregisterContext - More efficiently handles bulk reregistering of static mesh components, by removing and
 * adding scene and physics debug render data in bulk render commands, rather than one at a time.  A significant fraction of the cost
 * of reregistering components is the synchronization cost of issuing commands to the render thread.  Bulk render commands means you
 * only pay this cost once, rather than per component, potentially providing up to a 4x speedup.
 * 
 * When a context is active, the bBulkReregister flag is set on the primitive component.  This disables calls to SendRenderDebugPhysics,
 * AddPrimitive, RemovePrimitive, and ReleasePrimitive, where they would otherwise occur during re-registration, with the assumption
 * that the constructor and destructor of the context handles those tasks.  To allow the optimization to be applied to re-created
 * components, "AddSimpleConstructionScript" can be called to register simple construction scripts, so any components they create get
 * added to the context as well.  It's not a problem if re-reated components are missed by the context, they'll just lose performance
 * by going through the slower individual component code path.
 */
enum class EBulkReregister
{
	Component,			// Reregistering components
	RenderState			// Updating render state only -- limits reregistration to components with bRenderStateDirty set, and skips ReleasePrimitive
};

class FStaticMeshComponentBulkReregisterContext
{
public:
	/**
	  * Initialization constructor.  Note that it's OK to pass things that aren't static meshes.  This class will filter those out.
	  */
	ENGINE_API FStaticMeshComponentBulkReregisterContext(FSceneInterface* InScene, TArrayView<UActorComponent*> InComponents, EBulkReregister ReregisterType = EBulkReregister::Component);
	ENGINE_API ~FStaticMeshComponentBulkReregisterContext();

	ENGINE_API void AddSimpleConstructionScript(USimpleConstructionScript* SCS);

	/** Removes any static mesh components that have had their proxy created after the context is created*/
	void SanitizeMeshComponents();
private:
	/** Called by USCS_Node (child of USimpleConstructionScript) to track re-created components */
	ENGINE_API void AddConstructedComponent(USceneComponent* SceneComp);
	friend class USCS_Node;

	FSceneInterface* Scene;
	TArray<UPrimitiveComponent*> StaticMeshComponents;
	TArray<USimpleConstructionScript*> SCSs;
};

/*-----------------------------------------------------------------------------
	FStaticMeshInstanceData
-----------------------------------------------------------------------------*/

/** The implementation of the static mesh instance data storage type. */
class FStaticMeshInstanceData
{
	template<typename F>
	struct FInstanceTransformMatrix
	{
		F InstanceTransform1[4];
		F InstanceTransform2[4];
		F InstanceTransform3[4];

		friend FArchive& operator<<(FArchive& Ar, FInstanceTransformMatrix& V)
		{
			return Ar
				<< V.InstanceTransform1[0]
				<< V.InstanceTransform1[1]
				<< V.InstanceTransform1[2]
				<< V.InstanceTransform1[3]

				<< V.InstanceTransform2[0]
				<< V.InstanceTransform2[1]
				<< V.InstanceTransform2[2]
				<< V.InstanceTransform2[3]

				<< V.InstanceTransform3[0]
				<< V.InstanceTransform3[1]
				<< V.InstanceTransform3[2]
				<< V.InstanceTransform3[3];
		}

	};

	struct FInstanceLightMapVector
	{
		int16 InstanceLightmapAndShadowMapUVBias[4];

		friend FArchive& operator<<(FArchive& Ar, FInstanceLightMapVector& V)
		{
			return Ar
				<< V.InstanceLightmapAndShadowMapUVBias[0]
				<< V.InstanceLightmapAndShadowMapUVBias[1]
				<< V.InstanceLightmapAndShadowMapUVBias[2]
				<< V.InstanceLightmapAndShadowMapUVBias[3];
		}
	};

public:
	FStaticMeshInstanceData()
	{
	}

	/**
	 * Constructor
	 * @param bInUseHalfFloat - true if device has support for half float in vertex arrays
	 */
	FStaticMeshInstanceData(bool bInUseHalfFloat)
	:	bUseHalfFloat(PLATFORM_BUILTIN_VERTEX_HALF_FLOAT || bInUseHalfFloat)
	{
		AllocateBuffers(0);
	}

	~FStaticMeshInstanceData()
	{
		delete InstanceOriginData;
		delete InstanceLightmapData;
		delete InstanceTransformData;
		delete InstanceCustomData;
	}

	void Serialize(FArchive& Ar);
	
	void AllocateInstances(int32 InNumInstances, int32 InNumCustomDataFloats, EResizeBufferFlags BufferFlags, bool DestroyExistingInstances)
	{
		NumInstances = InNumInstances;
		NumCustomDataFloats = InNumCustomDataFloats;

		if (DestroyExistingInstances)
		{
			InstanceOriginData->Empty(NumInstances);
			InstanceLightmapData->Empty(NumInstances);
			InstanceTransformData->Empty(NumInstances);
			InstanceCustomData->Empty(NumCustomDataFloats * NumInstances);
		}

		// We cannot write directly to the data on all platforms,
		// so we make a TArray of the right type, then assign it
		InstanceOriginData->ResizeBuffer(NumInstances, BufferFlags);
		InstanceOriginDataPtr = InstanceOriginData->GetDataPointer();

		InstanceLightmapData->ResizeBuffer(NumInstances, BufferFlags);
		InstanceLightmapDataPtr = InstanceLightmapData->GetDataPointer();

		InstanceTransformData->ResizeBuffer(NumInstances, BufferFlags);
		InstanceTransformDataPtr = InstanceTransformData->GetDataPointer();

		InstanceCustomData->ResizeBuffer(NumCustomDataFloats * NumInstances, BufferFlags);
		InstanceCustomDataPtr = InstanceCustomData->GetDataPointer();
	}

	inline int32 IsValidIndex(int32 Index) const
	{
		return InstanceOriginData->IsValidIndex(Index);
	}

	inline void GetInstanceTransform(int32 InstanceIndex, FRenderTransform& Transform) const
	{
		FVector4f TransformVec[3];
		if (bUseHalfFloat)
		{
			GetInstanceTransformInternal<FFloat16>(InstanceIndex, TransformVec);
		}
		else
		{
			GetInstanceTransformInternal<float>(InstanceIndex, TransformVec);
		}

		Transform.TransformRows[0] = FVector3f(TransformVec[0].X, TransformVec[0].Y, TransformVec[0].Z);
		Transform.TransformRows[1] = FVector3f(TransformVec[1].X, TransformVec[1].Y, TransformVec[1].Z);
		Transform.TransformRows[2] = FVector3f(TransformVec[2].X, TransformVec[2].Y, TransformVec[2].Z);

		FVector4f Origin;
		GetInstanceOriginInternal(InstanceIndex, Origin);
		Transform.Origin = FVector3f(Origin.X, Origin.Y, Origin.Z);
	}

	inline void GetInstanceRandomID(int32 InstanceIndex, float& RandomInstanceID) const
	{
		FVector4f Origin;
		GetInstanceOriginInternal(InstanceIndex, Origin);
		RandomInstanceID = Origin.W;
	}


#if WITH_EDITOR
	inline void GetInstanceEditorData(int32 InstanceIndex, FColor& HitProxyColorOut, bool & bSelectedOut) const
	{
		// TODO: put this into a sensible format
		FVector4f InstanceTransform[3];
		if (bUseHalfFloat)
		{
			GetInstanceTransformInternal<FFloat16>(InstanceIndex, InstanceTransform);
		}
		else
		{
			GetInstanceTransformInternal<float>(InstanceIndex, InstanceTransform);
		}
		bSelectedOut = InstanceTransform[0][3] > 255.0f;
		HitProxyColorOut.R = uint8(InstanceTransform[0][3] - (bSelectedOut ? 256.0f : 0.0f));
		HitProxyColorOut.G = uint8(InstanceTransform[1][3]);
		HitProxyColorOut.B = uint8(InstanceTransform[2][3]);
	}
#endif 

	inline void GetInstanceLightMapData(int32 InstanceIndex, FVector4f& InstanceLightmapAndShadowMapUVBias) const
	{
		GetInstanceLightMapDataInternal(InstanceIndex, InstanceLightmapAndShadowMapUVBias);
	}

	inline void GetInstanceCustomDataValues(int32 InstanceIndex, TArrayView<float> OutCustomData) const
	{
		GetInstanceCustomDataInternal(InstanceIndex, OutCustomData);
	}
	
	inline void SetInstance(int32 InstanceIndex, const FMatrix44f& Transform, float RandomInstanceID, const FVector2D& LightmapUVBias, const FVector2D& ShadowmapUVBias)
	{
		FVector4f Origin(Transform.M[3][0], Transform.M[3][1], Transform.M[3][2], RandomInstanceID);
		SetInstanceOriginInternal(InstanceIndex, Origin);

		FVector4f InstanceTransform[3];
		InstanceTransform[0] = FVector4f(Transform.M[0][0], Transform.M[0][1], Transform.M[0][2], 0.0f);
		InstanceTransform[1] = FVector4f(Transform.M[1][0], Transform.M[1][1], Transform.M[1][2], 0.0f);
		InstanceTransform[2] = FVector4f(Transform.M[2][0], Transform.M[2][1], Transform.M[2][2], 0.0f);

		if (bUseHalfFloat)
		{
			SetInstanceTransformInternal<FFloat16>(InstanceIndex, InstanceTransform);
		}
		else
		{
			SetInstanceTransformInternal<float>(InstanceIndex, InstanceTransform);
		}

		SetInstanceLightMapDataInternal(InstanceIndex, FVector4f((float)LightmapUVBias.X, (float)LightmapUVBias.Y, (float)ShadowmapUVBias.X, (float)ShadowmapUVBias.Y));

		for (int32 i = 0; i < NumCustomDataFloats; ++i)
		{
			SetInstanceCustomDataInternal(InstanceIndex, i, 0);
		}
	}

	inline void SetInstance(int32 InstanceIndex, const FMatrix44f& Transform, float RandomInstanceID)
	{
		const FVector2D& LightmapUVBias  = FVector2D::ZeroVector;
		const FVector2D& ShadowmapUVBias = FVector2D::ZeroVector;
		SetInstance(InstanceIndex, Transform, RandomInstanceID, LightmapUVBias, ShadowmapUVBias);
	}

	inline void SetInstance(int32 InstanceIndex, const FMatrix44f& Transform, const FVector2D& LightmapUVBias, const FVector2D& ShadowmapUVBias)
	{
		float RandomInstanceID;
		GetInstanceRandomID(InstanceIndex, RandomInstanceID);
		SetInstance(InstanceIndex, Transform, RandomInstanceID, LightmapUVBias, ShadowmapUVBias);
	}

	inline void SetInstance(int32 InstanceIndex, const FMatrix44f& Transform)
	{
		const FVector2D& LightmapUVBias = FVector2D::ZeroVector;
		const FVector2D& ShadowmapUVBias = FVector2D::ZeroVector;
		float RandomInstanceID;
		GetInstanceRandomID(InstanceIndex, RandomInstanceID);
		SetInstance(InstanceIndex, Transform, RandomInstanceID, LightmapUVBias, ShadowmapUVBias);
	}

	inline void SetInstanceLightMapData(int32 InstanceIndex, const FVector2D& LightmapUVBias, const FVector2D& ShadowmapUVBias)
	{
		SetInstanceLightMapDataInternal(InstanceIndex, FVector4f((float)LightmapUVBias.X, (float)LightmapUVBias.Y, (float)ShadowmapUVBias.X, (float)ShadowmapUVBias.Y));
	}
	
	inline void SetInstanceCustomData(int32 InstanceIndex, int32 Index, float CustomData)
	{
		SetInstanceCustomDataInternal(InstanceIndex, Index, CustomData);
	}
	
	inline void NullifyInstance(int32 InstanceIndex)
	{
		SetInstanceOriginInternal(InstanceIndex, FVector4f(0, 0, 0, 0));

		FVector4f InstanceTransform[3];
		InstanceTransform[0] = FVector4f(0, 0, 0, 0);
		InstanceTransform[1] = FVector4f(0, 0, 0, 0);
		InstanceTransform[2] = FVector4f(0, 0, 0, 0);

		if (bUseHalfFloat)
		{
			SetInstanceTransformInternal<FFloat16>(InstanceIndex, InstanceTransform);
		}
		else
		{
			SetInstanceTransformInternal<float>(InstanceIndex, InstanceTransform);
		}

		SetInstanceLightMapDataInternal(InstanceIndex, FVector4f(0, 0, 0, 0));

		for (int32 i = 0; i < NumCustomDataFloats; ++i)
		{
			SetInstanceCustomDataInternal(InstanceIndex, i, 0);
		}
	}

	inline void SetInstanceEditorData(int32 InstanceIndex, FColor HitProxyColor, bool bSelected)
	{
		FVector4f InstanceTransform[3];
		if (bUseHalfFloat)
		{
			GetInstanceTransformInternal<FFloat16>(InstanceIndex, InstanceTransform);
			InstanceTransform[0][3] = ((float)HitProxyColor.R) + (bSelected ? 256.f : 0.0f);
			InstanceTransform[1][3] = (float)HitProxyColor.G;
			InstanceTransform[2][3] = (float)HitProxyColor.B;
			SetInstanceTransformInternal<FFloat16>(InstanceIndex, InstanceTransform);
		}
		else
		{
			GetInstanceTransformInternal<float>(InstanceIndex, InstanceTransform);
			InstanceTransform[0][3] = ((float)HitProxyColor.R) + (bSelected ? 256.f : 0.0f);
			InstanceTransform[1][3] = (float)HitProxyColor.G;
			InstanceTransform[2][3] = (float)HitProxyColor.B;
			SetInstanceTransformInternal<float>(InstanceIndex, InstanceTransform);
		}
	}

	inline void ClearInstanceEditorData(int32 InstanceIndex)
	{
		FVector4f InstanceTransform[3];
		if (bUseHalfFloat)
		{
			GetInstanceTransformInternal<FFloat16>(InstanceIndex, InstanceTransform);
			InstanceTransform[0][3] = 0.0f;
			InstanceTransform[1][3] = 0.0f;
			InstanceTransform[2][3] = 0.0f;
			SetInstanceTransformInternal<FFloat16>(InstanceIndex, InstanceTransform);
		}
		else
		{
			GetInstanceTransformInternal<float>(InstanceIndex, InstanceTransform);
			InstanceTransform[0][3] = 0.0f;
			InstanceTransform[1][3] = 0.0f;
			InstanceTransform[2][3] = 0.0f;
			SetInstanceTransformInternal<float>(InstanceIndex, InstanceTransform);
		}
	}

	inline void SwapInstance(int32 Index1, int32 Index2)
	{
		if (bUseHalfFloat)
		{
			FInstanceTransformMatrix<FFloat16>* ElementData = reinterpret_cast<FInstanceTransformMatrix<FFloat16>*>(InstanceTransformDataPtr);
			uint32 CurrentSize = InstanceTransformData->Num() * InstanceTransformData->GetStride();
			check((void*)((&ElementData[Index1]) + 1) <= (void*)(InstanceTransformDataPtr + CurrentSize));
			check((void*)((&ElementData[Index1]) + 0) >= (void*)(InstanceTransformDataPtr));
			check((void*)((&ElementData[Index2]) + 1) <= (void*)(InstanceTransformDataPtr + CurrentSize));
			check((void*)((&ElementData[Index2]) + 0) >= (void*)(InstanceTransformDataPtr));

			FInstanceTransformMatrix<FFloat16> TempStore = ElementData[Index1];
			ElementData[Index1] = ElementData[Index2];
			ElementData[Index2] = TempStore;
		}
		else
		{
			FInstanceTransformMatrix<float>* ElementData = reinterpret_cast<FInstanceTransformMatrix<float>*>(InstanceTransformDataPtr);
			uint32 CurrentSize = InstanceTransformData->Num() * InstanceTransformData->GetStride();
			check((void*)((&ElementData[Index1]) + 1) <= (void*)(InstanceTransformDataPtr + CurrentSize));
			check((void*)((&ElementData[Index1]) + 0) >= (void*)(InstanceTransformDataPtr));
			check((void*)((&ElementData[Index2]) + 1) <= (void*)(InstanceTransformDataPtr + CurrentSize));
			check((void*)((&ElementData[Index2]) + 0) >= (void*)(InstanceTransformDataPtr));
			
			FInstanceTransformMatrix<float> TempStore = ElementData[Index1];
			ElementData[Index1] = ElementData[Index2];
			ElementData[Index2] = TempStore;
		}
		{

			FVector4f* ElementData = reinterpret_cast<FVector4f*>(InstanceOriginDataPtr);
			uint32 CurrentSize = InstanceOriginData->Num() * InstanceOriginData->GetStride();
			check((void*)((&ElementData[Index1]) + 1) <= (void*)(InstanceOriginDataPtr + CurrentSize));
			check((void*)((&ElementData[Index1]) + 0) >= (void*)(InstanceOriginDataPtr));
			check((void*)((&ElementData[Index2]) + 1) <= (void*)(InstanceOriginDataPtr + CurrentSize));
			check((void*)((&ElementData[Index2]) + 0) >= (void*)(InstanceOriginDataPtr));

			FVector4f TempStore = ElementData[Index1];
			ElementData[Index1] = ElementData[Index2];
			ElementData[Index2] = TempStore;
		}
		{
			FInstanceLightMapVector* ElementData = reinterpret_cast<FInstanceLightMapVector*>(InstanceLightmapDataPtr);
			uint32 CurrentSize = InstanceLightmapData->Num() * InstanceLightmapData->GetStride();
			check((void*)((&ElementData[Index1]) + 1) <= (void*)(InstanceLightmapDataPtr + CurrentSize));
			check((void*)((&ElementData[Index1]) + 0) >= (void*)(InstanceLightmapDataPtr));
			check((void*)((&ElementData[Index2]) + 1) <= (void*)(InstanceLightmapDataPtr + CurrentSize));
			check((void*)((&ElementData[Index2]) + 0) >= (void*)(InstanceLightmapDataPtr));
			
			FInstanceLightMapVector TempStore = ElementData[Index1];
			ElementData[Index1] = ElementData[Index2];
			ElementData[Index2] = TempStore;
		}
		{
			float* ElementData = reinterpret_cast<float*>(InstanceCustomDataPtr);
			const uint32 CurrentSize = InstanceCustomData->Num() * InstanceCustomData->GetStride();

			for (int32 i = 0; i < NumCustomDataFloats; ++i)
			{
				const int32 CustomDataIndex1 = NumCustomDataFloats * Index1 + i;
				const int32 CustomDataIndex2 = NumCustomDataFloats * Index2 + i;

				check((void*)((&ElementData[CustomDataIndex1]) + 1) <= (void*)(InstanceCustomDataPtr + CurrentSize));
				check((void*)((&ElementData[CustomDataIndex1]) + 0) >= (void*)(InstanceCustomDataPtr));
				check((void*)((&ElementData[CustomDataIndex2]) + 1) <= (void*)(InstanceCustomDataPtr + CurrentSize));
				check((void*)((&ElementData[CustomDataIndex2]) + 0) >= (void*)(InstanceCustomDataPtr));

				float TempStore = ElementData[CustomDataIndex1];
				ElementData[CustomDataIndex1] = ElementData[CustomDataIndex2];
				ElementData[CustomDataIndex2] = TempStore;
			}
		}
	}

	inline int32 GetNumInstances() const
	{
		return NumInstances;
	}

	inline int32 GetNumCustomDataFloats() const
	{
		return NumCustomDataFloats;
	}

	inline void SetAllowCPUAccess(bool InNeedsCPUAccess)
	{
		if (InstanceOriginData)
		{
			InstanceOriginData->GetResourceArray()->SetAllowCPUAccess(InNeedsCPUAccess);
		}
		if (InstanceLightmapData)
		{
			InstanceLightmapData->GetResourceArray()->SetAllowCPUAccess(InNeedsCPUAccess);
		}
		if (InstanceTransformData)
		{
			InstanceTransformData->GetResourceArray()->SetAllowCPUAccess(InNeedsCPUAccess);
		}
		if (InstanceCustomData)
		{
			InstanceCustomData->GetResourceArray()->SetAllowCPUAccess(InNeedsCPUAccess);
		}
	}

	inline bool GetTranslationUsesHalfs() const
	{
		return bUseHalfFloat;
	}

	inline FResourceArrayInterface* GetOriginResourceArray()
	{
		return InstanceOriginData->GetResourceArray();
	}

	inline FResourceArrayInterface* GetTransformResourceArray()
	{
		return InstanceTransformData->GetResourceArray();
	}

	inline FResourceArrayInterface* GetLightMapResourceArray()
	{
		return InstanceLightmapData->GetResourceArray();
	}

	inline FResourceArrayInterface* GetCustomDataResourceArray()
	{
		return InstanceCustomData->GetResourceArray();
	}

	inline uint32 GetOriginStride()
	{
		return InstanceOriginData->GetStride();
	}

	inline uint32 GetTransformStride()
	{
		return InstanceTransformData->GetStride();
	}

	inline uint32 GetLightMapStride()
	{
		return InstanceLightmapData->GetStride();
	}

	inline uint32 GetCustomDataStride()
	{
		return InstanceCustomData->GetStride();
	}

	inline SIZE_T GetResourceSize() const
	{
		return	InstanceOriginData->GetResourceSize() + 
				InstanceTransformData->GetResourceSize() + 
				InstanceLightmapData->GetResourceSize() +
				InstanceCustomData->GetResourceSize();
	}

private:
	template<typename T>
	inline void GetInstanceTransformInternal(int32 InstanceIndex, FVector4f (&Transform)[3]) const
	{
		FInstanceTransformMatrix<T>* ElementData = reinterpret_cast<FInstanceTransformMatrix<T>*>(InstanceTransformDataPtr);
		uint32 CurrentSize = InstanceTransformData->Num() * InstanceTransformData->GetStride();

		if (ensure((void*)((&ElementData[InstanceIndex]) + 1) <= (void*)(InstanceTransformDataPtr + CurrentSize))
			&& ensure((void*)((&ElementData[InstanceIndex]) + 0) >= (void*)(InstanceTransformDataPtr)))
		{		
			Transform[0][0] = ElementData[InstanceIndex].InstanceTransform1[0];
			Transform[0][1] = ElementData[InstanceIndex].InstanceTransform1[1];
			Transform[0][2] = ElementData[InstanceIndex].InstanceTransform1[2];
			Transform[0][3] = ElementData[InstanceIndex].InstanceTransform1[3];
		
			Transform[1][0] = ElementData[InstanceIndex].InstanceTransform2[0];
			Transform[1][1] = ElementData[InstanceIndex].InstanceTransform2[1];
			Transform[1][2] = ElementData[InstanceIndex].InstanceTransform2[2];
			Transform[1][3] = ElementData[InstanceIndex].InstanceTransform2[3];
		
			Transform[2][0] = ElementData[InstanceIndex].InstanceTransform3[0];
			Transform[2][1] = ElementData[InstanceIndex].InstanceTransform3[1];
			Transform[2][2] = ElementData[InstanceIndex].InstanceTransform3[2];
			Transform[2][3] = ElementData[InstanceIndex].InstanceTransform3[3];
		}
		else
		{
			Transform[0] = FVector4f(1.0f, 0.0f, 0.0f, 0.0f);
			Transform[1] = FVector4f(0.0f, 1.0f, 0.0f, 0.0f);
			Transform[2] = FVector4f(0.0f, 0.0f, 1.0f, 0.0f);
		}
	}

	inline void GetInstanceOriginInternal(int32 InstanceIndex, FVector4f &Origin) const
	{
		FVector4f* ElementData = reinterpret_cast<FVector4f*>(InstanceOriginDataPtr);
		uint32 CurrentSize = InstanceOriginData->Num() * InstanceOriginData->GetStride();

		if (ensure((void*)((&ElementData[InstanceIndex]) + 1) <= (void*)(InstanceOriginDataPtr + CurrentSize))
			&& ensure((void*)((&ElementData[InstanceIndex]) + 0) >= (void*)(InstanceOriginDataPtr)))
		{
			Origin = ElementData[InstanceIndex];
		}
		else
		{
			Origin = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		}
	}

	inline void GetInstanceLightMapDataInternal(int32 InstanceIndex, FVector4f &LightmapData) const
	{
		FInstanceLightMapVector* ElementData = reinterpret_cast<FInstanceLightMapVector*>(InstanceLightmapDataPtr);
		uint32 CurrentSize = InstanceLightmapData->Num() * InstanceLightmapData->GetStride();

		if (ensure((void*)((&ElementData[InstanceIndex]) + 1) <= (void*)(InstanceLightmapDataPtr + CurrentSize))
			&& ensure((void*)((&ElementData[InstanceIndex]) + 0) >= (void*)(InstanceLightmapDataPtr)))
		{
			LightmapData = FVector4f
			(
				float(ElementData[InstanceIndex].InstanceLightmapAndShadowMapUVBias[0]) / 32767.0f, 
				float(ElementData[InstanceIndex].InstanceLightmapAndShadowMapUVBias[1]) / 32767.0f,
				float(ElementData[InstanceIndex].InstanceLightmapAndShadowMapUVBias[2]) / 32767.0f,
				float(ElementData[InstanceIndex].InstanceLightmapAndShadowMapUVBias[3]) / 32767.0f
			);
		}
		else
		{
			LightmapData = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		}
	}

	inline void GetInstanceCustomDataInternal(int32 InstanceIndex, TArrayView<float> OutCustomData) const
	{
		check(OutCustomData.Num() == NumCustomDataFloats);

		float* ElementData = reinterpret_cast<float*>(InstanceCustomDataPtr);
		const uint32 CurrentSize = InstanceCustomData->Num() * InstanceCustomData->GetStride();

		for (int32 i = 0; i < NumCustomDataFloats; ++i)
		{
			int32 CustomDataIndex = NumCustomDataFloats * InstanceIndex + i;
			
			if (ensure((void*)((&ElementData[CustomDataIndex]) + 1) <= (void*)(InstanceCustomDataPtr + CurrentSize))
				&& ensure((void*)((&ElementData[CustomDataIndex]) + 0) >= (void*)(InstanceCustomDataPtr)))
			{
				OutCustomData[i] = ElementData[CustomDataIndex];
			}
		}
	}

	template<typename T>
	inline void SetInstanceTransformInternal(int32 InstanceIndex, FVector4f(Transform)[3]) const
	{
		FInstanceTransformMatrix<T>* ElementData = reinterpret_cast<FInstanceTransformMatrix<T>*>(InstanceTransformDataPtr);
		uint32 CurrentSize = InstanceTransformData->Num() * InstanceTransformData->GetStride();

		if (ensure((void*)((&ElementData[InstanceIndex]) + 1) <= (void*)(InstanceTransformDataPtr + CurrentSize))
			&& ensure((void*)((&ElementData[InstanceIndex]) + 0) >= (void*)(InstanceTransformDataPtr)))
		{
			ElementData[InstanceIndex].InstanceTransform1[0] = Transform[0][0];
			ElementData[InstanceIndex].InstanceTransform1[1] = Transform[0][1];
			ElementData[InstanceIndex].InstanceTransform1[2] = Transform[0][2];
			ElementData[InstanceIndex].InstanceTransform1[3] = Transform[0][3];

			ElementData[InstanceIndex].InstanceTransform2[0] = Transform[1][0];
			ElementData[InstanceIndex].InstanceTransform2[1] = Transform[1][1];
			ElementData[InstanceIndex].InstanceTransform2[2] = Transform[1][2];
			ElementData[InstanceIndex].InstanceTransform2[3] = Transform[1][3];

			ElementData[InstanceIndex].InstanceTransform3[0] = Transform[2][0];
			ElementData[InstanceIndex].InstanceTransform3[1] = Transform[2][1];
			ElementData[InstanceIndex].InstanceTransform3[2] = Transform[2][2];
			ElementData[InstanceIndex].InstanceTransform3[3] = Transform[2][3];
		}
	}

	inline void SetInstanceOriginInternal(int32 InstanceIndex, const FVector4f& Origin) const
	{
		FVector4f* ElementData = reinterpret_cast<FVector4f*>(InstanceOriginDataPtr);
		uint32 CurrentSize = InstanceOriginData->Num() * InstanceOriginData->GetStride();

		if (ensureMsgf((void*)((&ElementData[InstanceIndex]) + 1) <= (void*)(InstanceOriginDataPtr + CurrentSize), TEXT("OOB Instance Set Under: %i, %u, %p, %p"), InstanceIndex, CurrentSize, &ElementData, InstanceOriginDataPtr)
			&& ensureMsgf((void*)((&ElementData[InstanceIndex]) + 0) >= (void*)(InstanceOriginDataPtr), TEXT("OOB Instance Set: %i, %u, %p, %p"), InstanceIndex, CurrentSize, &ElementData, InstanceOriginDataPtr))
		{
			ElementData[InstanceIndex] = Origin;
		}
	}

	inline void SetInstanceLightMapDataInternal(int32 InstanceIndex, const FVector4f& LightmapData) const
	{
		FInstanceLightMapVector* ElementData = reinterpret_cast<FInstanceLightMapVector*>(InstanceLightmapDataPtr);
		uint32 CurrentSize = InstanceLightmapData->Num() * InstanceLightmapData->GetStride();
		
		if (ensure((void*)((&ElementData[InstanceIndex]) + 1) <= (void*)(InstanceLightmapDataPtr + CurrentSize))
			&& ensure((void*)((&ElementData[InstanceIndex]) + 0) >= (void*)(InstanceLightmapDataPtr)))
		{

			ElementData[InstanceIndex].InstanceLightmapAndShadowMapUVBias[0] = (int16)FMath::Clamp<int32>(FMath::TruncToInt(LightmapData.X * 32767.0f), MIN_int16, MAX_int16);
			ElementData[InstanceIndex].InstanceLightmapAndShadowMapUVBias[1] = (int16)FMath::Clamp<int32>(FMath::TruncToInt(LightmapData.Y * 32767.0f), MIN_int16, MAX_int16);
			ElementData[InstanceIndex].InstanceLightmapAndShadowMapUVBias[2] = (int16)FMath::Clamp<int32>(FMath::TruncToInt(LightmapData.Z * 32767.0f), MIN_int16, MAX_int16);
			ElementData[InstanceIndex].InstanceLightmapAndShadowMapUVBias[3] = (int16)FMath::Clamp<int32>(FMath::TruncToInt(LightmapData.W * 32767.0f), MIN_int16, MAX_int16);
		}

	}

	inline void SetInstanceCustomDataInternal(int32 InstanceIndex, int32 DataIndex, float CustomData)
	{
		if (DataIndex >= NumCustomDataFloats)
		{
			return;
		}

		float* ElementData = reinterpret_cast<float*>(InstanceCustomDataPtr);
		const uint32 CurrentSize = InstanceCustomData->Num() * InstanceCustomData->GetStride();

		const int32 CustomDataIndex = NumCustomDataFloats * InstanceIndex + DataIndex;

		if (ensure((void*)((&ElementData[CustomDataIndex]) + 1) <= (void*)(InstanceCustomDataPtr + CurrentSize))
			&& ensure((void*)((&ElementData[CustomDataIndex]) + 0) >= (void*)(InstanceCustomDataPtr)))
		{
			ElementData[CustomDataIndex] = CustomData;
		}

	}

	void AllocateBuffers(int32 InNumInstances, EResizeBufferFlags BufferFlags = EResizeBufferFlags::None)
	{
		delete InstanceOriginData;
		InstanceOriginDataPtr = nullptr;
		
		delete InstanceTransformData;
		InstanceTransformDataPtr = nullptr;
		
		delete InstanceLightmapData;
		InstanceLightmapDataPtr = nullptr;
		 		
		delete InstanceCustomData;
		InstanceCustomData = nullptr;
		 		
		InstanceOriginData = new TStaticMeshVertexData<FVector4f>();
		InstanceOriginData->ResizeBuffer(InNumInstances, BufferFlags);
		InstanceLightmapData = new TStaticMeshVertexData<FInstanceLightMapVector>();
		InstanceLightmapData->ResizeBuffer(InNumInstances, BufferFlags);
		if (bUseHalfFloat)
		{
			InstanceTransformData = new TStaticMeshVertexData<FInstanceTransformMatrix<FFloat16>>();
		}
		else
		{
			InstanceTransformData = new TStaticMeshVertexData<FInstanceTransformMatrix<float>>();
		}
		InstanceTransformData->ResizeBuffer(InNumInstances, BufferFlags);
		
		InstanceCustomData = new TStaticMeshVertexData<float>();
		InstanceCustomData->ResizeBuffer(NumCustomDataFloats * InNumInstances, BufferFlags);
	}

	FStaticMeshVertexDataInterface* InstanceOriginData = nullptr;
	uint8* InstanceOriginDataPtr = nullptr;

	FStaticMeshVertexDataInterface* InstanceTransformData = nullptr;
	uint8* InstanceTransformDataPtr = nullptr;

	FStaticMeshVertexDataInterface* InstanceLightmapData = nullptr;
	uint8* InstanceLightmapDataPtr = nullptr;	

	FStaticMeshVertexDataInterface* InstanceCustomData = nullptr;
	uint8* InstanceCustomDataPtr = nullptr;

	int32 NumInstances = 0;
	int32 NumCustomDataFloats = 0;
	bool bUseHalfFloat = false;
};
	
#if WITH_EDITOR
/**
 * Remaps painted vertex colors when the renderable mesh has changed.
 * @param InPaintedVertices - The original position and normal for each painted vertex.
 * @param InOverrideColors - The painted vertex colors.
 * @param NewPositions - Positions of the new renderable mesh on which colors are to be mapped.
 * @param OptionalVertexBuffer - [optional] Vertex buffer containing vertex normals for the new mesh.
 * @param OutOverrideColors - Will contain vertex colors for the new mesh.
 */
ENGINE_API void RemapPaintedVertexColors(
	const TArray<FPaintedVertex>& InPaintedVertices,
	const FColorVertexBuffer* InOverrideColors,
	const FPositionVertexBuffer& OldPositions,
	const FStaticMeshVertexBuffer& OldVertexBuffer,
	const FPositionVertexBuffer& NewPositions,	
	const FStaticMeshVertexBuffer* OptionalVertexBuffer,
	TArray<FColor>& OutOverrideColors
	);
#endif // #if WITH_EDITOR
