// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMesh.h: Static mesh class definition.
=============================================================================*/

#pragma once

#include "Components/StaticMeshComponent.h"
#include "PrimitiveSceneProxy.h"
#include "RayTracingInstance.h"
#include "SceneManagement.h"
#include "RayTracingGeometry.h"
#include "RenderUtils.h"

class FLocalVertexFactoryUniformShaderParameters;
class FRawStaticIndexBuffer;
struct FAdditionalStaticMeshIndexBuffers;
struct FStaticMeshSection;
struct FStaticMeshVertexFactories;
using FStaticMeshVertexFactoriesArray = TArray<FStaticMeshVertexFactories>;
struct FStaticMeshSceneProxyDesc;
class FTextureResource;
enum class EPSOPrecachePriority : uint8;

ENGINE_API EPSOPrecachePriority GetStaticMeshComponentBoostPSOPrecachePriority();

void GetStaticMeshRayTracingWPOConfig(bool& bOutHasRayTracingRepresentation, bool& bOutDynamicRayTracingGeometry);
bool ShouldStaticMeshEvaluateWPOInRayTracing(FVector ViewCenter, const FBoxSphereBounds& Bounds);

/**
 * A static mesh component scene proxy.
 */
class FStaticMeshSceneProxy : public FPrimitiveSceneProxy
{
public:
	ENGINE_API SIZE_T GetTypeHash() const override;

	/** Initialization constructor. */
	ENGINE_API FStaticMeshSceneProxy(UStaticMeshComponent* Component, bool bForceLODsShareStaticLighting);

	/** Initialization constructor. */
	ENGINE_API FStaticMeshSceneProxy(const FStaticMeshSceneProxyDesc& InDesc, bool bForceLODsShareStaticLighting);

	ENGINE_API virtual ~FStaticMeshSceneProxy();

	/** Gets the number of mesh batches required to represent the proxy, aside from section needs. */
	virtual int32 GetNumMeshBatches() const
	{
		return 1;
	}

	/** Sets up a shadow FMeshBatch for a specific LOD. */
	ENGINE_API virtual bool GetShadowMeshElement(int32 LODIndex, int32 BatchIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch, bool bDitheredLODTransition) const;

	/** Sets up a FMeshBatch for a specific LOD and element. */
	ENGINE_API virtual bool GetMeshElement(
		int32 LODIndex, 
		int32 BatchIndex, 
		int32 SectionIndex,
		uint8 InDepthPriorityGroup, 
		bool bUseSelectionOutline,
		bool bAllowPreCulledIndices,
		FMeshBatch& OutMeshBatch) const;

	/** Sets up a ray tracing FMeshBatch for a specific LOD. */
	ENGINE_API virtual bool GetRayTracingMeshElement(int32 LODIndex, int32 BatchIndex, int32 SectionIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch) const;

	ENGINE_API virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;

	ENGINE_API virtual void DestroyRenderThreadResources() override;

	/** Sets up a wireframe FMeshBatch for a specific LOD. */
	ENGINE_API virtual bool GetWireframeMeshElement(int32 LODIndex, int32 BatchIndex, const FMaterialRenderProxy* WireframeRenderProxy, uint8 InDepthPriorityGroup, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const;

	/** Sets up a collision FMeshBatch for a specific LOD and element. */
	ENGINE_API virtual bool GetCollisionMeshElement(
		int32 LODIndex,
		int32 BatchIndex,
		int32 ElementIndex,
		uint8 InDepthPriorityGroup,
		const FMaterialRenderProxy* RenderProxy,
		FMeshBatch& OutMeshBatch) const;

	ENGINE_API virtual void SetEvaluateWorldPositionOffsetInRayTracing(FRHICommandListBase& RHICmdList, bool bEvaluate);

	virtual uint8 GetCurrentFirstLODIdx_RenderThread() const final override
	{
		return GetCurrentFirstLODIdx_Internal();
	}

#if !UE_BUILD_SHIPPING
	ENGINE_API virtual uint8 GetStreamingLODWantedDeficit() const override;
#endif

	virtual FDesiredLODLevel GetDesiredLODLevel_RenderThread(const FSceneView* View) const final override
	{
		return FDesiredLODLevel::CreateFirst(GetCurrentFirstLODIdx_Internal());
	}

	ENGINE_API virtual int32 GetLightMapCoordinateIndex() const override;

	ENGINE_API virtual bool GetInstanceWorldPositionOffsetDisableDistance(float& OutWPODisableDistance) const override;

	ENGINE_API virtual void GetStreamableRenderAssetInfo(const FBoxSphereBounds& InPrimitiveBounds, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamableRenderAssets) const override;

	virtual bool CanApplyStreamableRenderAssetScaleFactor() const override { return true; }

protected:
	UE_DEPRECATED(5.6, "Provide FStaticMeshSection, IndexBuffer and AdditionalIndexBuffers instead of LODIndex, etc.")
	ENGINE_API uint32 SetMeshElementGeometrySource(
		int32 LODIndex,
		int32 ElementIndex,
		bool bWireframe,
		bool bUseInversedIndices,
		bool bAllowPreCulledIndices,
		const FVertexFactory* VertexFactory,
		FMeshBatch& OutMeshElement) const;

	/** Configures mesh batch vertex / index state. Returns the number of primitives used in the element. */
	ENGINE_API uint32 SetMeshElementGeometrySource(
		const FStaticMeshSection& Section,
		const FRawStaticIndexBuffer& IndexBuffer,
		const FAdditionalStaticMeshIndexBuffers* AdditionalIndexBuffers,
		const FVertexFactory* VertexFactory,
		bool bWireframe,
		bool bUseInversedIndices,
		FMeshBatch& OutMeshElement) const;

	/** Sets the screen size on a mesh element. */
	ENGINE_API void SetMeshElementScreenSize(int32 LODIndex, bool bDitheredLODTransition, FMeshBatch& OutMeshBatch) const;

	/** Returns whether this mesh should render back-faces instead of front-faces - either with reversed indices or reversed cull mode */
	ENGINE_API bool ShouldRenderBackFaces() const;

	/** Returns whether this mesh needs reverse culling when using reversed indices. */
	ENGINE_API bool IsReversedCullingNeeded(bool bUseReversedIndices) const;

	ENGINE_API bool IsCollisionView(const FEngineShowFlags& EngineShowFlags, bool& bDrawSimpleCollision, bool& bDrawComplexCollision) const;

	/** Only call on render thread timeline */
	ENGINE_API uint8 GetCurrentFirstLODIdx_Internal() const;

	ENGINE_API virtual void OnEvaluateWorldPositionOffsetChanged_RenderThread() override;

public:
	// FPrimitiveSceneProxy interface.
#if WITH_EDITOR
	ENGINE_API virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
	ENGINE_API virtual HHitProxy* CreateHitProxies(IPrimitiveComponent* ComponentInterface, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
#endif
	ENGINE_API virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override;
	ENGINE_API virtual int32 GetLOD(const FSceneView* View) const override;
	ENGINE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	ENGINE_API virtual bool CanBeOccluded() const override;
	virtual bool AllowInstanceCullingOcclusionQueries() const override { return true; }
	ENGINE_API virtual bool IsUsingDistanceCullFade() const override;
	ENGINE_API virtual void GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const override;
	ENGINE_API virtual void GetDistanceFieldAtlasData(const FDistanceFieldVolumeData*& OutDistanceFieldData, float& SelfShadowBias) const override;
	ENGINE_API virtual bool HasDistanceFieldRepresentation() const override;
	ENGINE_API virtual bool StaticMeshHasPendingStreaming() const override;
	ENGINE_API virtual bool HasDynamicIndirectShadowCasterRepresentation() const override;
	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	SIZE_T GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() + LODs.GetAllocatedSize() ); }

	ENGINE_API virtual void GetMeshDescription(int32 LODIndex, TArray<FMeshBatch>& OutMeshElements) const override;

	ENGINE_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	ENGINE_API virtual const FCardRepresentationData* GetMeshCardRepresentation() const override;

	virtual bool IsCullingReversedByComponent() const override { return bReverseCulling; }
	
	virtual FUintVector2 GetMeshPaintTextureDescriptor() const override { return MeshPaintTextureDescriptor; }

#if RHI_RAYTRACING
	ENGINE_API virtual TArray<FRayTracingGeometry*> GetStaticRayTracingGeometries() const override;
	ENGINE_API virtual void GetStaticRayTracingInstances(FStaticRayTracingInstances& OutRayTracingInstances) override;
	ENGINE_API virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override;
	ENGINE_API virtual bool HasRayTracingRepresentation() const override;
	virtual bool IsRayTracingRelevant() const override { return true; }
	virtual bool IsRayTracingStaticRelevant() const override 
	{ 
		const bool bAllowStaticLighting = IsStaticLightingAllowed();
		const bool bIsStaticInstance = !bDynamicRayTracingInstances && !bDynamicRayTracingGeometry;
		return bIsStaticInstance && !HasViewDependentDPG() && !(bAllowStaticLighting && HasStaticLighting() && !HasValidSettingsForStaticLighting());
	}
	ENGINE_API virtual RayTracing::FGeometryGroupHandle GetRayTracingGeometryGroupHandle() const override;
	ENGINE_API virtual FRaytracingVFAttributeData GetRaytracingVFAttributeData(const FSceneView* View) override;
#endif // RHI_RAYTRACING

	ENGINE_API virtual void GetLCIs(FLCIArray& LCIs) override;

#if WITH_EDITORONLY_DATA
	ENGINE_API virtual bool GetPrimitiveDistance(int32 LODIndex, int32 SectionIndex, const FVector& ViewOrigin, float& PrimitiveDistance) const override;
	ENGINE_API virtual bool GetMeshUVDensities(int32 LODIndex, int32 SectionIndex, FVector4& WorldUVDensities) const override;
	ENGINE_API virtual bool GetMaterialTextureScales(int32 LODIndex, int32 SectionIndex, const FMaterialRenderProxy* MaterialRenderProxy, FVector4f* OneOverScales, FIntVector4* UVChannelIndices) const override;
#endif

#if STATICMESH_ENABLE_DEBUG_RENDERING
	virtual int32 GetLightMapResolution() const override { return LightMapResolution; }
#endif

protected:
	/** Information used by the proxy about a single LOD of the mesh. */
	class FLODInfo : public FLightCacheInterface
	{
	public:

		/** Information about an element of a LOD. */
		struct FSectionInfo
		{
			/** The material with which to render this section. */
			UMaterialInterface* Material = nullptr;

		#if WITH_EDITOR
			/** True if this section should be rendered as selected (editor only). */
			bool bSelected = false;

			/** The editor needs to be able to individual sub-mesh hit detection, so we store a hit proxy on each mesh. */
			HHitProxy* HitProxy = nullptr;
		#endif
			
			// The material index from the component. Used by the texture streaming accuracy viewmodes.
			int32 MaterialIndex = 0;
			
			/** The overlay material with which to render a second time this section. */
			UMaterialInterface* OverlayMaterial = nullptr;
		};

		/** Per-section information. */
		TArray<FSectionInfo, TInlineAllocator<1>> Sections;

		/** Vertex color data for this LOD (or NULL when not overridden), FStaticMeshComponentLODInfo handle the release of the memory */
		FColorVertexBuffer* OverrideColorVertexBuffer;

		TUniformBufferRef<FLocalVertexFactoryUniformShaderParameters> OverrideColorVFUniformBuffer;

		/** Initialization constructor. */
		FLODInfo(const FStaticMeshSceneProxyDesc& InProxyDesc, const FStaticMeshVertexFactoriesArray& InLODVertexFactories, int32 InLODIndex, int32 InClampedMinLOD, bool bLODsShareStaticLighting);

		bool UsesMeshModifyingMaterials() const { return bUsesMeshModifyingMaterials; }

		// FLightCacheInterface.
		ENGINE_API virtual FLightInteraction GetInteraction(const FLightSceneProxy* LightSceneProxy) const override;

	private:
		TArray<FGuid> IrrelevantLights;

		/** True if any elements in this LOD use mesh-modifying materials **/
		bool bUsesMeshModifyingMaterials;
	};

	/** Information used by the proxy about a single ray tracing proxy LOD of the mesh. */
	class FRayTracingLODInfo
	{
	public:

		struct FSectionInfo
		{
			UMaterialInterface* Material = nullptr;
		};

		TArray<FSectionInfo, TInlineAllocator<1>> Sections;

		// TODO: Support OverrideColorVertexBuffer (needs to be generated similar to what is done for regular LODs)

		FRayTracingLODInfo(const FStaticMeshSceneProxyDesc& InProxyDesc, int32 InLODIndex);
	};

	FStaticMeshRenderData* RenderData;

	TArray<FLODInfo> LODs;
	TArray<FRayTracingLODInfo> RayTracingLODInfos;

	const FDistanceFieldVolumeData* DistanceFieldData;	
	const FCardRepresentationData* CardRepresentationData;	

	UMaterialInterface* OverlayMaterial;
	float OverlayMaterialMaxDrawDistance;
	TArray<TObjectPtr<UMaterialInterface>> MaterialSlotsOverlayMaterial;

#if RHI_RAYTRACING
	ENGINE_API virtual void GetRayTracingWPOConfig(bool& bOutHasRayTracingRepresentation, bool& bOutDynamicRayTracingGeometry) const;
	ENGINE_API virtual bool ShouldEvaluateWPOInRayTracing(FVector ViewCenter) const;

	ENGINE_API void CreateDynamicRayTracingGeometries(FRHICommandListBase& RHICmdList);
	ENGINE_API void ReleaseDynamicRayTracingGeometries();

	bool bSupportRayTracing : 1;
	bool bHasRayTracingRepresentation : 1;
	bool bDynamicRayTracingInstances : 1;
	bool bDynamicRayTracingGeometry : 1;
	bool bCreateDynamicRayTracingGeometries : 1;

	TArray<FRayTracingGeometry> DynamicRayTracingGeometries;
	TArray<FMeshBatch> CachedRayTracingMaterials;
	int16 CachedRayTracingMaterialsLODIndex = INDEX_NONE;

	RayTracing::FGeometryGroupHandle RayTracingGeometryGroupHandle;
#endif
	/**
	 * The forcedLOD set in the static mesh editor, copied from the mesh component
	 */
	int32 ForcedLodModel;

	/** Minimum LOD index to use.  Clamped to valid range [0, NumLODs - 1]. */
	int32 ClampedMinLOD;

	uint32 bCastShadow : 1;

	/** This primitive has culling reversed */
	uint32 bReverseCulling : 1;

	/** The coordinate index to use for texture color painting. */
	uint32 MeshPaintTextureCoordinateIndex : 2;

	/** The view relevance for all the static mesh's materials. */
	FMaterialRelevance MaterialRelevance;

	/** The distance at which to disable World Position Offset (0 = no max). */
	float WPODisableDistance;

#if WITH_EDITORONLY_DATA
	/** The component streaming distance multiplier */
	float StreamingDistanceMultiplier;
	/** The cached GetTextureStreamingTransformScale */
	float StreamingTransformScale;
	/** Material bounds used for texture streaming. */
	TArray<uint32> MaterialStreamingRelativeBoxes;

	/** Index of the section to preview. If set to INDEX_NONE, all section will be rendered */
	int32 SectionIndexPreview;
	/** Index of the material to preview. If set to INDEX_NONE, all section will be rendered */
	int32 MaterialIndexPreview;

	/** Whether selection should be per section or per entire proxy. */
	bool bPerSectionSelection;
#endif

private:

	const UStaticMesh* StaticMesh;

	FTextureResource* MeshPaintTextureResource = nullptr;
	FUintVector2 MeshPaintTextureDescriptor = FUintVector2(0, 0);

#if STATICMESH_ENABLE_DEBUG_RENDERING
	UObject* Owner;
	/** LightMap resolution used for VMI_LightmapDensity */
	int32 LightMapResolution;
	/** Body setup for collision debug rendering */
	UBodySetup* BodySetup;
	/** Collision trace flags */
	ECollisionTraceFlag		CollisionTraceFlag;
	/** Collision Response of this component */
	FCollisionResponseContainer CollisionResponse;
	/** LOD used for collision */
	int32 LODForCollision;
	/** Draw mesh collision if used for complex collision */
	uint32 bDrawMeshCollisionIfComplex : 1;
	/** Draw mesh collision if used for simple collision */
	uint32 bDrawMeshCollisionIfSimple : 1;

protected:
	/** Hierarchical LOD Index used for rendering */
	uint8 HierarchicalLODIndex;
#endif

public:

	/**
	 * Returns the display factor for the given LOD level
	 *
	 * @Param LODIndex - The LOD to get the display factor for
	 */
	ENGINE_API float GetScreenSize(int32 LODIndex) const;

	/**
	 * Returns the LOD mask for a view, this is like the ordinary LOD but can return two values for dither fading
	 */
	ENGINE_API FLODMask GetLODMask(const FSceneView* View) const;

private:
	ENGINE_API void AddSpeedTreeWind();
	ENGINE_API void RemoveSpeedTreeWind();
};
