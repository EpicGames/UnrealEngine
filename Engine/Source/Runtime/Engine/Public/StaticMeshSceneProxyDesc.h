// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PrimitiveSceneProxyDesc.h"
#include "Components/StaticMeshComponent.h" // STATICMESH_ENABLE_DEBUG_RENDERING

class UBodySetup;
class FTextureResource;
struct FStaticMeshComponentLODInfo;

struct FStaticMeshSceneProxyDesc : public FPrimitiveSceneProxyDesc
{
	ENGINE_API FStaticMeshSceneProxyDesc();
	ENGINE_API FStaticMeshSceneProxyDesc(const UStaticMeshComponent*);
	ENGINE_API ~FStaticMeshSceneProxyDesc();

	ENGINE_API void InitializeFromStaticMeshComponent(const UStaticMeshComponent*);

	TArrayView<TObjectPtr<UMaterialInterface>>	OverrideMaterials;
	TArray<TObjectPtr<UMaterialInterface>> MaterialSlotsOverlayMaterial;
	TArray<UMaterialCacheVirtualTexture*> MaterialCacheTextures;

	TArrayView<FStaticMeshComponentLODInfo> LODData;

	UStaticMesh* StaticMesh = nullptr;
	UBodySetup* BodySetup = nullptr;
	UObject* LODParentPrimitive = nullptr;
	UTexture* MeshPaintTexture = nullptr;

	// Optional pre-resolved resource pointer for MeshPaintTexture. Populated GT-side by callers
	// that build the desc on a thread where UTexture::GetResource() is not safe to call (e.g.
	// FastGeo's cooked async render-state worker). When non-null, GetMeshPaintTextureResource()
	// returns this directly and skips the live GetResource() call. Stays null for callers that
	// don't pre-resolve, preserving the original behavior.
	FTextureResource* CachedMeshPaintTextureResource = nullptr;

	const Nanite::FResources* NaniteResources = nullptr;

	TObjectPtr<UMaterialInterface> OverlayMaterial;
	FMaterialRelevance MaterialRelevance;

	float OverlayMaterialMaxDrawDistance = 0.0f;
	float NanitePixelProgrammableDistance = 0.0f;

	float DistanceFieldSelfShadowBias = 0;
	float DistanceFieldIndirectShadowMinVisibility = 0.1f;

	int32 ForcedLodModel = 0;
	int32 MinLOD = 0;
	int32 WorldPositionOffsetDisableDistance = 0;
	int32 StaticLightMapResolution = 0;
	int32 MeshPaintTextureCoordinateIndex = 0;
	
	FBox2f MaterialCacheUVRegion = FBox2f(FVector2f::Zero(), FVector2f::One());
	int32 MaterialCacheTextureCoordinateIndex = 0;

	//@todo: share color selection logic according to mobility with USMC?
	FColor	WireframeColor = FColor(0, 255, 255, 255);

	uint32 bReverseCulling : 1 = false;
#if STATICMESH_ENABLE_DEBUG_RENDERING
	uint32 bDrawMeshCollisionIfComplex : 1;
	uint32 bDrawMeshCollisionIfSimple : 1;
#endif
	uint32 bEvaluateWorldPositionOffset : 1;
	uint32 bOverrideMinLOD : 1;

	uint32 bCastDistanceFieldIndirectShadow : 1;
	uint32 bOverrideDistanceFieldSelfShadowBias : 1;
	uint32 bEvaluateWorldPositionOffsetInRayTracing : 1;
	uint32 bSortTriangles : 1;

	uint32 bDisplayNaniteFallbackMesh : 1;
	uint32 bDisallowNanite : 1;
	uint32 bForceDisableNanite : 1;
	uint32 bForceNaniteForMasked : 1;

	uint32 bUseProvidedMaterialRelevance : 1;
	uint32 bUseProvidedCollisionResponseContainer : 1;

	TOptional<FCollisionResponseContainer> CollisionResponseContainer;

	ELightmapType LightmapType = ELightmapType::Default;

#if WITH_EDITORONLY_DATA
	float StreamingDistanceMultiplier = 1.0f;
	TArrayView<uint32> MaterialStreamingRelativeBoxes;
	int32 SectionIndexPreview = INDEX_NONE;
	int32 MaterialIndexPreview = INDEX_NONE;
	int32 SelectedEditorMaterial = INDEX_NONE;
	int32 SelectedEditorSection = INDEX_NONE;

	float TextureStreamingTransformScale = 1.0f;
#endif

	bool IsReverseCulling() const
	{
		return bReverseCulling;
	}

	bool IsDisallowNanite() const
	{
		return bDisallowNanite;
	}

	bool IsForceDisableNanite() const
	{
		return bForceDisableNanite;
	}

	bool IsForceNaniteForMasked() const
	{
		return bForceNaniteForMasked;
	}

	int32 GetForcedLodModel() const
	{
		return ForcedLodModel;
	}

	int32 GetWorldPositionOffsetDisableDistance() const
	{
		return WorldPositionOffsetDisableDistance;
	}

	float GetNanitePixelProgrammableDistance() const
	{
		return NanitePixelProgrammableDistance;
	}

	bool EvaluateWorldPositionOffsetInRayTracing() const
	{
		return bEvaluateWorldPositionOffsetInRayTracing;
	}

	bool IsDisplayNaniteFallbackMesh() const
	{
		return bDisplayNaniteFallbackMesh;
	}

	UStaticMesh* GetStaticMesh() const
	{
		return StaticMesh;
	}

	UObject* GetLODParentPrimitive() const
	{
		return LODParentPrimitive;
	}

	const Nanite::FResources* GetNaniteResources() const
	{
		return NaniteResources;
	}

	UMaterialInterface* GetOverlayMaterial() const
	{
		return OverlayMaterial;
	}

	float GetOverlayMaterialMaxDrawDistance() const
	{
		return OverlayMaterialMaxDrawDistance;
	}

	int32 GetStaticLightMapResolution() const
	{
		return StaticLightMapResolution;
	}

	UObject* GetObjectForPropertyColoration() const
	{
		return Component;
	}

	FColor GetWireframeColor() const
	{
		return WireframeColor;
	}

	// Begin FPrimitiveSceneProxyDesc interface
	ENGINE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	// End FPrimitiveSceneProxyDesc interface

	ENGINE_API UMaterialInterface* GetMaterial(int32 MaterialIndex, bool bDoingNaniteMaterialAudit = false, bool bIgnoreNaniteOverrideMaterials = false) const;
	ENGINE_API int32 GetNumMaterials() const;

	bool ShouldCreateMaterialCacheProxy() const;

	ENGINE_API const UStaticMeshComponent* GetUStaticMeshComponent() const;
	ENGINE_API void GetMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotsOverlayMaterial) const;

	ENGINE_API void SetMaterialRelevance(const FMaterialRelevance& InRelevance);
	ENGINE_API FMaterialRelevance GetMaterialRelevance(EShaderPlatform ShaderPlatform) const;

	UE_DEPRECATED(5.7, "Please use GetMaterialRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API FMaterialRelevance GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const;

	FTextureResource* GetMeshPaintTextureResource() const;

	ENGINE_API UBodySetup* GetBodySetup() const;

	ENGINE_API bool HasValidNaniteData() const;
	ENGINE_API bool ShouldCreateNaniteProxy(Nanite::FMaterialAudit* OutNaniteMaterials = nullptr) const;
	ENGINE_API bool UseNaniteOverrideMaterials(bool bDoingMaterialAudit) const;
	ENGINE_API UMaterialInterface* GetNaniteAuditMaterial(int32 MaterialIndex) const;

	ENGINE_API void SetCollisionResponseToChannels(const FCollisionResponseContainer& InContainer);
	ENGINE_API const FCollisionResponseContainer& GetCollisionResponseToChannels() const;

};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "StaticMeshSceneProxy.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "Engine/Engine.h"
#include "StaticMeshComponentLODInfo.h"
#include "Rendering/NaniteResources.h"
#endif
