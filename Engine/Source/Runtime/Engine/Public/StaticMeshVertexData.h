// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Rendering/StaticMeshVertexDataInterface.h"

/** The implementation of the static mesh vertex data storage type. */
template<typename VertexDataType>
class TStaticMeshVertexData :
	public FStaticMeshVertexDataInterface
{
	using FVertexResourceArray = TResourceArray<VertexDataType, VERTEXBUFFER_ALIGNMENT>;
	FVertexResourceArray Data;
public:

	/**
	* Constructor
	* @param InNeedsCPUAccess - true if resource array data should be CPU accessible
	*/
	TStaticMeshVertexData(bool InNeedsCPUAccess=false)
		: Data(InNeedsCPUAccess)
	{
	}

	/**
	* Resizes the vertex data buffer, discarding any data which no longer fits.
	*
	* @param NumVertices - The number of vertices to allocate the buffer for.
	* @param BufferFlags - Flags to define the expected behavior of the buffer
	*/
	void ResizeBuffer(uint32 NumVertices, EResizeBufferFlags BufferFlags = EResizeBufferFlags::None) override
	{
		if ((uint32)Data.Num() < NumVertices)
		{
			// Enlarge the array.
			if (!EnumHasAnyFlags(BufferFlags, EResizeBufferFlags::AllowSlackOnGrow))
			{
				Data.Reserve(NumVertices);
			}

			Data.AddUninitialized(NumVertices - Data.Num());
		}
		else if ((uint32)Data.Num() > NumVertices)
		{
			// Shrink the array.
			EAllowShrinking AllowShinking = EnumHasAnyFlags(BufferFlags, EResizeBufferFlags::AllowSlackOnReduce) ? EAllowShrinking::No : EAllowShrinking::Yes;
			Data.RemoveAt(NumVertices, Data.Num() - NumVertices, AllowShinking);
		}
	}

	void Empty(uint32 NumVertices) override
	{
		Data.Empty(NumVertices);
	}

	bool IsValidIndex(uint32 Index) override
	{
		return Data.IsValidIndex(Index);
	}

	/**
	* @return stride of the vertex type stored in the resource data array
	*/
	uint32 GetStride() const override
	{
		return sizeof(VertexDataType);
	}
	/**
	* @return uint8 pointer to the resource data array
	*/
	uint8* GetDataPointer() override
	{
		return (uint8*)Data.GetData();
	}

	/**
	* @return resource array interface access
	*/
	FResourceArrayInterface* GetResourceArray() override
	{
		return &Data;
	}

	const FResourceArrayInterface* GetResourceArray() const
	{
		return &Data;
	}

	/**
	* Serializer for this class
	*
	* @param Ar - archive to serialize to
	* @param B - data to serialize
	*/
	void Serialize(FArchive& Ar, bool bForcePerElementSerialization = false) override
	{
		Data.FVertexResourceArray::BulkSerialize(Ar, bForcePerElementSerialization);
	}
	/**
	* Assignment. This is currently the only method which allows for 
	* modifying an existing resource array
	*/
	void Assign(const TArray<VertexDataType>& Other)
	{
		ResizeBuffer(Other.Num());
		if (Other.Num())
		{
			memcpy(GetDataPointer(), &Other[0], Other.Num() * sizeof(VertexDataType));
		}
	}

	/**
	* Helper function to return the amount of memory allocated by this
	* container.
	*
	* @returns Number of bytes allocated by this container.
	*/
	SIZE_T GetResourceSize() const override
	{
		return Data.GetAllocatedSize();
	}

	/**
	* Helper function to return the number of elements by this
	* container.
	*
	* @returns Number of elements allocated by this container.
	*/
	virtual int32 Num() const override
	{
		return Data.Num();
	}

	bool GetAllowCPUAccess() const override
	{
		return Data.GetAllowCPUAccess();
	}

	void OverrideFreezeSizeAndAlignment(int64& Size, int32& Alignment) const override
	{
		Size = sizeof(*this);
	}
};
