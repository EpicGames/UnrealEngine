// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Hash/xxhash.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialLayersFunctions.h"
#include "Misc/Guid.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/NameTypes.h"
#include "UObject/RenderingObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "StaticParameterSet.generated.h"

class FShaderKeyGenerator;
class FSHA1;
struct FStaticParameterSet;

/**
Base parameter properties
*/
USTRUCT()
struct FStaticParameterBase
{
	GENERATED_USTRUCT_BODY();

	UPROPERTY()
		FMaterialParameterInfo ParameterInfo;

	UPROPERTY()
		bool bOverride;

	UPROPERTY()
		FGuid ExpressionGUID;

	FStaticParameterBase() :
		bOverride(false),
		ExpressionGUID(0, 0, 0, 0)
	{ }

	FStaticParameterBase(const FMaterialParameterInfo& InInfo, bool InOverride, FGuid InGuid) :
		ParameterInfo(InInfo),
		bOverride(InOverride),
		ExpressionGUID(InGuid)
	{ }

	bool IsOverride() const { return bOverride; }

	friend FArchive& operator<<(FArchive& Ar, FStaticParameterBase& P)
	{
		// This method should never be called, derived structures need to implement their own code (to retain compatibility) or call SerializeBase (for new classes)
		check(false);

		return Ar;
	}

	bool operator==(const FStaticParameterBase& Reference) const
	{
		return ParameterInfo == Reference.ParameterInfo && bOverride == Reference.bOverride && ExpressionGUID == Reference.ExpressionGUID;
	}

	void SerializeBase(FArchive& Ar)
	{
		Ar << ParameterInfo;
		Ar << bOverride;
		Ar << ExpressionGUID;
	}

	template<typename THasher>
	void UpdateHash(THasher& HashState) const
	{
		const FString ParameterName = ParameterInfo.ToString();
		HashState.Update((const uint8*)*ParameterName, ParameterName.Len() * sizeof(TCHAR));
		HashState.Update((const uint8*)&ExpressionGUID, sizeof(ExpressionGUID));
		uint8 Override = bOverride;
		HashState.Update((const uint8*)&Override, sizeof(Override));
	}

	ENGINE_API void AppendKeyString(FString& KeyString) const;
	ENGINE_API void Append(FShaderKeyGenerator& KeyGen) const;
};


/**
* Holds the information for a static switch parameter
*/
USTRUCT()
struct FStaticSwitchParameter : public FStaticParameterBase
{
	GENERATED_USTRUCT_BODY();

	UPROPERTY()
	bool Value;

	FStaticSwitchParameter() :
		FStaticParameterBase(),
		Value(false)
	{ }

	FStaticSwitchParameter(const FMaterialParameterInfo& InInfo, bool InValue, bool InOverride, FGuid InGuid) :
		FStaticParameterBase(InInfo, InOverride, InGuid),
		Value(InValue)
	{ }

	bool operator==(const FStaticSwitchParameter& Reference) const
	{
		return FStaticParameterBase::operator==(Reference) && Value == Reference.Value;
	}

	friend FArchive& operator<<(FArchive& Ar, FStaticSwitchParameter& P)
	{
		Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
		if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MaterialAttributeLayerParameters)
		{
			Ar << P.ParameterInfo.Name;
		}
		else
		{
			Ar << P.ParameterInfo;
		}
		Ar << P.Value << P.bOverride << P.ExpressionGUID;
		return Ar;
	}

	void UpdateHash(FXxHash64Builder& HashState) const
	{
		FStaticParameterBase::UpdateHash(HashState);
		uint8 HashValue = Value;
		HashState.Update((const uint8*)&HashValue, sizeof(HashValue));
	}

	ENGINE_API void AppendKeyString(FString& KeyString) const;
	ENGINE_API void Append(FShaderKeyGenerator& KeyGen) const;

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = Value;
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
#endif
	}

	bool IsValid() const { return true; }
};

/**
* Holds the information for a static component mask parameter
*/
USTRUCT()
struct FStaticComponentMaskParameter : public FStaticParameterBase
{
	GENERATED_USTRUCT_BODY();

	UPROPERTY()
	bool R;
	
	UPROPERTY()
	bool G;

	UPROPERTY()
	bool B;

	UPROPERTY()
	bool A; 

	FStaticComponentMaskParameter() :
		FStaticParameterBase(),
		R(false),
		G(false),
		B(false),
		A(false)
	{ }

	FStaticComponentMaskParameter(const FMaterialParameterInfo& InInfo, bool InR, bool InG, bool InB, bool InA, bool InOverride, FGuid InGuid) :
		FStaticParameterBase(InInfo, InOverride, InGuid),
		R(InR),
		G(InG),
		B(InB),
		A(InA)
	{ }

	bool operator==(const FStaticComponentMaskParameter& Reference) const
	{
		return FStaticParameterBase::operator==(Reference) && R == Reference.R && G == Reference.G && B == Reference.B && A == Reference.A;
	}

	friend FArchive& operator<<(FArchive& Ar, FStaticComponentMaskParameter& P)
	{
		Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
		if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MaterialAttributeLayerParameters)
		{
			Ar << P.ParameterInfo.Name;
		}
		else
		{
			Ar << P.ParameterInfo;
		}
		Ar << P.R << P.G << P.B << P.A << P.bOverride << P.ExpressionGUID;
		return Ar;
	}

	UE_DEPRECATED(5.8, "Use overload accepting an FXxHash64Builder; SHA1 hashes are no longer used")
	void UpdateHash(FSHA1& HashState) const {}

	void UpdateHash(FXxHash64Builder& HashState) const
	{
		FStaticParameterBase::UpdateHash(HashState);
		uint8 Values[4];
		Values[0] = R;
		Values[1] = G;
		Values[2] = B;
		Values[3] = A;
		HashState.Update((const uint8*)&Values, sizeof(Values));
	}

	ENGINE_API void AppendKeyString(FString& KeyString) const;
	ENGINE_API void Append(FShaderKeyGenerator& KeyGen) const;

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = FMaterialParameterValue(R, G, B, A);
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
#endif
	}

	bool IsValid() const { return true; }
};

/**
* Stores information that maps a terrain layer to a particular weightmap index
* Despite the name, these are not actually material parameters. These bindings are automatically generated by landscape when materials are initialized
* Still stored in FStaticParameterSet, since it influences generation of shaders on the MI
*/
USTRUCT()
struct FStaticTerrainLayerWeightParameter
{
	GENERATED_USTRUCT_BODY();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// These are necessary to prevent deprecation warnings from UE_DEPRECATED on some compilers because of auto-generated code
	FStaticTerrainLayerWeightParameter() = default;
	FStaticTerrainLayerWeightParameter(const FStaticTerrainLayerWeightParameter&) = default;
	FStaticTerrainLayerWeightParameter(FStaticTerrainLayerWeightParameter&&) = default;
	FStaticTerrainLayerWeightParameter& operator=(const FStaticTerrainLayerWeightParameter&) = default;
	FStaticTerrainLayerWeightParameter& operator=(FStaticTerrainLayerWeightParameter&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(all, "ParameterInfo_DEPRECATED remains in the code for backwards-compatibility reasons but shall not be used")
	UPROPERTY()
	FMaterialParameterInfo ParameterInfo_DEPRECATED;

	UE_DEPRECATED(5.7, "ExpressionGUID_DEPRECATED is now unused and will be removed in a future version")
	UPROPERTY()
	FGuid ExpressionGUID_DEPRECATED;

	UE_DEPRECATED(all, "bOverride_DEPRECATED remains in the code for backwards-compatibility reasons but shall not be used")
	UPROPERTY()
	bool bOverride_DEPRECATED = true;

	UPROPERTY()
	FName LayerName;

	UPROPERTY()
	int32 WeightmapIndex = INDEX_NONE;

	UE_DEPRECATED(5.7, "bIsRepeatedLayer is now unused and will be removed in a future version")
	UPROPERTY()
	bool bIsRepeatedLayer = false;

	UE_DEPRECATED(5.7, "bWeightBasedBlend is now unused and will be removed in a future version")
	UPROPERTY()
	bool bWeightBasedBlend = false;

	UE_DEPRECATED(5.7, "bWeightBasedBlend being deprecated, this constructor is now useless")
	FStaticTerrainLayerWeightParameter(const FName& InName, int32 InWeightmapIndex, bool InWeightBasedBlend) 
		: LayerName(InName)
		, WeightmapIndex(InWeightmapIndex)
	{ }

	UE_DEPRECATED(5.7, "bWeightBasedBlend being deprecated, this constructor is now useless")
	FStaticTerrainLayerWeightParameter(const FName& InName, int32 InWeightmapIndex, bool InWeightBasedBlend, bool InIsRepeatedLayer)
		: LayerName(InName)
		, WeightmapIndex(InWeightmapIndex)
	{
	}

	FStaticTerrainLayerWeightParameter(const FName& InName, int32 InWeightmapIndex)
		: LayerName(InName)
		, WeightmapIndex(InWeightmapIndex)
	{}

	bool operator==(const FStaticTerrainLayerWeightParameter& Reference) const
	{
		return LayerName == Reference.LayerName && WeightmapIndex == Reference.WeightmapIndex;
	}

	friend FArchive& operator<<(FArchive& Ar, FStaticTerrainLayerWeightParameter& P)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
		Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
		if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MaterialAttributeLayerParameters)
		{
			Ar << P.LayerName;
		}
		else if(Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::TerrainLayerWeightsAreNotParameters)
		{
			Ar << P.ParameterInfo_DEPRECATED;
			P.LayerName = P.ParameterInfo_DEPRECATED.Name;
		}
		else
#endif // WITH_EDITORONLY_DATA
		{
			Ar << P.LayerName;
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::StaticParameterTerrainLayerWeightBlendType)
		{
			// For backwards compatibility reasons, we now serialize a bogus value and this code shall remain here forever, but the member 
			//  variable bWeightBasedBlend can be removed after the deprecation grace period has passed
			bool bDummyWeightBasedBlend = false;
			Ar << bDummyWeightBasedBlend;
		}

		Ar << P.WeightmapIndex;
#if WITH_EDITORONLY_DATA
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::TerrainLayerWeightsAreNotParameters)
		{
			Ar << P.bOverride_DEPRECATED;
			// For backwards compatibility reasons, we now serialize a bogus value and this code shall remain here forever, but the member 
			//  variable ExpressionGUID_DEPRECATED can be removed after the deprecation grace period has passed
			FGuid DummyExpressionGUID_DEPRECATED;
			Ar << DummyExpressionGUID_DEPRECATED;
		}
#endif // WITH_EDITORONLY_DATA
		return Ar;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.8, "Use overload accepting an FXxHash64Builder; SHA1 hashes are no longer used")
	void UpdateHash(FSHA1& HashState) const {}

	void UpdateHash(FXxHash64Builder& HashState) const
	{
		const FString LayerNameString = LayerName.ToString();
		HashState.Update((const uint8*)*LayerNameString, LayerNameString.Len() * sizeof(TCHAR));
		HashState.Update((const uint8*)&WeightmapIndex, sizeof(WeightmapIndex));
	}
	ENGINE_API void AppendKeyString(FString& KeyString) const;
	ENGINE_API void Append(FShaderKeyGenerator& KeyGen) const;
};

struct UE_DEPRECATED(5.0, "Material layers are no longer material parameters, use FStaticParameterSet::MaterialLayers") FStaticMaterialLayersParameter;
USTRUCT()
struct FStaticMaterialLayersParameter : public FStaticParameterBase
{
	GENERATED_USTRUCT_BODY();
#if WITH_EDITOR
	struct ID
	{
		FStaticParameterBase ParameterID;
		FMaterialLayersFunctions::ID Functions;

		friend FArchive& operator<<(FArchive& Ar, ID& P)
		{
			P.ParameterID.SerializeBase(Ar);
			P.Functions.SerializeForDDC(Ar);
			return Ar;
		}
	};
#endif // WITH_EDITOR

	UPROPERTY()
	FMaterialLayersFunctions Value;

#if WITH_EDITOR
	friend FArchive& operator<<(FArchive& Ar, FStaticMaterialLayersParameter& P)
	{
		Ar << P.ParameterInfo << P.bOverride << P.ExpressionGUID;
		Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
		if (Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::MaterialLayersParameterSerializationRefactor)
		{
			P.Value.SerializeLegacy(Ar);
		}
		return Ar;
	}
#endif // WITH_EDITOR
};

USTRUCT()
struct FStaticParameterSetRuntimeData
{
	GENERATED_BODY()

	FStaticParameterSetRuntimeData() : bHasMaterialLayers(false) {}
	FStaticParameterSetRuntimeData(const FStaticParameterSetRuntimeData& Rhs) = default;
	FStaticParameterSetRuntimeData(const FStaticParameterSet& Rhs) = delete;
	FStaticParameterSetRuntimeData& operator=(const FStaticParameterSetRuntimeData& Rhs) = default;
	FStaticParameterSetRuntimeData& operator=(const FStaticParameterSet& Rhs) = delete;

	/** An array of static switch parameters in this set */
	UPROPERTY()
	TArray<FStaticSwitchParameter> StaticSwitchParameters;

	/** Material layers for this set */
	UPROPERTY()
	FMaterialLayersFunctionsRuntimeData MaterialLayers;

	UPROPERTY()
	uint8 bHasMaterialLayers : 1;

	void Empty()
	{
		StaticSwitchParameters.Empty();
		MaterialLayers.Empty();
		bHasMaterialLayers = false;
	}

	bool IsEmpty() const
	{
		return  StaticSwitchParameters.Num() == 0 && !bHasMaterialLayers;
	}
};

USTRUCT()
struct FStaticParameterSetEditorOnlyData
{
	GENERATED_BODY()

	/** An array of static switch parameters in this set */
	UPROPERTY()
	TArray<FStaticSwitchParameter> StaticSwitchParameters_DEPRECATED;

	/** An array of static component mask parameters in this set */
	UPROPERTY()
	TArray<FStaticComponentMaskParameter> StaticComponentMaskParameters;

	/** An array of terrain layer weight parameters in this set */
	UPROPERTY()
	TArray<FStaticTerrainLayerWeightParameter> TerrainLayerWeightParameters;

	/** Material layers for this set */
	UPROPERTY()
	FMaterialLayersFunctionsEditorOnlyData MaterialLayers;

#if WITH_EDITOR
	void Empty()
	{
		StaticComponentMaskParameters.Empty();
		TerrainLayerWeightParameters.Empty();
		MaterialLayers.Empty();
	}

	bool IsEmpty() const
	{
		return StaticComponentMaskParameters.Num() == 0 && TerrainLayerWeightParameters.Num() == 0;
	}
#endif // WITH_EDITOR
};

/** Contains all the information needed to identify a single permutation of static parameters. */
USTRUCT()
struct FStaticParameterSet : public FStaticParameterSetRuntimeData
{
	GENERATED_USTRUCT_BODY();

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FStaticParameterSetEditorOnlyData EditorOnly;
#endif // WITH_EDITORONLY_DATA

	FStaticParameterSet() = default;
	ENGINE_API FStaticParameterSet(const FStaticParameterSet& InValue);
	FStaticParameterSet(const FStaticParameterSetRuntimeData&) = delete;

	ENGINE_API FStaticParameterSet& operator=(const FStaticParameterSet& InValue);
	FStaticParameterSet& operator=(const FStaticParameterSetRuntimeData& InValue) = delete;

	FStaticParameterSetRuntimeData& GetRuntime() { return *this; }
	const FStaticParameterSetRuntimeData& GetRuntime() const { return *this; }

	/** 
	* Checks if this set contains any parameters
	* 
	* @return	true if this set has no parameters
	*/
	bool IsEmpty() const
	{
		if (!FStaticParameterSetRuntimeData::IsEmpty()) return false;
#if WITH_EDITOR
		if (!EditorOnly.IsEmpty()) return false;
#endif
		return true;
	}

	void Empty()
	{
		FStaticParameterSetRuntimeData::Empty();
#if WITH_EDITOR
		EditorOnly.Empty();
#endif
		// Ensure that we're actually empty now
		check(IsEmpty());
	}

	bool GetMaterialLayers(FMaterialLayersFunctions& OutMaterialLayers) const;

#if WITH_EDITOR
	void SerializeLegacy(FArchive& Ar);
	void UpdateLegacyTerrainLayerWeightData();
	void UpdateLegacyMaterialLayersData();

	static void Validate(const FStaticParameterSetRuntimeData& Runtime, const FStaticParameterSetEditorOnlyData& EditorOnly);
	void Validate()
	{
		Validate(GetRuntime(), EditorOnly);
	}
#endif // WITH_EDITOR

	void PostSerialize(const FArchive& Ar);

	/** 
	* Tests this set against another for equality
	* 
	* @param ReferenceSet	The set to compare against
	* @return				true if the sets are equal
	*/
	bool operator==(const FStaticParameterSet& ReferenceSet) const;

	bool operator!=(const FStaticParameterSet& ReferenceSet) const
	{
		return !(*this == ReferenceSet);
	}

	ENGINE_API bool Equivalent(const FStaticParameterSet& ReferenceSet) const;

#if WITH_EDITORONLY_DATA
	ENGINE_API void SetParameterValue(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& Meta, EMaterialSetParameterValueFlags Flags = EMaterialSetParameterValueFlags::None);
	void AddParametersOfType(EMaterialParameterType Type, const TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& Values);
#endif // WITH_EDITORONLY_DATA

private:
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TArray<FStaticMaterialLayersParameter> MaterialLayersParameters_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	TArray<FStaticSwitchParameter> StaticSwitchParameters_DEPRECATED;

	UPROPERTY()
	TArray<FStaticComponentMaskParameter> StaticComponentMaskParameters_DEPRECATED;

	UPROPERTY()
	TArray<FStaticTerrainLayerWeightParameter> TerrainLayerWeightParameters_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	void SortForEquivalent();

#if WITH_EDITORONLY_DATA
	void SetStaticSwitchParameterValue(const FMaterialParameterInfo& ParameterInfo, const FGuid& ExpressionGuid, bool Value);
	void SetStaticComponentMaskParameterValue(const FMaterialParameterInfo& ParameterInfo, const FGuid& ExpressionGuid, bool R, bool G, bool B, bool A);
#endif // WITH_EDITORONLY_DATA

	friend bool operator==(const FStaticParameterSet&, const FStaticParameterSetRuntimeData&) = delete;
	friend bool operator==(const FStaticParameterSetRuntimeData&, const FStaticParameterSet&) = delete;
	friend bool operator!=(const FStaticParameterSet&, const FStaticParameterSetRuntimeData&) = delete;
	friend bool operator!=(const FStaticParameterSetRuntimeData&, const FStaticParameterSet&) = delete;
};

template<>
struct TStructOpsTypeTraits<FStaticParameterSet> : TStructOpsTypeTraitsBase2<FStaticParameterSet>
{
	enum { WithPostSerialize = true	};
};
