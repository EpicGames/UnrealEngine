// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"

/**
 * To add flags to StereoLayerComponents from plugins, please follow this procedure:
 * 1. Implement the IStereoLayersFlagsSupplier interface.
 * 
 * 2. Register the modular feature:
 *	  IModularFeatures::Get().RegisterModularFeature(IStereoLayersFlagsSupplier::GetModularFeatureName(), this);
 * 
 * 3. Unregister the modular feature when the plugin is torn down:
 *    IModularFeatures::Get().UnregisterModularFeature(IStereoLayersFlagsSupplier::GetModularFeatureName(), this);
 * 
 * 4. Implement the EnumerateFlags(TSet<FName>& OutFlags) function, where you can add your flags. A simple 
 *	  implementation might be:
 *		void MyFlagSupplier::EnumerateFlags(TSet<FName>& OutFlags)
 *		{
 *			OutFlags.Add(FName("MY_FLAG_1"));
 *			OutFlags.Add(FName("MY_FLAG_2"));
 *		}
 * 5. It's up to the developers to make sure the flag values are not overflowing.
 * 6. At runtime, use 
 *		TSharedPtr<FStereoLayerAdditionalFlagsManager> FlagsManager = FStereoLayerAdditionalFlagsManager::Get();
 *		FlagsManager->GetFlagValue(Flag);
 *	  to get flags value to check against the flags in your StereoLayerComponents.
 */

DECLARE_LOG_CATEGORY_EXTERN(LogStereoLayerFlags, Log, All);

class FStereoLayerAdditionalFlagsManager final
{
	// Required for creating a TSharedPtr because this class has a private constructor.
	template<typename, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

public:
	/** Returns instance of this class. If it doesn't exist yet, it creates one and collects all the flags. */
	static TSharedPtr<FStereoLayerAdditionalFlagsManager> Get();

	/** Collect all flags to use in StereoLayerComponents. */
	static void CollectFlags(TSet<FName>& OutFlags);

	/** Destroys manager when a game or PIE session is ending. */
	static void Destroy();

	/** Returns the value of the flag if it has been found, 0 otherwise. */
	uint32 GetFlagValue(const FName Flag) const;

private:
	FStereoLayerAdditionalFlagsManager() {};

	/** Create runtime mapping between flag names and values. */
	void CreateRuntimeFlagsMap();

	/** Parameterless private function to collect flags and store them in the UniqueFlags. */
	void CollectFlags();

	static TSharedPtr<FStereoLayerAdditionalFlagsManager> Instance;

	// All the unique flags provided by suppliers.
	TSet<FName> UniqueFlags;
	// Maps flags with their runtime values.
	TMap<FName, uint32> RuntimeFlags;
};



