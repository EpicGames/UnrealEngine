// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenRendering.h: Screen rendering definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"

/**
 * A vertex shader for rendering a transformed textured element.
 */
class FStereoLayerVS : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FStereoLayerVS,Global,ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FStereoLayerVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FGlobalShader(Initializer)
	{
		InQuadAdjust.Bind(Initializer.ParameterMap, TEXT("InQuadAdjust"));
		InUVAdjust.Bind(Initializer.ParameterMap, TEXT("InUVAdjust"));
		InViewProjection.Bind(Initializer.ParameterMap, TEXT("InViewProjection"));
		InWorld.Bind(Initializer.ParameterMap, TEXT("InWorld"));
	}
	FStereoLayerVS() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FVector2D QuadSize, FBox2D UVRect, const FMatrix& ViewProjection, const FMatrix& World)
	{
		if (InQuadAdjust.IsBound())
		{
			SetShaderValue(BatchedParameters, InQuadAdjust, FVector2f(QuadSize));
		}

		if (InUVAdjust.IsBound())
		{
			FVector4f UVAdjust;
			UVAdjust.X = static_cast<float>(UVRect.Min.X);
			UVAdjust.Y = static_cast<float>(UVRect.Min.Y);
			UVAdjust.Z = static_cast<float>(UVRect.Max.X - UVRect.Min.X);
			UVAdjust.W = static_cast<float>(UVRect.Max.Y - UVRect.Min.Y);
			SetShaderValue(BatchedParameters, InUVAdjust, UVAdjust);
		}

		if (InViewProjection.IsBound())
		{
			SetShaderValue(BatchedParameters, InViewProjection, (FMatrix44f)ViewProjection);
		}

		if (InWorld.IsBound())
		{
			SetShaderValue(BatchedParameters, InWorld, (FMatrix44f)World);
		}
	}

private:
	LAYOUT_FIELD(FShaderParameter, InQuadAdjust);
	LAYOUT_FIELD(FShaderParameter, InUVAdjust);
	LAYOUT_FIELD(FShaderParameter, InViewProjection);
	LAYOUT_FIELD(FShaderParameter, InWorld);
};

class FStereoLayerPS_Base : public FGlobalShader
{
	DECLARE_TYPE_LAYOUT(FStereoLayerPS_Base, NonVirtual);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI, bool bIsOpaque)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, SamplerStateRHI, TextureRHI);

		if (InIsOpaque.IsBound())
		{
			const float OpaqueVal = bIsOpaque ? 1.0 : 0.0;
			SetShaderValue(BatchedParameters, InIsOpaque, OpaqueVal);
		}
	}

protected:
	FStereoLayerPS_Base(const ShaderMetaType::CompiledShaderInitializerType& Initializer, const TCHAR* TextureParamName) :
		FGlobalShader(Initializer) 
	{
		InTexture.Bind(Initializer.ParameterMap, TextureParamName, SPF_Mandatory);
		InTextureSampler.Bind(Initializer.ParameterMap, TEXT("InTextureSampler"));
		InIsOpaque.Bind(Initializer.ParameterMap, TEXT("InIsOpaque"));
	}
	FStereoLayerPS_Base() {}

	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
	LAYOUT_FIELD(FShaderParameter, InIsOpaque);
};

/**
 * A pixel shader for rendering a transformed textured element.
 */
class FStereoLayerPS : public FStereoLayerPS_Base
{
	DECLARE_EXPORTED_SHADER_TYPE(FStereoLayerPS,Global,ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FStereoLayerPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FStereoLayerPS_Base(Initializer, TEXT("InTexture"))	{}
	FStereoLayerPS() {}
};

/**
 * A pixel shader for rendering a transformed external texture element.
 */
class FStereoLayerPS_External : public FStereoLayerPS_Base
{
	DECLARE_EXPORTED_SHADER_TYPE(FStereoLayerPS_External, Global, ENGINE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FStereoLayerPS_External(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FStereoLayerPS_Base(Initializer, TEXT("InExternalTexture")) {}
	FStereoLayerPS_External() {}
};
