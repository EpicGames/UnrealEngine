// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
/** Different layer shapes are defined as subclass of ILayerShape.
  * Currently, only FQuadLayer is supported by all platforms, but plugins may define their own custom shapes.
  * When encountering an unsupported shape type, implementations should treat the layer as a quad layer.
  */

#define STEREO_LAYER_SHAPE_BOILERPLATE(ClassName) \
public: \
	ENGINE_API static const FName ShapeName; \
	virtual FName GetShapeName() override { return ShapeName; } \
	virtual IStereoLayerShape* Clone() const override { return new ClassName(*this); }

class IStereoLayerShape
{
public:
	/** Shape name is used to identify the shape type. */
	virtual FName GetShapeName() = 0;
	virtual IStereoLayerShape* Clone() const = 0;
	virtual ~IStereoLayerShape() = default;
};

/** Quad layer is the default layer shape and contains no additional settings */
class FQuadLayer : public IStereoLayerShape
{
	STEREO_LAYER_SHAPE_BOILERPLATE(FQuadLayer)
};

/** Class describing additional settings for cylinder layers. Currently only supported by Oculus. */
class FCylinderLayer : public IStereoLayerShape
{
	STEREO_LAYER_SHAPE_BOILERPLATE(FCylinderLayer)
public:

	FCylinderLayer() {}
	FCylinderLayer(float InRadius, float InOverlayArc, float InHeight) :
		Radius(InRadius),
		OverlayArc(InOverlayArc),
		Height(InHeight)
	{}	

	float	Radius;
	float	OverlayArc;
	float	Height;
};

/** Class describing additional settings for cube map layers. Currently only supported by Oculus. */
class FCubemapLayer : public IStereoLayerShape
{
	STEREO_LAYER_SHAPE_BOILERPLATE(FCubemapLayer)
};

/** Class describing additional settings for equirect layers. Currently only supported by Oculus. */
class FEquirectLayer : public IStereoLayerShape
{
	STEREO_LAYER_SHAPE_BOILERPLATE(FEquirectLayer)
public:

	FEquirectLayer() {}
	FEquirectLayer(FBox2D InLeftUVRect, FBox2D InRightUVRect, FVector2D InLeftScale, FVector2D InRightScale, FVector2D InLeftBias, FVector2D InRightBias, float InRadius) :
		LeftUVRect(InLeftUVRect),
		RightUVRect(InRightUVRect),
		LeftScale(InLeftScale),
		RightScale(InRightScale),
		LeftBias(InLeftBias),
		RightBias(InRightBias),
		Radius(InRadius)
	{}

	/** Left source texture UVRect, specifying portion of input texture corresponding to left eye. */
	FBox2D LeftUVRect;

	/** Right source texture UVRect, specifying portion of input texture corresponding to right eye. */
	FBox2D RightUVRect;

	/** Left eye's texture coordinate scale after mapping to 2D. */
	FVector2D LeftScale;

	/** Right eye's texture coordinate scale after mapping to 2D. */
	FVector2D RightScale;

	/** Left eye's texture coordinate bias after mapping to 2D. */
	FVector2D LeftBias;

	/** Right eye's texture coordinate bias after mapping to 2D. */
	FVector2D RightBias;

	/** Sphere radius. As of UE 5.3, equirect layers are supported only by the Oculus OpenXR runtime and only with a radius of 0 (infinite sphere).*/
	float Radius;
};
