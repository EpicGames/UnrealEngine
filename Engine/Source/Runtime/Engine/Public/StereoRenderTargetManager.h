// Copyright Epic Games, Inc.All Rights Reserved.

/*=============================================================================
StereoRenderTargetManager.h: Abstract interface returned from IStereoRendering to support rendering into a texture
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/GameViewportClient.h"
#include "PixelFormat.h"

class FRHICommandListBase;
class FRHITexture;
enum class ETextureCreateFlags : uint64;
enum EShaderPlatform : uint16;
typedef TRefCountPtr<FRHITexture> FTextureRHIRef;

/** 
 * The IStereoRenderTargetManager can be returned from IStereoRendering::GetRenderTargetManager() implementations.
 * Implement this interface if a stereo rendering device requires all output to be rendered into separate render targets and/or to customize how
 * separate render targets are allocated.
 */
class IStereoRenderTargetManager
{
public:
	/** 
	 * Whether a separate render target should be used or not.
	 * In case the stereo rendering implementation does not require special handling of separate render targets 
	 * at all, it can leave out implementing this interface completely and simply let the default implementation 
	 * of IStereoRendering::GetRenderTargetManager() return nullptr.
	 */
	virtual bool ShouldUseSeparateRenderTarget() const = 0;

	/**
	 * Calculates dimensions of the render target texture for direct rendering of distortion.
	 */
	virtual void CalculateRenderTargetSize(const class FViewport& Viewport, uint32& InOutSizeX, uint32& InOutSizeY) = 0;

	/**
	 * Returns true, if render target texture must be re-calculated.
	 */
	virtual bool NeedReAllocateViewportRenderTarget(const class FViewport& Viewport) = 0;

	/**
	* Returns true, if shading rate texture must be re-calculated.
	*/
	virtual bool NeedReAllocateShadingRateTexture(const TRefCountPtr<struct IPooledRenderTarget>& ShadingRateTarget) { return false; }

	/**
	 * Allocates the render target textures, which includes the color textures and optionally other textures like depth.
	 * The default implementation always returns false to indicate that the default texture allocation should be used instead.
	 *
	 * @return				true, if textures were allocated; false, if the default texture allocation should be used.
	 */
	virtual bool AllocateRenderTargetTextures(FRHICommandListBase& RHICmdList, uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumLayers, ETextureCreateFlags Flags, ETextureCreateFlags TargetableTextureFlags, TArray<FTextureRHIRef>& OutTargetableTextures, TArray<FTextureRHIRef>& OutShaderResourceTextures, uint32 NumSamples = 1)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return AllocateRenderTargetTextures(SizeX, SizeY, Format, NumLayers, Flags, TargetableTextureFlags, OutTargetableTextures, OutShaderResourceTextures, NumSamples);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.8, "RHI command list is now a required argument")
	virtual bool AllocateRenderTargetTextures(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumLayers, ETextureCreateFlags Flags, ETextureCreateFlags TargetableTextureFlags, TArray<FTextureRHIRef>& OutTargetableTextures, TArray<FTextureRHIRef>& OutShaderResourceTextures, uint32 NumSamples = 1) { return false; }

	/**
	 * Returns pixel format that the device created its swapchain with (which can be different than what was requested in AllocateRenderTargetTexture)
	 */
	virtual EPixelFormat GetActualColorSwapchainFormat() const { return PF_Unknown; }

	/**
	 * Acquires the next available color texture.
	 * 
	 * @return				the index of the texture in the array returned by AllocateRenderTargetTexture.
	 */
	virtual int32 AcquireColorTexture() { return -1; }
	
	/**
	 * Acquires the next available depth texture.
	 * 
	 * @return				the index of the texture in the array returned by AllocateRenderTargetTexture.
	 */
	virtual int32 AcquireDepthTexture() { return -1; }

	/**
	 * Allocates a depth texture.
	 *
	 * @param Index			(in) index of the buffer, changing from 0 to GetNumberOfBufferedFrames()
	 * @return				true, if texture was allocated; false, if the default texture allocation should be used.
	 */
	virtual bool AllocateDepthTexture(uint32 Index, uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ETextureCreateFlags TargetableTextureFlags, FTextureRHIRef& OutTargetableTexture, FTextureRHIRef& OutShaderResourceTexture, uint32 NumSamples = 1) { return false; }

	/**
	 * Allocates a shading rate texture.
	 * The default implementation always returns false to indicate that the default texture allocation should be used instead.
	 *
	 * @param Index			(in) index of the buffer, changing from 0 to GetNumberOfBufferedFrames()
	 * @return				true, if texture was allocated; false, if the default texture allocation should be used.
	 */
	virtual bool AllocateShadingRateTexture(uint32 Index, uint32 RenderSizeX, uint32 RenderSizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ETextureCreateFlags TargetableTextureFlags, FTextureRHIRef& OutTexture, FIntPoint& OutTextureSize) { return false; }

	/**
	 * Retrieves HDR information about the stereo device, if any is available.
	 * The default implementation always returns false to indicate that the information from the monitor can be used instead.
	 *
	 * @param OutDisplayOutputFormat	(out) encoding used by the stereo device
	 * @param OutDisplayColorGamut		(out) color space used by the stereo device
	 * @param OutbHDRSupported			(out) whether HDR is supported by the stereo device
	 * @return							true, if HDR information is available for the stereo device
	 */
	virtual bool HDRGetMetaDataForStereo(EDisplayOutputFormat& OutDisplayOutputFormat, EDisplayColorGamut& OutDisplayColorGamut, bool& OutbHDRSupported) { return false; }

	/**
	 * In the editor, we may switch between preview shader platforms, which support various single-pass rendering methods
	 * (e.g. InstancedStereo or MobileMultiView). Sometimes RT managers have their own state that depends on the method, so this
	 * API provides them with the possibility to reconfigure it
	 */
	virtual bool ReconfigureForShaderPlatform(EShaderPlatform NewShaderPlatform) { return false; };

	/**
	 * Gets the size of the motion vector texture recommended by the stereo device.
	 * Allocating a motion vector texture of any other size may be detrimental to performance or be unsupported by the stereo device.
	 *
	 * @param OutTextureSize			(out) The size of the motion vector texture.
	 * @return							true, if the stereo device can recommend a texture size; false, if not.
	 */
	virtual bool GetRecommendedMotionVectorTextureSize(FIntPoint& OutTextureSize) { return false; }
	
	/**
	 * Allocates a motion vector texture.
	 * The default implementation always returns false to indicate that the stereo device does not support motion vector textures.
	 *
	 * @param Size						(in) requested motion vector texture size
	 * @param Format					(in) requested motion vector texture format
	 * @param NumMips					(in) requested motion vector texture mips count
	 * @param Flags						(in) requested motion vector texture flags
	 * @param OutTexture				(out) motion vector texture resource allocated by the stereo device
	 * @param NumSamples				(in) requested motion vector texture samples count
	 * @return							true, if texture was allocated; false, if the stereo device can't provide a texture.
	 */
	virtual bool GetMotionVectorTexture(uint32 Index, const FIntPoint& Size, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, FTextureRHIRef& OutTexture, uint32 NumSamples = 1) { return false; }
	
	/**
	 * Allocates a motion vector depth texture.
	 * The default implementation always returns false to indicate that the stereo device doesn't support motion vector depth textures.
	 *
	 * @param Size						(in) requested motion vector depth texture size
	 * @param Format					(in) requested motion vector depth texture format
	 * @param NumMips					(in) requested motion vector depth texture mips count
	 * @param Flags						(in) requested motion vector depth texture flags
	 * @param OutTexture				(out) motion vector depth texture resource allocated by the stereo device
	 * @param NumSamples				(in) requested motion vector depth texture samples count
	 * @return							true, if texture was allocated; false, if the stereo device can't provide a texture.
	 */
	virtual bool GetMotionVectorDepthTexture(uint32 Index, const FIntPoint& Size, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, FTextureRHIRef& OutTexture, uint32 NumSamples = 1) { return false; }
	
	static EPixelFormat GetStereoLayerPixelFormat() { return PF_B8G8R8A8; }
};
