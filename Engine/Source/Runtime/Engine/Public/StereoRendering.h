// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StereoRendering.h: Abstract stereoscopic rendering interface
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphFwd.h"

class FSceneView;
class IStereoLayers;
class IStereoRenderTargetManager;
class FSceneViewStateInterface;

/**
* Stereoscopic rendering passes.  FULL implies stereoscopic rendering isn't enabled for this pass
* PRIMARY implies the view needs its own pass, while SECONDARY implies the view can be instanced
*/
enum class EStereoscopicPass
{
	eSSP_FULL,
	eSSP_PRIMARY,
	eSSP_SECONDARY
};

/**
* Helper enum to identify eye view indices.
*/
enum EStereoscopicEye : int32
{
	eSSE_MONOSCOPIC = INDEX_NONE,
	eSSE_LEFT_EYE = 0,
	eSSE_RIGHT_EYE = 1,
	eSSE_LEFT_EYE_SIDE = 2,
	eSSE_RIGHT_EYE_SIDE = 3,
};

class IStereoRendering
{
public:
	virtual ~IStereoRendering() = default;

	/** 
	 * Whether or not stereo rendering is on this frame.
	 */
	virtual bool IsStereoEnabled() const = 0;

	/** 
	 * Whether or not stereo rendering is on on next frame. Useful to determine if some preparation work
	 * should be done before stereo got enabled in next frame. 
	 */
	virtual bool IsStereoEnabledOnNextFrame() const { return IsStereoEnabled(); }

	/** 
	 * Switches stereo rendering on / off. Returns current state of stereo.
	 * @return Returns current state of stereo (true / false).
	 */
	virtual bool EnableStereo(bool stereo = true) = 0;

	/**
	* Returns the desired number of views, so that devices that require additional views can allocate them.
	* Default is two viewports if using stereo rendering.
	*/
	virtual int32 GetDesiredNumberOfViews(bool bStereoRequested) const { return (bStereoRequested) ? 2 : 1; }

	/**
	* For the specified view index in the view family, assign a stereoscopic pass type based on the extension's usage
	*/
	virtual EStereoscopicPass GetViewPassForIndex(bool bStereoRequested, int32 ViewIndex) const
	{
		if (!bStereoRequested)
			return EStereoscopicPass::eSSP_FULL;
		else if (ViewIndex == 0)
			return EStereoscopicPass::eSSP_PRIMARY;
		else
			return EStereoscopicPass::eSSP_SECONDARY;
	}

	/**
	* Static helper. Return true if this pass is for a stereo eye view
	*/
	static bool IsStereoEyePass(EStereoscopicPass Pass)
	{
		return !(Pass == EStereoscopicPass::eSSP_FULL);
	}

	/**
	* Static helper. Return true if this is a stereoscopic view
	*/
	static ENGINE_API bool IsStereoEyeView(const FSceneView& View);

	/**
	* Static helper. Return true if this pass is for a view we do all the work for (ie this view can't borrow from another)
	*/
	static bool IsAPrimaryPass(EStereoscopicPass Pass)
	{
		return Pass == EStereoscopicPass::eSSP_FULL || Pass == EStereoscopicPass::eSSP_PRIMARY;
	}

	/**
	* Static helper. Return true if primary view
	*/
	static ENGINE_API bool IsAPrimaryView(const FSceneView& View);

	/**
	* Static helper. Return true if this pass is for a view for which we share some work done for eSSP_PRIMARY (ie borrow some intermediate state from that view)
	*/
	static bool IsASecondaryPass(EStereoscopicPass Pass)
	{
		return Pass == EStereoscopicPass::eSSP_SECONDARY;
	}

	/**
	* Static helper. Return true if secondary view
	*/
	static ENGINE_API bool IsASecondaryView(const FSceneView& View);

	/**
	 * Return the index of the view that is used for selecting LODs
	 */
	virtual uint32 GetLODViewIndex() const
	{
		return 0;
	}

	/**
	 * True when a device has only stereo display (like a self contained mobile vr headset).
	 */
	virtual bool IsStandaloneStereoOnlyDevice() const
	{ 
		return false; 
	}

	/**
	 * Adjusts the viewport rectangle for stereo, based on which eye pass is being rendered.
	 */
	virtual void AdjustViewRect(const int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const = 0;

	/**
	* Provides the final view rect that the renderer will render into.
	*/
	virtual void SetFinalViewRect(class FRHICommandListImmediate& RHICmdList, const int32 ViewIndex, const FIntRect& FinalViewRect) {}

	/**
	 * Gets the percentage bounds of the safe region to draw in.  This allows things like stat rendering to appear within the readable portion of the stereo view.
	 * @return	The centered percentage of the view that is safe to draw readable text in
	 */
	virtual FVector2D GetTextSafeRegionBounds() const { return FVector2D(0.75f, 0.75f); }

	/**
	 * Calculates the offset for the camera position, given the specified position, rotation, and world scale
	 * Specifying eSSE_MONOSCOPIC for the view index returns a center offset behind the stereo views
	 */
	virtual void CalculateStereoViewOffset(const int32 ViewIndex, FRotator& ViewRotation, const float WorldToMeters, FVector& ViewLocation) = 0;

	/**
	 * Gets a projection matrix for the device, given the specified view index
	 * Specifying eSSE_MONOSCOPIC for the view index returns a center projection matrix encompassing all views
	 */
	virtual FMatrix GetStereoProjectionMatrix(const int32 ViewIndex) const = 0;


	/**
	* True if the device manages the ViewState for the specified view index.
	*/
	virtual bool HasExternalViewState(const int32 ViewIndex) const
	{
		return false;
	}

	/**
	* Gets a ViewState for the device, given the specified view index.
	* Note: The device may return nullptr, but it is expected to configure
	*       rendering properly (disable TemporalAA, Postprocessing, etc.).
	*/
	virtual FSceneViewStateInterface* GetExternalViewState(const int32 ViewIndex)
	{
		return nullptr;
	}

	// Renders texture into a backbuffer. Could be empty if no rendertarget texture is used, or if direct-rendering 
	// through RHI bridge is implemented. 
	UE_DEPRECATED(5.6, "Use the FRDGBuilder overload instead")
	virtual void RenderTexture_RenderThread(class FRHICommandListImmediate& RHICmdList, class FRHITexture* BackBuffer, class FRHITexture* SrcTexture, FVector2D WindowSize) const final {}
	virtual ENGINE_API void RenderTexture_RenderThread(class FRDGBuilder& GraphBuilder, FRDGTextureRef BackBuffer, FRDGTextureRef SrcTexture, FVector2f WindowSize) const;

	/**
	 * Returns currently active render target manager.
	 */
	virtual IStereoRenderTargetManager* GetRenderTargetManager() { return nullptr; }

	/**
	 * Returns an IStereoLayers implementation, if one is present
	 */
	virtual IStereoLayers* GetStereoLayers () { return nullptr; }

	
	virtual void StartFinalPostprocessSettings(struct FPostProcessSettings* StartPostProcessingSettings, const enum EStereoscopicPass StereoPassType, const int32 StereoViewIndex) {}
	virtual bool OverrideFinalPostprocessSettings(struct FPostProcessSettings* OverridePostProcessingSettings, const enum EStereoscopicPass StereoPassType, const int32 StereoViewIndex, float& BlendWeight) { return false; }
	virtual void EndFinalPostprocessSettings(struct FPostProcessSettings* FinalPostProcessingSettings, const enum EStereoscopicPass StereoPassType, const int32 StereoViewIndex) {}

	/**
	* Static helper. Return true if the Start in VR setting is set to true, or if we have the "-vr" commandline argument
	*/
	static ENGINE_API bool IsStartInVR();
};
