// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/ActorComponent.h"
#include "Misc/Guid.h"
#include "Engine/TextureStreamingTypes.h"
#include "ActorTextureStreamingBuildDataComponent.generated.h"

class UTexture;

USTRUCT()
struct FStreamableTexture
{
	GENERATED_BODY()

	FStreamableTexture() {}

	FStreamableTexture(const FString& InName, const FGuid& InGuid)
#if WITH_EDITORONLY_DATA
		: Name(InName)
		, Guid(InGuid)
#endif
	{}

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString Name;

	UPROPERTY()
	FGuid Guid;
#endif

#if WITH_EDITOR
	ENGINE_API uint32 ComputeHash() const;
#endif
};

UCLASS(MinimalAPI)
class UActorTextureStreamingBuildDataComponent : public UActorComponent, public ITextureStreamingContainer
{
	GENERATED_BODY()

public:

	//~Begin UObject Interface.
	virtual bool IsEditorOnly() const override { return true; }
	//~End UObject Interface.

#if WITH_EDITOR
	//~Begin ITextureStreamingContainer Interface.
	ENGINE_API virtual void InitializeTextureStreamingContainer(uint32 InPackedTextureStreamingQualityLevelFeatureLevel) override;
	ENGINE_API virtual uint16 RegisterStreamableTexture(UTexture* InTexture) override;
	ENGINE_API virtual bool GetStreamableTexture(uint16 InTextureIndex, FString& OutTextureName, FGuid& OutTextureGuid) const override;
	//~End ITextureStreamingContainer Interface.

	uint32 GetPackedTextureStreamingQualityLevelFeatureLevel() const { return PackedTextureStreamingQualityLevelFeatureLevel; }
	ENGINE_API uint32 ComputeHash() const;
#endif

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FStreamableTexture> StreamableTextures;

	UPROPERTY()
	uint32 PackedTextureStreamingQualityLevelFeatureLevel;
#endif
};
