// Copyright Epic Games, Inc. All Rights Reserved.
// Globally-bound delegates for level streaming status updates

#pragma once

#include "Delegates/Delegate.h"

class UWorld;
class ULevelStreaming;
class ULevel;

enum class ELevelStreamingState : uint8;
enum class ELevelStreamingTargetState : uint8;

// TODO: what is the best way to capture a level which is queued for unload while it is during async load or add to world?
//	Are level async loads actually cancelled?

struct FLevelStreamingDelegates
{
	// Called when a new target state for a streaming level is determined. 
	// Can be used to measure latency between requests and application
	// (e.g. limits on current level loads & visibility changes)
	static ENGINE_API TMulticastDelegate<void(
		UWorld*, 
		const ULevelStreaming*, 
		ULevel* LevelIfLoaded,
		ELevelStreamingState Current,
		ELevelStreamingTargetState PrevTarget, 
		ELevelStreamingTargetState NewTarget)> OnLevelStreamingTargetStateChanged;

	// Called when a streaming level enters a new state.
	// When a level is newly added to the streaming system, PreviousState and NewState may be equal.
	static ENGINE_API TMulticastDelegate<void(
		UWorld*, 
		const ULevelStreaming*,
		ULevel* LevelIfLoaded,
		ELevelStreamingState PreviousState,
		ELevelStreamingState NewState
		)> OnLevelStreamingStateChanged;

	// Notifications on when work actually beings on making a level visible or invisible, so profiling tools can measure the difference between time spent waiting to be added to the world and time spend actively working
	static ENGINE_API TMulticastDelegate<void(UWorld*, const ULevelStreaming*, ULevel* LoadedLevel)> OnLevelBeginMakingVisible;
	static ENGINE_API TMulticastDelegate<void(UWorld*, const ULevelStreaming*, ULevel* LoadedLevel)> OnLevelBeginMakingInvisible;
};
