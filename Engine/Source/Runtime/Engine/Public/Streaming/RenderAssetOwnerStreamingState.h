// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"

struct FRenderAssetOwnerStreamingState
{
	FRenderAssetOwnerStreamingState()
		: bAttachedToStreamingManagerAsStatic(0)
		, bAttachedToStreamingManagerAsDynamic(0)
		, bHandledByStreamingManagerAsDynamic(0)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, bAttachedToCoarseMeshStreamingManager(0)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		, bIgnoreStreamingManagerUpdate(0)
	{
	}

	/** Whether this primitive is referenced by a FLevelRenderAssetManager  */
	mutable uint8 bAttachedToStreamingManagerAsStatic : 1;

	/** Whether this primitive is referenced by a FDynamicRenderAssetInstanceManager */
	mutable uint8 bAttachedToStreamingManagerAsDynamic : 1;

	/** Whether this primitive is handled as dynamic, although it could have no references */
	mutable uint8 bHandledByStreamingManagerAsDynamic : 1;

	/** Whether this primitive is referenced by a Nanite::FCoarseMeshStreamingManager  */
	UE_DEPRECATED(5.8, "Coarse Mesh Streaming is deprecated.")
	mutable uint8 bAttachedToCoarseMeshStreamingManager : 1;

	/** When true, texture streaming manager won't update the component state. Used to perform early exits when updating component. */
	mutable uint8 bIgnoreStreamingManagerUpdate : 1;

	void SetAttachedToStreamingManagerAsStatic(bool bValue) const
	{
		bAttachedToStreamingManagerAsStatic = bValue;
	}

	bool IsAttachedToStreamingManagerAsStatic() const
	{
		return bAttachedToStreamingManagerAsStatic;
	}

	void SetAttachedToStreamingManagerAsDynamic(bool bValue) const
	{
		bAttachedToStreamingManagerAsDynamic = bValue;
	}

	bool IsAttachedToStreamingManagerAsDynamic() const
	{
		return bAttachedToStreamingManagerAsDynamic;
	}

	void SetHandledByStreamingManagerAsDynamic(bool bValue) const
	{
		bHandledByStreamingManagerAsDynamic = bValue;
	}

	bool IsHandledByStreamingManagerAsDynamic() const
	{
		return bHandledByStreamingManagerAsDynamic;
	}

	void SetIgnoreStreamingManagerUpdate(bool bValue) const
	{
		bIgnoreStreamingManagerUpdate = bValue;
	}

	bool IsIgnoreStreamingManagerUpdate() const
	{
		return bIgnoreStreamingManagerUpdate;
	}

	UE_DEPRECATED(5.8, "Coarse Mesh Streaming is deprecated.")
	void SetAttachedToCoarseMeshStreamingManager(bool bValue) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bAttachedToCoarseMeshStreamingManager = bValue;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.8, "Coarse Mesh Streaming is deprecated.")
	bool IsAttachedToCoarseMeshStreamingManager() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bAttachedToCoarseMeshStreamingManager;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Whether this primitive is referenced by the streaming manager and should sent callbacks when detached or destroyed */
	bool IsAttachedToStreamingManager() const
	{
		return IsAttachedToStreamingManagerAsStatic() || IsAttachedToStreamingManagerAsDynamic();
	}
};
