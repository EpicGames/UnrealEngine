// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "ServerStreamingLevelsVisibility.generated.h"

class ULevelStreaming;

/**
 * Actor used to replicate server's visible level streaming
 */
UCLASS(notplaceable, transient, MinimalAPI)
class AServerStreamingLevelsVisibility : public AActor
{
	GENERATED_UCLASS_BODY()

public:
	static ENGINE_API AServerStreamingLevelsVisibility* SpawnServerActor(UWorld* World);
	ENGINE_API bool Contains(const FName& InPackageName) const;
	ENGINE_API void SetIsVisible(ULevelStreaming* InStreamingLevel, bool bInIsVisible);
	ENGINE_API ULevelStreaming* GetVisibleStreamingLevel(const FName& InPackageName) const;
private:
	TMap<FName, TWeakObjectPtr<ULevelStreaming>> ServerVisibleStreamingLevels;
};
