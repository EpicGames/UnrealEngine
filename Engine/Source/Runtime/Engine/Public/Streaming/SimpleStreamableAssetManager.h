// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Components/PrimitiveComponent.h"
#include "Misc/SpinLock.h"
#include "Engine/StreamableRenderAsset.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/TextureDefines.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Streaming/SimpleStreamableAssetManagerHandle.h"

class FPrimitiveSceneProxy;
class IPrimitiveComponent;

struct FStreamingRenderAssetPrimitiveInfo;
struct FBoundsViewInfo;
struct FStreamingViewInfo;
struct FStreamingViewInfoExtra;
struct FRenderAssetStreamingSettings;
struct FBounds4;

#define SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER 0

template<typename T>
class TLocklessGrowingStorage
{
public:

	void Push(T&& In)
	{
		Queue.ProduceItem(MoveTemp(In));
		ItemCount.fetch_add(1, std::memory_order_relaxed);
	}

	void ExtractAll(TArray<T>& Out)
	{
		const int32 PreviousNum = Out.Num();
		Queue.ConsumeAllFifo([&Out](T Item)
		{
			Out.Add(MoveTemp(Item));
		});
		// Subtract only what we drained so concurrent producers' increments stand. ItemCount
		// is informational (memory stats) and still races with live pushes by construction.
		const int32 DrainedCount = Out.Num() - PreviousNum;
		ItemCount.fetch_sub(DrainedCount, std::memory_order_relaxed);
	}

	uint32 GetAllocatedSize() const
	{
		// TConsumeAllMpmcQueue allocates a private FNode (next-pointer + storage for T)
		// per outstanding item. Approximation: treat each node as a next-ptr + the item.
		// ItemCount is a relaxed counter racing with live producers/consumers and can briefly
		// observe negative skew (ExtractAll's fetch_sub completing before a concurrent Push's
		// fetch_add). Clamp at zero so the unsigned multiplication does not produce an
		// astronomical reported size in memory stats.
		constexpr uint32 PerItemBytes = sizeof(T) + sizeof(void*);
		const int32 ClampedItemCount = FMath::Max<int32>(0, ItemCount.load(std::memory_order_relaxed));
		return static_cast<uint32>(ClampedItemCount) * PerItemBytes;
	}

private:
	UE::TConsumeAllMpmcQueue<T> Queue;
	std::atomic<int32> ItemCount{0};
};

/**
 * FSimpleStreamableAssetManager (SSAM) -- EXPERIMENTAL.
 *
 * SceneProxy-based streaming asset manager. When enabled, replaces the legacy streaming-manager
 * integration's per-UPrimitiveComponent path with a proxy-driven flow that supports both
 * UPrimitiveComponent-backed and proxy-only primitives.
 *
 * Disabled by default (s.StreamableAssets.UseSimpleStreamableAssetManager = 0). API, data
 * structures, and runtime behavior may change without notice; do not build production code or
 * shipping licensee integrations against SSAM until it reaches stable status.
 *
 * Required by FastGeo: FastGeo's async render state creation produces proxy-only primitives
 * (no UPrimitiveComponent). The legacy streaming path keys off UPrimitiveComponent, so
 * FastGeo content is invisible to it. SSAM must be enabled for FastGeo content to participate
 * in mip / LOD streaming.
 */
class FSimpleStreamableAssetManager
{
	template<typename TagT> friend struct TSSAMHandle;  // for TSSAMHandle::IsStale

public:

	struct FScopedLock
	{
		FScopedLock(FCriticalSection* InCriticalSection, bool bInShouldLock)
			: CriticalSection(InCriticalSection)
			// GetCriticalSection() returns nullptr while Instance is null (pre-Init / post-Shutdown);
			// fold the null check into bShouldLock so Lock/Unlock are only issued on a live CS.
			, bShouldLock(bInShouldLock && InCriticalSection != nullptr)
		{
			if (bShouldLock)
			{
				CriticalSection->Lock();
			}
		}
		~FScopedLock()
		{
			if (bShouldLock)
			{
				CriticalSection->Unlock();
			}
		}
	private:
		FCriticalSection* CriticalSection = nullptr;
		bool bShouldLock = false;
	};
	
private:
	// Record types consumed by the internal PushXxxRecord primitives. Kept private because no
	// public API takes them: external callers go through Register / Unregister / UpdateStreamingState /
	// UpdateLastRenderTime, which construct these internally.

	struct FUnregister
	{
		FSSAMPrimitiveHandle PrimitiveHandle{};

#if SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER
		// Diagnostic key only -- treat as an opaque numeric identifier, NOT a live pointer.
		// The proxy/component this points to may have been destroyed.
		union
		{
			UPTRINT ObjectKey = 0;
			const FPrimitiveSceneProxy* DebugSceneProxy;
			const UPrimitiveComponent*  DebugPrimitiveComponent;
		};
#endif

		FUnregister() = default;
		FUnregister(const FUnregister&) = default;
		FUnregister& operator=(const FUnregister&) = default;
		FUnregister(FUnregister&&) = default;
		FUnregister& operator=(FUnregister&&) = default;

		FUnregister(FSSAMPrimitiveHandle InPrimitiveHandle, const void* InDebugObject = nullptr)
			: PrimitiveHandle(InPrimitiveHandle)
#if SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER
			, ObjectKey(reinterpret_cast<UPTRINT>(InDebugObject))
#endif
		{}
	};

	struct FUpdateLastRenderTime
	{
		FSSAMPrimitiveHandle PrimitiveHandle{};
#if SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER
		union
		{
			UPTRINT ObjectKey = 0;
			const FPrimitiveSceneProxy* DebugSceneProxy;
			const UPrimitiveComponent*  DebugPrimitiveComponent;
		};
#endif
		float LastRenderedTime = -1000.0f;
		FUpdateLastRenderTime() = default;
		FUpdateLastRenderTime(const FUpdateLastRenderTime&) = default;
		FUpdateLastRenderTime& operator=(const FUpdateLastRenderTime&) = default;
		FUpdateLastRenderTime(FUpdateLastRenderTime&&) = default;
		FUpdateLastRenderTime& operator=(FUpdateLastRenderTime&&) = default;

		template<typename T>
		FUpdateLastRenderTime(const T* InObject, const float InLastRenderedTime)
				: PrimitiveHandle(InObject->GetSSAMPrimitiveHandle())
#if SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER
				, ObjectKey(reinterpret_cast<UPTRINT>(InObject))
#endif
				, LastRenderedTime(InLastRenderedTime)
		{}
	};

	struct FUpdate
	{
		FSSAMPrimitiveHandle PrimitiveHandle{};
		FBoxSphereBounds ObjectBounds{};
#if SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER
		union
		{
			UPTRINT ObjectKey = 0;
			const FPrimitiveSceneProxy* DebugSceneProxy;
			const UPrimitiveComponent*  DebugPrimitiveComponent;
		};
#endif
		float StreamingScaleFactor = 1.f;
		float MinDistance = 0.0f;
		float MaxDistance = FLT_MAX;
		float LastRenderedTime = -1000.0f;
		uint8 bForceMipStreaming : 1 = false;

		FUpdate() = default;
		FUpdate(const FUpdate&) = default;
		FUpdate& operator=(const FUpdate&) = default;
		FUpdate(FUpdate&&) = default;
		FUpdate& operator=(FUpdate&&) = default;

		template<typename T>
		FUpdate(
			const T* InObject,
			const FBoxSphereBounds& InBounds,
			float InStreamingScaleFactor,
			const float InMinDistance, const float InMaxDistance, const float InLastRenderedTime, bool InForceMipStreaming)
				: PrimitiveHandle(InObject->GetSSAMPrimitiveHandle())
				, ObjectBounds(InBounds)
#if SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER
				, ObjectKey(reinterpret_cast<UPTRINT>(InObject))
#endif
				, StreamingScaleFactor(InStreamingScaleFactor)
				, MinDistance(InMinDistance)
				, MaxDistance(InMaxDistance)
				, LastRenderedTime(InLastRenderedTime)
				, bForceMipStreaming(InForceMipStreaming)
		{}
	};

	struct FRegisteredAsset
	{
		FSSAMAssetHandle AssetHandle{};
		FStreamingRenderAssetPrimitiveInfo AssetPrimitiveInfo;
	};

	struct FRegister : public FUpdate
	{
		TArray<FRegisteredAsset> RegisteredAssets{};

		FRegister() = default;
		FRegister(const FRegister&) = default;
		FRegister& operator=(const FRegister&) = default;
		FRegister(FRegister&&) = default;
		FRegister& operator=(FRegister&&) = default;
		FRegister(const FPrimitiveSceneProxy*, const FMatrix& InRenderMatrix, const FBoxSphereBounds& InBounds);
		FRegister(const UPrimitiveComponent*, const TArray<FStreamingRenderAssetPrimitiveInfo>& Assets);
	};

	struct FAssetBoundElement
	{
		int32 ObjectRegistrationIndex = INDEX_NONE;
		float TexelFactor = 0.0f;
		uint32 bForceLOD : 1 = 0;
		// Mirrors FStreamingRenderAssetPrimitiveInfo::bAffectedByComponentScale.
		// Texture queries multiply TexelFactor by ComponentScale only when this is set;
		// mesh entries (which encode physical world size in TexelFactor) clear it.
		uint32 bAffectedByComponentScale : 1 = 1;
	};

	struct FAssetRecord
	{
		// Identity-stable across slot recycling: the handle's generation makes "this exact
		// registration" distinguishable from "the slot was recycled to a different asset",
		// which a cached AssetRegistrationIndex (recycled) could not express. Consumers go
		// through GetAssetRegistrationIndex() to resolve the live index, which returns
		// INDEX_NONE for stale handles and prevents wrong-asset access.
		FSSAMAssetHandle AssetHandle{};
		int32 AssetElementIndex = INDEX_NONE;
#if SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER
		// Diagnostic key only -- treat as an opaque numeric identifier, NOT a live pointer.
		// The asset this points to may have been destroyed before the record is drained.
		UPTRINT DebugStreamableRenderAsset = 0;
#endif
		friend bool operator==(const FAssetRecord& A, const FAssetRecord& B) { return A.AssetHandle == B.AssetHandle; }
		friend uint32 GetTypeHash(const FAssetRecord& Object) { return GetTypeHash(Object.AssetHandle); }
	};

	struct FRemovedAssetRecord
	{
		FSSAMAssetHandle AssetHandle{};
		int32 AssetElementIndex = INDEX_NONE;
#if SIMPLE_STREAMABLE_ASSET_MANAGER_ALLOW_DEBUG_POINTER
		// Diagnostic key only -- treat as an opaque numeric identifier, NOT a live pointer.
		// The asset this points to may have been destroyed before the record is drained.
		UPTRINT DebugStreamableRenderAsset = 0;
#endif
		friend bool operator==(const FRemovedAssetRecord& A, const FRemovedAssetRecord& B) { return A.AssetHandle == B.AssetHandle; }
	};

	struct FObjectBoundsRecord
	{
		int32 BoundsIndex = INDEX_NONE;
	};

	template <typename T>
	struct TSimpleSparseArray
	{
		int32 Add(T&& InElement)
		{
			if (UsedElementsCount == UsedElements.Num())
			{
				UsedElements.Add(false, GSimpleStreamableAssetManagerSparseArrayGrowSize);
				Elements.AddDefaulted(GSimpleStreamableAssetManagerSparseArrayGrowSize);
			}
			const int32 Index = UsedElements.FindAndSetFirstZeroBit(FreeElementIndexHint);
			check(Index != INDEX_NONE);
			FreeElementIndexHint = Index + 1;
			++UsedElementsCount;
			Elements[Index] = MoveTemp(InElement);
			return Index;
		}
		
		void Reset(int32 Index)
		{
			if (UsedElements.Num() > Index && UsedElements[Index])
			{
				UsedElements[Index] = false;
				FreeElementIndexHint = FMath::Min(FreeElementIndexHint, Index);
				Elements[Index] = T{};
				--UsedElementsCount;
			}
		}
		
		void Empty()
		{
			FreeElementIndexHint = 0;
			UsedElementsCount = 0;
			UsedElements.Empty();
			Elements.Empty();
		}

		int32 Num() const { return UsedElementsCount; }

		SIZE_T GetAllocatedSize(void) const
		{
			return sizeof(TSimpleSparseArray)
				+ UsedElements.GetAllocatedSize()
				+ Elements.GetAllocatedSize();
		}

		// Returns a contiguous view over [0..LastUsed+1) covering every set bit in UsedElements
		// up to the highest, or an empty view when no bit is set (FindLast returns INDEX_NONE,
		// so the view length is 0). The view INCLUDES holes at unused indices in between --
		// consumers must filter via the per-element validity check (typically
		// ObjectRegistrationIndex == INDEX_NONE) before using an entry. When the array is
		// sparse the per-iteration cost still scales with the prefix length rather than the
		// used-element count; a future tightening could iterate UsedElements directly via
		// TConstSetBitIterator on the screen-size hot path. Cheap when UsedElementsCount is
		// close to LastUsed+1.
		TConstArrayView<T> GetSparseView() const
		{
			const int32 Max = UsedElements.FindLast(true);
			return TConstArrayView<T>{ Elements.GetData(), Max + 1};
		}
		private:
		int32 FreeElementIndexHint = 0;
		int32 UsedElementsCount = 0;
		TBitArray<> UsedElements;
		TArray<T> Elements;
	};

	static FAutoConsoleVariableRef CVarUseSimpleStreamableAssetManager;
	ENGINE_API static int32 GUseSimpleStreamableAssetManager;

	static FAutoConsoleVariableRef CVarUseSimpleStreamableAssetManagerSparseArrayGrowSize;
	ENGINE_API static int32 GSimpleStreamableAssetManagerSparseArrayGrowSize;

	static FAutoConsoleVariableRef CVarUseSimpleStreamableAssetManagerEnsureAssetUniqueOnRegistration;
	ENGINE_API static int32 GSimpleStreamableAssetManagerEnsureAssetUniqueOnRegistration;

	static FAutoConsoleVariableRef CVarSimpleStreamableAssetManagerConsiderVisibility;
	ENGINE_API static int32 GSimpleStreamableAssetManagerConsiderVisibility;
	
	static FSimpleStreamableAssetManager* Instance;
	FCriticalSection CriticalSection;

	static FDelegateHandle PostGCDelegateHandle;

	// Static (process-lifetime): UStreamableRenderAsset CDOs allocate handles during
	// ProcessNewlyLoadedUObjects, which runs before Init() / before Instance exists.
	static ENGINE_API FSSAMAssetHandleTable AssetHandleTable;

	// Instance-owned (unlike the static AssetHandleTable): primitive handles only exist during
	// scene-proxy registration, which requires a live world -- always after Init().
	FSSAMPrimitiveHandleTable PrimitiveHandleTable;

	/** Primitive handle table operations. External callers go through Register /
	 *  Unregister; these are for SSAM-internal paths only. */
	FSSAMPrimitiveHandle AllocatePrimitiveHandle()                                               { return PrimitiveHandleTable.Allocate(); }
	int32                GetPrimitiveRegistrationIndex(FSSAMPrimitiveHandle Handle) const        { return PrimitiveHandleTable.GetIndex(Handle); }
	void                 SetPrimitiveRegistrationIndex(FSSAMPrimitiveHandle Handle, int32 Index) { PrimitiveHandleTable.SetIndex(Handle, Index); }
	int32                ReleasePrimitiveHandle(FSSAMPrimitiveHandle Handle)                     { return PrimitiveHandleTable.Release(Handle); }
	bool                 IsPrimitiveHandleLive(FSSAMPrimitiveHandle Handle) const                { return PrimitiveHandleTable.IsValid(Handle); }

	/** Primitive-handle field sync between proxy and component. Private because only SSAM's own
	 *  Register / Unregister drive this -- external code never directly stamps or
	 *  clears the handle field on a proxy/component. */
	static void AssignPrimitiveHandle(FPrimitiveSceneProxy* Proxy, UPrimitiveComponent* Component, FSSAMPrimitiveHandle PrimitiveHandle);
	static FSSAMPrimitiveHandle ClearPrimitiveHandle(FPrimitiveSceneProxy* Proxy, UPrimitiveComponent* Component);

	// Queued entries awaiting GT-side asset gather. The handle is captured at push time so the
	// drain can distinguish "still the active registration" from "superseded by a later
	// RecreateRenderState" by comparing against the component's current handle.
	struct FFallbackEntry
	{
		TWeakObjectPtr<UPrimitiveComponent> Component;
		FSSAMPrimitiveHandle                Handle;
	};
	TLocklessGrowingStorage<FFallbackEntry> FallbackComponentRegistration;

	TLocklessGrowingStorage<FRemovedAssetRecord> RemovedAssetsRecords;
	TLocklessGrowingStorage<FRegister> RegisterRecords;
	TLocklessGrowingStorage<FUnregister> UnregisterRecords;
	TLocklessGrowingStorage<FUpdate> UpdateRecords;
	TLocklessGrowingStorage<FUpdateLastRenderTime> UpdateLastRenderTimeRecords;

	TArray<FUpdate> Pending_UpdateRecords;
	TArray<FUpdateLastRenderTime> Pending_UpdateLastRenderTimeRecords;
	TArray<FRegister> Pending_RegisterRecords;
	TArray<FUnregister> Pending_UnregisterRecords;
	TArray<FRemovedAssetRecord> Pending_RemovedAssetRecords;

	// ** Variables to manage Object registration ** //
	int32 RegisteredObjectCount = 0;
	int32 MaxObjects = 0;

	int32 FreeObjectIndexHint = 0;
	TBitArray<> ObjectUsedIndices;
	TArray<TArray<FAssetRecord>> ObjectRegistrationIndexToAssetProperty;
	TArray<FBounds4> ObjectBounds4;

	void RegisterRecord(FRegister& Record);
	void UpdateRecord(FUpdate& Record);
	void UpdateRecord(FUpdateLastRenderTime& Record);
	void UnregisterRecord(FUnregister& Record);

	// Low-level record-queue primitives. Direct callers bypass the fallback-queueing logic in
	// Register() and the handle-clearing logic in Unregister() -- internal SSAM use only.
	static void PushRegisterRecord(FRegister&& Record);
	static void PushUnregisterRecord(FUnregister&& Record);
	static void PushUpdateRecord(FUpdate&& Record);
	static void PushUpdateLastRenderTimeRecord(FUpdateLastRenderTime&& Record);

	// ** Variables to manage Bounds registration ** //
	void SetBounds(int32 BoundsIndex, const FBoxSphereBounds& Bounds, uint32 PackedRelativeBox, float StreamingScaleFactor, float LastRenderTime, const FVector4& RangeOrigin, float MinDistanceSq, float MinRangeSq, float MaxRangeSq);
	FBoxSphereBounds GetBounds(int32 BoundsIndex) const;

	// ** Variables to manage Assets registration ** //
	int32 FreeAssetIndexHint = 0;
	int32 UsedAssetIndices = 0;
	TBitArray<> AssetUsedIndices;
	TArray<TSimpleSparseArray<FAssetBoundElement>> AssetIndexToBounds4Index;
	// Parallel to AssetIndexToBounds4Index. Stores the asset's handle
	// so SSAM can invalidate the asset registration when all objects are removed,
	// allowing the index slot to be reclaimed.
	TArray<FSSAMAssetHandle> AssetIndexToHandle;
	void AddRenderAssetElements(const TArrayView<FRegisteredAsset>& RegisteredAssets, int32 ObjectRegistrationIndex, bool bForceMipStreaming);
	void RemoveRenderAssetElements(int32 ObjectRegistrationIndex);

	// Used to update last render time
	void TrySetLastRenderTime(int32 BoundsIndex, float LastRenderTime);

	// ** Background task data** //
	TArray<FBoundsViewInfo> BoundsViewInfos;
	
	void GetRenderAssetScreenSize_Impl(
		EStreamableRenderAssetType AssetType,
		const int32 InAssetIndex,
		float& MaxSize,
		float& MaxSize_VisibleOnly,
		int32& MaxNumForcedLODs,
		const float MaxAssetSize,
		const int32 MaxAllowedMip,
		const TCHAR* LogPrefix) const;

	void UpdateBoundSizes_Impl(
		const TArray<FStreamingViewInfo>& ViewInfos,
		const TArray<FStreamingViewInfoExtra, TInlineAllocator<4>>& ViewInfoExtras,
		float LastUpdateTime,
		const FRenderAssetStreamingSettings& Settings);

	static void GetDistanceAndRange(
		const FUpdate& Record,
		float& MinDistanceSq, float& MinRangeSq, float& MaxRangeSq);

	void UpdateTask_Async();
	
	void GetAssetReferenceBounds_Impl(const UStreamableRenderAsset* Asset, TArray<FBox>& AssetBoxes) const;
	uint32 GetAllocatedSize_Impl() const;

	static void EnqueueOrReleaseCapturedHandle(FSSAMAssetHandle CapturedHandle, UStreamableRenderAsset* DebugAsset);
public:
	static FCriticalSection* GetCriticalSection() { return Instance ? &Instance->CriticalSection : nullptr; }

	/** Asset handle lifecycle. Callable from any thread, including before Init and after Shutdown.
	 *  The asset table is a process-lifetime singleton so there is no "no instance" failure mode. */
	static ENGINE_API FSSAMAssetHandle AllocateAssetHandle();
	static ENGINE_API int32            ReleaseAssetHandle(FSSAMAssetHandle Handle);

	/** Queue a captured asset handle for release through the standard RemovedAssetsRecords drain.
	 *  Use this instead of ReleaseAssetHandle when the caller owns a handle that may have an
	 *  active AssetIndex with bound elements -- the drain is what clears AssetUsedIndices /
	 *  AssetIndexToBounds4Index / AssetIndexToHandle. Releasing such a handle directly orphans
	 *  the FAssetBoundElement rows and permanently pins the AssetIndex slot.
	 *  Falls back to direct release when Instance is null (no drain to process the record). */
	static ENGINE_API void QueueRegisteredAssetHandleRelease(FSSAMAssetHandle Handle);

	/** Cheap predicate for "is this asset tracked by SSAM". Hides the INDEX_NONE sentinel from
	 *  external callers; returns false for default-constructed or stale handles. */
	static ENGINE_API bool IsAssetRegistered(FSSAMAssetHandle Handle);

	/** Asset registration index accessors. Module-internal (no ENGINE_API export): the raw index
	 *  is an implementation detail of SSAM's arrays. Tests in Engine/Private/Tests call these
	 *  directly; other modules should use the handle-taking operations above. */
	static int32 GetAssetRegistrationIndex(FSSAMAssetHandle Handle);
	static void  SetAssetRegistrationIndex(FSSAMAssetHandle Handle, int32 NewIndex);

	/** Primary registration entry point. Safe to call from GT, ParallelGT, and from true worker
	 *  threads (FastGeo). True-worker callers must guarantee that every UStreamableRenderAsset
	 *  reached via FPrimitiveSceneProxy::GetStreamableRenderAssetInfo remains GC-reachable for
	 *  the full lifetime of this call -- see the block comment at FRegister::FRegister in the .cpp. */
	ENGINE_API static void Register(IPrimitiveComponent* ComponentInterface, FPrimitiveSceneProxy* SceneProxy, const FMatrix& RenderMatrix, const FBoxSphereBounds& WorldBounds);
	ENGINE_API static void Unregister(IPrimitiveComponent* ComponentInterface, FPrimitiveSceneProxy* SceneProxy);

	/** Refresh the streaming-relevant state for a registered primitive: bounds, streaming scale
	 *  factor (derived from LocalToWorld), MinDrawDistance, MaxDrawDistance, on-screen
	 *  LastRenderTime, and ForceMipStreaming. Callers only supply transform + bounds; everything
	 *  else is pulled from the proxy. No-op if SSAM is disabled or the proxy isn't registered.
     *
     *  Does NOT re-walk GetStreamableRenderAssetInfo -- the registered asset set is fixed at
     *  Register time. If the proxy's asset set has changed (different materials, swapped meshes,
     *  etc.), the caller must Unregister and Register again; UpdateStreamingState alone will
     *  leave SSAM tracking the original asset set. The normal proxy-recreation path
     *  (RecreateRenderState_Concurrent) handles this implicitly via Unregister + Register. */
	ENGINE_API static void UpdateStreamingState(const FPrimitiveSceneProxy* Proxy, const FMatrix& LocalToWorld, const FBoxSphereBounds& WorldBounds);

	/** Last-rendered-time update. No-op if SSAM is disabled, visibility tracking is off, or the
	 *  proxy isn't registered. */
	ENGINE_API static void UpdateLastRenderTime(const FPrimitiveSceneProxy* Proxy, float CurrentWorldTime);

	/** Recovery path for component / proxy teardown that discovers a still-live SSAM handle
	 *  (a missed Unregister). The caller has already cleared its handle field; this queues an
	 *  unregister so the streaming arrays drop their references and the slot is reclaimed. */
	ENGINE_API static void RecoverMissedUnregister(FSSAMPrimitiveHandle Handle);

	static bool IsEnabled() { return GUseSimpleStreamableAssetManager != 0; }
	static bool ShouldConsiderVisibility() { return GSimpleStreamableAssetManagerConsiderVisibility != 0; }
	static void Init();
	static void Shutdown();
	static void Process();
	static void ProcessFallbackRegistrations_GameThread();

	static void UnregisterAsset(UStreamableRenderAsset* InAsset);

	static uint32 GetAllocatedSize();

	static void UpdateBoundSizes(
		const TArray<FStreamingViewInfo>& ViewInfos,
		const TArray<FStreamingViewInfoExtra, TInlineAllocator<4>>& ViewInfoExtras,
		float LastUpdateTime,
		const FRenderAssetStreamingSettings& Settings);

	/** Screen-size query for a tracked asset. Takes a handle so callers never touch the internal
	 *  registration index. No-op (leaves out-params unchanged) when the handle is invalid, stale,
	 *  or unregistered. */
	static void GetRenderAssetScreenSize(
		EStreamableRenderAssetType AssetType,
		FSSAMAssetHandle Handle,
		float& MaxSize,
		float& MaxSize_VisibleOnly,
		int32& MaxNumForcedLODs,
		const float MaxAssetSize,
		const int32 MaxAllowedMip,
		const TCHAR* LogPrefix);

	static void GetAssetReferenceBounds(const UStreamableRenderAsset* Asset, TArray<FBox>& AssetBoxes);

	ENGINE_API static float GetStreamingScaleFactor(const FPrimitiveSceneProxy* Object, const FMatrix& LocalToWorld);
};

// IsStale specializations -- access the private tables via the friend declaration above.

template<>
inline bool TSSAMHandle<FSSAMAssetHandleTag>::IsStale() const
{
	return IsValid() && !FSimpleStreamableAssetManager::AssetHandleTable.IsValid(*this);
}

template<>
inline bool TSSAMHandle<FSSAMPrimitiveHandleTag>::IsStale() const
{
	if (!IsValid())
	{
		return false;
	}
	// No Instance -> no table -> the slot this handle references cannot be live.
	if (!FSimpleStreamableAssetManager::Instance)
	{
		return true;
	}
	return !FSimpleStreamableAssetManager::Instance->PrimitiveHandleTable.IsValid(*this);
}
