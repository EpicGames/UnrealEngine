// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Containers/LockFreeList.h"
#include "AutoRTFM/ReturnFromOpenMode.h"
#include <atomic>

/** Phantom tags to distinguish asset vs primitive handles at compile time. */
struct FSSAMAssetHandleTag {};
struct FSSAMPrimitiveHandleTag {};

/** Opaque handle into an SSAM handle table. 8 bytes, POD, trivially copyable.
 *  Separate tag types for asset vs primitive prevent accidental cross-table use. */
template<typename TagT>
struct TSSAMHandle
{
	static constexpr uint32 INVALID_SLOT = ~uint32(0);

	uint32 SlotIndex  = INVALID_SLOT;
	uint32 Generation = 0;

	bool IsValid() const { return SlotIndex != INVALID_SLOT; }
	/** True when the handle's slot was released since the handle was minted (generation mismatch).
	 *  Returns false when !IsValid(). */
	bool IsStale() const;
	/** Reset to default. */
	void Reset() { *this = TSSAMHandle{}; }
	bool operator==(const TSSAMHandle&) const = default;
	bool operator!=(const TSSAMHandle&) const = default;

#if UE_AUTORTFM
	/** Opt-in for AutoRTFM::Open() return values: the handle is two trivially-copyable uint32s,
	 *  safe to memcpy across the open -> closed boundary. Lets BeginDestroy capture the live
	 *  handle in the open (where the atomic Exchange must run) and forward it to a closed-side
	 *  AutoRTFM::OnCommit (which would run immediately if registered from within the open nest). */
	static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode =
		AutoRTFM::EReturnFromOpenMode::CopyConstructInClosed;
#endif

	friend uint32 GetTypeHash(const TSSAMHandle& Handle)
	{
		return HashCombineFast(::GetTypeHash(Handle.SlotIndex), ::GetTypeHash(Handle.Generation));
	}
};

using FSSAMAssetHandle     = TSSAMHandle<FSSAMAssetHandleTag>;
using FSSAMPrimitiveHandle = TSSAMHandle<FSSAMPrimitiveHandleTag>;

static_assert(sizeof(FSSAMAssetHandle)     == 8, "FSSAMAssetHandle is expected to be 8 bytes");
static_assert(sizeof(FSSAMPrimitiveHandle) == 8, "FSSAMPrimitiveHandle is expected to be 8 bytes");

/** Storage wrapper for an SSAM handle field that may be mutated post-publication and observed
 *  concurrently from other threads. Packs the 8-byte handle into a std::atomic<uint64> so
 *  loads/stores are guaranteed lock-free hardware-atomic on every UE-supported platform --
 *  no torn reads, no C++ data-race UB. The handle table's generation check still provides the
 *  safety net for stale/recycled-slot scenarios; this wrapper just ensures the read of the
 *  handle itself is well-defined under the C++ memory model.
 *
 *  Value semantics: implicit conversion to/from the underlying handle, plus the same
 *  IsValid / IsStale / Reset surface as TSSAMHandle, so call sites that touched the plain
 *  handle continue to compile unchanged. */
template<typename TagT>
struct alignas(8) TSSAMAtomicHandle
{
public:
	using HandleT = TSSAMHandle<TagT>;

	TSSAMAtomicHandle() = default;

	// A handle holder owns a single SSAM slot. Two object instances must never share it: a
	// Release on either copy would silently invalidate the other. To enforce that:
	//  - Copy default-initializes the destination. The source is unchanged. Cloning a live
	//    handle is impossible; the duplicate behaves as a fresh, unregistered holder. This
	//    keeps containing classes (FPrimitiveSceneProxy, derived FDebugRenderSceneProxy and
	//    its 35+ subclasses) backward-compatible -- they can keep their `=default` copy ctors
	//    and the wrapper just no-ops the slot field on copy.
	//  - Move transfers the slot. Source becomes empty, destination receives the handle.
	//    Single-owner invariant preserved across the FPrimitiveSceneProxy move chain.
	// UObject duplication (StaticDuplicateObject) uses serialization, not C++ copy, so the
	// non-UPROPERTY handle field is naturally default-initialized on the duplicate either way.
	TSSAMAtomicHandle(const TSSAMAtomicHandle& /*Other*/) noexcept
	{
		Storage.store(DefaultStorage, std::memory_order_relaxed);
	}
	TSSAMAtomicHandle& operator=(const TSSAMAtomicHandle& /*Other*/) noexcept
	{
		Storage.store(DefaultStorage, std::memory_order_relaxed);
		return *this;
	}
	TSSAMAtomicHandle(TSSAMAtomicHandle&& Other) noexcept
	{
		const uint64 Bits = Other.Storage.exchange(DefaultStorage, std::memory_order_relaxed);
		Storage.store(Bits, std::memory_order_relaxed);
	}
	TSSAMAtomicHandle& operator=(TSSAMAtomicHandle&& Other) noexcept
	{
		const uint64 Bits = Other.Storage.exchange(DefaultStorage, std::memory_order_relaxed);
		Storage.store(Bits, std::memory_order_relaxed);
		return *this;
	}

	HandleT Load(std::memory_order Order = std::memory_order_acquire) const
	{
		const uint64 Bits = Storage.load(Order);
		return HandleT{ static_cast<uint32>(Bits & 0xFFFFFFFFu), static_cast<uint32>(Bits >> 32) };
	}
	void Store(HandleT Handle, std::memory_order Order = std::memory_order_release)
	{
		const uint64 Bits = static_cast<uint64>(Handle.SlotIndex)
		                  | (static_cast<uint64>(Handle.Generation) << 32);
		Storage.store(Bits, Order);
	}
	HandleT Exchange(HandleT Desired, std::memory_order Order = std::memory_order_acq_rel)
	{
		const uint64 Bits = static_cast<uint64>(Desired.SlotIndex)
		                  | (static_cast<uint64>(Desired.Generation) << 32);
		const uint64 Old = Storage.exchange(Bits, Order);
		return HandleT{ static_cast<uint32>(Old & 0xFFFFFFFFu), static_cast<uint32>(Old >> 32) };
	}

	operator HandleT() const                    { return Load(); }
	TSSAMAtomicHandle& operator=(HandleT Handle) { Store(Handle); return *this; }
	bool operator==(HandleT Other) const        { return Load() == Other; }
	bool operator!=(HandleT Other) const        { return Load() != Other; }

	bool IsValid() const { return Load().IsValid(); }
	bool IsStale() const { return Load().IsStale(); }
	void Reset()         { Store(HandleT{}); }

private:
	// Packed: low 32 bits = SlotIndex, high 32 bits = Generation. Default-constructed value
	// matches a default-constructed TSSAMHandle (INVALID_SLOT, generation 0).
	static constexpr uint64 DefaultStorage = static_cast<uint64>(HandleT::INVALID_SLOT);
	std::atomic<uint64> Storage{ DefaultStorage };
};

using FSSAMAtomicAssetHandle     = TSSAMAtomicHandle<FSSAMAssetHandleTag>;
using FSSAMAtomicPrimitiveHandle = TSSAMAtomicHandle<FSSAMPrimitiveHandleTag>;

static_assert(sizeof(FSSAMAtomicPrimitiveHandle) == 8, "FSSAMAtomicPrimitiveHandle is expected to be 8 bytes");
static_assert(sizeof(FSSAMAtomicAssetHandle)     == 8, "FSSAMAtomicAssetHandle is expected to be 8 bytes");

/** A single slot in an SSAM handle table. RegistrationIndex is written by SSAM when the
 *  registration is processed; Generation is bumped on Release to invalidate stale handles. */
struct alignas(8) FSSAMSlot
{
	std::atomic<int32>  RegistrationIndex{INDEX_NONE};
	std::atomic<uint32> Generation{0};
};
static_assert(sizeof(FSSAMSlot) == 8, "FSSAMSlot is expected to be 8 bytes");

/** Chunked handle table with generational ABA protection.
 *
 *  Storage is backed by TLockFreeAllocOnceIndexedAllocator (chunked pages, pointer-stable
 *  GetItem). Slot allocation (FreeList pop / NextIndex++) and lazy page materialization
 *  share a single mutex. Serializing producers guarantees a newly-allocated slot's memory
 *  is fully initialized before any consumer observes the slot index; register/unregister
 *  traffic isn't hot enough to justify a lockless design at this layer.
 *
 *  GetIndex / SetIndex / IsValid are lock-free on the slot's own atomics (generation +
 *  registration index); these are called outside Allocate/Release and don't need the
 *  table-level mutex. GetIndex additionally range-checks the handle against an atomic
 *  NextIndex (release-stored after page materialization in Allocate), rejecting forged
 *  or corrupted handles before any storage access.
 *
 *  Generation-based ABA protection: every Release bumps the slot's Generation, invalidating
 *  every outstanding copy of the previous handle. A stale handle (generation mismatch):
 *    - Returns INDEX_NONE from GetIndex.
 *    - Silently no-ops in SetIndex (no clobber of a recycled slot).
 *    - Silently no-ops in Release (no double-bump, no double-push to the free list).
 *  These checks are enforced with real runtime logic, not check() macros, so the protection
 *  holds in shipping builds.
 */
template<typename HandleT>
class TSSAMHandleTable
{
public:
	HandleT Allocate()
	{
		UE::TUniqueLock Lock(Mutex);

		uint32 SlotIndex;
		bool   bAllocatedFreshIndex = false;
		if (FreeList.Num() > 0)
		{
			SlotIndex = FreeList.Pop(EAllowShrinking::No);
		}
		else
		{
			// Read under the mutex; no race with other writers. Defer publishing the new
			// NextIndex via release-store until after the backing page is materialized so
			// lock-free readers (GetIndex) can rely on the range check as a safety gate.
			SlotIndex = NextIndex.load(std::memory_order_relaxed);
			bAllocatedFreshIndex = true;
		}

		// Lazily materialize the slot's backing page + placement-new. Under the mutex there
		// is only one materializer at a time, so plain (non-atomic) visibility within the
		// critical section is sufficient.
		while (NextMaterializedItem <= SlotIndex + 1)
		{
			SlotStorage.Alloc(1);
			++NextMaterializedItem;
		}

		FSSAMSlot& Slot = *SlotStorage.GetItem(SlotIndex + 1);
		const uint32 Generation = Slot.Generation.load(std::memory_order_relaxed);
		Slot.RegistrationIndex.store(INDEX_NONE, std::memory_order_relaxed);

		if (bAllocatedFreshIndex)
		{
			// Release-store ordered after SlotStorage.Alloc above. Once a lock-free reader
			// observes SlotIndex < NextIndex via acquire-load, the slot's backing page
			// allocation is also visible -- GetItem is then guaranteed not to land on a
			// nullptr page.
			NextIndex.store(SlotIndex + 1, std::memory_order_release);
		}
		return HandleT{ SlotIndex, Generation };
	}

	/** Invalidate the handle and recycle the slot.
	 *  Returns the slot's RegistrationIndex at the moment of invalidation, or INDEX_NONE if
	 *  the handle was already stale or never assigned an index. Convenience for cleanup
	 *  callers that want the last-assigned index to drive SSAM-internal teardown without a
	 *  separate GetIndex call.
	 *
	 *  Stale handles (generation mismatch) are a safe no-op: the CAS fails, no recycling,
	 *  INDEX_NONE returned. */
	int32 Release(HandleT SSAMHandle)
	{
		if (!SSAMHandle.IsValid()) return INDEX_NONE;
		FSSAMSlot& Slot = *SlotStorage.GetItem(SSAMHandle.SlotIndex + 1); // +1: indexed allocator skips 0

		// CAS generation to atomically guarantee only the holder of the live generation proceeds.
		// The CAS runs outside the table mutex so concurrent Releases for different slots don't
		// serialize on each other. If the CAS fails, the slot has already been recycled
		// (possibly to a different handle) and we must not touch it.
		uint32 Expected = SSAMHandle.Generation;
		if (!Slot.Generation.compare_exchange_strong(Expected, Expected + 1, std::memory_order_acq_rel, std::memory_order_acquire))
		{
			return INDEX_NONE;
		}

		// Own the invalidation window between CAS (generation bumped) and the slot being
		// returned to the free pool: no other SetIndex can succeed here (generation check
		// fails), no other Release can win the CAS, and no Allocate can re-pick this slot
		// (not in the free list yet). Atomic exchange captures the last live index in one op.
		const int32 OldIndex = Slot.RegistrationIndex.exchange(INDEX_NONE, std::memory_order_acq_rel);

		UE::TUniqueLock Lock(Mutex);
		FreeList.Push(SSAMHandle.SlotIndex);
		return OldIndex;
	}

	int32 GetIndex(HandleT SSAMHandle) const
	{
		if (!SSAMHandle.IsValid())
		{
			return INDEX_NONE;
		}

		// Defensive range check. Rejects garbage / forged / corrupted handles whose SlotIndex
		// is outside the allocated range BEFORE any SlotStorage access. Pairs with the
		// release-store on NextIndex in Allocate (slot's backing page is materialized before
		// NextIndex is bumped), so observing SlotIndex < NextIndex via acquire-load guarantees
		// the GetItem call below cannot land on an unallocated page.
		if (SSAMHandle.SlotIndex >= NextIndex.load(std::memory_order_acquire))
		{
			return INDEX_NONE;
		}

		const FSSAMSlot* Slot = SlotStorage.GetItem(SSAMHandle.SlotIndex + 1);
		if (!Slot)
		{
			return INDEX_NONE;
		}

		const uint32 CurrentGen = Slot->Generation.load(std::memory_order_acquire);
		if (CurrentGen != SSAMHandle.Generation)
		{
			return INDEX_NONE;
		}

		return Slot->RegistrationIndex.load(std::memory_order_acquire);
	}

	/** Store a new registration index. Stale handles (generation mismatch) are silently no-op
	 *  in both dev and ship builds to prevent clobbering a slot that has been recycled to a
	 *  different handle. */
	void SetIndex(HandleT SSAMHandle, int32 NewIndex)
	{
		if (!SSAMHandle.IsValid())
		{
			return;
		}

		FSSAMSlot& Slot = *SlotStorage.GetItem(SSAMHandle.SlotIndex + 1);
		const uint32 CurrentGen = Slot.Generation.load(std::memory_order_acquire);
		if (CurrentGen != SSAMHandle.Generation)
		{
			return;
		}

		Slot.RegistrationIndex.store(NewIndex, std::memory_order_release);
	}

	bool IsValid(HandleT SSAMHandle) const
	{
		if (!SSAMHandle.IsValid())
		{
			return false;
		}

		const FSSAMSlot* Slot = SlotStorage.GetItem(SSAMHandle.SlotIndex + 1);
		return Slot && Slot->Generation.load(std::memory_order_acquire) == SSAMHandle.Generation;
	}

	uint32 GetAllocatedSize() const
	{
		UE::TUniqueLock Lock(Mutex);
		return static_cast<uint32>((NextIndex.load(std::memory_order_relaxed) - static_cast<uint32>(FreeList.Num())) * sizeof(FSSAMSlot));
	}

private:
	static constexpr uint32 MaxSlots     = 1u << 20;  // 1M handles per table
	static constexpr uint32 SlotsPerPage = 4096;      // 256 pages * 4K slots

	using FStorage = TLockFreeAllocOnceIndexedAllocator<FSSAMSlot, MaxSlots, SlotsPerPage>;

	// SlotStorage is mutable because TLockFreeAllocOnceIndexedAllocator declares GetItem
	// non-const even though it's semantically a read-only atomic load. Our const accessors
	// (GetIndex, IsValid) need to call through it without forcing callers to hold a non-const
	// table reference. Mutex is mutable so const accessors (GetAllocatedSize) can lock it.
	mutable FStorage    SlotStorage;
	mutable UE::FMutex  Mutex;
	TArray<uint32>      FreeList;
	// Monotonic high-water mark of slot indices ever issued. Atomic so GetIndex (called
	// lock-free, including from the render thread) can range-check incoming handles before
	// touching SlotStorage. Updated only inside Allocate under Mutex with release-store
	// after the slot's backing page has been materialized; readers acquire-load and gate
	// on SlotIndex < NextIndex.
	std::atomic<uint32> NextIndex{0};
	uint32              NextMaterializedItem = 1; // indexed allocator skips 0
};

using FSSAMAssetHandleTable     = TSSAMHandleTable<FSSAMAssetHandle>;
using FSSAMPrimitiveHandleTable = TSSAMHandleTable<FSSAMPrimitiveHandle>;
