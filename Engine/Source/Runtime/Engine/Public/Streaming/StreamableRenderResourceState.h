// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
StreamableRenderResourceState.h: Render resource state for streamable assets.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

/** 
 * Define the streaming capabilities of a UStreamableRenderAsset render resources. 
 * The properties defines the current render states, coherent on the gamethread for UStreamableRenderAsset::CachedSSRState.
 * If used within render resources, like in FStreamableTextureResource::State, then it is coherent on the renderthread.
 *
 * Is it only expected to be valid, if the render resource are created, with InitRHI command sent, in order to simplify logic.
 * Also, { bSupportsStreaming, NumNonStreamingLODs, NumNonOptionalLODs, MaxNumLODs, AssetLODBias } are expected to be constant throughout the resource lifetime.
 *
 * An important concept within this structure is that the resource first LOD is not required to be the first asset LOD, the offset being defined by AssetLODBias.
 * This allows the streaming logic to ignore invalid, non relevant entries, or too many entries in the asset.
 */
struct FStreamableRenderResourceState
{
	// The maximum number of streaming LODs the streaming supports.
	static const uint32 MAX_LOD_COUNT = 16;

	FStreamableRenderResourceState() : Data(0) {}

	// Manually implement copy operations, because MSVC produces bad codegen for bitfields in unions
	// (as of the 14.35.32217 toolchain)
	inline FStreamableRenderResourceState(const FStreamableRenderResourceState& Other)
		: Data(Other.Data)
	{
	}
	inline void operator=(const FStreamableRenderResourceState& Other)
	{
		Data = Other.Data;
	}

	union
	{
		struct
		{
			/* Whether the resource LODs can be streamed or not. Even when false, the structure can contain valid information if IsValid(). */
			uint8 bSupportsStreaming : 1;

			/** Whether the resource uses virtual texture streaming. Virtual textures need to use a specific memory category. */
			uint8 bSupportsVirtualStreaming : 1;

			/* Whether the render resources have pending InitRHI(). Used in HasPendingInitOrStreaming() to avoid dereferencing the different resource pointers unless required. Once false, expected to stay false. */
			mutable uint8 bHasPendingInitHint : 1;

			/* The number of always loaded LODs, which can not be streamed out. */
			uint8 NumNonStreamingLODs; 
			/* The number of LODs (streaming or not) that are guarantied to be installed, for example in PAKs. */
			uint8 NumNonOptionalLODs; // >= NumberOfNonStreamingLODs
			/* The maximum number of LODs the resource can possibly have. Might be less than the asset LOD count. */
			uint8 MaxNumLODs; // >= NumberOfNonOptionalLODs

			/** The asset LOD index of the resource LOD 0. Non-zero when the render resource doesn't use all the asset LODs, for example ignore the first one. */
			uint8 AssetLODBias;
			
			/** The current number of LODs the resource has. Between NumNonStreamingLODs and MaxNumLODs. */
			uint8 NumResidentLODs;
			/** The expected number of LODs after the current streaming request completes. Between NumNonStreamingLODs and MaxNumLODs. */
			uint8 NumRequestedLODs;

			/** An additional LOD bias modifier set during asset loading based on the current RHI feature level or other settings */
			uint8 LODBiasModifier;
		};
		uint64 Data;
	};

	/** Considering the given render resource LOD count, return the corresponding first LOD index within the asset LOD array. */
	inline int32 LODCountToAssetFirstLODIdx(int32 InLODCount) const
	{
		return AssetLODBias + MaxNumLODs - InLODCount;
	}

	/** Considering the given render resource LOD count, return the corresponding first LOD index within the render resource LOD array. */
	inline int32 LODCountToFirstLODIdx(int32 InLODCount) const
	{
		return MaxNumLODs - InLODCount;
	}

	/** Return the first resident LOD index within the render resource LOD array. */
	inline int32 ResidentFirstLODIdx() const
	{
		return LODCountToFirstLODIdx(NumResidentLODs);
	}

	/** Return the first requested LOD index within the render resource LOD array. */
	inline int32 RequestedFirstLODIdx() const
	{
		return LODCountToFirstLODIdx(NumRequestedLODs);
	}

	/** Return whether this has any information to be relied on. */
	inline bool IsValid() const { return Data != 0; }

	inline void Clear() { Data = 0; }

	/** Validate that everything is valid and makes sense for a streaming requests. */
	inline bool IsValidForStreamingRequest() const
	{
		return bSupportsStreaming
			&& NumNonStreamingLODs <= NumNonOptionalLODs && NumNonOptionalLODs <= MaxNumLODs && MaxNumLODs <= MAX_LOD_COUNT 
			&& FMath::IsWithinInclusive(NumResidentLODs, NumNonStreamingLODs, MaxNumLODs)
			&& FMath::IsWithinInclusive(NumRequestedLODs, NumNonStreamingLODs, MaxNumLODs)
			&& NumResidentLODs != NumRequestedLODs;
	}

	/** Validate that everything is valid and makes sense for a streaming requests. */
	inline bool StreamIn(int32 InLODCount)
	{
		// InLODCount can be higher than MaxNumLODs, as long as there are some LODs to yet stream in.
		if (bSupportsStreaming && InLODCount > NumResidentLODs && NumResidentLODs < MaxNumLODs && NumRequestedLODs == NumResidentLODs)
		{
			NumRequestedLODs = (uint8)FMath::Min<int32>(InLODCount, MaxNumLODs);
			return true;
		}
		else
		{
			return false;
		}
	}

	inline bool StreamOut(int32 InLODCount)
	{
		// InLODCount can be lower than MaxNumLODs, as long as there are some LODs to yet stream in.
		if (bSupportsStreaming && InLODCount < NumResidentLODs && NumResidentLODs > NumNonStreamingLODs && NumRequestedLODs == NumResidentLODs)
		{
			NumRequestedLODs = (uint8)FMath::Max<int32>(InLODCount, NumNonStreamingLODs);
			return true;
		}
		else
		{
			return false;
		}
	}
};