// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "StreamingWorldSubsystemInterface.generated.h"

/** Interface for world subsystems that require an update for streaming (called by UWorld::InternalUpdateStreamingState) */
UINTERFACE(meta = (CannotImplementInterfaceInBlueprint))
class UStreamingWorldSubsystemInterface : public UInterface
{
	GENERATED_BODY()
};

class IStreamingWorldSubsystemInterface
{
	GENERATED_BODY()

public:

	virtual void OnUpdateStreamingState() { }
	virtual void OnFlushStreaming() {}
};
