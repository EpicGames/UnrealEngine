// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
TextureMipDataProvider.h: Base class for providing the mip data used by FTextureStreamIn.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "PixelFormat.h"
#include "RHIDefinitions.h"
#include "RHIFwd.h"
#include "Streaming/StreamableRenderResourceState.h"

class UTexture;
class UStreamableRenderAsset;
class FStreamableTextureResource;
struct FTexture2DMipMap;

// Describes a mip that gets updated through FTextureStreamIn.
struct FTextureMipInfo
{
	void* DestData = nullptr; // The mip data where the content must be copied too.
	uint64 DataSize = 0; // Optional, might be 0 in cooked. Mostly for safeguard.
	EPixelFormat Format = PF_Unknown;
	uint32 SizeX = 0;
	uint32 SizeY = 0;
	uint32 SizeZ = 0; // For 3d textures
	uint32 ArraySize = 0; // For texture arrays / cubemaps
	uint32 RowPitch = 0;
	uint32 DepthPitch = 0;	// For volume texture, texture array only.
	ECubeFace CubeFace = CubeFace_MAX; // Cubemap only.
};

// An array of mips mapped to a given texture asset. 0 based on the highest mip count for this texture.
typedef TArray<FTextureMipInfo, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> FTextureMipInfoArray;

/**
* Different options to schedule correctly the next tick in FTextureStreamIn.
* It allows the asset update logic to postpone the next tick until the effect of the last tick is completed.
* This helps for things like async operations.
*
* If no sync options are used in the tick of either FTextureMipDataProvider or FTextureMipAllocator,
* the tick must at least advance to the next state through AdvanceTo() to prevent infinite loops.
*/
struct FTextureUpdateSyncOptions
{
	typedef TFunction<void()> FCallback;

	// A bool to tell whether the tick should be postponed by a small duration (undefined by design).
	bool* bSnooze = nullptr;
	// A counter that will be incremented before doing an operation and then decremented once has completed.
	FThreadSafeCounter* Counter = nullptr;

	// A callback to force a new tick on the relevant thread.
	FCallback RescheduleCallback;
};

/**
* A constant context used throughout the texture stream in update tick.
* Hold useful information for either FTextureMipDataProvider or FTextureMipAllocator.
*/
struct FTextureUpdateContext
{
	typedef int32 EThreadType;

	FTextureUpdateContext(const UTexture* InTexture, EThreadType InCurrentThread);
	FTextureUpdateContext(const UStreamableRenderAsset* InTexture, EThreadType InCurrentThread);

	EThreadType GetCurrentThread() const { return CurrentThread; }

	// The texture to update, this must be the same one as the one used when creating the FTextureUpdate object.
	const UTexture* Texture = nullptr;

	/** The current streamabble resource of this texture. */
	FStreamableTextureResource* Resource = nullptr;

	/** The array view of streamable mips from the asset. Takes into account FStreamableRenderResourceState::AssetLODBias and FStreamableRenderResourceState::MaxNumLODs. */
	TArrayView<const FTexture2DMipMap*> MipsView;

	// The current executing thread.
	EThreadType CurrentThread;
};

/**
* FTextureMipDataProvider defines the update steps and interface to implement the mip data strategy
* used in FTextureStreamIn. It allows to decouples where the texture mip data source from the texture update.
* Typical implementations are using DDC, disk files, internet server or dynamically generated.
*/
class FTextureMipDataProvider
{
public:

	enum class ETickState : uint32
	{
		Init,					// Call Init(), which must initialize the mip data provider so that it can efficiently provide the mips in GetMips.
		GetMips,				// Call GetMips(), which must fill mip data from highest to lowest (all mips might not be available thought).
		PollMips,				// Call PollMips(), which stalls the mip update until the mip have been set.
		PostprocessGPUMipData,	// Call PostprocessGPUMipData(), which allows for mip updates after the texture has been uploaded to the GPU
		CleanUp,				// Call CleanUp(), which must release all temporary resource created in the update process.
		Done
	};

	enum class ETickThread : uint32
	{
		Async,			// Tick must run on an async thread.
		Render,			// Tick must run on the renderthread.
		None,			// No tick must run, only valid in ETickState::Done. 
	};

	// Constructor, defining the first tick step and thread.
	ENGINE_API FTextureMipDataProvider(const UTexture* Texture, ETickState InTickState, ETickThread InTickThread);

	virtual ~FTextureMipDataProvider() = default;

	// Get the next tick state and thread for the mip allocator. Used by FTextureStreamIn to schedule correctly the update between FTextureMipAllocator and FTextureMipDataProvider.
	inline ETickState GetNextTickState() const { return NextTickState; }
	inline ETickThread GetNextTickThread() const { return NextTickThread; }

	/**
	* Initialize data prelimary to the GetMips() step. Can be called several time (it does not have to advance to GetMips immediately). Mostly useful to simplify the logic in GetMips().
	* This is because GetMips is a chained call between all mip data providers, each taking some mips to handle, and is not compatible with multi step process.
	* This means that GetMips() must return immediately and can not postpone or delay return by not advancing to the next steps.
	*
	* @param Context - An update context constant throughout the FTextureStreamIn update. Gives things like which texture asset is updated and what mips are streamed in.
	* @param SyncOptions - Different sync options to control when the next tick of FTextureStreamIn can be scheduled.
	*/
	virtual void Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) = 0;

	/**
	* Acquire the mips this provider will handle. Non handled mips must be handled by the next mip data provider or the update will be cancelled altogether.
	* GetMips() must typically advance to PollMips() if it wants to be able to notify FTextureStreamIn that something went wrong and that the update must be cancelled.
	*
	* @param Context - An update context constant throughout the FTextureStreamIn update. Gives things like which texture asset is updated and what mips are streamed in.
	* @param StartingMipIndex - The current starting mip index, somewhere between FTextureUpdateContext::PendingFirstMipIndex and FTextureUpdateContext::CurrentFirstMipIndex inclusively.
	* @param MipInfos - The array of FTextureMipInfo that hold the information relative to each mip for which data must be provided.
	* @param SyncOptions - Different sync options to control when the next tick of FTextureStreamIn can be scheduled.
	*
	* Return the next value StartingMipIndex (for the next provider). Must be FTextureUpdateContext::CurrentFirstMipIndex to indicate that all mips have been handled.
	*/
	virtual int32 GetMips(const FTextureUpdateContext& Context, int32 StartingMipIndex, const FTextureMipInfoArray& MipInfos, const FTextureUpdateSyncOptions& SyncOptions) = 0;

	/**
	* Check if each mip handled by this mip data provider have been updated correctly. Must move to CleanUp() or Done() when done.
	*
	* @param SyncOptions - Different sync options to control when the next tick of FTextureStreamIn can be scheduled.
	*
	* Return true unless the texture update needs to be aborted because the mip won't be updated correctly (for example IO error).
	*/
	virtual bool PollMips(const FTextureUpdateSyncOptions& SyncOptions) = 0;

	/*
	* Once the texture has been uploaded to the GPU, the mip provider has the opportunity to perform any modifications necessary
	* before the texture is finalized and applied to the texture resource.
	*/
	UE_DEPRECATED(5.7, "Implement the version that takes FirstMip and NumMips.")
	virtual bool PostprocessGPUMipData(const FTextureUpdateContext& Context, const FTextureRHIRef& NewTextureRHI, const FTextureUpdateSyncOptions& SyncOptions) { return true; }

	/*
	* Once the texture has been uploaded to the GPU, the mip provider has the opportunity to perform any modifications necessary
	* before the texture is finalized and applied to the texture resource.
	*/
	ENGINE_API virtual bool PostprocessGPUMipData(const FTextureUpdateContext& Context, const FTextureRHIRef& TextureRHI, int32 FirstMip, int32 NumMips, const FTextureUpdateSyncOptions& SyncOptions);

	/**
	* Abort anything that could be stalling the update in PollMips(). Called from an async task.
	* Expected behavior is to cancel any pending IO operations. 
	* Called when the FTextureStreamIn update is canceled and the current step is to be waiting on upon PollMips().
	* This is because Cancel() can only be executed when any pending operations are completed (see FTextureUpdateSyncOptions).
	* Will be used if executing PollMips
	*/
	virtual void AbortPollMips() {}

	/**
	* Abort anything that could be stalling the update in PostprocessGPUMipData(). Called from an async task.
	* Expected behavior is to cancel any pending IO operations.
	* Called when the FTextureStreamIn update is canceled and the current step is to be waiting on upon AbortPostprocessGPUMipData().
	* This is because Cancel() can only be executed when any pending operations are completed (see FTextureUpdateSyncOptions).
	* Will be used if executing PostprocessGPUMipData
	*/
	virtual void AbortPostprocessGPUMipData() {}

	/**
	* Release any temporary data and objects that where used for the update. Final step executed after the texture has been updated correctly.
	*
	* @param SyncOptions - Different sync options to control when the next tick of FTextureStreamIn can be scheduled.
	*/
	virtual void CleanUp(const FTextureUpdateSyncOptions& SyncOptions) = 0;

	/**
	* Cancel the progression and release any temporary resources. 
	* Called within the FTextureStreamIn update when the stream in request is aborted or cannot complete correctly
	* The Cancel() function is called on the thread returned by GetCancelThread().
	*
	* @param SyncOptions - Different sync options to control when the next tick of FTextureStreamIn can be scheduled.
	*/
	virtual void Cancel(const FTextureUpdateSyncOptions& SyncOptions) = 0;

	// Returns on which thread Cancel() must be called in the texture update (from FTextureStreamIn) to release ressources safely and correctly. ETickThread::None when ready to delete this.
	virtual ETickThread GetCancelThread() const = 0;

protected:

	// Helper to set the next tick state and thread. Validates that the progression is coherent.
	inline void AdvanceTo(ETickState InState, ETickThread InThread)
	{
		// State must advance or at least stay to the same step.
		check((uint32)NextTickState <= (uint32)InState);
		// Thread must be valid unless in done state.
		check((InState == ETickState::Done) == (InThread == ETickThread::None));

		NextTickState = InState;
		NextTickThread = InThread;
	}

	/** The streamable state requested. */
	const FStreamableRenderResourceState ResourceState;
	// The resident first LOD resource index. With domain = [0, ResourceState.NumLODs[. NOT THE ASSET LOD INDEX!
	const int32 CurrentFirstLODIdx = INDEX_NONE;
	// The requested first LOD resource index. With domain = [0, ResourceState.NumLODs[. NOT THE ASSET LOD INDEX!
	const int32 PendingFirstLODIdx = INDEX_NONE;

private:

	// The next tick function that should be called.
	ETickState NextTickState = ETickState::Done;
	// The next tick thread from where to call the tick function.
	ETickThread NextTickThread = ETickThread::None;
};

typedef TArray<TUniquePtr<FTextureMipDataProvider>, TInlineAllocator<2>> FTextureMipDataProviderArray;
