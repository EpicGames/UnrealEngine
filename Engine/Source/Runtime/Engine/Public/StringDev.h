// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

class FOptionalStringNull
{
public:
	inline FOptionalStringNull()
	{}

	inline FOptionalStringNull(const FString& InString)
	{}

	inline  const FString& GetString() const
	{
		static FString String;
		return String;
	}
	
	inline void SetString(const FString& InString)
	{}

	friend inline FArchive& operator<<(FArchive& Ar, FOptionalStringNull& InString)
	{
		FString String;
		return Ar << String;
	}
};

class FOptionalStringNonNull
{
	FString String;

public:
	inline FOptionalStringNonNull()
	{}

	inline FOptionalStringNonNull(const FString& InString)
	{
		SetString(InString);
	}

	inline const FString& GetString() const
	{
		return String;
	}

	inline void SetString(const FString& InString)
	{
		String = InString;
	}

	friend inline FArchive& operator<<(FArchive& Ar, FOptionalStringNonNull& InString)
	{
		return Ar << InString.String;
	}
};

/**
 * A string that is present in memory only in editor and debug builds.
 */
#if WITH_EDITOR || UE_BUILD_DEBUG
using FStringDebug = FOptionalStringNonNull;
#else
using FStringDebug = FOptionalStringNull;
#endif

/**
 * A string that is present in memory only in editor, debug and development builds.
 */
#if WITH_EDITOR || UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
using FStringDev = FOptionalStringNonNull;
#else
using FStringDev = FOptionalStringNull;
#endif

/**
 * A string that is present in memory only in editor, debug, development and test builds.
 */
#if WITH_EDITOR || !UE_BUILD_SHIPPING
using FStringTest = FOptionalStringNonNull;
#else
using FStringTest = FOptionalStringNull;
#endif