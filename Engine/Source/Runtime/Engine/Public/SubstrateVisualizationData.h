// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class FSubstrateViewMode : uint8
{
	None,
	MaterialProperties,
	MaterialCount,
	MaterialByteCount,
	SubstrateInfo,
	AdvancedMaterialProperties,
	MaterialClassification,
	RoughRefractionClassification,
	DecalClassification,
};

class FSubstrateVisualizationData
{
public:

	/** Describes a single available visualization mode. */
	struct FModeRecord
	{
		FString   ModeString;
		FName     ModeName;
		FText     ModeText;
		FText     ModeDesc;
		FSubstrateViewMode ViewMode;

		// Whether or not this mode (by default) composites with regular scene depth.
		bool      bDefaultComposited;

		bool      bAvailableCommand;
		FText	  UnavailableReason;
	};

	/** Mapping of FName to a visualization mode record. */
	typedef TMultiMap<FName, FModeRecord> TModeMap;

public:
	FSubstrateVisualizationData()
	: bIsInitialized(false)
	{
	}

	/** Initialize the system. */
	void Initialize();

	/** Check if system was initialized. */
	inline bool IsInitialized() const { return bIsInitialized; }

	/** Get the display name of a named mode from the available mode map. **/
	ENGINE_API FText GetModeDisplayName(const FName& InModeName) const;

	ENGINE_API FSubstrateViewMode GetViewMode(const FName& InModeName) const;

	ENGINE_API static uint32 GetViewMode();

	ENGINE_API bool GetModeDefaultComposited(const FName& InModeName) const;

	inline const TModeMap& GetModeMap() const
	{
		return ModeMap;
	}

	/** Return the console command name for enabling single mode visualization. */
	static const TCHAR* GetVisualizeConsoleCommandName()
	{
		return TEXT("r.Substrate.ViewMode");
	}

private:

	/** The name->mode mapping table */
	TModeMap ModeMap;

	/** Storage for console variable documentation strings. **/
	FString ConsoleDocumentationVisualizationMode;

	/** Flag indicating if system is initialized. **/
	bool bIsInitialized;

	/** CVar for handling view modes. **/
	IConsoleVariable* CVarViewModes = nullptr;
};

ENGINE_API FSubstrateVisualizationData& GetSubstrateVisualizationData();
