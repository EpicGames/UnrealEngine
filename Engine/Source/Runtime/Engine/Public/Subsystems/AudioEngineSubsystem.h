// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDeviceManager.h"
#include "Subsystem.h"
#include "SubsystemCollection.h"
#include "AudioEngineSubsystem.generated.h"

// Forward Declarations 
class FAudioDevice;
namespace Audio
{
	class FMixerDevice;
	class FMixerSourceManager;
}

/**
 * UAudioSubsystemCollectionRoot
 * Root UObject used to anchor UAudioEngineSubsystems to the FAudioDevice
 */
UCLASS(MinimalAPI)
class UAudioSubsystemCollectionRoot final : public UObject
{
	GENERATED_BODY()

public:

	/** Set the ID of the owning audio device */
	void inline SetAudioDeviceID(Audio::FDeviceId DeviceID) { OwningDeviceID = DeviceID; }

	/** Get the ID of the owning audio device */
	Audio::FDeviceId GetAudioDeviceID() const { return OwningDeviceID; }

protected:

	Audio::FDeviceId OwningDeviceID = Audio::FDeviceId(INDEX_NONE);
};

/**
 * UAudioEngineSubsystem
 * Base class for auto instanced and initialized systems that share the lifetime of the audio device
 */
UCLASS(Abstract, MinimalAPI)
class UAudioEngineSubsystem : public UDynamicSubsystem
{
	GENERATED_BODY()

public:

	ENGINE_API UAudioEngineSubsystem();

	/**
	 * Override to get an update call during AudioDevice::Update
	 *  Note: This call will happen on the audio thread
	 */
	virtual void Update() {}

	/** Returns the owning audio device Id */
	ENGINE_API virtual Audio::FDeviceId GetAudioDeviceId() const final;

	/** Returns the owning audio device handle */
	ENGINE_API virtual FAudioDeviceHandle GetAudioDeviceHandle() const final;

	/** Return a mutable version of the source manager associated with the owning device handle */
	ENGINE_API virtual Audio::FMixerSourceManager* GetMutableSourceManager() final;

	/** Return the source manager associated with the owning device handle */
	ENGINE_API virtual const Audio::FMixerSourceManager* GetSourceManager() const final;

	/** Return a mutable version of the mixer device from the owning device handle */
	ENGINE_API virtual Audio::FMixerDevice* GetMutableMixerDevice() final;

	/** Return the mixer device from the owning device handle */
	ENGINE_API virtual const Audio::FMixerDevice* GetMixerDevice() const final;
};

/**
 * FAudioSubsystemCollection - Subsystem collection specifically targeting UAudioEngineSubsystems
 */
class FAudioSubsystemCollection : public FSubsystemCollection<UAudioEngineSubsystem>
{
public:

	template<class ArgType>
	void ForEachSubsystem(TFunctionRef<bool(ArgType*)> InFunction) const
	{
		UClass* ArgClass = nullptr;
		if constexpr (TIsIInterface<ArgType>::Value)
		{
			ArgClass = ArgType::UClassType::StaticClass();
		}
		else
		{
			ArgClass = ArgType::StaticClass();
		}

		ForEachSubsystemOfClass(ArgClass, [Function=MoveTemp(InFunction)](USubsystem* Subsystem)
		{
			Function(CastChecked<ArgType>(Subsystem));
		});
	}
};

