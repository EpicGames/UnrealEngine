// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystem.h"

#include "EngineSubsystem.generated.h"

/**
 * UEngineSubsystem
 * Base class for auto instanced and initialized systems that share the lifetime of the Engine
 *
 * UEngineSubsystems are dynamic and will be initialized when the module is loaded if necessary.
 * This means that after StartupModule() is called on the module containing a subsystem,
 * the subsystem collection with instantiate and initialize the subsystem automatically.
 * If the subsystem collection is created post module load then the instances will be created at
 * collection initialization time.
 */
UCLASS(Abstract, MinimalAPI)
class UEngineSubsystem : public UDynamicSubsystem
{
	GENERATED_BODY()
};
