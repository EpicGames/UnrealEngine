// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/Subsystem.h"

#include "GameInstanceSubsystem.generated.h"

class UGameInstance;

/**
 * UGameInstanceSubsystem
 * Base class for auto instanced and initialized systems that share the lifetime of the game instance
 */
UCLASS(Abstract, Within = GameInstance, MinimalAPI)
class UGameInstanceSubsystem : public USubsystem
{
	GENERATED_BODY()

public:
	/**
	 * Returns a pointer to the UGameInstance this subsystem is contained within.
	 * @note This will return null for default objects
	 */
	ENGINE_API UGameInstance* GetGameInstance() const;

};
