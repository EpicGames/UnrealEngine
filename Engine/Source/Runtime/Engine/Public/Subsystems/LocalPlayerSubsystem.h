// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/Subsystem.h"

#include "LocalPlayerSubsystem.generated.h"

class ULocalPlayer;
class APlayerController;

/**
 * ULocalPlayerSubsystem
 * Base class for auto instanced and initialized subsystem that share the lifetime of local players
 */
UCLASS(Abstract, Within = LocalPlayer, MinimalAPI)
class ULocalPlayerSubsystem : public USubsystem
{
	GENERATED_BODY()

public:
	template<typename LocalPlayerType = ULocalPlayer>
	LocalPlayerType* GetLocalPlayer() const
	{
		return Cast<LocalPlayerType>(GetOuter());
	}

	template<typename LocalPlayerType = ULocalPlayer>
	LocalPlayerType* GetLocalPlayerChecked() const
	{
		return CastChecked<LocalPlayerType>(GetOuter());
	}

	/** Callback for when the player controller is changed on this subsystem's owning local player */
	virtual void PlayerControllerChanged(APlayerController* NewPlayerController) {}
};
