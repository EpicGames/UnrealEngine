// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "Subsystem.generated.h"

class FSubsystemCollectionBase;

/** 
 *	Subsystems are auto instanced classes that share the lifetime of certain engine constructs
 *	Currently supported Subsystem lifetimes are:
 *		Engine		 -> inherit UEngineSubsystem
 *		Editor		 -> inherit UEditorSubsystem
 *		GameInstance -> inherit UGameInstanceSubsystem
 *		World		 -> inherit UWorldSubsystem
 *		LocalPlayer	 -> inherit ULocalPlayerSubsystem
 *
 *	Normal Example:
 *		class UMySystem : public UGameInstanceSubsystem
 *	Which can be accessed by:
 *		UGameInstance* GameInstance = ...;
 *		UMySystem* MySystem = GameInstance->GetSubsystem<UMySystem>();
 *
 *	or the following if you need protection from a null GameInstance
 *		UGameInstance* GameInstance = ...;
 *		UMyGameSubsystem* MySubsystem = UGameInstance::GetSubsystem<MyGameSubsystem>(GameInstance);
 *
 *	You can get also define interfaces that can have multiple implementations.
 *	Interface Example :
 *      MySystemInterface
 *    With 2 concrete derivative classes:
 *      MyA : public MySystemInterface
 *      MyB : public MySystemInterface
 *
 *	Which can be accessed by:
 *		UGameInstance* GameInstance = ...;
 *		const TArray<UMyGameSubsystem*>& MySubsystems = GameInstance->GetSubsystemArray<MyGameSubsystem>();
 */

UCLASS(Abstract, MinimalAPI)
class USubsystem : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Override to control if the Subsystem should be created at all.
	 * For example you could only have your system created on servers.
	 * For subsystems that can return false, you must null check whenever getting the Subsystem.
	 *
	 * Note: This function is called on the CDO prior to instances being created!
	 */
	virtual bool ShouldCreateSubsystem(UObject* Outer) const { return true; }

	/** Implement this for initialization of instances of the system */
	virtual void Initialize(FSubsystemCollectionBase& Collection) {}

	/** Implement this for deinitialization of instances of the system */
	virtual void Deinitialize() {}

	/** Overridden to check global network context */
	ENGINE_API virtual int32 GetFunctionCallspace(UFunction* Function, FFrame* Stack) override;

private:
	friend class FSubsystemCollectionBase;
	FSubsystemCollectionBase* InternalOwningSubsystem = nullptr;
};

/** 
 * Dynamic Subsystems auto populate/depopulate existing collections when modules are loaded and unloaded.
 * This abstract base class is used for singletons like UEngineSubsystem that are automatically created.
 * 
 * If instances of your subsystem aren't being created it maybe that the module they are in isn't being explicitly loaded,
 * make sure there is a LoadModule("ModuleName") to load the module.
 */
UCLASS(Abstract, MinimalAPI)
class UDynamicSubsystem : public USubsystem
{
	GENERATED_BODY()
};
