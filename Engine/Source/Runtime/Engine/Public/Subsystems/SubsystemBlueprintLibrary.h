// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Subsystems/EngineSubsystem.h"
#include "Subsystems/GameInstanceSubsystem.h"
#include "Subsystems/LocalPlayerSubsystem.h"
#include "Subsystems/AudioEngineSubsystem.h"

#include "SubsystemBlueprintLibrary.generated.h"

UCLASS(MinimalAPI)
class USubsystemBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Get a Game Instance Subsystem from the Game Instance associated with the provided context */
	UFUNCTION(BlueprintPure, Category = "Engine Subsystems", meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API UEngineSubsystem* GetEngineSubsystem(TSubclassOf<UEngineSubsystem> Class);

	/** Get a Game Instance Subsystem from the Game Instance associated with the provided context */
	UFUNCTION(BlueprintPure, Category = "GameInstance Subsystems", meta = (WorldContext = "ContextObject", BlueprintInternalUseOnly = "true"))
	static ENGINE_API UGameInstanceSubsystem* GetGameInstanceSubsystem(UObject* ContextObject, TSubclassOf<UGameInstanceSubsystem> Class);

	/** Get a Local Player Subsystem from the Local Player associated with the provided context */
	UFUNCTION(BlueprintPure, Category = "LocalPlayer Subsystems", meta = (WorldContext = "ContextObject", BlueprintInternalUseOnly = "true"))
	static ENGINE_API ULocalPlayerSubsystem* GetLocalPlayerSubsystem(UObject* ContextObject, TSubclassOf<ULocalPlayerSubsystem> Class);

	/** Get a World Subsystem from the World associated with the provided context */
	UFUNCTION(BlueprintPure, Category = "GameInstance Subsystems", meta = (WorldContext = "ContextObject", BlueprintInternalUseOnly = "true"))
	static ENGINE_API UWorldSubsystem* GetWorldSubsystem(UObject* ContextObject, TSubclassOf<UWorldSubsystem> Class);

	/** Get an AudioEngine Subsystem from the AudioDevice associated with the provided context */
	UFUNCTION(BlueprintPure, Category = "AudioEngine Subsystems", meta = (WorldContext = "ContextObject", BlueprintInternalUseOnly = "true"))
	static ENGINE_API UAudioEngineSubsystem* GetAudioEngineSubsystem(UObject* ContextObject, TSubclassOf<UAudioEngineSubsystem> Class);

	/** 
	 * Get a Local Player Subsystem from the LocalPlayer associated with the provided context
	 * If the player controller isn't associated to a LocalPlayer nullptr is returned
	 */
	UFUNCTION(BlueprintPure, Category = "LocalPlayer Subsystems", meta = (BlueprintInternalUseOnly = "true"))
	static ENGINE_API ULocalPlayerSubsystem* GetLocalPlayerSubSystemFromPlayerController(APlayerController* PlayerController, TSubclassOf<ULocalPlayerSubsystem> Class);

private:
	static UWorld* GetWorldFrom(UObject* ContextObject);
};
