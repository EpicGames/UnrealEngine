// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/GCObject.h"

class USubsystem;
class UDynamicSubsystem;

struct FSubsystemCollectionInitialization;

DECLARE_LOG_CATEGORY_EXTERN(LogSubsystemCollection, Log, All);

class FSubsystemCollectionBase
{
public:
	/** Initialize the collection of systems, systems will be created and initialized */
	ENGINE_API void Initialize(UObject* NewOuter);

	/* Clears the collection, while deinitializing the systems */
	ENGINE_API void Deinitialize();

	/** Returns true if collection was already initialized */
	bool IsInitialized() const { return Outer != nullptr; }

	/** Get the collection BaseType */
	const UClass* GetBaseType() const { return BaseType; }

	/** 
	 * Only call from Initialize() of Systems to ensure initialization order
	 * Note: Dependencies only work within a collection
	 */
	ENGINE_API USubsystem* InitializeDependency(TSubclassOf<USubsystem> SubsystemClass);

	/**
	 * Only call from Initialize() of Systems to ensure initialization order
	 * Note: Dependencies only work within a collection
	 */
	template <typename TSubsystemClass>
	TSubsystemClass* InitializeDependency()
	{
		return Cast<TSubsystemClass>(InitializeDependency(TSubsystemClass::StaticClass()));
	}

	/** Registers and adds instances of the specified Subsystem class to all existing SubsystemCollections of the correct type.
	 *  Should be used by specific subsystems in plug ins when plugin is activated.
	 */
	static ENGINE_API void ActivateExternalSubsystem(UClass* SubsystemClass);

	/** Unregisters and removed instances of the specified Subsystem class from all existing SubsystemCollections of the correct type.
	 *  Should be used by specific subsystems in plug ins when plugin is deactivated.
	 */
	static ENGINE_API void DeactivateExternalSubsystem(UClass* SubsystemClass);

	/** Collect references held by this collection */
	ENGINE_API void AddReferencedObjects(UObject* Referencer, FReferenceCollector& Collector);
protected:
	struct FSubsystemArray
	{
		TArray<USubsystem*> Subsystems;
		mutable bool bIsIterating = false; // Safety check to avoid removals during iteration but allow index-based iteration
	};

	/** protected constructor - for use by the template only(FSubsystemCollection<TBaseType>) */
	ENGINE_API FSubsystemCollectionBase(UClass* InBaseType);

	/** protected constructor - Use the FSubsystemCollection<TBaseType> class */
	ENGINE_API FSubsystemCollectionBase();
	
	/** destructor will be called from virtual ~FGCObject in GC cleanup **/
	ENGINE_API virtual ~FSubsystemCollectionBase();

	/** Get a Subsystem by type */
	ENGINE_API USubsystem* GetSubsystemInternal(UClass* SubsystemClass) const;

	// Fetch a list of subsystems that derive from the given class, populating the cache if necessary
	UE_DEPRECATED("5.8", "FindAndPopulateSubsystemArrayInternal has been deprecated. Use GetSubsystemArrayCopy instead.")
	ENGINE_API FSubsystemArray& FindAndPopulateSubsystemArrayInternal(UClass* SubsystemClass) const;

	/** Get a list of Subsystems by type */
	TArray<USubsystem*> GetSubsystemArrayCopy(UClass* SubsystemClass) const
	{
		return GetSubsystemArrayCopy<USubsystem>(SubsystemClass);
	}

	/** Get a list of Subsystems by type */
	template<typename SubsystemType>
	TArray<SubsystemType*> GetSubsystemArrayCopy(UClass* SubsystemClass) const
	{
		if (const FSubsystemArray* Referenced = FindAndPopulateSubsystemArray(SubsystemClass))
		{
			return TArray<SubsystemType*>(reinterpret_cast<SubsystemType* const*>(Referenced->Subsystems.GetData()), Referenced->Subsystems.Num());
		}
		return {};
	}

	/** 
	 *  Run the given operation on each registered subsystem.
	 *  Any new subsystems registered during this operation will also be visited.
	 *  It is not permitted to remove subsystems (e.g. by calling DeactivateExternalSubsystem) during this operation.
	 */
	ENGINE_API void ForEachSubsystem(TFunctionRef<void(USubsystem*)> Operation) const;

	/** Perform an operation on all subsystems that derive from the given class */
	ENGINE_API void ForEachSubsystemOfClass(UClass* SubsystemClass, TFunctionRef<void(USubsystem*)> Operation) const;

	/** Remove all subsystems in this list of packages */
	ENGINE_API void RemoveSubsystemsInPackages(TConstArrayView<UPackage*> Packages);

	/** Get the outer object associated with this collection */
	UObject* GetOuter() const { return Outer; }

	/** Adds a subsystem to the collection and initializes it */
	ENGINE_API USubsystem* AddAndInitializeSubsystem(UClass* SubsystemClass);

	/** Add and initialize many subsystems at once, handling dependencies between them as with Initialize */
	ENGINE_API void AddAndInitializeSubsystems(TConstArrayView<UClass*> SubsystemClasses);

	/** Removes a subsystem to the collection and deinitializes it */
	ENGINE_API void RemoveAndDeinitializeSubsystem(USubsystem* Subsystem);

	// True if we're adding subsystems either from Initialize or piecemeal
	[[nodiscard]] bool IsPopulating() const;

	/* Hooks for subclasses such as FWorldSubsystemCollection */
	// Called after a subsystem has been initialized and added to the collection
	virtual void OnSubsystemAdded(TNonNullPtr<USubsystem> Subsystem) { }

	// Called before a subsystem is de-initialized and remove from the collection 
	virtual void OnPreRemoveSubsystem(TNonNullPtr<USubsystem> Subsystem) { }

private:

	TMap<TObjectPtr<UClass>, TObjectPtr<USubsystem>> SubsystemMap;

	// Keys for this map should be a base class or interface implemented by a class in SubsystemMap
	mutable TMap<UClass*, TUniquePtr<FSubsystemArray>> SubsystemArrayMap;

	UClass* BaseType;

	UObject* Outer;

	// Non-null inside Initialize and other places that we add subsystems
	FSubsystemCollectionInitialization* Initialization = nullptr;

	mutable bool bIterating = false; // True if iterating over SubsystemMap

	bool bDeinitComplete = false; // True if the subsystem collection has finished being Deinitialized

	FDelegateHandle ModulesUnloadedHandle;

private:
	friend class FSubsystemModuleWatcher;

	/** For a class which has already been validated as something we should create, initialize the subsystem and add it to the collection */
	USubsystem* AddAndInitializeValidatedSubsystem(UClass* SubsystemClass);

	/**
	 * Private to keep space for making this threadsafe in future as it returns a reference
	 * If this returns a non-null pointer, there will be at least one element in the array
	 */
	ENGINE_API const FSubsystemArray* FindAndPopulateSubsystemArray(UClass* SubsystemClass) const;

	/** Add Instances of the specified Subsystem classes to all existing SubsystemCollections of the correct type */
	static ENGINE_API void AddAllInstances(TConstArrayView<UClass*> SubsystemClasses);

	/** Remove Instances of the specified Subsystem classes from all existing SubsystemCollections of the correct type */
	static ENGINE_API void RemoveAllInstances(TConstArrayView<UClass*> SubsystemClasses);
};

template<typename TBaseType>
class FSubsystemCollection : public FSubsystemCollectionBase, public FGCObject
{
public:
	/** Get a Subsystem by type */
	template <typename TSubsystemClass>
	TSubsystemClass* GetSubsystem(const TSubclassOf<TSubsystemClass>& SubsystemClass) const
	{
		static_assert(TIsDerivedFrom<TSubsystemClass, TBaseType>::IsDerived, "TSubsystemClass must be derived from TBaseType");

		// A static cast is safe here because we know SubsystemClass derives from TSubsystemClass if it is not null
		return static_cast<TSubsystemClass*>(GetSubsystemInternal(SubsystemClass));
	}

	/** Get a list of Subsystems by type */
	template <typename TSubsystemClass>
	TArray<TSubsystemClass*> GetSubsystemArrayCopy(const TSubclassOf<TSubsystemClass>& SubsystemClass) const
	{
		// Force a compile time check that TSubsystemClass derives from TBaseType, the internal code only enforces it's a USubsystem
		TSubclassOf<TBaseType> SubsystemBaseClass = SubsystemClass;
		return FSubsystemCollectionBase::GetSubsystemArrayCopy<TSubsystemClass>(SubsystemBaseClass);
	}

	/** Perform an operation on all subsystems of a given type in the collection */
	void ForEachSubsystem(TFunctionRef<void(TBaseType*)> Operation, const TSubclassOf<TBaseType>& SubsystemClass = {}) const
	{
		// Force a compile time check that TSubsystemClass derives from TBaseType, the internal code only enforces it's a USubsystem
		ForEachSubsystemOfClass(SubsystemClass , [Operation=MoveTemp(Operation)](USubsystem* Subsystem){
			Operation(CastChecked<TBaseType>(Subsystem));
		});
	}

	/* FGCObject Interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		FSubsystemCollectionBase::AddReferencedObjects(nullptr, Collector);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("FSubsystemCollection");
	}
	
public:

	/** Construct a FSubsystemCollection, pass in the owning object almost certainly (this). */
	FSubsystemCollection()
		: FSubsystemCollectionBase(TBaseType::StaticClass())
	{
	}
};

/** Subsystem collection which delegates UObject references to its owning UObject (object needs to implement AddReferencedObjects and forward call to Collection */
template<typename TBaseType>
class FObjectSubsystemCollection : public FSubsystemCollectionBase
{
public:
	/** Get a Subsystem by type */
	template <typename TSubsystemClass>
	TSubsystemClass* GetSubsystem(const TSubclassOf<TSubsystemClass>& SubsystemClass) const
	{
		static_assert(TIsDerivedFrom<TSubsystemClass, TBaseType>::IsDerived, "TSubsystemClass must be derived from TBaseType");

		// A static cast is safe here because we know SubsystemClass derives from TSubsystemClass if it is not null
		return static_cast<TSubsystemClass*>(GetSubsystemInternal(SubsystemClass));
	}

	/** Get a list of Subsystems by type */
	template <typename TSubsystemClass>
	TArray<TSubsystemClass*> GetSubsystemArrayCopy(const TSubclassOf<TSubsystemClass>& SubsystemClass) const
	{
		// Force a compile time check that TSubsystemClass derives from TBaseType, the internal code only enforces it's a USubsystem
		TSubclassOf<TBaseType> SubsystemBaseClass = SubsystemClass;
		return FSubsystemCollectionBase::GetSubsystemArrayCopy<TSubsystemClass>(SubsystemBaseClass);
	}

	/** Perform an operation on all subsystems in the collection */
	void ForEachSubsystem(TFunctionRef<void(TBaseType*)> Operation, const TSubclassOf<TBaseType>& SubsystemClass = {}) const
	{
		ForEachSubsystemOfClass(SubsystemClass, [Operation=MoveTemp(Operation)](USubsystem* Subsystem){
			Operation(CastChecked<TBaseType>(Subsystem));
		});
	}

	template <typename TSubsystemInterface>
	void ForEachSubsystemWithInterface(TFunctionRef<void(TBaseType*)> Operation) const
	{
		UClass* SubsystemInterfaceClass = TSubsystemInterface::StaticClass();
		ForEachSubsystemOfClass(SubsystemInterfaceClass, [Operation = MoveTemp(Operation)](USubsystem* Subsystem)
		{
			Operation(CastChecked<TBaseType>(Subsystem));
		});
	}

public:

	/** Construct a FSubsystemCollection, pass in the owning object almost certainly (this). */
	FObjectSubsystemCollection()
		: FSubsystemCollectionBase(TBaseType::StaticClass())
	{
	}
};