// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystem.h"
#include "Engine/EngineTypes.h"
#include "Tickable.h"
#include "WorldSubsystem.generated.h"

/**
 * UWorldSubsystem
 * Base class for auto instanced and initialized systems that share the lifetime of a UWorld
 * FWorldSubsystemCollection handles calling PostInitialize and OnWorldBeginPlay for already initialized worlds
 */
UCLASS(Abstract, MinimalAPI)
class UWorldSubsystem : public USubsystem
{
	GENERATED_BODY()

public:
	/**
	 * Returns a pointer to the UWorld this subsystem is contained within.
	 * @note This will return null for default objects
	 */
	ENGINE_API virtual UWorld* GetWorld() const override final;

	/**
	 * Returns a reference to the UWorld this subsystem is contained within.
	 * @note This should not be called on default object since the method assumes a valid outer world.
	 */
	ENGINE_API UWorld& GetWorldRef() const;

	/** Calls DoesSupportWorldType to check if this subsystem should be created */
	ENGINE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;

	/** Called once all UWorldSubsystems have been initialized */
	ENGINE_API virtual void PostInitialize();

	/** Called once all UWorldSubsystems are about to be deinitialized */
	virtual void PreDeinitialize() {}
	
	/** Called when world is ready to start gameplay before the game mode transitions to the correct state and call BeginPlay on all actors */
	ENGINE_API virtual void OnWorldBeginPlay(UWorld& InWorld);

	/** Called when world is ready to stop gameplay before the game mode transitions to the correct state and call EndPlay on all actors */
	virtual void OnWorldEndPlay(UWorld& InWorld) {}

	/** Called after world components (e.g. line batcher and all level components) have been updated */
	virtual void OnWorldComponentsUpdated(UWorld& World) {}

	/** Returns true once the subsystem has executed PostInitialize **/
	bool HasCalledPostInitialize() const { return bHasCalledPostInitialize; }

	/** Returns true once the subsystem has executed OnWorldBeginPlay **/
	bool HasCalledBeginPlay() const { return bHasCalledBeginPlay; }

	/** Called during initialization to ensure the correct callbacks have happened */
	ENGINE_API void EnsureHasCalledPostInitialize() const;
	ENGINE_API void EnsureHasCalledBeginPlay() const;

protected:
	/** 
	 * Override this to change what types of worlds support this subsystem.
	 * By default, world subsystems support game, PIE, and editor worlds but not preview worlds.
	 */
	ENGINE_API virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const;

	bool bHasCalledPostInitialize : 1 = false;
	bool bHasCalledBeginPlay : 1 = false;
};

/**
 * UTickableWorldSubsystem
 * Base class for auto instanced and initialized systems that share the lifetime of a UWorld and are ticking along with it.
 * With the default implementation, it will start ticking after Initialize and stop during Deinitialize,
 * and it will call IsTickable every frame (defaults to true) before calling Tick.
 * Subclasses must forward calls to the Initialize/Deinitialize functions to correctly enable ticking.
 */
UCLASS(Abstract, MinimalAPI)
class UTickableWorldSubsystem : public UWorldSubsystem, public FTickableGameObject
{
	GENERATED_BODY()

public:
	ENGINE_API UTickableWorldSubsystem();

	// FTickableGameObject implementation Begin
	ENGINE_API UWorld* GetTickableGameObjectWorld() const override;
	ENGINE_API virtual ETickableTickType GetTickableTickType() const override;
	ENGINE_API virtual bool IsAllowedToTick() const override final;
	ENGINE_API virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override PURE_VIRTUAL(UTickableWorldSubsystem::GetStatId, return TStatId(););
	// FTickableGameObject implementation End

	// USubsystem implementation Begin
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;
	ENGINE_API virtual void BeginDestroy() override;
	// USubsystem implementation End

	/** Returns true if Initialize has been called but Deinitialize has not */
	bool IsInitialized() const { return bInitialized; }

private:
	bool bInitialized = false;
};
