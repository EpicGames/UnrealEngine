// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "UObject/ObjectKey.h"

#define SUBTITLE_SCREEN_DEPTH_FOR_3D 0.1f

class FCanvas;

/** HACK: Delegate for allowing displays to hijack subtitle text from the subtitle manager and get around the Canvas display */
DECLARE_MULTICAST_DELEGATE_OneParam(FSubtitleManagerSetSubtitleText, const FText& /* SubtitleText */)

/**
 * A collection of subtitles, rendered at a certain priority.
 */
class FActiveSubtitle
{
public:

	FActiveSubtitle(int32 InIndex, float InPriority, bool InbSplit, bool InbSingleLine, TArray<FSubtitleCue> InSubtitles)
		: Index(InIndex)
		, Priority(InPriority)
		, bSplit(InbSplit)
		, bSingleLine(InbSingleLine)
		, Subtitles(MoveTemp(InSubtitles))
	{ }

	/** Index into the Subtitles array for the currently active subtitle in this set. */
	int32 Index;

	/** Priority for this set of subtitles, used by FSubtitleManager to determine which subtitle to play. */
	float Priority;
	
	/** Whether subtitles have come in pre-split, or after they have been split programmatically. */
	bool bSplit;
	
	/** Whether subtitles should be displayed as a sequence of single lines. */
	bool bSingleLine;
	
	/** A set of subtitles. */
	TArray<FSubtitleCue>	Subtitles;
};


struct FQueueSubtitleParams
{
	FQueueSubtitleParams(const TArray<FSubtitleCue>& InSubtitles)
		: Subtitles(InSubtitles)
	{ }

	uint64 AudioComponentID;
	TWeakObjectPtr<UWorld> WorldPtr;
	PTRINT WaveInstance;
	float SubtitlePriority;
	float Duration;
	uint8 bManualWordWrap : 1;
	uint8 bSingleLine : 1;
	const TArray<FSubtitleCue>& Subtitles;
	float RequestedStartTime;
};


/**
 * Subtitle manager. Handles prioritization and rendering of subtitles.
 */
class FSubtitleManager
{
public:

	/** Kill all the subtitles. */
	void KillAllSubtitles(void);

	/**
	 * Kill specified subtitles.
	 *
	 * @param SubtitleId Identifier of the subtitles to kill.
	 */
	void KillSubtitles(PTRINT SubtitleID);

	/**
	 * Trim the SubtitleRegion to the safe 80% of the canvas.
	 *
	 * @param Canvas The canvas to trim to.
	 * @param InOutSubtitleRegion The trim region.
	 */
	void TrimRegionToSafeZone(FCanvas* Canvas, FIntRect& InOutSubtitleRegion);

	/**
	 * If any of the active subtitles need to be split into multiple lines, do so now.
	 *
	 * Note: This assumes the width of the subtitle region does not change while the subtitle is active.
	 */
	void SplitLinesToSafeZone(FCanvas* Canvas, FIntRect & SubtitleRegion);

	/**
	 * Find the highest priority subtitle from the list of currently active ones.
	 *
	 * @param CurrentTime The time at which to find the subtitle.
	 * @return Found subtitle identifier.
	 */
	PTRINT FindHighestPrioritySubtitle(float CurrentTime);

	/**
	 * Add an array of subtitles to the active list
	 *
	 * @param SubTitleID The controlling id that groups sets of subtitles together.
	 * @param Priority Used to prioritize subtitles; higher values have higher priority.  Subtitles with a priority 0.0f are not displayed.
	 * @param StartTime Time at which the subtitles start (in seconds).
	 * @param SoundDuration Time  after which the subtitles do not display (in seconds).
	 * @param Subtitles Collection of lines of subtitle and time offset to play them.
	 */
	ENGINE_API void QueueSubtitles(PTRINT SubtitleID, float Priority, bool bManualWordWrap, bool bSingleLine, float SoundDuration, const TArray<FSubtitleCue>& Subtitles, float InStartTime, float InCurrentTime);

	static void QueueSubtitles(const FQueueSubtitleParams& QueueSubtitlesParams);

	/**
	 * Draws a subtitle at the specified pixel location.
	 */
	void DisplaySubtitle(FCanvas* Canvas, FActiveSubtitle* Subtitle, FIntRect & Parms, const FLinearColor& Color);

	/**
	 * Display the currently queued subtitles and cleanup after they have finished rendering.
	 *
	 * @param Canvas Where to render the subtitles.
	 * @param CurrentTime Current world time.
	 */
	ENGINE_API void DisplaySubtitles(FCanvas* InCanvas, FIntRect & SubtitleRegion, float InAudioTimeSeconds);

	/**
	 * Whether there are any active subtitles.
	 *
	 * @return true if there are any active subtitles, false otherwise.
	 */
	bool HasSubtitles() 
	{ 
		return (ActiveSubtitles.Num() > 0); 
	}

	/**
	 * Get the height of currently rendered subtitles.
	 *
	 * @return Subtitle height (in pixels), or 0 if none are rendering.
	 */
	float GetCurrentSubtitlesHeight()
	{
		return CurrentSubtitleHeight;
	}

	/**
	 * Get the subtitle manager singleton instance.
	 *
	 * @return Subtitle manager.
	 */
	ENGINE_API static FSubtitleManager* GetSubtitleManager();

	/** HACK: Returns the delegate for setting subtitle text */
	FSubtitleManagerSetSubtitleText&  OnSetSubtitleText() { return OnSetSubtitleTextDelegate; }

	/**
	 * HACK: Accept a movie subtitle and display it to all currently registered displays. This will not
	 * work with the old Canvas drawing system.
	 *
	 * @param SubtitleOwner		The owner of the movie subtitle. If this is null, the subtitle will not be set
	 * @param MovieSubtitles	The subtitle text to display to all currently active displays
	 */
	void ENGINE_API SetMovieSubtitle(UObject* SubtitleOwner, const TArray<FString>& Subtitles);

private:

	/** HACK: Displays a subtitle through objects that are bound to the delegate */
	void DisplaySubtitle_ToDisplays(FActiveSubtitle* Subtitle);

	/** HACK: Builds a subtitle string from all cues in the supplied subtitle */
	FString SubtitleCuesToString(FActiveSubtitle* Subtitle);

private:

	FSubtitleManagerSetSubtitleText OnSetSubtitleTextDelegate;

	/** The set of active, prioritized subtitles. */
	TMap<PTRINT, FActiveSubtitle> ActiveSubtitles;

	/** The current height of the subtitles. */
	float CurrentSubtitleHeight = 0;

	/** HACK: The currently active movie subtitle for given object owners. Each owner can have one movie subtitle at a time */
	TMap<FObjectKey, TSharedPtr<FActiveSubtitle>> ActiveMovieSubtitles;
};
