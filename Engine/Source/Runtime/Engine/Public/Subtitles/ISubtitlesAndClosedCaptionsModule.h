// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SubclassOf.h"

class UAssetUserData;

class ISubtitlesAndClosedCaptionsModule : public IModuleInterface, public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static const FName FeatureName = FName("SubtitlesAndClosedCaptions");
		return FeatureName;
	}

	virtual TSubclassOf<UAssetUserData> GetAssetUserDataClass() const = 0;
};
