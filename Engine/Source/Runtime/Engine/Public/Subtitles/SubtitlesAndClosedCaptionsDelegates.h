// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Delegates/Delegate.h"
#include "SubtitlesAndClosedCaptionsTypes.h"

#define UE_API ENGINE_API

struct FSubtitleAssetData;
struct FQueueSubtitleParameters;
enum class ESubtitleTiming : uint8;

class FSubtitlesAndClosedCaptionsDelegates
{
public:
	// Have the subtitle subsystem to queue a subtitle to be displayed
	static UE_API TDelegate<void(FQueueSubtitleParameters, const ESubtitleTiming)> QueueSubtitle;

	static UE_API TDelegate<void(const FName& SoundName, const FName& PackageName, const float Duration), FDefaultTSDelegateUserPolicy> QueueSubtitleFromSoundWaveName;

	static UE_API TDelegate<bool(FSubtitleAssetData)> IsSubtitleActive;

	static UE_API TDelegate<void(FSubtitleAssetData)> StopSubtitle;

	static UE_API TDelegate<void()> StopAllSubtitles;
};

#undef UE_API
