// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/AssetUserData.h"
#include "SubtitlesAndClosedCaptionsTypes.generated.h"

#define UE_API ENGINE_API

class FCanvas;
class UAssetUserData;

// Externally-Timed subtitles must be manually added and removed with USubtitlesSubsystem::QueueSubtitle and ::StopSubtitle.
// For the initial delay before becoming visible, use USubtitleAssetUserData::StartOffset instead of this enum.
UENUM()
enum class ESubtitleTiming : uint8
{
	InternallyTimed,
	ExternallyTimed
};

// SubtitlesAudioSubsystem will automatically queue and stop subtitles embedded in USoundBase sounds using play/stop callbacks.
// This enum controls whether the stopping should be done automatically, or if the subtitle should be timed using its existing duration.
UENUM(BlueprintType)
enum class ESubtitleDurationType : uint8
{
	UseSoundDuration UMETA(ToolTip = "Automatically stops this subtitle when the sound stops playing (only useful when attached as AssetUserData"),
	UseDurationProperty UMETA(ToolTip = "Time this subtitle using its Duration property")
};

// Subtitle type for type-specific rendering.
UENUM()
enum class ESubtitleType : uint8
{
	Subtitle,
	ClosedCaption,
	AudioDescription
};

// Minimum duration to display subtitle.
static constexpr float SubtitleMinDuration		= 0.05f;

// Default value to initialize subtitle duration to. Used by SoundWaves to check if they should manually set the duration.
static constexpr float SubtitleDefaultDuration	= 3.f;

USTRUCT(MinimalAPI, BlueprintType)
struct FSubtitleAssetData
{
	GENERATED_BODY()

	// #SUBTITLES_PRD: carried over from FSubtitleCue, still required

	// The text to appear in the subtitle.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subtitles", meta = (Multiline = true))
	FText Text;

	// #SUBTITLES_PRD: carried over from FSubtitleCue, still required

	// Whether to automatically unqueue the subtitle using an associated Sound, or to use the duration supplied below.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subtitles|Duration")
	ESubtitleDurationType SubtitleDurationType = ESubtitleDurationType::UseSoundDuration;

#if WITH_EDITORONLY_DATA
	// Editor-only helper value for SubtitleDefaultDuration's EditCondition.
	UPROPERTY(Transient, VisibleDefaultsOnly, Category = "Subtitles|Duration")
	bool bCanEditDuration = (SubtitleDurationType == ESubtitleDurationType::UseDurationProperty);
#endif // WITH_EDITORONLY_DATA

	// Time to display in seconds.
	// Defaulted to 3 seconds so when adding new subtitles it's not required to enter a placeholder Duration.
	// Duration can be be set by ingestion pipelines when importing Subtitles in bulk
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subtitles|Duration", meta = (ClampMin = 0.05f, EditCondition = "bCanEditDuration"))
	float Duration = SubtitleDefaultDuration;

	// Some subtitles have a delay before they're allowed to be displayed (primarily from the legacy system).
	// StartOffset measures how long in Seconds, after queuing, before the subtitle is allowed to enter the active subtitles queue.
	// ESubtitleTiming::ExternallyTimed does not effect this initial delay.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subtitles", meta = (ClampMin = 0.f))
	float StartOffset = 0.f;

	// #SUBTITLES_PRD: Priority comes from USoundBase::GetSubtitlePriority, USoundCue::GetSubtitlePriority and USoundWave::GetSubtitlePriority
	// Consolidate various subtitle properties throughout sound/audio code into this new subtitles plugin.

	// The priority of the subtitle.  Defaults to 1.  Higher values will play instead of lower values.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subtitles")
	float Priority = 1.f;

	// Subtitle type for type-specific rendering.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Subtitles")
	ESubtitleType SubtitleType = ESubtitleType::Subtitle;

#if WITH_EDITORONLY_DATA
	// Author comments, editor-only.
	UPROPERTY(EditAnywhere, Category = "Subtitles")
	FString Comment;
#endif // WITH_EDITORONLY_DATA

	// The struct needs a custom equality test operator because FText doesn't have one overloaded: need to use EqualTo().
	// Don't compare the Comment property because it isn't compiled outside the editor.
	const bool operator==(const FSubtitleAssetData& Other) const
	{
		return ((Text.EqualTo(Other.Text) && (SubtitleDurationType == Other.SubtitleDurationType) && (Duration == Other.Duration) 
			&& (StartOffset == Other.StartOffset) && (Priority == Other.Priority) && (SubtitleType == Other.SubtitleType)));
	}
};

/**
 *  Base class for subtitle data being attached to assets
 */
UCLASS(MinimalAPI, BlueprintType)
class USubtitleAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Subtitles")
	TArray<FSubtitleAssetData> Subtitles;

	// In seconds, the longest any subtitle in the Subtitles array will last, including any initial offset starts.
	float GetMaximumDuration() const
	{
		float Duration = 0.f;
		for (const FSubtitleAssetData& Entry : Subtitles)
		{
			const float EntryDuration = Entry.Duration + Entry.StartOffset;
			Duration = FMath::Max(Duration, EntryDuration);
		}

		return Duration;
	};

	// These functions ensure Duration values are only editable when DurationType == UseDurationProperty:
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
#endif // WITH_EDITOR
};

struct FQueueSubtitleParameters
{
	const FSubtitleAssetData Subtitle;
	TOptional<float> Duration;
	TOptional<float> StartOffset;	// Scrubbing subtitles in the sequencer needs an override for StartOffset.
};

#undef UE_API
