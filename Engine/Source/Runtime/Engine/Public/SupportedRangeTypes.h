// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

namespace RangeVisibilityUtils
{
	extern ENGINE_API TSet<FName> StructsSupportingRangeVisibility;
}