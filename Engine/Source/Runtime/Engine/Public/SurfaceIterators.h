// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SurfaceIterators.h: Model surface iterators.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Model.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Level filters
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Level filter that passes all levels.
 */
class FAllSurfaceLevelFilter
{
public:
	static inline bool IsSuitable(const ULevel* Level)
	{
		return true;
	}
};

/**
 * Level filter that passes the current level.
 */
class FCurrentLevelSurfaceLevelFilter
{
public:
	static inline bool IsSuitable(const ULevel* Level)
	{
		return Level->IsCurrentLevel();
	}
};


/** The default level filter. */
typedef FAllSurfaceLevelFilter DefaultSurfaceLevelFilter;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// TSurfaceIteratorBase
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Iterates over the selected surfaces of all levels in the specified UWorld.
 */
template< class SurfaceFilter, class LevelFilter=DefaultSurfaceLevelFilter >
class TSurfaceIteratorBase
{
public:
	typedef SurfaceFilter	SurfaceFilterType;
	typedef LevelFilter		LevelFilterType;

	inline FBspSurf* operator*()
	{
		check( CurrentSurface );
		return CurrentSurface;
	}
	inline FBspSurf* operator->()
	{
		check( CurrentSurface );
		return CurrentSurface;
	}
	inline explicit operator bool() const
	{
		return !bReachedEnd;
	}

	inline UModel* GetModel()
	{
		check( !bReachedEnd );
		return World->GetLevel(LevelIndex)->Model;
	}

	inline int32 GetSurfaceIndex() const
	{
		check( !bReachedEnd );
		return SurfaceIndex;
	}

	inline int32 GetLevelIndex() const
	{
		check( !bReachedEnd );
		return LevelIndex;
	}

	inline UWorld* GetWorld()
	{
		check( !bReachedEnd );
		return World;
	}
	inline ULevel* GetLevel() const
	{
		check( !bReachedEnd );
		return World->GetLevel(LevelIndex);
	}
	void operator++()
	{
		CurrentSurface = NULL;

		ULevel* Level = World->GetLevel(LevelIndex);
		while ( !bReachedEnd && !CurrentSurface )
		{
			// Skip over unsuitable levels or levels for whom all surfaces have been visited.
			if ( !LevelFilter::IsSuitable( Level ) || ++SurfaceIndex >= Level->Model->Surfs.Num() )
			{
				if ( ++LevelIndex >= World->GetNumLevels() )
				{
					// End of level list.
					bReachedEnd = true;
					LevelIndex = 0;
					SurfaceIndex = 0;
					CurrentSurface = NULL;
					break;
				}
				else
				{
					// Get the next level.
					Level = World->GetLevel(LevelIndex);
					if ( !LevelFilter::IsSuitable( Level ) )
					{
						continue;
					}

					SurfaceIndex = 0;
					// Gracefully handle levels with no surfaces.
					if ( SurfaceIndex >= Level->Model->Surfs.Num() )
					{
						continue;
					}
				}
			}
			CurrentSurface = &Level->Model->Surfs[SurfaceIndex];
			if ( !SurfaceFilterType::IsSuitable( CurrentSurface ) )
			{
				CurrentSurface = NULL;
			}
		}
	}

protected:
	TSurfaceIteratorBase(UWorld* InWorld)
		:	bReachedEnd( false )
		,	World( InWorld )
		,	LevelIndex( 0 )
		,	SurfaceIndex( -1 )
		,	CurrentSurface( NULL )
	{}

private:
	/** true if the iterator has reached the end. */
	bool		bReachedEnd;

	/** The world whose surfaces we're iterating over. */
	UWorld*		World;

	/** Current index into UWorld's Levels array. */
	int32			LevelIndex;

	/** Current index into the current level's Surfs array. */
	int32			SurfaceIndex;

	/** Current surface pointed at by the iterator. */
	FBspSurf*	CurrentSurface;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// TSurfaceIterator
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Surface filter that passes all surfaces.
 */
class FAllSurfaceFilter
{
public:
	static inline bool IsSuitable(const FBspSurf* Surface)
	{
		return true;
	}
};

/**
 * Iterates over selected surfaces of the specified UWorld.
 */
template< class LevelFilter=DefaultSurfaceLevelFilter >
class TSurfaceIterator : public TSurfaceIteratorBase<FAllSurfaceFilter, LevelFilter>
{
public:
	typedef TSurfaceIteratorBase<FAllSurfaceFilter, LevelFilter> Super;

	TSurfaceIterator(UWorld* InWorld)
		: Super( InWorld )
	{
		// Initialize members by advancing to the first valid surface.
		++(*this);
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// TSelectedSurfaceIterator
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Surface filter that passes selected surfaces.
 */
class FSelectedSurfaceFilter
{
public:
	static inline bool IsSuitable(const FBspSurf* Surface)
	{
		return (Surface->PolyFlags & PF_Selected) ? true : false;
	}
};

/**
 * Iterates over selected surfaces of the specified UWorld.
 */
template< class LevelFilter=DefaultSurfaceLevelFilter >
class TSelectedSurfaceIterator : public TSurfaceIteratorBase<FSelectedSurfaceFilter, LevelFilter>
{
public:
	typedef TSurfaceIteratorBase<FSelectedSurfaceFilter, LevelFilter> Super;

	TSelectedSurfaceIterator(UWorld* InWorld)
		: Super( InWorld )
	{
		// Initialize members by advancing to the first valid surface.
		++(*this);
	}
};

