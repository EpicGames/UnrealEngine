// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SystemSettings.cpp: Unreal engine HW compat scalability system.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ShowFlags.h"

/** 
 * Struct that holds the actual data for the system settings. 
 *
 * Uses the insane derivation for backwards compatibility purposes only. Would be cleaner to use composition.
 */
struct FSystemSettingsData 
{
	/** ctor */
	FSystemSettingsData();

	/** loads settings from the given section in the given ini */
	ENGINE_API void LoadFromIni(const TCHAR* IniSection, const FString& IniFilename = GEngineIni, bool bAllowMissingValues = true, bool* FoundValues=NULL);
};

UE_DEPRECATED(5.1, "Use UE::ConfigUtilities::OnSetCVarFromIniEntry")
ENGINE_API void OnSetCVarFromIniEntry(const TCHAR *IniFile, const TCHAR *Key, const TCHAR* Value, uint32 SetBy, bool bAllowCheating = false);


/**
 * Class that actually holds the current system settings
 */
class FSystemSettings : public FSystemSettingsData, public FExec, private FNoncopyable
{
public:
	/** Constructor, initializing all member variables. */
	FSystemSettings();

	/**
	 * Initializes system settings and included texture LOD settings.
	 *
	 * @param bSetupForEditor	Whether to initialize settings for Editor
	 */
	ENGINE_API void Initialize( bool bSetupForEditor );

	/** Applies setting overrides based on command line options. */
	void ApplyOverrides();
	
	//~ Begin Exec Interface
#if UE_ALLOW_EXEC_COMMANDS
	virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override;
#endif
	//~ End Exec Interface

	/** Mask where 1 bits mean we want to force the engine show flag to be off */
	const FEngineShowFlags& GetForce0Mask() const { return Force0Mask; }
	/** Mask where 1 bits mean we want to force the engine show flag to be on */
	const FEngineShowFlags& GetForce1Mask() const { return Force1Mask; }

private:
	/** Since System Settings is called into before GIsEditor is set, we must cache this value. */
	bool bIsEditor;

	/** Mask where 1 bits mean we want to force the engine show flag to be off */
	FEngineShowFlags Force0Mask;
	/** Mask where 1 bits mean we want to force the engine show flag to be on */
	FEngineShowFlags Force1Mask;

	/**
	 * Loads settings from the ini. (purposely override the inherited name so people can't accidentally call it.)
	 */
	void LoadFromIni();

	/** 
	 * Makes System Settings take effect on the rendering thread 
	 */
	void RegisterShowFlagConsoleVariables();

	/** 
	 * Console variable sink - called when cvars have changed
	 */
	void CVarSink();
};

/**
 * Global system settings accessor
 */
extern ENGINE_API FSystemSettings GSystemSettings;
