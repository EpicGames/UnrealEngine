// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/World.h"
#include "Async/TaskGraphFwd.h"
#include "Delegates/Delegate.h"

#include "TaskSyncManager.generated.h"

// NOTE: This header file is considered Experimental and will remain so until the release of 5.6. It will be split into multiple headers

namespace UE::Tick
{

/** The type of event to execute at the sync point */
UENUM()
enum class ESyncPointEventType : uint8
{
	Invalid,

	/** A simple event that cannot activate any code directly */
	SimpleEvent,

	/** A task that executes code on the game thread, can be used for batching */
	GameThreadTask,

	/** High priority game thread task, will run before normal ticks */
	GameThreadTask_HighPriority,

	/** A task that executes code on a worker thread, can be used for batching */
	WorkerThreadTask,

	/** A task that executes code on a worker thread, can be used for batching */
	WorkerThreadTask_HighPriority,
};

/** Rules for when a sync point's task will be activated/dispatched during a frame. Execution will also need to wait on any task dependencies */
UENUM()
enum class ESyncPointActivationRules : uint8
{
	Invalid,

	/** Always activate, dispatch during FirstPossibleTickGroup */
	AlwaysActivate,

	/** Triggered once manually with TriggerSyncPoint, or as a backup from LastPossibleTickGroup */
	WaitForTrigger,

	/** Trigger once when there is any requested work and no reserved work */
	WaitForAllWork,

	/** Can activate multiple times per frame, whenever there is any requested work to perform */
	ActivateForAnyWork,
};

/**
 * Description of a registered sync event that will happen once per frame according to specific rules
 */
USTRUCT()
struct FSyncPointDescription
{
	GENERATED_BODY()

	/** Name of the sync point that will be used for lookup */
	UPROPERTY(EditDefaultsOnly, Category="Task")
	FName RegisteredName;

	/** Name of what added this description, default means it was loaded from settings */
	FName SourceName;

	/** The kind of task async task that is used to implement this sync point */
	UPROPERTY(EditDefaultsOnly, Category="Task")
	ESyncPointEventType EventType = ESyncPointEventType::Invalid;

	/** Rules for when and how a sync point can be activated */
	UPROPERTY(EditDefaultsOnly, Category="Task")
	ESyncPointActivationRules ActivationRules = ESyncPointActivationRules::Invalid;

	/** The first tick group this could be triggered during */
	UPROPERTY(EditDefaultsOnly, Category="Task")
	TEnumAsByte<enum ETickingGroup> FirstPossibleTickGroup = TG_PrePhysics;

	/** The last possible tick group this will be triggered during, and when it will be forcibly triggered as a backup */
	UPROPERTY(EditDefaultsOnly, Category="Task")
	TEnumAsByte<enum ETickingGroup> LastPossibleTickGroup = TG_LastDemotable;

	/** Array of other sync groups that this will tick after. To match the normal tick behavior these will be ignored if thee sync point is not active */
	UPROPERTY(EditDefaultsOnly, Category="Task")
	TArray<FName> PrerequisiteSyncGroups;

	/** True if this valid and has a name */
	inline bool IsValid() const
	{
		return !RegisteredName.IsNone();
	}

	/** True if this was loaded from settings and has a default source */
	inline bool WasLoadedFromSettings() const 
	{
		return SourceName.IsNone();
	}
};

/** Settings for synchronizing tasks and ticking across the engine */
UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "Task Synchronization"), MinimalAPI)
class UTaskSyncManagerSettings : public UDeveloperSettings
{
	GENERATED_BODY()

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool SupportsAutoRegistration() const override;
#endif

public:
	/** List of all registered task events */
	UPROPERTY(config, EditAnywhere, Category = "Task Synchronization")
	TArray<FSyncPointDescription> RegisteredSyncPoints;
};

/** Simple lambda type */
typedef TUniqueFunction<void()> FWorkFunction;

/** Enum describing the success/failure of a task sync operation */
enum class ESyncOperationResult : uint8
{
	/** Unknown or invalid status */
	Invalid,

	/** The requested operation was successful */
	Success,

	// Error conditions TODO some of these are redundant

	/** The FSyncPointId is completely invalid */
	SyncPointInvalid,
	/** Sync point is not registered */
	SyncPointNotRegistered,
	/** Sync point does not exist for the specific batch */
	SyncPointNotFound,
	/** Sync point was specifically disabled */
	SyncPointDisabled,
	
	/** Sync Point status is incorrect for the requested operation, check the status */
	SyncPointStatusIncorrect,
	/** The sync point's event type does not support this operation, such as trying to add a tick function to a simple event */
	EventTypeIncorrect,
	/** This is not supported for the sync point's activation rules */
	ActivationRulesIncorrect,

	/** BatchContextId was not found */
	BatchNotFound,
	/** WorldContextId was not found */
	WorldNotFound,
	/** Operation called on incorrect thread or it cannot determine thread context */
	ThreadIncorrect,
};

/** Enum describing the current status of a specific sync point in the current frame */
enum class ESyncPointStatus : uint8
{
	/** Status of sync point is not known, probably because it could not be found */
	Unknown,

	/** Sync point tick function has not yet been registered with tick system */
	TaskNotRegistered,
	/** Sync point tick is registered but no task has been made yet, this is the state between tick frames */
	TaskNotCreated,
	/** Task has been created but not yet dispatched. It may be before the first tick group */
	TaskCreated,

	/** Has not been dispatched, waiting for a trigger */
	DispatchWaitingForTrigger,
	/** Has not been dispatched, waiting for work requests */
	DispatchWaitingForWork,
	
	/** Has been dispatched, but has not started execution. This could be waiting on a prerequisite task */
	Dispatched,
	/** Was dispatched and has started execution */
	Executing,
	/** A follow up task has been dispatched to waiting for more work before executing again */
	WaitingForMoreWork,
	/** Not executing, but can start a manual task if needed */
	WaitingForManualTask,
	/** Completely done executing for the frame */
	ExecutionComplete,
};

/** 
 * Result structure returned from all task sync manager operations.
 * This holds an error code as well as the status of the sync point at operation time, if known.
 */
struct FTaskSyncResult
{
	ESyncOperationResult OperationResult;
	ESyncPointStatus SyncStatus;

	/** Invalid result by default */
	FTaskSyncResult()
		: OperationResult(ESyncOperationResult::Invalid)
		, SyncStatus(ESyncPointStatus::Unknown)
	{}

	/** Create from an operation result, defaulting to an unknown sync point status */
	FTaskSyncResult(ESyncOperationResult Result, ESyncPointStatus Status = ESyncPointStatus::Unknown)
		: OperationResult(Result)
		, SyncStatus(Status)
	{}

	/** True if the result has been initialized by an operation */
	inline bool IsValid() const
	{
		return OperationResult != ESyncOperationResult::Invalid;
	}

	/** True if the operation was successful */
	inline bool WasSuccessful() const
	{
		return OperationResult == ESyncOperationResult::Success;
	}

	/** Treat successful as true for if statements */
	inline operator bool() const
	{
		return WasSuccessful();
	}

	/** True if a low level task was created this frame for this sync point. The task may have been destroyed if it has completed */
	inline bool WasTaskCreatedForFrame() const
	{
		return SyncStatus >= ESyncPointStatus::TaskCreated;
	}

	/** True if the sync point has already been dispatched for this frame */
	inline bool WasActivatedForFrame() const 
	{
		return SyncStatus >= ESyncPointStatus::Dispatched;
	}
};


/** Identifies a specific WorldContext that has objects which are allowed to tick and run tasks */
struct FWorldContextId final
{
	// TODO Consider moving this concept into World.h
	typedef int32 FInternalId;

	static constexpr FInternalId InvalidWorldContextId = -1;

	/** Default id which is used for the default game world outside the editor */
	static constexpr FInternalId DefaultWorldContextId = 0;

	/** Construct from a world pointer, if it is null or not part of a tickable world context it will return an invalid id */
	static ENGINE_API FWorldContextId GetContextIdForWorld(const UWorld* World);

	/** Construct an invalid world context id */
	FWorldContextId()
		: WorldId(InvalidWorldContextId)
	{}

	/** Construct from an internal id TODO: May make private */
	explicit FWorldContextId(FInternalId InWorldId)
		: WorldId(InWorldId)
	{}

	/** Construct from a world pointer, if it is null or not part of a tickable world context it will return an invalid id */
	FWorldContextId(const UWorld* InWorld)
	{
		WorldId = GetContextIdForWorld(InWorld).WorldId;
	}

	/** Returns true if this points to a valid tickable world context */
	inline bool IsValid() const
	{
		return WorldId > InvalidWorldContextId;
	}

	inline bool operator==(const FWorldContextId& Other) const
	{
		return WorldId == Other.WorldId;
	}

	inline bool operator!=(const FWorldContextId& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FWorldContextId& Context)
	{
		return static_cast<uint32>(Context.WorldId);
	}

	FInternalId WorldId;
};

/** Identifies a specific batch of tasks, which will be associated with a set of objects in a world context */
struct FBatchContextId final
{
	// Internal type is sometimes used as an array index
	typedef int32 FInternalId;

	static constexpr FInternalId InvalidBatch = -1;

	/** Construct an invalid batch */
	FBatchContextId()
		: BatchId(InvalidBatch)
	{}

	/** Construct from an internal id TODO: May make private */
	explicit FBatchContextId(FInternalId InBatchId)
		: BatchId(InBatchId)
	{}

	/** Returns true if this has been initialized to a valid batch */
	inline bool IsValid() const
	{
		return BatchId > InvalidBatch;
	}

	inline bool operator==(const FBatchContextId& Other) const
	{
		return BatchId == Other.BatchId;
	}

	inline bool operator!=(const FBatchContextId& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FBatchContextId& Context)
	{
		return static_cast<uint32>(Context.BatchId);
	}

	FInternalId BatchId;
};

/** Identifies a specific registered sync point and batch, this can be safely passed around threads */
struct FSyncPointId final
{
	// Internal type is sometimes used as an array index
	typedef int32 FInternalId;

	static constexpr FInternalId InvalidSyncPoint = -1;

	/** Construct an invalid context */
	FSyncPointId()
		: SyncId(InvalidSyncPoint)
	{}

	/** Construct from explicit ids TODO: May make private */
	explicit FSyncPointId(FInternalId InSyncId, FBatchContextId InBatchContext)
		: SyncId(InSyncId)
		, BatchContext(InBatchContext)
	{}

	/** Returns true if this has been initialized to a registered sync point */
	inline bool IsValid() const
	{
		return SyncId > InvalidSyncPoint;
	}

	/** Returns the batch context this was initialized with */
	inline FBatchContextId GetBatchContext() const
	{
		return BatchContext;
	}

	inline bool operator==(const FSyncPointId& Other) const
	{
		return BatchContext == Other.BatchContext && SyncId == Other.SyncId;
	}

	inline bool operator!=(const FSyncPointId& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FSyncPointId& SyncPoint)
	{
		return GetTypeHash(SyncPoint.BatchContext) + SyncPoint.SyncId;
	}

	FInternalId SyncId;
	FBatchContextId BatchContext;
};

/** Used to specify how many times work should be executed as part of a sync point */
enum class ESyncWorkRepetition : uint8
{
	/** Do not perform this work, used to cancel previous requests */
	Never,

	/** Work will be reserved or requested once per call to this function, and will reset for the next frame */
	Once,

	/** Work will be reserved or requested once every frame until it is abandoned, including the current frame if possible */
	EveryFrame,
};

/** 
 * Handle pointing to a FActiveSyncPoint that can be used to reserve or request work to execute as part of that sync point.
 * These handles cannot be copied, but can be safely moved between threads and will cancel all reservations and requests on destruction.
 */
struct FActiveSyncWorkHandle final
{
	FActiveSyncWorkHandle() = default;
	ENGINE_API ~FActiveSyncWorkHandle();

	FActiveSyncWorkHandle(FActiveSyncWorkHandle&&) = default;
	FActiveSyncWorkHandle& operator=(FActiveSyncWorkHandle&&) = default;

	// These handles can be moved but not copied
	FActiveSyncWorkHandle(const FActiveSyncWorkHandle&) = delete;
	FActiveSyncWorkHandle& operator=(const FActiveSyncWorkHandle&) = delete;

	/** True if this points to a real sync point */
	ENGINE_API bool IsValid() const;

	/**
	 * Returns a raw pointer to the tick function that implements this handle which can be passed into AddPrerequisite. 
	 * This is only valid to call on the game thread and should not be cached.
	 */
	ENGINE_API FTickFunction* GetDependencyTickFunction();

	/** True if this handle has been used to reserve work. This will still be true until work is abandoned */
	ENGINE_API bool HasReservedWork() const;

	/** True if this handle has been used to request work. This will still be true until work is abandoned */
	ENGINE_API bool HasRequestedWork() const;

	/** True if this handle is associated with a work queue that can be added to */
	ENGINE_API bool HasWorkQueue() const;

	/** 
	 * Request to reserve work using this handle, which must be filled to complete the frame's tasks. 
	 * If EveryFrame is passed in, it will reserve work at the start of every frame that must be requested or abandoned.
	 */
	ENGINE_API bool ReserveFutureWork(ESyncWorkRepetition Repeat);

	/**
	 * Requests that a tick function be executed by the sync point, this fills any reservations and may trigger tasks.
	 * If EveryFrame is passed in, it will request the same function every frame until it is abandoned.
	 */
	ENGINE_API bool RequestWork(FTickFunction* FunctionToExecute, ESyncWorkRepetition Repeat);

	/** Adds an arbitrary work function to the queue of work to execute later, this will fail if work has already been requested or reserved */
	ENGINE_API bool QueueWorkFunction(FWorkFunction&& WorkFunction);

	/** Sends any previously queued work to be executed as part of the sync point. Will call DontCompleteUntil on passed in TaskToExtend if valid */
	ENGINE_API bool SendQueuedWork(FGraphEventRef* TaskToExtend = nullptr);

	/** 
	 * Abandon any requested or reserved work and return to how it was at initial registration.
	 * This clears HasReserved/HasRequested, but does not invalidate the handle.
	 * Return true if there was any requested/reserved work to abandon.
	 */
	ENGINE_API bool AbandonWork();

	/** Completely reset this handle, which will abandon work and prevent any future use. This returns true if it was valid before */
	ENGINE_API bool Reset();

private:
	/** The sync point this handle is from */
	TSharedPtr<struct FActiveSyncPoint> SyncPoint;

	/** Index into the ActiveWork array */
	uint32 WorkIndex : 24 = 0;

	/** Max amount of work per sync point */
	static constexpr uint32 MaxWorkIndex = (1 << 24) - 1;

	/** True if this may represent active reserved work */
	uint32 bWorkReserved : 1 = 0;

	/** True if this may represent active requested work */
	uint32 bWorkRequested : 1 = 0;

	/** True if this represents a function work queue */
	uint32 bHasWorkQueue : 1 = 0;

	/** Get index as signed int */
	inline int32 GetIndex() const
	{
		return static_cast<int32>(WorkIndex);
	}

	/** Reset a handle to point to nothing */
	inline void ResetInternal()
	{
		SyncPoint.Reset();
		WorkIndex = 0;
		bWorkReserved = false;
		bWorkRequested = false;
		bHasWorkQueue = false;
	}

	friend struct FActiveSyncPoint;
	friend class FTaskSyncManager;
};

/**
 * A Tick function that can be used to execute a set of arbitrary work. This can be used as a normal tick function or passed to the work request functions.
 * This is NOT threadsafe on it's own, it needs to be protected by something like the locks in SendQueuedWork to use from multiple threads.
 */
struct FWorkQueueTickFunction : public FTickFunction
{
	ENGINE_API FWorkQueueTickFunction();
	ENGINE_API virtual ~FWorkQueueTickFunction();

	/** Return true if this is currently executing, if so no work can be added to the queue */
	inline bool IsWorkExecuting() const
	{
		return WorkExecutionIndex != INDEX_NONE;
	}

	/** Return true if any work has been queued */
	inline bool HasQueuedWork() const
	{
		return IsWorkExecuting() || QueuedWork.Num() > 0;
	}

	/** Return true if work can be added to the queue */
	inline bool IsQueueOpen() const
	{
		return !IsWorkExecuting() && bSetAsOpen;
	}

	/** Explicitly close or open the queue, can be used to finalize before execution */
	ENGINE_API void SetQueueOpen(bool bShouldBeOpen);

	/** Try to add a new function to the queue */
	ENGINE_API bool AddWork(FWorkFunction&& Work);

	/** Clear the queue, not allowed if execution has started */
	ENGINE_API void ClearWork();

	/** Actually execute the work, also called by execute tick if this scheduled as a normal tick */
	ENGINE_API void ExecuteWork();

	/** Sets the debug name used by DiagnosticContext */
	ENGINE_API void SetDebugName(FName InDebugName, const FString& InDetailString = FString());

	/** Sets if it should clear the queue after every execution (default behavior) */
	ENGINE_API void SetClearAfterExecute(bool bShouldClear);

	/** Sets if it should reopen a closed queue after every execution (default behavior) */
	ENGINE_API void SetOpenAfterExecute(bool bShouldOpen);

protected:

	// FTickFunction interface
	ENGINE_API void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	ENGINE_API FString DiagnosticMessage() override;
	ENGINE_API FName DiagnosticContext(bool bDetailed) override;

	bool bSetAsOpen;
	bool bClearAfterExecute;
	bool bOpenAfterExecute;
	int32 WorkExecutionIndex;

	FName DebugName;
	FString DebugDetailString;

	// Simple queue of work to execute, could be changed to a different type in the future
	TArray<FWorkFunction> QueuedWork;
};


/** Struct defining an active sync point inside a specific world context, implemented as a tick function that may not be registered */
struct FActiveSyncPoint final : public TSharedFromThis<FActiveSyncPoint, ESPMode::ThreadSafe>, private FTickFunction
{
	ENGINE_API FActiveSyncPoint();

	/** Returns the registered data  */
	inline const FSyncPointDescription& GetDescription() const { return SyncPointDescription; }

	/** Returns the sync point id that created this, including the current batch context */
	inline const FSyncPointId& GetSyncPointId() const { return SyncPointId; }

	/** Returns the world context */
	inline const FWorldContextId& GetWorldContextId() const { return WorldContextId; }

	/** Returns current status */
	inline ESyncPointStatus GetFrameStatus() const { return FrameStatus; }

	/** True if this is a special work function that is never registered */
	inline bool IsTickGroupWork() const { return !SyncPointId.IsValid(); }

	/** Registers a new work handle that can be used to reserve and request work */
	ENGINE_API FActiveSyncWorkHandle RegisterWorkHandle();

	/** Uses handle to reserve work for later, that can be requested when ready */
	ENGINE_API bool ReserveFutureWork(FActiveSyncWorkHandle& Handle, ESyncWorkRepetition Repeat);

	/** Creates a new item of work bound to a specific function */
	ENGINE_API bool RequestWork(FActiveSyncWorkHandle& Handle, FTickFunction* WorkFunction, ESyncWorkRepetition Repeat);

	/** Adds an arbitrary work function to the queue of work to execute later, this will fail if work has already been requested or reserved */
	ENGINE_API bool QueueWorkFunction(FActiveSyncWorkHandle& Handle, FWorkFunction&& WorkFunction);

	/** Sends any previously queued work to be executed as part of the sync point. Will call DontCompleteUntil on passed in TaskToExtend if valid */
	ENGINE_API bool SendQueuedWork(FActiveSyncWorkHandle& Handle, FGraphEventRef* TaskToExtend = nullptr);

	/** Abandons requested or reserved work, which could trigger other work to start */
	ENGINE_API bool AbandonWork(FActiveSyncWorkHandle& Handle);

	/** Resets a handle */
	ENGINE_API bool ResetWorkHandle(FActiveSyncWorkHandle& Handle);

private:

	/** Struct defining a unit of work that is executed as part of an FActiveSyncPoint */
	struct FActiveSyncWork
	{
		union
		{
			/** Default state of 0 means it can be reused immediately */
			uint32 StateValue;

			struct
			{
				/** If true, this work corresponds to an active handle */
				uint32 bHasActiveHandle : 1;

				/** If true, this work has been reserved for execution later in the frame */
				uint32 bWorkReserved : 1;

				/** If true, this work can be executed this frame */
				uint32 bWorkRequested : 1;

				/** If true, work will be reserved at the start of every frame */
				uint32 bReserveEveryFrame : 1;

				/** If true, work will be requested at the start of every frame */
				uint32 bRequestEveryFrame : 1;

				/** If true, this work is in the middle of being executed and is in the ExecutingWork array */
				uint32 bCurrentlyExecuting : 1;

				/** If true, this work has completely finished executing this frame */
				uint32 bAlreadyExecuted : 1;

				/** If true, a late work request was created for this handle */
				uint32 bLateWorkRequested : 1;

				/** If true, this points to a FWorkQueueTickFunction that can be added to and sent */
				uint32 bIsWorkQueueFunction : 1;

				/** If true, the tick function allocated by this work and should be freed when reset */
				uint32 bAllocatedTickFunction : 1;
			};
		};

		/** Function to actually execute */
		FTickFunction* TickFunction;

		FActiveSyncWork()
			: StateValue(0)
			, TickFunction(nullptr)
		{
		}

		~FActiveSyncWork()
		{
			if (bAllocatedTickFunction && TickFunction)
			{
				delete TickFunction;
			}
		}

		inline void SetTickFunction(FTickFunction* InTickFunction, bool bWasAllocatedInternally)
		{
			if (bAllocatedTickFunction && TickFunction)
			{
				delete TickFunction;
			}
			TickFunction = InTickFunction;
			bAllocatedTickFunction = bWasAllocatedInternally;
		}

		inline void Reset()
		{
			SetTickFunction(nullptr, false);
			StateValue = 0;
		}

		inline bool IsInitialized() const
		{
			return StateValue != 0;
		}

		FActiveSyncWork(const FActiveSyncWork&) = delete;
		FActiveSyncWork& operator=(const FActiveSyncWork&) = delete;
	};

	/** Struct used to actually execute callbacks */
	struct FExecutingSyncWork
	{
		/** Source index in the ActiveWork array */
		int32 ActiveWorkIndex;

		/** Function to run, if this is null it was disabled */
		FTickFunction* TickFunction;

		FExecutingSyncWork(int32 InIndex, const FActiveSyncWork& InWork)
			: ActiveWorkIndex(InIndex),
			TickFunction(InWork.TickFunction)
		{}

		inline bool IsValid() const
		{
			return TickFunction != nullptr;
		}

		/** Call from outside execution to invalidate even if already in queue */
		inline void Invalidate()
		{
			TickFunction = nullptr;
		}
	};

	// FTickFunction interface
	virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
	virtual FName DiagnosticContext(bool bDetailed) override;
	virtual bool DispatchManually() override;

	/** Execute work from the game thread instead of from task graph */
	void ExecuteFromGameThread(float DeltaTime, ELevelTick TickType);

	/** Resets tick function at start of frame, this can automatically reserve/request work */
	void ResetWorkForFrame();

	/** Called on the first tick group to set status properly. Returns true if this function should be dispatched immediately */
	bool HandleFirstTickGroup();

	/** Returns true if this sync point is ready to trigger due to work */
	bool IsReadyToProcessWork() const;

	/** Returns true if we're too late in execution to schedule work directly */
	bool IsTooLateToAddWork(bool bWorkReserved) const;

	/** Handle copying of work into / out of ExecutingWork array, returns true if there is anything to do */
	bool GetWorkToExecute(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);

	/** Cancels work that may already be in the execution queue */
	void CancelRequestedWork(int32 ActiveWorkIndex, FActiveSyncWork& CurrentWork);

	/** Gets index of new work slot, could resize array so must be called locked and not during iteration */
	uint32 AllocateActiveWork();

	/** Dispatches the task to handle work, this should be called outside the work lock */
	void DispatchWorkTask(FGraphEventRef* TaskToExtend = nullptr);

	/** Registered sync point that created this */
	FSyncPointDescription SyncPointDescription;
	
	/** Sync point this corresponds to, including batch */
	FSyncPointId SyncPointId;

	/** World Context this was created in */
	FWorldContextId WorldContextId;

	/** Current state of this sync point in the current frame */
	ESyncPointStatus FrameStatus;

	/** Event to signal if there is work to process after initial activation */
	FGraphEventRef ReactivationEvent;

	/** Lock to handle access to work array and other internal structs */
	mutable FTransactionallySafeCriticalSection WorkLock;

	/** Array of work to execute as part of tick, this can only be read or written with WorkLock */
	TArray<FActiveSyncWork> ActiveWork;

	/** Second buffer of work used during active execution, this can only reallocate with WorkLock but is read outside it */
	TArray<FExecutingSyncWork> ExecutingWork;

	friend class FTaskSyncManager;
	friend struct FActiveSyncWorkHandle;
};


/**
 * Global singleton manager that can be used to synchronize tasks across different engine systems.
 * This provides event registration and utility functions that wrap functionality in the base Task and TaskGraph systems.
 */
class FTaskSyncManager
{
public:
	ENGINE_API FTaskSyncManager();
	ENGINE_API ~FTaskSyncManager();
	

	// Thread safe accessors, safe at any time

	/** Return the global singleton if it exists */
	static ENGINE_API FTaskSyncManager* Get();

	/** Gets the default batch for a world context. This will return an invalid id if the world context is not set up for ticking */
	ENGINE_API FBatchContextId FindDefaultBatch(FWorldContextId WorldContext) const;

	/** 
	 * Initializes a sync point id that can be passed between threads and used in the functions below.
	 * This will return an invalid id if the name is not currently registered.
	 * This function does not check that the sync point is enabled for the specific batch, that is handled in the other functions below.
	 */
	ENGINE_API FSyncPointId FindSyncPoint(FBatchContextId Batch, FName RegisteredName);

	/** Same as above, but using the default batch for the world context */
	ENGINE_API FSyncPointId FindSyncPoint(FWorldContextId WorldContext, FName RegisteredName);


	// These functions are thread safe and can be called by worker or game threads during a tick, if the sync point is in the proper state

	/** 
	 * Attempts to return the current frame's task graph event for this sync point, to pass as a dependency to other tasks.
	 * This will fail if the task has not yet been created this frame or it has finished executing.
	 */
	ENGINE_API FTaskSyncResult GetTaskGraphEvent(FSyncPointId SyncPoint, FGraphEventRef& OutEventRef);

	/** Manually triggers a sync point, this can only be called once per frame */
	ENGINE_API FTaskSyncResult TriggerSyncPoint(FSyncPointId SyncPoint);

	/** Tells the sync point to trigger at the completion of the passed in event/task, this is only possible if it hasn't already been triggered */
	ENGINE_API FTaskSyncResult TriggerSyncPointAfterEvent(FSyncPointId SyncPoint, FGraphEventRef EventToWairFor);

	/** 
	 * Tries to creates a new sync work handle that can be used to reserve or request work to happen as part of a sync point.
	 * If it was successful, OutWorkHandle will be set to a handle you can pass between threads and later fulfill or abandon.
	 */
	ENGINE_API FTaskSyncResult RegisterWorkHandle(FSyncPointId SyncPoint, FActiveSyncWorkHandle& OutWorkHandle);

	/**
	 * Tries to creates a new sync work handle that can be used to reserve or request work to happen on the game thread during a tick group.
	 * If it was successful, OutWorkHandle will be set to a handle you can use to request work for this frame or every frame (but not reserve).
	 */
	ENGINE_API FTaskSyncResult RegisterTickGroupWorkHandle(FWorldContextId WorldContext, ETickingGroup TickGroup, FActiveSyncWorkHandle& OutWorkHandle);

	/**
	 * Creates (but does not dispatch) a simple task that will execute a tick function. 
	 * The caller is responsible for dispatching the task and must keep the tick function valid until the task completes.
	 * This is useful for cases where you want to spawn a tick halfway through a tick group
	 * If default delta time is passed, it will try to use the currently ticking world's info
	 */
	ENGINE_API FGraphEventRef CreateManualTickTask(FWorldContextId WorldContext, FTickFunction* TickFunction, float DeltaTime = -1.0f, ELevelTick TickType = LEVELTICK_All);


	// Game thread-only functions, in general these will not affect the current frame

	/** Refresh the registered data from settings */
	ENGINE_API void ReloadRegisteredData();

	/** Searches for a registered sync point, if ones is foundw ith that name it will return true and fill in OutDescription */
	ENGINE_API bool GetSyncPointDescription(FName RegisteredName, FSyncPointDescription& OutDescription) const;

	/** Registers a new sync point at runtime. RegisteredName and SourceName must be filled out */
	ENGINE_API bool RegisterNewSyncPoint(const FSyncPointDescription& NewDescription);

	/** Unregisters a sync point, this will only delete if if both the registered and source names match */
	ENGINE_API bool UnregisterSyncPoint(FName RegisteredName, FName SourceName);

	/** Allocates a new batch for the specified world, this will duplicate the events */
	ENGINE_API FBatchContextId CreateNewBatch(FWorldContextId WorldContext);

	/** Gets the correct task world context for the current thread context. This will return an invalid context if called on worker threads */
	ENGINE_API FWorldContextId GetCurrentWorldContext() const;

	/** Returns the tick function representing the specified event, if this returns a valid pointer you can use it for setting dependencies but it should not be modified or stored */
	ENGINE_API FTickFunction* GetTickFunctionForSyncPoint(FSyncPointId SyncPoint);

	/** Tells the manager that it is the start of a frame, which will register the appropriate events */
	ENGINE_API void StartFrame(const UWorld* InWorld, float InDeltaSeconds, ELevelTick InTickType);

	/** Tells the manager that a tick group is starting for a specific world */
	ENGINE_API void StartTickGroup(const UWorld* InWorld, ETickingGroup TickGroup, TArray<FTickFunction*>& TicksToManualDispatch);

	/** Tells the manager that a tick group is complete for a specific world, which could trigger events */
	ENGINE_API void EndTickGroup(const UWorld* InWorld, ETickingGroup TickGroup);

	/** Tells the manager that it is the end of a frame and it will not start/end any more tick groups */
	ENGINE_API void EndFrame(const UWorld* InWorld);

	/** Tells the manager that it should destroy all tracking info for a world context. This only needs to be called when destroying a test or preview world */
	ENGINE_API void ReleaseWorldContext(FWorldContextId WorldContext);
private:
		
	struct FTemporaryWorkRequest
	{
		FTemporaryWorkRequest(FActiveSyncPoint* InRequestingSyncPoint, int32 InRequestingHandle)
			: RequestingSyncPoint(InRequestingSyncPoint)
			, RequestingHandle(InRequestingHandle)
		{}

		FActiveSyncPoint* RequestingSyncPoint;
		int32 RequestingHandle;
		FActiveSyncWorkHandle WorkHandle;
	};

	struct FBatchData
	{
		/** The batch of objects with index in batch array */
		FBatchContextId BatchContext;

		/** The world this is associated with */
		FWorldContextId WorldContext;

		/** Array of specific user-defined sync points */
		TMap<FSyncPointId::FInternalId, TSharedRef<FActiveSyncPoint>> SyncPointData;

		/** Array of general gamethread work per tickgroup, these are not scheduled as real tasks and only exist in default batch */
		TArray<TSharedPtr<FActiveSyncPoint>> TickGroupWork;

		/** List of sync points that were reset during the frame, will be deleted at end */
		TArray<TSharedPtr<FActiveSyncPoint>> SyncPointsToDelete;

		/** Array of temporary work handles, dropped at end of frame */
		TArray<FTemporaryWorkRequest> TemporaryWorkRequests;

		inline void Reset()
		{
			SyncPointData.Reset();
			TickGroupWork.Reset();
			SyncPointsToDelete.Reset();
			TemporaryWorkRequests.Reset();
			BatchContext = FBatchContextId();
			WorldContext = FWorldContextId();
		}
	};

	struct FRegisteredSyncPointData
	{
		FSyncPointId::FInternalId RegisteredId = FSyncPointId::InvalidSyncPoint;
		FSyncPointDescription RegisteredPoint;
	};

	// Store as a simple array for fast access, there are a small number of active batches
	TArray<FBatchData> BatchList;

	TMap<FSyncPointId::FInternalId, FRegisteredSyncPointData> RegisteredDataMap;
	TMap<FName, FSyncPointId::FInternalId> RegisteredNameMap;
	FSyncPointId::FInternalId HighestSyncId = FSyncPointId::InvalidSyncPoint;

	/** Lock to support multithreaded access. TODO: Investigate if RWLock or command buffer is worth it */
	mutable FTransactionallySafeCriticalSection ManagerLock;

	/** World that is currently ticking, cannot modify certain operations during a tick or tick two worlds at once */
	const UWorld* CurrentTickWorld = nullptr;
	float CurrentDeltaTime = 0.0f;
	ELevelTick CurrentTickType = LEVELTICK_All;
	ETickingGroup CurrentTickGroup = ETickingGroup::TG_MAX;

	/** True if this is currently ticking a world */
	inline bool IsTicking() const { return CurrentTickWorld != nullptr; }

	/** Internal accessor to get a tick function's raw data, does not lock */
	FTaskSyncResult FindActiveSyncPoint(FSyncPointId SyncPoint, TSharedPtr<FActiveSyncPoint>& OutData);

	/** Try to handle forwarded work from a SyncPoint, will add to TickGroupWork if it makes sense */
	bool HandleLateWorkRequest(FActiveSyncPoint* RequestedSyncPoint, int32 RequestingHandle, FTickFunction* TickFunction);

	/** Cancels any scheduled temporary work requests matching the point and handle */
	void CancelTemporaryWorkRequest(FActiveSyncPoint* RequestedSyncPoint, int32 RequestingHandle);

	void RegisterSyncPointInternal(const FSyncPointDescription& InDescription);
	void OnWorldContextRemove(FWorldContext& InWorldContext);
	void InitializeBatchForFrame(FBatchData& BatchData, ULevel* PersistentLevel);
	void InitializeForWorld(FWorldContextId WorldContext);
	void OnPostWorldInitialization(UWorld* World, const UWorld::InitializationValues IVS);
	void InitializeBatchForNewWorld(FBatchData& BatchData);
#if WITH_EDITOR
	void EndPIE(bool bStartSimulate);
#endif

	FActiveSyncPoint* GetOrCreateSyncPoint(FBatchData& BatchData, FRegisteredSyncPointData& SyncData);
	bool DestroySyncPointIfValid(FBatchData& BatchData, FSyncPointId::FInternalId SyncId);
	FActiveSyncPoint* GetOrCreateTickGroupWork(FBatchData& BatchData, ETickingGroup TickGroup);

	friend FActiveSyncPoint;
};

} // namespace UE::Task
