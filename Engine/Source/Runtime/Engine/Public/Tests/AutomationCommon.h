// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Engine/GameViewportClient.h"
#include "Engine/World.h"

class AActor;
class SWindow;
class SWidget;

#if WITH_AUTOMATION_TESTS

///////////////////////////////////////////////////////////////////////
// Common Latent commands which are used across test type. I.e. Engine, Network, etc...

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogEditorAutomationTests, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogEngineAutomationTests, Log, All);

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnEditorAutomationMapLoad, const FString&, bool, FString*);

#endif

/** 
 * Utility class for creating and destroying a temporary test world, can be used for automation or performance testing.
 * This can be used with any test framework but has utility functions for FAutomationTestBase.
 */
struct FTestWorldWrapper
{
	/** This will properly shut down and destroy the test world as needed */
	ENGINE_API virtual ~FTestWorldWrapper();

	/** Gets the wrapped world, can be null */
	inline UWorld* GetTestWorld() const {return TestWorld;}

	/** Creates a world of the appropriate world type, returns false on failure */
	ENGINE_API virtual bool CreateTestWorld(EWorldType::Type WorldType);

	/** Destroys the test world and handles any required cleanup */
	ENGINE_API virtual bool DestroyTestWorld(bool bForceGarbageCollect);

	/** Starts play in the test world to simulate gameplay */
	ENGINE_API virtual bool BeginPlayInTestWorld();

	/** Ticks the test world for one frame, defaults to 100 fps */
	ENGINE_API virtual bool TickTestWorld(float DeltaTime = 0.01f);

	/** Stops play properly */
	ENGINE_API virtual bool EndPlayInTestWorld();

	/** Registers an error message and marks test as failed, called by the functions above */
	ENGINE_API virtual void ReportFailure(const TCHAR* ErrorMessage);

	/** Clears any failures and initial state */
	ENGINE_API virtual void ClearFailureState();

	/** Returns true if there are any errors that should stop further execution */
	ENGINE_API virtual bool HasFailed() const;

	/** Gets the actual error messages for reporting to the automation framework */
	ENGINE_API virtual void AppendErrorMessages(TArray<FString>& OutErrorMessages) const;

	/** Reports error messages to a passed in automation test */
	ENGINE_API virtual void ForwardErrorMessages(FAutomationTestBase* AutomationTest) const;

protected:
	UWorld* TestWorld = nullptr;
	uint64 CachedFrameCounter = 0;
	TArray<FString> FailureErrors;
};

/**
 * Utility for setting and restoring of a Console Variable (CVar).
 * 
 * Note that race conditions are possible when multiple `FTestConsoleVariable` objects refer to the same Console Variable and get restored/destroyed in an order different than they were set.
 * Please use `FScopedTestEnvironment` to manage multiple `FTestConsoleVariable` objects.
 */
struct FTestConsoleVariable
{
	ENGINE_API FTestConsoleVariable(const FString& InConsoleVariableName);
	ENGINE_API FTestConsoleVariable(FTestConsoleVariable&& Other);

	ENGINE_API ~FTestConsoleVariable();

	/**
	 * Sets a Console Variable to the specified value. Will keep a reference to the original value regardless of how many times the value has been set.
	 * @param Value		Value to set on the Console Variable
	 */
	ENGINE_API void Set(const FString& Value);

	/** Returns the current value of the Console Variable */
	ENGINE_API FString Get();

	/** Returns the Console Variable to the original value */
	ENGINE_API void Restore();

private:
	bool bModified;
	FString ConsoleVariableName;
	FString OriginalValue;
};

/**
 * Utility for setting and management of temporary CVars
 * Will handle restoring the Console Variables (CVars) back to the original state on destruction.
 */
struct FScopedTestEnvironment
{
	/** Restores all set Console Variables back to the original value */
	ENGINE_API ~FScopedTestEnvironment();

	/**
	 * Returns a shared pointer to current instance of the scoped test environment. Will create an instance if current instance is invalid.
	 * @return the shared pointer to the instance
	 */
	static ENGINE_API TSharedPtr<FScopedTestEnvironment> Get();

	/**
	 * Sets a Console Variable to the specified value. Will keep a reference to the original value regardless of how many times the value has been set.
	 * @param VariableName	Name of the Console Variable to set
	 * @param Value			Value to be applied
	 */
	ENGINE_API void SetConsoleVariableValue(const FString& ConsoleVariableName, const FString& Value);

	/**
	 * Gets the current overridden value for the specified Console Variable
	 * @param VariableName	Name of the Console Variable to fetch the current value from
	 * @param OutValue		Current value of the Console Variable if it was overridden.
	 * @return true if the Console Variable was overridden.
	 */
	ENGINE_API bool TryGetConsoleVariableValue(const FString& ConsoleVariableName, FString* OutValue);

	/** Restores all set Console Variables back to the original value */
	ENGINE_API void Restore();

private:
	FScopedTestEnvironment() = default;

	FScopedTestEnvironment(FScopedTestEnvironment&&) = delete;
	FScopedTestEnvironment(const FScopedTestEnvironment&) = delete;
	FScopedTestEnvironment& operator=(FScopedTestEnvironment&&) = delete;
	FScopedTestEnvironment& operator=(const FScopedTestEnvironment&) = delete;

	TMap<FString, FTestConsoleVariable> Variables;

	static TWeakPtr<struct FScopedTestEnvironment> EnvironmentInstance;
};

/** Common automation functions */
namespace AutomationCommon
{
#if WITH_AUTOMATION_TESTS

	/** Get a string contains the render mode we are currently in */
	ENGINE_API FString GetRenderDetailsString();


	/** Gets a name to be used for this screenshot.  This will return something like 
		TestName/PlatformName/DeviceName.png. It's important to understand that a screenshot
		generated on a device will likely have a different absolute path than the editor so this
		name should be used with	*/
	ENGINE_API FString GetScreenshotPath(const FString& TestName);

	/** 
	This function takes the result of GetScreenshotName and will return the complete path to where a
	screenshot can/should be found on the local device. This cannot reliably be used when communicating between
	the editor and a test worker!
	*/
	ENGINE_API FString GetLocalPathForScreenshot(const FString& InScreenshotName);

	ENGINE_API FAutomationScreenshotData BuildScreenshotData(const FString& MapOrContext, const FString& TestName, const FString& ScreenShotName, int32 Width, int32 Height);

	ENGINE_API extern FOnEditorAutomationMapLoad OnEditorAutomationMapLoad;
	static FOnEditorAutomationMapLoad& OnEditorAutomationMapLoadDelegate()
	{
		return OnEditorAutomationMapLoad;
	}

	ENGINE_API TArray<uint8> CaptureFrameTrace(const FString& MapOrContext, const FString& TestName);

	/**
	 * Given the FName of a FTagMetaData will find all the corresponding widgets.
	 * @param Tag the meta data tag searched for
	 * @return the found widget or nullptr
	 */
	ENGINE_API SWidget* FindWidgetByTag(const FName Tag);

	ENGINE_API UWorld* GetAnyGameWorld();

#endif
	ENGINE_API UGameViewportClient* GetAnyGameViewportClient();

	/* Get the adjusted World name to use for screenshot paths */
	ENGINE_API FString GetWorldContext(UWorld* InWorld);
}

#if WITH_AUTOMATION_TESTS

/**
 * Parameters to the Latent Automation command FTakeEditorScreenshotCommand
 */
struct WindowScreenshotParameters
{
	FString ScreenshotName;
	TSharedPtr<SWindow> CurrentWindow;
};

/**
 * If Editor, Opens map and PIES.  If Game, transitions to map and waits for load
 */
ENGINE_API bool AutomationOpenMap(const FString& MapName, bool bForceReload = false);

/**
 * Wait for the given amount of time
 */
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FWaitLatentCommand, float, Duration);

/**
 * Write a string to editor automation tests log
 */
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FEditorAutomationLogCommand, FString, LogText);


/**
 * Take a screenshot of the active window
 */
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FTakeActiveEditorScreenshotCommand, FString, ScreenshotName);

/**
 * Take a screenshot of the active window
 */
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FTakeEditorScreenshotCommand, WindowScreenshotParameters, ScreenshotParameters);

/**
 * Latent command to load a map in game
 */
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FLoadGameMapCommand, FString, MapName);

/**
 * Latent command to exit the current game
 */
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND(FExitGameCommand);

/**
 * Latent command that requests exit
 */
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND( FRequestExitCommand );

/**
* Latent command to wait for map to complete loading
*/
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND(FWaitForMapToLoadCommand);

/**
* Latent command to wait for map to complete loading
*/
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FWaitForSpecifiedMapToLoadCommand, FString, MapName);


/**
* Execute command string
*/
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FExecStringLatentCommand, FString, ExecCommand);


/**
* Wait for the given amount of time
*/
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FEngineWaitLatentCommand, float, Duration);

/**
* Wait until data is streamed in
*/
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FStreamAllResourcesLatentCommand, float, Duration);


/**
* Latent command to run an exec command that also requires a UWorld.
*/
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FExecWorldStringLatentCommand, FString, ExecCommand);


/**
* Waits for shaders to finish compiling before moving on to the next thing.
*/
DEFINE_ENGINE_LATENT_AUTOMATION_COMMAND(FWaitForShadersToFinishCompilingInGame);


/**
 * Waits until the average framerate meets to exceeds the specified value. Mostly intended as a way to ensure that a level load etc has completed
 * and an interactive framerate is present.
 *
 * If values are zero the defaults in [/Script/Engine.AutomationTestSetting] will be used
 */
class FWaitForInteractiveFrameRate : public IAutomationLatentCommand
{
public:

	ENGINE_API FWaitForInteractiveFrameRate(float InDesiredFrameRate = 0, float InDuration = 0, float InMaxWaitTime = 0);

	ENGINE_API bool Update() override;

public:

	// Framerate we want to see
	float DesiredFrameRate;

	// How long must we maintain this framerate
	float Duration;

	// Max time so spend waiting
	float MaxWaitTime;

private:

	// Add a sample to the rolling buffer where we hold tick rate
	void AddTickRateSample(const double Value);

	// return the average tick rate
	double CurrentAverageTickRate() const;

	// Time we began executing
	double StartTimeOfWait;

	// how many seconds we've been at the desired framerate
	double StartTimeOfAcceptableFrameRate;	

	// time we last logged we are waiting
	double LastReportTime;

	// time of last tick
	double LastTickTime;

	// buffer of recent tick rate
	TArray<double> RollingTickRateBuffer;

	// index into the buffer
	int32 BufferIndex;

	// We tick at 60Hz
	const double kTickRate = 60.0;

	// How many samples we hold
	const int kSampleCount = (int32)kTickRate * 5;
};

/**
 * Latent command to wait for one engine frame
 */
class FWaitForNextEngineFrameCommand : public IAutomationLatentCommand
{
public:

	ENGINE_API bool Update() override;

private:

	// Frame to be passed to consider the waiting over
	uint64 LastFrame = 0;
};

/**
 * Latent command to wait for a given number of engine frames
 */
class FWaitForEngineFramesCommand : public IAutomationLatentCommand
{
public:
	ENGINE_API explicit FWaitForEngineFramesCommand(int32 InFramesToWait = 1);

	ENGINE_API bool Update() override;

private:
	int32 FrameCounter = 0;
	int32 FramesToWait = 1;
};


/**
* Request an Image Comparison and queue the result to the test report
* @param InImageName	Name used to identify the comparison
* @param InContext		Optional context used to identify the comparison, by default the full name of the test is used
**/
ENGINE_API void RequestImageComparison(const FString& InImageName, int32 InWidth, int32 InHeight, const TArray<FColor>& InImageData, EAutomationComparisonToleranceLevel InTolerance = EAutomationComparisonToleranceLevel::Low, const FString& InContext = TEXT(""), const FString& InNotes = TEXT(""));


#endif
