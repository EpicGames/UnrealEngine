// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Engine/NetDriver.h"
#include "UObject/UObjectIterator.h"

#include "Containers/Array.h"
#include "GameFramework/PlayerController.h"
#include "Kismet/GameplayStatics.h"
#include "Tests/EnsureScope.h"

class IConsoleVariable;
class UGameInstance;
class UReplicationSystem;
class UObjectReplicationBridge;

struct FGameInstancePIEParameters;

namespace UE::Net
{

/**
 * Temporarily overrides network emulation settings for a specific NetDriver while in scope.
 * Only works when DO_ENABLE_NET_TEST is true.
 */
class [[nodiscard]] FScopedNetEmulationOverride
{
public:
	ENGINE_API FScopedNetEmulationOverride(UNetDriver* InNetDriver, const FPacketSimulationSettings& InSettings);
	ENGINE_API ~FScopedNetEmulationOverride();

private:
	TWeakObjectPtr<UNetDriver> WeakNetDriver;
	FPacketSimulationSettings SavedSettings;
};


#if WITH_EDITOR

template<class T>
int32 NumActors(UWorld* World)
{
	TArray<AActor*> Actors;
	UGameplayStatics::GetAllActorsOfClass(Cast<UObject>(World), T::StaticClass(), Actors);
	return Actors.Num();
}

template<class T>
int32 NumObjects(UWorld* World)
{
	int32 Count = 0;
	for (TObjectIterator<T> It; It; ++It)
	{
		if (IsValid(*It) && It->GetWorld() == World)
		{
			Count++;
		}
	}

	return Count;
}

/*
 * Used to set a cvar and restore it to it's original value when destroyed
 */
class [[nodiscard]] FScopedCVarOverrideInt
{
public:
	ENGINE_API FScopedCVarOverrideInt(const TCHAR* VariableName, int32 Value);
	ENGINE_API ~FScopedCVarOverrideInt();

	FScopedCVarOverrideInt(FScopedCVarOverrideInt&&) = delete;
	FScopedCVarOverrideInt(const FScopedCVarOverrideInt&) = delete;
	FScopedCVarOverrideInt& operator=(FScopedCVarOverrideInt&&) = delete;
	FScopedCVarOverrideInt& operator=(const FScopedCVarOverrideInt&) = delete;

private:
	IConsoleVariable* Variable = nullptr;
	int32 SavedValue = 0;
};

class [[nodiscard]] FScopedCVarOverrideFloat
{
public:
	ENGINE_API FScopedCVarOverrideFloat(const TCHAR* VariableName, float Value);
	ENGINE_API ~FScopedCVarOverrideFloat();

	FScopedCVarOverrideFloat(FScopedCVarOverrideFloat&&) = delete;
	FScopedCVarOverrideFloat(const FScopedCVarOverrideFloat&) = delete;
	FScopedCVarOverrideFloat& operator=(FScopedCVarOverrideFloat&&) = delete;
	FScopedCVarOverrideFloat& operator=(const FScopedCVarOverrideFloat&) = delete;

private:
	IConsoleVariable* Variable = nullptr;
	float SavedValue = 0;
};

/**
* Sets and restores  cvars needed to use FNetTestWorldInstances within a scope.
* Meant to be used within a single function.
*/
class FScopedTestSettings
{
public:
	ENGINE_API FScopedTestSettings();
	ENGINE_API ~FScopedTestSettings();

	FScopedTestSettings(FScopedTestSettings&&) = delete;
	FScopedTestSettings(const FScopedTestSettings&) = delete;
	FScopedTestSettings& operator=(FScopedTestSettings&&) = delete;
	FScopedTestSettings& operator=(const FScopedTestSettings&) = delete;

private:
	FScopedCVarOverrideInt AddressResolutionDisabled;
	FScopedCVarOverrideInt BandwidthThrottlingDisabled;
	FScopedCVarOverrideInt RepGraphBandwidthThrottlingDisabled;
	FScopedCVarOverrideInt RandomNetUpdateDelayDisabled;
	FScopedCVarOverrideInt GameplayDebuggerDisabled;
#if WITH_EDITOR
	FScopedCVarOverrideInt ClientsShareAudioDevice;
#endif

	FDelegateHandle NetDriverCreatedHandle;
};

/**
* Stores and restore GWorld and PIE settings modified by the creation of the temporary test worlds
*/
class FScopedNetTestPIERestoration
{
public:

	ENGINE_API FScopedNetTestPIERestoration();
	ENGINE_API ~FScopedNetTestPIERestoration();

	FScopedNetTestPIERestoration(FScopedNetTestPIERestoration&&) = delete;
	FScopedNetTestPIERestoration(const FScopedNetTestPIERestoration&) = delete;
	FScopedNetTestPIERestoration& operator=(FScopedNetTestPIERestoration&&) = delete;
	FScopedNetTestPIERestoration& operator=(const FScopedNetTestPIERestoration&) = delete;

private:

	// Restores PIE world context
	UWorld* OldGWorld = nullptr;
	int32 OldPIEID = 0;
	bool OldGIsPlayInEditorWorld = false;
};

/** 
 * Scope used when a test is expected to trigger an ensure. 
 * Prevents the ensure from triggering a DEBUG_BREAK when running with a debugger attached.
 */

class FNetTestEnsureScope : public FEnsureScope
{
public:
#if !UE_BUILD_SHIPPING
	FNetTestEnsureScope()	
		: FEnsureScope()
		, bIgnoreDebugger(GIgnoreDebugger)
	{
		GIgnoreDebugger = true;
	}

	~FNetTestEnsureScope()
	{
		GIgnoreDebugger = bIgnoreDebugger;
	}

private:
	bool bIgnoreDebugger;
#else
public:
	FNetTestEnsureScope() = default;
	~FNetTestEnsureScope() = default;
#endif //!UE_BUILD_SHIPPING
};


/**
 * Properly scoped/RAII wrapper around a GameInstance/WorldContext/World that makes it easier to write tests
 * involving full UWorld functionality within the scope of one function.
 */
struct FTestWorldInstance
{
	struct FContext;

public:
	ENGINE_API static FTestWorldInstance CreateServer(const TCHAR* InURL, const EReplicationSystem ReplicationSystem = EReplicationSystem::Default);
	ENGINE_API static FTestWorldInstance CreateClient(int32 ServerPort, int32 NumLocalPlayers = 1);
	ENGINE_API static FTestWorldInstance CreateProxy();

	ENGINE_API ~FTestWorldInstance();

	FTestWorldInstance(const FTestWorldInstance&) = delete;
	FTestWorldInstance& operator=(const FTestWorldInstance&) = delete;

	explicit FTestWorldInstance(bool bDelayedInit)
	{
		// GameInstance will be created manually later
	}
	
	FTestWorldInstance(FTestWorldInstance&& Other)	
		: GameInstance(Other.GameInstance)
		, bIsLoaded(Other.bIsLoaded)
	{
		Other.GameInstance = nullptr;
		Other.bIsLoaded = false;
	}

	ENGINE_API FTestWorldInstance& operator=(FTestWorldInstance&& Other);

	ENGINE_API FWorldContext* GetWorldContext() const;	

	ENGINE_API FContext GetTestContext() const;

	ENGINE_API UWorld* GetWorld() const;
	ENGINE_API UNetDriver* GetNetDriver() const;

	ENGINE_API int32 GetPort() const;

	ENGINE_API void Tick(float DeltaSeconds = 0.0166f);

	ENGINE_API void LoadStreamingLevel(FName LevelName);
	ENGINE_API void UnloadStreamingLevel(FName LevelName);

	/** Returns true if the world was correctly loaded and ready to play. */
	bool IsLoaded() const
	{
		return bIsLoaded;
	}

public:

	struct FContext
	{
		UWorld* World = nullptr;
		UNetDriver* NetDriver = nullptr;
		UReplicationSystem* IrisRepSystem = nullptr;
		UObjectReplicationBridge* IrisRepBridge = nullptr;

		bool IsValid() const
		{
			return World && NetDriver && IrisRepSystem && IrisRepBridge;
		}
	};

public:

	UGameInstance* GameInstance = nullptr;
	bool bIsLoaded = false;

private:
	explicit FTestWorldInstance(const FGameInstancePIEParameters& InstanceParams);

	void Shutdown();

	static int32 FindUnusedPIEInstance();

	int32 LevelStreamRequestUUID = 0;
};

/**
 * Stores FTestWorldInstances for a server and clients and allows synchronously ticking them.
 * Can be used within a single function to make automated tests that use the whole world & net driver flow.
 */
struct FTestWorlds
{
	/** Creates a server world using the default map (Entry) and default GameMode class (GameMode) */
	ENGINE_API FTestWorlds(const EReplicationSystem ReplicationSystem = EReplicationSystem::Default);

	/** 
	 * Creates a server world using a specific map name and GameMode class
	 * @param MapName Full path name of the map. ex: /Engine/Maps/Entry
	 * @param GameModeName Full path name of the game mode. ex: /Script/EngineTest.MyGameMode
	 */
	ENGINE_API FTestWorlds(const FString& MapName, const FString& GameModeName, const EReplicationSystem ReplicationSystem = EReplicationSystem::Default);

	/** Creates a server world using the given URL. */
	ENGINE_API explicit FTestWorlds(const TCHAR* ServerURL, const EReplicationSystem ReplicationSystem = EReplicationSystem::Default);

	ENGINE_API ~FTestWorlds();

	ENGINE_API void SetDescription(const FString& Description);
	FString GetDescription() const { return Description; }

	ENGINE_API void SetTickInSeconds(float TickInSeconds);
	float GetTickDeltaSeconds() const { return TickDeltaSeconds; }
	
	/** Create a client and fully connect it to the server. Returns true if the connection succeeded */
	ENGINE_API bool CreateAndConnectClient();

	/** Create a new client and connect it to the server. Returns the ClientID of the new client or INDEX_NONE if it failed to connect. */
	ENGINE_API int32 ConnectNewClient();

	/** Returns the ClientId associated with the passed world or INDEX_NONE if it's not a known client world. */
	ENGINE_API int32 FindClientIdFromWorld(const UWorld* ClientWorld) const;

	/** Returns the world context data for an existing client. */
	ENGINE_API FTestWorldInstance::FContext GetClientContext(int32 ClientId) const;

	/** Ticks all server & client worlds NumTick times synchronously. */
	ENGINE_API void TickAll(int32 NumTicks=1);
	ENGINE_API void TickServer();
	ENGINE_API void TickClients();

	/** Tick the world and drop all outgoing packets */
	ENGINE_API void TickServerAndDrop();
	ENGINE_API void TickClientsAndDrop();

	/** Tick the world but delay the packets that would be sent */
	ENGINE_API void TickServerAndDelay(uint32 NumFramesToDelay = 1);
	ENGINE_API void TickClientsAndDelay(uint32 NumFramesToDelay = 1);

	/**
	 * Ticks all server & client worlds until Predicate returns true, or MaxTicks is reached.
	 * Returns true if Predicate did, false if it didn't within MaxTicks.
	 */
	template<class PredicateT>
	bool TickAllUntil(const PredicateT& Predicate, float DeltaSeconds = 0.0166f, int32 MaxTicks = 60);

	/**
	 * Ticks all server & client worlds until Predicate returns true, or MaxTicks is reached using the default Delta
	 * Returns true if Predicate did, false if it didn't within MaxTicks.
	 */
	template<class PredicateT>
	bool TickUntil(const PredicateT& Predicate, int32 MaxTicks = 60);

	/** Ticks all server & client worlds until the passed in client world has a valid client PlayerController. */
	ENGINE_API bool WaitForClientConnect(FTestWorldInstance& Client);

	/** Return the Server's player state corresponding to a specific client */
	ENGINE_API APlayerController* GetServerPlayerControllerOfClient(uint32 ClientIndex);

	/** Move a client's PlayerController to a location on the server */
	ENGINE_API void MovePlayerToLocation(uint32 ClientIndex, const FVector& Location);

	/**
	 * Find the remote instance of a replicated object and ensures if it can't find it
	 * @param ServerObject The server object that you want the remote version of.
	 * @param ClientIndex The client you want a remote instance of. If not set the function will use the first client available.
	 * @return Return the remote (client) instance of the same object if it exists.
	 */
	ENGINE_API UObject* FindReplicatedObjectOnClient(UObject* ServerObject, uint32 ClientIndex = UINT32_MAX) const;

	template<typename T>
	T* FindReplicatedObjectOnClient(T* ServerObject, uint32 ClientIndex = UINT32_MAX) const
	{
		return Cast<T>(FindReplicatedObjectOnClient(Cast<UObject>(ServerObject), ClientIndex));
	}

	/**
	 * Returns if the remote instance of a replicated object exists on the client or not.
	 * @param ServerObject The server object you want to see if it is replicated
	 * @param ClientIndex The client you want to test on. If not set the function will use the first client available.
	 */
	ENGINE_API bool DoesReplicatedObjectExistOnClient(UObject* ServerObject, uint32 ClientIndex = UINT32_MAX) const;

	/**
	 * Returns if the object is registered for replication on the server.
	 * @param ServerObject The server object you want to see if it is replicated
	 */
	ENGINE_API bool IsServerObjectReplicated(const UObject* ServerObject) const;

public:

	/** Delegate called before a TickAll is executed */
	DECLARE_MULTICAST_DELEGATE(FPreTickAll);
	FPreTickAll PreTickAllDelegate;

	/** Delegate called after a client finished connecting to the server */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnClientConnected, const FTestWorldInstance& Client);
	FOnClientConnected OnClientConnected;

public:

	/** Server and Client Worlds */
	FTestWorldInstance Server;
	TArray<FTestWorldInstance> Clients;

private:

	float TickDeltaSeconds = 0.0166f;

	// Sets up important settings for the networking system to run optimally
	FScopedTestSettings TestSettings;

	// Restore GWorld and other PIE settings
	FScopedNetTestPIERestoration PIERestoration;

	// Testworld description.
	FString Description;
};

//------------------------------------------------------------------------
// Inline functions
//------------------------------------------------------------------------

template<class PredicateT>
inline bool FTestWorlds::TickAllUntil(const PredicateT& Predicate, float DeltaSeconds, int32 MaxTicks)
{
	PreTickAllDelegate.Broadcast();

	int32 TickCount = 0;
	bool bPredicateResult = Predicate();
		
	while (!bPredicateResult && TickCount < MaxTicks)
	{
		Server.Tick(DeltaSeconds);
		for (FTestWorldInstance& Client : Clients)
		{
			Client.Tick(DeltaSeconds);
		}
		TickCount++;
		GFrameCounter++;
		bPredicateResult = Predicate();
	}

	return bPredicateResult;
}

template<class PredicateT>
inline bool FTestWorlds::TickUntil(const PredicateT& Predicate, int32 MaxTicks)
{
	PreTickAllDelegate.Broadcast();

	int32 TickCount = 0;
	bool bPredicateResult = Predicate();

	while (!bPredicateResult && TickCount < MaxTicks)
	{
		TickServer();
		TickClients();
		TickCount++;
		GFrameCounter++;
		bPredicateResult = Predicate();
	}

	return bPredicateResult;
}

ENGINE_API bool IsClientConnected(FTestWorldInstance& Instance);
ENGINE_API bool IsServerConnected(FTestWorldInstance& Instance);

#endif //WITH_EDITOR

} // end namespace UE::Net