// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Engine.h"
#include "TextImportTest.generated.h"

UENUM(Flags)
enum class ETextImportTestFlags : uint32
{
	Default = 0,
	FlagA = 1 << 0,
	FlagB = 1 << 1,
	FlagC = 1 << 2,
	FlagD = 1 << 3,
	FlagE = 1 << 4,
	TestStructDefault = 1 << 5,
};

ENUM_CLASS_FLAGS(ETextImportTestFlags)

USTRUCT()
struct FTextImportTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	ETextImportTestFlags EmbeddedFlags = ETextImportTestFlags::TestStructDefault;

	UPROPERTY()
	int32 TestInt = 1;

	UPROPERTY()
	FString TestString = FString("DefaultString");

	inline bool operator==(const FTextImportTestStruct& OtherStruct)
	{
		return EmbeddedFlags == OtherStruct.EmbeddedFlags && TestInt == OtherStruct.TestInt && TestString == OtherStruct.TestString;
	}

	inline FString ToString() const
	{
		return FString::Printf(TEXT("EmbeddedFlags: %d TestInt: %d TestString: %s"), static_cast<int32>(EmbeddedFlags), TestInt, *TestString);
	}
};


UCLASS()
class UTextImportContainer : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FTextImportTestStruct ResultStruct;
};
