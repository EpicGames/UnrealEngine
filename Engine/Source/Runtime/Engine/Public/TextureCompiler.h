// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/WeakObjectPtr.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Containers/Set.h"
#include "IAssetCompilingManager.h"

#if WITH_EDITOR

class FAsyncCompilationNotification;
class UTexture;
class FQueuedThreadPool;
enum class EQueuedWorkPriority : uint8;

DECLARE_MULTICAST_DELEGATE_OneParam(FTexturePostCompileEvent, const TArrayView<UTexture* const>&);

class FTextureCompilingManager : IAssetCompilingManager
{
public:
	ENGINE_API static FTextureCompilingManager& Get();

	/**
	 * Returns true if the feature is currently activated.
	 */
	ENGINE_API bool IsAsyncTextureCompilationEnabled() const;

	/**
	 * Returns true if the cancelation feature is currently activated.
	 */
	ENGINE_API bool IsAsyncCompilationCancelable() const;

	/** 
	 * Returns the number of outstanding texture compilations.
	 */
	ENGINE_API int32 GetNumRemainingTextures() const;

	/** 
	 * Adds textures compiled asynchronously so they are monitored. 
	 */
	ENGINE_API void AddTextures(TArrayView<UTexture* const> InTextures);

	/** 
	 * Forces textures to be recompiled asynchronously later. 
	 */
	ENGINE_API void ForceDeferredTextureRebuildAnyThread(TArrayView<const TWeakObjectPtr<UTexture>> InTextures);

	/** 
	 * Blocks until completion of the requested textures.
	 */
	ENGINE_API void FinishCompilation(TArrayView<UTexture* const> InTextures);

	/** 
	 * Blocks until completion of all async texture compilation.
	 */
	ENGINE_API void FinishAllCompilation() override;

	/**
	 * Returns if asynchronous compilation is allowed for this texture.
	 */
	ENGINE_API bool IsAsyncCompilationAllowed(UTexture* InTexture) const;

	/**
	 * Request that the texture be processed at the specified priority.
	 */
	ENGINE_API bool RequestPriorityChange(UTexture* InTexture, EQueuedWorkPriority Priority);

	/**
	 * Returns the priority at which the given texture should be scheduled.
	 */
	ENGINE_API EQueuedWorkPriority GetBasePriority(UTexture* InTexture) const;

	/**
	 * Returns the threadpool where texture compilation should be scheduled.
	 */
	ENGINE_API FQueuedThreadPool* GetThreadPool() const;

	/**
	 * Cancel any pending work and blocks until it is safe to shut down.
	 */
	ENGINE_API void Shutdown() override;

	/** Get the name of the asset type this compiler handles */
	ENGINE_API static FName GetStaticAssetTypeName();

	/** Return true if the texture is currently compiled */
	ENGINE_API bool IsCompilingTexture(UTexture* InTexture) const;

	FTexturePostCompileEvent& OnTexturePostCompileEvent() {return TexturePostCompileEvent;}

private:
	friend class FAssetCompilingManager;

	FTextureCompilingManager();

	/** Handle generic finish compilation */
	void FinishCompilationForObjects(TArrayView<UObject* const> InObjects) override;
	
	/** Mark compilation of the provided static meshes as canceled. */
	void MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects) override;

	FName GetAssetTypeName() const override;
	FTextFormat GetAssetNameFormat() const override;
	TArrayView<FName> GetDependentTypeNames() const override;
	int32 GetNumRemainingAssets() const override;
	void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;
	void ProcessAsyncTasks(const AssetCompilation::FProcessAsyncTaskParams& Params) override;

	void FinishCompilationsForGame();
	void ProcessTextures(bool bLimitExecutionTime, int32 MaximumPriority = -1);
	void UpdateCompilationNotification();

	bool GetCurrentPriority(UTexture* InTexture, EQueuedWorkPriority& OutPriority);
	[[nodiscard]] bool PostCompilation(UTexture* Texture);
	void PostCompilation(TArrayView<UTexture* const> InCompiledTextures);

	void ProcessDeferredRequests();


	FString CurrentPostCompilationDDCKey;
	double LastReschedule = 0.0f;
	bool bHasShutdown = false;
	bool bIsRoutingPostCompilation = false;
	bool bIsRoutingPostCompilationError = false;
	UTexture* PostCompilationTexture = nullptr;

	UE::TConsumeAllMpmcQueue<TWeakObjectPtr<UTexture>> DeferredRebuildRequestQueue;
	TArray<TSet<TWeakObjectPtr<UTexture>>> RegisteredTextureBuckets;
	TUniquePtr<FAsyncCompilationNotification> Notification;

	/** Event issued at the end of the compile process */
	FTexturePostCompileEvent TexturePostCompileEvent;
};

#endif
