// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/DeveloperSettings.h"
#include "Engine/TextureDefines.h"
#include "TextureEncodingSettings.generated.h"


// For encoders that support it (i.e. Oodle), this controls how much time to
// spend on finding better encoding.
// (These need to match the Oodle OodleTex_EncodeEffortLevel values if you are 
// using Oodle.)
UENUM()
enum class ETextureEncodeEffort : uint8
{
	Default = 0		UMETA(ToolTip = "Let the encoder decide what's best."),
	Low = 10		UMETA(ToolTip = "Faster encoding, lower quality. Probably don't ship textures encoded at this effort level"),
	Normal = 20		UMETA(ToolTip = "Reasonable compromise"),
	High = 30		UMETA(ToolTip = "More time, better quality - good for nightlies / unattended cooks.")
};

// enum values must match exactly with OodleTex_RDO_UniversalTiling
UENUM()
enum class ETextureUniversalTiling : uint8
{
	Disabled = 0,
	Enabled_256KB = 1,
	Enabled_64KB = 2
};


// Enum that allows for not overriding what the existing setting is - all the
// other values have the same meaning as ETextureEncodeSpeed
UENUM()
enum class ETextureEncodeSpeedOverride : uint8
{
	Disabled = 255, // don't override.
	Final = 0,
	FinalIfAvailable = 1,
	Fast = 2
};

//
// This is the public, thread safe class for accessing the encoding settings. They are
// filled out as part of the engine class default object init loop during PreInit, and
// are safe to access at any point during CDO construction as well.
//
struct FResolvedTextureEncodingSettings
{
	// Properties mirrored from UTextureEncodingUserSettings - look there for 
	// documentation
	struct 
	{
		ETextureEncodeSpeedOverride ForceEncodeSpeed;
	} User;

	// Properties mirrored from UTextureEncodingProjectSettings - look there for 
	// documentation
	struct
	{
		uint32 bSharedLinearTextureEncoding : 1;
		uint32 bFinalUsesRDO : 1;
		uint32 bFastUsesRDO : 1;
		int8 FinalRDOLambda;
		int8 FastRDOLambda;
		ETextureEncodeEffort FinalEffortLevel;
		ETextureUniversalTiling FinalUniversalTiling;
		ETextureEncodeEffort FastEffortLevel;
		ETextureUniversalTiling FastUniversalTiling;
		ETextureEncodeSpeed CookUsesSpeed;
		ETextureEncodeSpeed EditorUsesSpeed;
	} Project;

	// The resolved EncodeSpeed to use for this instance, taking in to account overrides.
	ETextureEncodeSpeed EncodeSpeed;

	static ENGINE_API FResolvedTextureEncodingSettings const& Get();
};

//
// Separate type so that the engine can check for custom encoding set in the texture
// editor module without needing to depend on it.
//
class FTextureEditorCustomEncode
{
public:
	// If we want to override Oodle specific encoding settings, we set this to true.
	bool bUseCustomEncode = false;

	// [0,100]
	uint8 OodleRDOLambda = 0;

	// enum ETextureEncodeEffort
	uint8 OodleEncodeEffort = 0;

	// enum ETextureUniversalTiling
	uint8 OodleUniversalTiling = 0;

};