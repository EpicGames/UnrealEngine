// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureLayout.h: Texture space allocation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"

enum class ETextureLayoutAspectRatio
{
	None,
	Force2To1,
	ForceSquare,
};

/**
 * An incremental texture space allocator.
 * For best results, add the elements ordered descending in size.
 */
class FTextureLayout
{
public:

	/**
	 * Minimal initialization constructor.
	 * @param	MinSizeX - The minimum width of the texture.
	 * @param	MinSizeY - The minimum height of the texture.
	 * @param	MaxSizeX - The maximum width of the texture.
	 * @param	MaxSizeY - The maximum height of the texture.
	 * @param	InPowerOfTwoSize - True if the texture size must be a power of two.
	 * @param	bInForce2To1Aspect - True if the texture size must have a 2:1 aspect.
	 * @param	bInAlignByFour - True if the texture size must be a multiple of 4..
	 */
	FTextureLayout(uint32 InMinSizeX, uint32 InMinSizeY, uint32 MaxSizeX, uint32 MaxSizeY, bool bInPowerOfTwoSize = false, ETextureLayoutAspectRatio InAspect = ETextureLayoutAspectRatio::None, bool bInAlignByFour = true):
		MinSizeX(InMinSizeX),
		MinSizeY(InMinSizeY),
		SizeX(InMinSizeX),
		SizeY(InMinSizeY),
		AspectRatio(InAspect),
		bPowerOfTwoSize(bInPowerOfTwoSize),
		bAlignByFour(bInAlignByFour)
	{
		Nodes.Emplace(0, 0, IntCastChecked<uint16>(MaxSizeX), IntCastChecked<uint16>(MaxSizeY));
	}

	/**
	 * Finds a free area in the texture large enough to contain a surface with the given size.
	 * If a large enough area is found, it is marked as in use, the output parameters OutBaseX and OutBaseY are
	 * set to the coordinates of the upper left corner of the free area and the function return true.
	 * Otherwise, the function returns false and OutBaseX and OutBaseY remain uninitialized.
	 * @param	OutBaseX - If the function succeeds, contains the X coordinate of the upper left corner of the free area on return.
	 * @param	OutBaseY - If the function succeeds, contains the Y coordinate of the upper left corner of the free area on return.
	 * @param	ElementSizeX - The size of the surface to allocate in horizontal pixels.
	 * @param	ElementSizeY - The size of the surface to allocate in vertical pixels.
	 * @return	True if succeeded, false otherwise.
	 */
	bool AddElement(uint32& OutBaseX, uint32& OutBaseY, uint32 ElementSizeX, uint32 ElementSizeY)
	{
		if (ElementSizeX == 0 || ElementSizeY == 0)
		{
			OutBaseX = 0;
			OutBaseY = 0;
			return true;
		}

		if (bAlignByFour)
		{
			// Pad to 4 to ensure alignment
			ElementSizeX = (ElementSizeX + 3) & ~3;
			ElementSizeY = (ElementSizeY + 3) & ~3;
		}

		// Try allocating space without enlarging the texture.
		int32	NodeIndex = AddSurfaceInner(0, ElementSizeX, ElementSizeY, false);
		if (NodeIndex == INDEX_NONE)
		{
			// Try allocating space which might enlarge the texture.
			NodeIndex = AddSurfaceInner(0, ElementSizeX, ElementSizeY, true);
		}

		if (NodeIndex != INDEX_NONE)
		{
			FTextureLayoutNode&	Node = Nodes[NodeIndex];
			Node.bUsed = true;
			OutBaseX = Node.MinX;
			OutBaseY = Node.MinY;

			UpdateSize(Node.MinX + ElementSizeX, Node.MinY + ElementSizeY);
			return true;
		}
		else
		{
			return false;
		}
	}

	/** 
	 * Removes a previously allocated element from the layout and collapses the tree as much as possible,
	 * In order to create the largest free block possible and return the tree to its state before the element was added.
	 * @return	True if the element specified by the input parameters existed in the layout.
	 */
	bool RemoveElement(uint32 ElementBaseX, uint32 ElementBaseY, uint32 ElementSizeX, uint32 ElementSizeY)
	{
		if (bAlignByFour)
		{
			// Pad to 4 to ensure alignment
			ElementSizeX = (ElementSizeX + 3) & ~3;
			ElementSizeY = (ElementSizeY + 3) & ~3;
		}

		int32 FoundNodeIndex = INDEX_NONE;
		// Search through nodes to find the element to remove
		//@todo - traverse the tree instead of iterating through all nodes
		for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
		{
			FTextureLayoutNode&	Node = Nodes[NodeIndex];

			if (Node.MinX == ElementBaseX
				&& Node.MinY == ElementBaseY
				&& Node.SizeX == ElementSizeX
				&& Node.SizeY == ElementSizeY)
			{
				FoundNodeIndex = NodeIndex;
				break;
			}
		}

		if (FoundNodeIndex != INDEX_NONE)
		{
			// Mark the found node as not being used anymore
			Nodes[FoundNodeIndex].bUsed = false;

			// Walk up the tree to find the node closest to the root that doesn't have any used children
			int32 ParentNodeIndex = FindParentNode(FoundNodeIndex);
			ParentNodeIndex = (ParentNodeIndex != INDEX_NONE && IsNodeUsed(ParentNodeIndex)) ? INDEX_NONE : ParentNodeIndex;
			int32 LastParentNodeIndex = ParentNodeIndex;
			while (ParentNodeIndex != INDEX_NONE 
				&& !IsNodeUsed(Nodes[ParentNodeIndex].ChildA) 
				&& !IsNodeUsed(Nodes[ParentNodeIndex].ChildB))
			{
				LastParentNodeIndex = ParentNodeIndex;
				ParentNodeIndex = FindParentNode(ParentNodeIndex);
			} 

			// Remove the children of the node closest to the root with only unused children,
			// Which restores the tree to its state before this element was allocated,
			// And allows allocations as large as LastParentNode in the future.
			if (LastParentNodeIndex != INDEX_NONE)
			{
				RemoveChildren(LastParentNodeIndex);
			}

			// Recalculate size
			{
				SizeX = MinSizeX;
				SizeY = MinSizeY;

				for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
				{
					const FTextureLayoutNode& Node = Nodes[NodeIndex];

					if (Node.bUsed)
					{
						UpdateSize(Node.MinX + Node.SizeX, Node.MinY + Node.SizeY);
					}
				}
			}

			return true;
		}

		return false;
	}

	/**
	 * Returns the minimum texture width which will contain the allocated surfaces.
	 */
	uint32 GetSizeX() const { return SizeX; }

	/**
	 * Returns the minimum texture height which will contain the allocated surfaces.
	 */
	uint32 GetSizeY() const { return SizeY; }

private:

	struct FTextureLayoutNode
	{
		int32		ChildA,
				ChildB;
		uint16	MinX,
				MinY,
				SizeX,
				SizeY;
		bool	bUsed;

		FTextureLayoutNode() {}

		FTextureLayoutNode(uint16 InMinX, uint16 InMinY, uint16 InSizeX, uint16 InSizeY):
			ChildA(INDEX_NONE),
			ChildB(INDEX_NONE),
			MinX(InMinX),
			MinY(InMinY),
			SizeX(InSizeX),
			SizeY(InSizeY),
			bUsed(false)
		{}
	};

	uint32 MinSizeX;
	uint32 MinSizeY;
	uint32 SizeX;
	uint32 SizeY;
	ETextureLayoutAspectRatio AspectRatio;
	bool bPowerOfTwoSize;
	bool bAlignByFour;
	TArray<FTextureLayoutNode,TInlineAllocator<5> > Nodes;

	/** Recursively traverses the tree depth first and searches for a large enough leaf node to contain the requested allocation. */
	int32 AddSurfaceInner(int32 NodeIndex, uint32 ElementSizeX, uint32 ElementSizeY, bool bAllowTextureEnlargement)
	{
		checkSlow(NodeIndex != INDEX_NONE);
		// Store a copy of the current node on the stack for easier debugging.
		// Can't store a pointer to the current node since Nodes may be reallocated in this function.
		const FTextureLayoutNode CurrentNode = Nodes[NodeIndex];
		// But do access this node via a pointer until the first recursive call. Prevents a ton of LHS.
		const FTextureLayoutNode* CurrentNodePtr = &Nodes[NodeIndex];
		if (CurrentNodePtr->ChildA != INDEX_NONE)
		{
			// Children are always allocated together
			checkSlow(CurrentNodePtr->ChildB != INDEX_NONE);

			// Traverse the children
			const int32 Result = AddSurfaceInner(CurrentNodePtr->ChildA, ElementSizeX, ElementSizeY, bAllowTextureEnlargement);
			
			// The pointer is now invalid, be explicit!
			CurrentNodePtr = 0;

			if (Result != INDEX_NONE)
			{
				return Result;
			}

			return AddSurfaceInner(CurrentNode.ChildB, ElementSizeX, ElementSizeY, bAllowTextureEnlargement);
		}
		// Node has no children, it is a leaf
		else
		{
			// Reject this node if it is already used
			if (CurrentNodePtr->bUsed)
			{
				return INDEX_NONE;
			}

			// Reject this node if it is too small for the element being placed
			if (CurrentNodePtr->SizeX < ElementSizeX || CurrentNodePtr->SizeY < ElementSizeY)
			{
				return INDEX_NONE;
			}

			if (!bAllowTextureEnlargement)
			{
				// Reject this node if this is an attempt to allocate space without enlarging the texture, 
				// And this node cannot hold the element without enlarging the texture.
				if (CurrentNodePtr->MinX + ElementSizeX > SizeX || CurrentNodePtr->MinY + ElementSizeY > SizeY)
				{
					return INDEX_NONE;
				}
			}
			else // Reject this node if this is an attempt to allocate space beyond max size.
			{
				const int32 MaxTextureSize = (1 << (MAX_TEXTURE_MIP_COUNT - 1));
				int32 ExpectedSizeX = CurrentNodePtr->MinX + ElementSizeX;
				int32 ExpectedSizeY = CurrentNodePtr->MinY + ElementSizeY;
				if (bPowerOfTwoSize)
				{
					ExpectedSizeX = FMath::RoundUpToPowerOfTwo(ExpectedSizeX);
					ExpectedSizeY = FMath::RoundUpToPowerOfTwo(ExpectedSizeY);

					switch (AspectRatio)
					{
					case ETextureLayoutAspectRatio::None:
						break;
					case ETextureLayoutAspectRatio::ForceSquare:
						ExpectedSizeX = FMath::Max(ExpectedSizeX, ExpectedSizeY);
						ExpectedSizeY = ExpectedSizeX;
						break;
					case ETextureLayoutAspectRatio::Force2To1:
						ExpectedSizeX = FMath::Max(ExpectedSizeX, ExpectedSizeY * 2);
						ExpectedSizeY = FMath::Max(ExpectedSizeY, ExpectedSizeX / 2);
						break;
					default:
						checkNoEntry();
						break;
					}
				}
				
				if (ExpectedSizeX > MaxTextureSize ||ExpectedSizeY > MaxTextureSize)
				{
					return INDEX_NONE;
				}
			}

			// Use this node if the size matches the requested element size
			if (CurrentNodePtr->SizeX == ElementSizeX && CurrentNodePtr->SizeY == ElementSizeY)
			{
				return NodeIndex;
			}

			const uint32 ExcessWidth = CurrentNodePtr->SizeX - ElementSizeX;
			const uint32 ExcessHeight = CurrentNodePtr->SizeY - ElementSizeY;

			// The pointer to the current node may be invalidated below, be explicit!
			CurrentNodePtr = 0;

			// Add new nodes, and link them as children of the current node.
			if (ExcessWidth > ExcessHeight)
			{
				// Update the child indices
				Nodes[NodeIndex].ChildA = Nodes.Num();

				// Create a child with the same width as the element being placed.
				// The height may not be the same as the element height yet, in that case another subdivision will occur when traversing this child node.
				Nodes.Emplace(
					CurrentNode.MinX,
					CurrentNode.MinY,
					IntCastChecked<uint16>(ElementSizeX),
					CurrentNode.SizeY
					);

				// Create a second child to contain the leftover area in the X direction
				Nodes[NodeIndex].ChildB = Nodes.Num();
				Nodes.Emplace(
					IntCastChecked<uint16>(CurrentNode.MinX + ElementSizeX),
					CurrentNode.MinY,
					IntCastChecked<uint16>(CurrentNode.SizeX - ElementSizeX),
					CurrentNode.SizeY
					);
			}
			else
			{
				Nodes[NodeIndex].ChildA = Nodes.Num();
				Nodes.Emplace(
					CurrentNode.MinX,
					CurrentNode.MinY,
					CurrentNode.SizeX,
					IntCastChecked<uint16>(ElementSizeY)
					);

				Nodes[NodeIndex].ChildB = Nodes.Num();
				Nodes.Emplace(
					CurrentNode.MinX,
					IntCastChecked<uint16>(CurrentNode.MinY + ElementSizeY),
					CurrentNode.SizeX,
					IntCastChecked<uint16>(CurrentNode.SizeY - ElementSizeY)
					);
			}

			// Only traversing ChildA, since ChildA is always the newly created node that matches the element size
			return AddSurfaceInner(Nodes[NodeIndex].ChildA, ElementSizeX, ElementSizeY, bAllowTextureEnlargement);
		}
	}

	void UpdateSize(uint32 ElementMaxX, uint32 ElementMaxY)
	{
		if (bPowerOfTwoSize)
		{
			SizeX = FMath::Max<uint32>(SizeX, FMath::RoundUpToPowerOfTwo(ElementMaxX));
			SizeY = FMath::Max<uint32>(SizeY, FMath::RoundUpToPowerOfTwo(ElementMaxY));
			
			switch(AspectRatio)
			{
			case ETextureLayoutAspectRatio::None:
				break;
			case ETextureLayoutAspectRatio::ForceSquare:
				SizeX = FMath::Max(SizeX, SizeY);
				SizeY = SizeX;
				break;
			case ETextureLayoutAspectRatio::Force2To1:
				SizeX = FMath::Max( SizeX, SizeY * 2 );
				SizeY = FMath::Max( SizeY, SizeX / 2 );
				break;
			default:
				checkNoEntry();
				break;
			}
		}
		else
		{
			SizeX = FMath::Max<uint32>(SizeX, ElementMaxX);
			SizeY = FMath::Max<uint32>(SizeY, ElementMaxY);
		}
	}

	/** Returns the index into Nodes of the parent node of SearchNode. */
	int32 FindParentNode(int32 SearchNodeIndex)
	{
		//@todo - could be a constant time search if the nodes stored a parent index
		for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
		{
			FTextureLayoutNode&	Node = Nodes[NodeIndex];
			if (Node.ChildA == SearchNodeIndex || Node.ChildB == SearchNodeIndex)
			{
				return NodeIndex;
			}
		}
		return INDEX_NONE;
	}

	/** Returns true if the node or any of its children are marked used. */
	bool IsNodeUsed(int32 NodeIndex)
	{
		bool bChildrenUsed = false;
		if (Nodes[NodeIndex].ChildA != INDEX_NONE)
		{
			checkSlow(Nodes[NodeIndex].ChildB != INDEX_NONE);
			bChildrenUsed = IsNodeUsed(Nodes[NodeIndex].ChildA) || IsNodeUsed(Nodes[NodeIndex].ChildB);
		}
		return Nodes[NodeIndex].bUsed || bChildrenUsed;
	}

	/** Recursively removes the children of a given node from the Nodes array and adjusts existing indices to compensate. */
	void RemoveChildren(int32 NodeIndex)
	{
		// Traverse the children depth first
		if (Nodes[NodeIndex].ChildA != INDEX_NONE)
		{
			RemoveChildren(Nodes[NodeIndex].ChildA);
		}
		if (Nodes[NodeIndex].ChildB != INDEX_NONE)
		{
			RemoveChildren(Nodes[NodeIndex].ChildB);
		}

		if (Nodes[NodeIndex].ChildA != INDEX_NONE)
		{
			// Store off the index of the child since it may be changed in the code below
			const int32 OldChildA = Nodes[NodeIndex].ChildA;

			// Remove the child from the Nodes array
			Nodes.RemoveAt(OldChildA);

			// Iterate over all the Nodes and fix up their indices now that an element has been removed
			for (int32 OtherNodeIndex = 0; OtherNodeIndex < Nodes.Num(); OtherNodeIndex++)
			{
				if (Nodes[OtherNodeIndex].ChildA >= OldChildA)
				{
					Nodes[OtherNodeIndex].ChildA--;
				}
				if (Nodes[OtherNodeIndex].ChildB >= OldChildA)
				{
					Nodes[OtherNodeIndex].ChildB--;
				}
			}
			// Mark the node as not having a ChildA
			Nodes[NodeIndex].ChildA = INDEX_NONE;
		}

		if (Nodes[NodeIndex].ChildB != INDEX_NONE)
		{
			const int32 OldChildB = Nodes[NodeIndex].ChildB;
			Nodes.RemoveAt(OldChildB);
			for (int32 OtherNodeIndex = 0; OtherNodeIndex < Nodes.Num(); OtherNodeIndex++)
			{
				if (Nodes[OtherNodeIndex].ChildA >= OldChildB)
				{
					Nodes[OtherNodeIndex].ChildA--;
				}
				if (Nodes[OtherNodeIndex].ChildB >= OldChildB)
				{
					Nodes[OtherNodeIndex].ChildB--;
				}
			}
			Nodes[NodeIndex].ChildB = INDEX_NONE;
		}
	}
};

/**
 * A simple binned style atlas layout, which uses FTextureLayout as the bin allocator.
 * Binning greatly reduces fragmentation over time with many allocations of different sizes and lifetimes, but ties up free memory in other bins.  
 * Binning also greatly reduces the FTextureLayout tree depth, and therefore allocation cost.
 * Best results when the number of unique allocation sizes are small.
 */
class FBinnedTextureLayout
{
public:

	FBinnedTextureLayout(FIntPoint InMaxSize, int32 InBinSizeTexels) :
		BinSizeTexels(InBinSizeTexels),
		MaxSize(InMaxSize),
		Layout(0, 0, InMaxSize.X, InMaxSize.Y, false, ETextureLayoutAspectRatio::None, false)
	{}

	/**
	 * Allocates an element from the atlas layout.  Returns true if allocation succeeds.
	 * Note: Sort allocations by descending size for best packing.
	 */
	ENGINE_API bool AddElement(FIntPoint ElementSize, FIntPoint& OutElementMin);

	/** Removes an element which was previously added. */
	ENGINE_API void RemoveElement(FIntRect Element);

private:

	class FBinAllocation
	{
	public:
		FIntRect LayoutAllocation;
		TArray<int32> FreeList;
	};

	class FBin
	{
	public:

		FBin(FIntPoint InElementSize)
		{
			ElementSize = InElementSize;
			bOutOfSpace = false;
		}

		FIntPoint ElementSize;
		bool bOutOfSpace;
		TArray<FBinAllocation, TInlineAllocator<1>> BinAllocations;
	};

	int32 BinSizeTexels;
	FIntPoint MaxSize;
	TArray<FBin> Bins;
	FTextureLayout Layout;
};

namespace TextureLayoutTools
{
	/**
	 * Computes the difference between two value arrays (templated)
	 *
	 * @param	ValuesA		First list of values
	 * @param	ValuesB		Second list of values
	 * @param	ValueCount	Number of values
	 * @param	OutValueDifferences		Difference between each value
	 */
	template<typename ValueType>
	void ComputeDifferenceArray(const ValueType* ValuesA, const ValueType* ValuesB, const int32 ValueCount, TArray< double >& OutValueDifferences)
	{
		OutValueDifferences.Reset();
		OutValueDifferences.AddUninitialized(ValueCount);
		for (int32 CurValueIndex = 0; CurValueIndex < ValueCount; ++CurValueIndex)
		{
			const ValueType CurValueA = ValuesA[CurValueIndex];
			const ValueType CurValueB = ValuesB[CurValueIndex];

			const double Difference = (double)CurValueA - (double)CurValueB;
			OutValueDifferences[CurValueIndex] = Difference;
		}
	}


	/**
	 * Computes the mean square root deviation for a set of values (templated)
	 *
	 * @param	Values		Array of values
	 * @param	ValueCount	Number of values
	 *
	 * @return	Mean square deviation for the value set
	 */
	template<typename ValueType>
	double ComputeRootMeanSquareDeviation(const ValueType* Values, const int32 ValueCount)
	{
		// Sum the values
		double ValuesSum = 0.0;
		for (int32 CurValueIndex = 0; CurValueIndex < ValueCount; ++CurValueIndex)
		{
			const ValueType CurValue = Values[CurValueIndex];
			ValuesSum += (double)CurValue;
		}

		// Compute the mean
		const double ValuesMean = (double)ValuesSum / (double)ValueCount;

		// Compute the squared sum of all mean deviations
		double ValuesSquaredDifferenceSum = 0;
		for (int32 CurValueIndex = 0; CurValueIndex < ValueCount; ++CurValueIndex)
		{
			const ValueType CurValue = Values[CurValueIndex];
			const double MeanDifference = (double)CurValue - ValuesMean;
			ValuesSquaredDifferenceSum += (MeanDifference * MeanDifference);
		}

		// Compute the root mean square deviation
		const double ValuesMeanSquaredDifference = ValuesSquaredDifferenceSum / (double)ValueCount;
		const double ValuesRootMeanSquareDeviation = sqrt( ValuesMeanSquaredDifference );

		return ValuesRootMeanSquareDeviation;
	}
}
