// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureLayout3d.h: Texture space allocation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

/**
 * An incremental texture space allocator.
 * For best results, add the elements ordered descending in size.
 */
class FTextureLayout3d
{
public:

	/**
	 * Minimal initialization constructor.
	 * @param	MinSizeX - The minimum width of the texture.
	 * @param	MinSizeY - The minimum height of the texture.
	 * @param	MaxSizeX - The maximum width of the texture.
	 * @param	MaxSizeY - The maximum height of the texture.
	 * @param	InPowerOfTwoSize - True if the texture size must be a power of two.
	 */
	FTextureLayout3d(uint32 InMinSizeX, uint32 InMinSizeY, uint32 InMinSizeZ, uint32 MaxSizeX, uint32 MaxSizeY, uint32 MaxSizeZ, bool bInPowerOfTwoSize = false, bool bInAlignByFour = true, bool bInAllowShrink = true):
		MinSizeX(InMinSizeX),
		MinSizeY(InMinSizeY),
		MinSizeZ(InMinSizeZ),
		SizeX(InMinSizeX),
		SizeY(InMinSizeY),
		SizeZ(InMinSizeZ),
		bPowerOfTwoSize(bInPowerOfTwoSize),
		bAlignByFour(bInAlignByFour),
		bAllowShrink(bInAllowShrink)
	{
		check(MaxSizeX < USHRT_MAX && MaxSizeY < USHRT_MAX && MaxSizeZ < USHRT_MAX);
		Nodes.Emplace(static_cast<uint16>(0), static_cast<uint16>(0), static_cast<uint16>(0), IntCastChecked<uint16>(MaxSizeX), IntCastChecked<uint16>(MaxSizeY), IntCastChecked<uint16>(MaxSizeZ), INDEX_NONE);
		UnusedLeaves.Emplace(static_cast<uint16>(0), static_cast<uint16>(0), static_cast<uint16>(0), IntCastChecked<uint16>(MaxSizeX), IntCastChecked<uint16>(MaxSizeY), IntCastChecked<uint16>(MaxSizeZ), 0);
	}

	/**
	 * Finds a free area in the texture large enough to contain a surface with the given size.
	 * If a large enough area is found, it is marked as in use, the output parameters OutBaseX and OutBaseY are
	 * set to the coordinates of the upper left corner of the free area and the function return true.
	 * Otherwise, the function returns false and OutBaseX and OutBaseY remain uninitialized.
	 * @param	OutBaseX - If the function succeeds, contains the X coordinate of the upper left corner of the free area on return.
	 * @param	OutBaseY - If the function succeeds, contains the Y coordinate of the upper left corner of the free area on return.
	 * @param	ElementSizeX - The size of the surface to allocate in horizontal pixels.
	 * @param	ElementSizeY - The size of the surface to allocate in vertical pixels.
	 * @return	True if succeeded, false otherwise.
	 */
	bool AddElement(uint32& OutBaseX, uint32& OutBaseY, uint32& OutBaseZ, uint32 ElementSizeX, uint32 ElementSizeY, uint32 ElementSizeZ)
	{
		if (ElementSizeX == 0 || ElementSizeY == 0 || ElementSizeZ == 0)
		{
			OutBaseX = 0;
			OutBaseY = 0;
			OutBaseZ = 0;
			return true;
		}

		if (bAlignByFour)
		{
			// Pad to 4 to ensure alignment
			ElementSizeX = (ElementSizeX + 3) & ~3;
			ElementSizeY = (ElementSizeY + 3) & ~3;
			ElementSizeZ = (ElementSizeZ + 3) & ~3;
		}
		
		int32 IdealNodeIndex = INDEX_NONE;
		int32 ContainingNodeIndex = INDEX_NONE;
		int32 RemoveIndex = INDEX_NONE;
		for (int32 Index = 0; Index < UnusedLeaves.Num(); ++Index)
		{
			if (ElementSizeX <= UnusedLeaves[Index].SizeX && ElementSizeY <= UnusedLeaves[Index].SizeY && ElementSizeZ <= UnusedLeaves[Index].SizeZ)
			{
				if (ContainingNodeIndex == INDEX_NONE)
				{
					ContainingNodeIndex = UnusedLeaves[Index].NodeIndex;
					RemoveIndex = Index;
				}

				if (UnusedLeaves[Index].MinX + ElementSizeX <= SizeX && UnusedLeaves[Index].MinY + ElementSizeY <= SizeY && UnusedLeaves[Index].MinZ + ElementSizeZ <= SizeZ)
				{
					IdealNodeIndex = UnusedLeaves[Index].NodeIndex;
					RemoveIndex = Index;
					break;
				}
			}
		}

		if (RemoveIndex != INDEX_NONE)
		{
			UnusedLeaves.RemoveAtSwap(RemoveIndex);
		}

		// Try allocating space without enlarging the texture.
		int32 NodeIndex = INDEX_NONE;
		if (IdealNodeIndex != INDEX_NONE)
		{
			NodeIndex = AddSurfaceInner(IdealNodeIndex, ElementSizeX, ElementSizeY, ElementSizeZ, false);
			check(NodeIndex != INDEX_NONE);
		}
		else if (ContainingNodeIndex != INDEX_NONE)
		{
			// Try allocating space which might enlarge the texture.
			NodeIndex = AddSurfaceInner(ContainingNodeIndex, ElementSizeX, ElementSizeY, ElementSizeZ, true);
		}

		if (NodeIndex != INDEX_NONE)
		{
			FTextureLayoutNode3d& Node = Nodes[NodeIndex];
			Node.bUsed = true;
			OutBaseX = Node.MinX;
			OutBaseY = Node.MinY;
			OutBaseZ = Node.MinZ;

			UsedLeaves.Add(FIntVector(OutBaseX, OutBaseY, OutBaseZ), NodeIndex);

			UpdateSize(OutBaseX + ElementSizeX, OutBaseY + ElementSizeY, OutBaseZ + ElementSizeZ);
			return true;
		}
		else
		{
			return false;
		}
	}

	/** 
	 * Removes a previously allocated element from the layout and collapses the tree as much as possible,
	 * In order to create the largest free block possible and return the tree to its state before the element was added.
	 * @return	True if the element specified by the input parameters existed in the layout.
	 */
	bool RemoveElement(uint32 ElementBaseX, uint32 ElementBaseY, uint32 ElementBaseZ, uint32 ElementSizeX, uint32 ElementSizeY, uint32 ElementSizeZ)
	{
		int32 FoundNodeIndex = INDEX_NONE;
		
		// Find the element to remove
		UsedLeaves.RemoveAndCopyValue(FIntVector(ElementBaseX, ElementBaseY, ElementBaseZ), FoundNodeIndex);

		if (FoundNodeIndex != INDEX_NONE)
		{
			check(Nodes[FoundNodeIndex].SizeX == (bAlignByFour ? ((ElementSizeX + 3u) & ~3u) : ElementSizeX)
				&& Nodes[FoundNodeIndex].SizeY == (bAlignByFour ? ((ElementSizeY + 3u) & ~3u) : ElementSizeY)
				&& Nodes[FoundNodeIndex].SizeZ == (bAlignByFour ? ((ElementSizeZ + 3u) & ~3u) : ElementSizeZ));

			// Mark the found node as not being used anymore
			Nodes[FoundNodeIndex].bUsed = false;

			// Walk up the tree to find the node closest to the root that doesn't have any used children
			int32 ParentNodeIndex = FindParentNode(FoundNodeIndex);
			int32 LastParentNodeIndex = INDEX_NONE;

			while (ParentNodeIndex != INDEX_NONE 
				&& !IsNodeUsed(Nodes[ParentNodeIndex].ChildA) 
				&& !IsNodeUsed(Nodes[ParentNodeIndex].ChildB))
			{
				LastParentNodeIndex = ParentNodeIndex;
				ParentNodeIndex = FindParentNode(ParentNodeIndex);
			} 

			// Remove the children of the node closest to the root with only unused children,
			// Which restores the tree to its state before this element was allocated,
			// And allows allocations as large as LastParentNode in the future.
			if (LastParentNodeIndex != INDEX_NONE)
			{
				RemoveChildren(LastParentNodeIndex);
				FoundNodeIndex = LastParentNodeIndex;
			}

			UnusedLeaves.Emplace(
				Nodes[FoundNodeIndex].MinX,
				Nodes[FoundNodeIndex].MinY,
				Nodes[FoundNodeIndex].MinZ,
				Nodes[FoundNodeIndex].SizeX,
				Nodes[FoundNodeIndex].SizeY,
				Nodes[FoundNodeIndex].SizeZ,
				FoundNodeIndex);

			// Recalculate size
			if (bAllowShrink)
			{
				SizeX = MinSizeX;
				SizeY = MinSizeY;
				SizeZ = MinSizeZ;

				for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
				{
					const FTextureLayoutNode3d& Node = Nodes[NodeIndex];

					if (Node.bUsed)
					{
						UpdateSize(Node.MinX + Node.SizeX, Node.MinY + Node.SizeY, Node.MinZ + Node.SizeZ);
					}
				}
			}

			return true;
		}

		return false;
	}

	/**
	 * Returns the minimum texture width which will contain the allocated surfaces.
	 */
	uint32 GetSizeX() const { return SizeX; }

	/**
	 * Returns the minimum texture height which will contain the allocated surfaces.
	 */
	uint32 GetSizeY() const { return SizeY; }

	uint32 GetSizeZ() const { return SizeZ; }

	FIntVector GetSize() const { return FIntVector(SizeX, SizeY, SizeZ); }

	uint32 GetMaxSizeX() const 
	{
		return Nodes[0].SizeX;
	}

	uint32 GetMaxSizeY() const 
	{
		return Nodes[0].SizeY;
	}

	uint32 GetMaxSizeZ() const 
	{
		return Nodes[0].SizeZ;
	}

private:

	struct FTextureLayoutNode3d
	{
		int32	ChildA,
				ChildB,
				Parent;
		uint16	MinX,
				MinY,
				MinZ,
				SizeX,
				SizeY,
				SizeZ;
		bool	bUsed;

		FTextureLayoutNode3d(uint16 InMinX, uint16 InMinY, uint16 InMinZ, uint16 InSizeX, uint16 InSizeY, uint16 InSizeZ, int32 InParent):
			ChildA(INDEX_NONE),
			ChildB(INDEX_NONE),
			Parent(InParent),
			MinX(InMinX),
			MinY(InMinY),
			MinZ(InMinZ),
			SizeX(InSizeX),
			SizeY(InSizeY),
			SizeZ(InSizeZ),
			bUsed(false)
		{}
	};

	struct FUnusedLeaf
	{
		uint16 MinX, MinY, MinZ;
		uint16 SizeX, SizeY, SizeZ;
		int32 NodeIndex;

		FUnusedLeaf(uint16 InMinX, uint16 InMinY, uint16 InMinZ, uint16 InSizeX, uint16 InSizeY, uint16 InSizeZ, int32 InNodeIndex) :
			MinX(InMinX),
			MinY(InMinY),
			MinZ(InMinZ),
			SizeX(InSizeX),
			SizeY(InSizeY),
			SizeZ(InSizeZ),
			NodeIndex(InNodeIndex)
		{}
	};

	uint32 MinSizeX;
	uint32 MinSizeY;
	uint32 MinSizeZ;
	uint32 SizeX;
	uint32 SizeY;
	uint32 SizeZ;
	bool bPowerOfTwoSize;
	bool bAlignByFour;
	bool bAllowShrink;
	TArray<FTextureLayoutNode3d,TInlineAllocator<5> > Nodes;
	TArray<int32> FreeNodeIndices;
	TArray<FUnusedLeaf> UnusedLeaves;
	TMap<FIntVector, int32> UsedLeaves;

	/** Recursively traverses the tree depth first and searches for a large enough leaf node to contain the requested allocation. */
	int32 AddSurfaceInner(int32 NodeIndex, uint32 ElementSizeX, uint32 ElementSizeY, uint32 ElementSizeZ, bool bAllowTextureEnlargement)
	{
		checkSlow(NodeIndex != INDEX_NONE);
		// But do access this node via a pointer until the first recursive call. Prevents a ton of LHS.
		const FTextureLayoutNode3d* CurrentNodePtr = &Nodes[NodeIndex];
		if (CurrentNodePtr->ChildA != INDEX_NONE)
		{
			// Children are always allocated together
			checkSlow(CurrentNodePtr->ChildB != INDEX_NONE);

			// Traverse the children
			const int32 Result = AddSurfaceInner(CurrentNodePtr->ChildA, ElementSizeX, ElementSizeY, ElementSizeZ, bAllowTextureEnlargement);
			
			// The pointer is now invalid, be explicit!
			CurrentNodePtr = 0;

			if (Result != INDEX_NONE)
			{
				return Result;
			}

			return AddSurfaceInner(Nodes[NodeIndex].ChildB, ElementSizeX, ElementSizeY, ElementSizeZ, bAllowTextureEnlargement);
		}
		// Node has no children, it is a leaf
		else
		{
			// Reject this node if it is already used
			if (CurrentNodePtr->bUsed)
			{
				return INDEX_NONE;
			}

			// Reject this node if it is too small for the element being placed
			if (CurrentNodePtr->SizeX < ElementSizeX || CurrentNodePtr->SizeY < ElementSizeY || CurrentNodePtr->SizeZ < ElementSizeZ)
			{
				return INDEX_NONE;
			}

			if (!bAllowTextureEnlargement)
			{
				// Reject this node if this is an attempt to allocate space without enlarging the texture, 
				// And this node cannot hold the element without enlarging the texture.
				if (CurrentNodePtr->MinX + ElementSizeX > SizeX || CurrentNodePtr->MinY + ElementSizeY > SizeY || CurrentNodePtr->MinZ + ElementSizeZ > SizeZ)
				{
					return INDEX_NONE;
				}
			}

			// Use this node if the size matches the requested element size
			if (CurrentNodePtr->SizeX == ElementSizeX && CurrentNodePtr->SizeY == ElementSizeY && CurrentNodePtr->SizeZ == ElementSizeZ)
			{
				return NodeIndex;
			}

			const uint32 ExcessWidth = CurrentNodePtr->SizeX - ElementSizeX;
			const uint32 ExcessHeight = CurrentNodePtr->SizeY - ElementSizeY;
			const uint32 ExcessDepth = CurrentNodePtr->SizeZ - ElementSizeZ;

			// The pointer to the current node may be invalidated below, be explicit!
			CurrentNodePtr = 0;

			// Store a copy of the current node on the stack for easier debugging.
			// Can't store a pointer to the current node since Nodes may be reallocated in this function.
			const FTextureLayoutNode3d CurrentNode = Nodes[NodeIndex];

			// Update the child indices
			int32 ChildANodeIndex = FreeNodeIndices.Num() ? FreeNodeIndices.Pop() : Nodes.AddUninitialized();
			int32 ChildBNodeIndex = FreeNodeIndices.Num() ? FreeNodeIndices.Pop() : Nodes.AddUninitialized();

			Nodes[NodeIndex].ChildA = ChildANodeIndex;
			Nodes[NodeIndex].ChildB = ChildBNodeIndex;

			// Add new nodes, and link them as children of the current node.
			if (ExcessWidth > ExcessHeight)
			{
				if (ExcessWidth > ExcessDepth)
				{
					// Create a child with the same width as the element being placed.
					// The height may not be the same as the element height yet, in that case another subdivision will occur when traversing this child node.
					new(&Nodes[ChildANodeIndex]) FTextureLayoutNode3d(
						CurrentNode.MinX,
						CurrentNode.MinY,
						CurrentNode.MinZ,
						IntCastChecked<uint16>(ElementSizeX),
						CurrentNode.SizeY,
						CurrentNode.SizeZ,
						NodeIndex);

					// Create a second child to contain the leftover area in the X direction
					new(&Nodes[ChildBNodeIndex]) FTextureLayoutNode3d(
						IntCastChecked<uint16>(CurrentNode.MinX + ElementSizeX),
						CurrentNode.MinY,
						CurrentNode.MinZ,
						IntCastChecked<uint16>(CurrentNode.SizeX - ElementSizeX),
						CurrentNode.SizeY,
						CurrentNode.SizeZ,
						NodeIndex);
				}
				else
				{
					new(&Nodes[ChildANodeIndex]) FTextureLayoutNode3d(
						CurrentNode.MinX,
						CurrentNode.MinY,
						CurrentNode.MinZ,
						CurrentNode.SizeX,
						CurrentNode.SizeY,
						IntCastChecked<uint16>(ElementSizeZ),
						NodeIndex);

					new(&Nodes[ChildBNodeIndex]) FTextureLayoutNode3d(
						CurrentNode.MinX,
						CurrentNode.MinY,
						IntCastChecked<uint16>(CurrentNode.MinZ + ElementSizeZ),
						CurrentNode.SizeX,
						CurrentNode.SizeY,
						IntCastChecked<uint16>(CurrentNode.SizeZ - ElementSizeZ),
						NodeIndex);
				}
			}
			else
			{
				if (ExcessHeight > ExcessDepth)
				{
					new(&Nodes[ChildANodeIndex]) FTextureLayoutNode3d(
						CurrentNode.MinX,
						CurrentNode.MinY,
						CurrentNode.MinZ,
						CurrentNode.SizeX,
						IntCastChecked<uint16>(ElementSizeY),
						CurrentNode.SizeZ,
						NodeIndex);

					new(&Nodes[ChildBNodeIndex]) FTextureLayoutNode3d(
						CurrentNode.MinX,
						IntCastChecked<uint16>(CurrentNode.MinY + ElementSizeY),
						CurrentNode.MinZ,
						CurrentNode.SizeX,
						IntCastChecked<uint16>(CurrentNode.SizeY - ElementSizeY),
						CurrentNode.SizeZ,
						NodeIndex);
				}
				else
				{
					new(&Nodes[ChildANodeIndex]) FTextureLayoutNode3d(
						CurrentNode.MinX,
						CurrentNode.MinY,
						CurrentNode.MinZ,
						CurrentNode.SizeX,
						CurrentNode.SizeY,
						IntCastChecked<uint16>(ElementSizeZ),
						NodeIndex);

					new(&Nodes[ChildBNodeIndex]) FTextureLayoutNode3d(
						CurrentNode.MinX,
						CurrentNode.MinY,
						IntCastChecked<uint16>(CurrentNode.MinZ + ElementSizeZ),
						CurrentNode.SizeX,
						CurrentNode.SizeY,
						IntCastChecked<uint16>(CurrentNode.SizeZ - ElementSizeZ),
						NodeIndex);
				}
			}

			const FTextureLayoutNode3d& ChildBNode = Nodes[ChildBNodeIndex];

			UnusedLeaves.Emplace(ChildBNode.MinX, ChildBNode.MinY, ChildBNode.MinZ, ChildBNode.SizeX, ChildBNode.SizeY, ChildBNode.SizeZ, ChildBNodeIndex);

			// Only traversing ChildA, since ChildA is always the newly created node that matches the element size
			return AddSurfaceInner(ChildANodeIndex, ElementSizeX, ElementSizeY, ElementSizeZ, bAllowTextureEnlargement);
		}
	}

	void UpdateSize(uint32 ElementMaxX, uint32 ElementMaxY, uint32 ElementMaxZ)
	{
		if (bPowerOfTwoSize)
		{
			SizeX = FMath::Max<uint32>(SizeX, FMath::RoundUpToPowerOfTwo(ElementMaxX));
			SizeY = FMath::Max<uint32>(SizeY, FMath::RoundUpToPowerOfTwo(ElementMaxY));
			SizeZ = FMath::Max<uint32>(SizeZ, FMath::RoundUpToPowerOfTwo(ElementMaxZ));
		}
		else
		{
			SizeX = FMath::Max<uint32>(SizeX, ElementMaxX);
			SizeY = FMath::Max<uint32>(SizeY, ElementMaxY);
			SizeZ = FMath::Max<uint32>(SizeZ, ElementMaxZ);
		}
	}

	/** Returns the index into Nodes of the parent node of SearchNode. */
	int32 FindParentNode(int32 SearchNodeIndex)
	{
		return Nodes[SearchNodeIndex].Parent;
	}

	/** Returns true if the node or any of its children are marked used. */
	bool IsNodeUsed(int32 NodeIndex)
	{
		bool bChildrenUsed = false;
		if (Nodes[NodeIndex].ChildA != INDEX_NONE)
		{
			checkSlow(Nodes[NodeIndex].ChildB != INDEX_NONE);
			bChildrenUsed = IsNodeUsed(Nodes[NodeIndex].ChildA) || IsNodeUsed(Nodes[NodeIndex].ChildB);
		}
		return bChildrenUsed || Nodes[NodeIndex].bUsed;
	}

	/** Recursively removes the children of a given node from the Nodes array and adjusts existing indices to compensate. */
	void RemoveChildren(int32 NodeIndex)
	{
		// Traverse the children depth first
		if (Nodes[NodeIndex].ChildA == INDEX_NONE)
		{
			check(Nodes[NodeIndex].ChildB == INDEX_NONE);
			return;
		}

		check(Nodes[NodeIndex].ChildB != INDEX_NONE);

		// Traverse the children depth first
		RemoveChildren(Nodes[NodeIndex].ChildA);
		RemoveChildren(Nodes[NodeIndex].ChildB);

		{
			const int32 ChildNodeIndex = Nodes[NodeIndex].ChildA;

			// Remove the child from the Nodes array
			FreeNodeIndices.Add(ChildNodeIndex);

			for (int32 Index = 0; Index < UnusedLeaves.Num(); ++Index)
			{
				if (UnusedLeaves[Index].NodeIndex == ChildNodeIndex)
				{
					UnusedLeaves.RemoveAtSwap(Index);
					break;
				}
			}
		}

		{
			const int32 ChildNodeIndex = Nodes[NodeIndex].ChildB;
			FreeNodeIndices.Add(ChildNodeIndex);

			for (int32 Index = 0; Index < UnusedLeaves.Num(); ++Index)
			{
				if (UnusedLeaves[Index].NodeIndex == ChildNodeIndex)
				{
					UnusedLeaves.RemoveAtSwap(Index);
					break;
				}
			}
		}

		Nodes[NodeIndex].ChildA = INDEX_NONE;
		Nodes[NodeIndex].ChildB = INDEX_NONE;
	}
};

