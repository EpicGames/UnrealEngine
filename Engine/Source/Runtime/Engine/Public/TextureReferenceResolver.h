// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture.h"
#include "RenderResource.h"
#include "RHI.h"
#include "RHITextureReference.h"

/** 
* Replaces the RHI reference of one texture with another.
* Allows one texture to be replaced with another at runtime and have all existing references to it remain valid.
*/
struct FTextureReferenceReplacer
{
	FTextureReferenceRHIRef OriginalRef;

	FTextureReferenceReplacer(UTexture* OriginalTexture)
	{
		if (OriginalTexture)
		{
			OriginalTexture->ReleaseResource();
			OriginalRef = OriginalTexture->TextureReference.TextureReferenceRHI;
		}
		else
		{
			OriginalRef = nullptr;
		}
	}

	void Replace(UTexture* NewTexture)
	{
		if (OriginalRef)
		{
			NewTexture->TextureReference.TextureReferenceRHI = OriginalRef;
		}
	}
};
 