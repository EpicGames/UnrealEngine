// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TextureResource.h"

/**
 * FTextureRenderTarget2DArrayResource type for 2DArray render target textures.
 */
class FTextureRenderTarget2DArrayResource : public FTextureRenderTargetResource
{
	friend class UTextureRenderTarget2DArray;
public:

	/**
	 * Constructor
	 * @param InOwner - 2DArray texture object to create a resource for
	 */
	FTextureRenderTarget2DArrayResource(const class UTextureRenderTarget2DArray* InOwner)
		: Owner(InOwner)
	{
	}

	/**
	 * 2DArray texture RT resource interface
	 */
	virtual class FTextureRenderTarget2DArrayResource* GetTextureRenderTarget2DArrayResource()
	{
		return this;
	}

	/**
	 * Initializes the dynamic RHI resource and/or RHI render target used by this resource.
	 * Called when the resource is initialized, or when reseting all RHI resources.
	 * Resources that need to initialize after a D3D device reset must implement this function.
	 * This is only called by the rendering thread.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/**
	 * Releases the dynamic RHI resource and/or RHI render target resources used by this resource.
	 * Called when the resource is released, or when reseting all RHI resources.
	 * Resources that need to release before a D3D device reset must implement this function.
	 * This is only called by the rendering thread.
	 */
	virtual void ReleaseRHI() override;

	// FRenderTarget interface.

	/**
	 * @return width of the target
	 */
	virtual uint32 GetSizeX() const override;

	/**
	 * @return height of the target
	 */
	virtual uint32 GetSizeY() const override;

	/**
	 * @return dimensions of the target
	 */
	virtual FIntPoint GetSizeXY() const override;

	/**
	 * @return UnorderedAccessView for rendering
	 */
	FUnorderedAccessViewRHIRef GetUnorderedAccessViewRHI() { return UnorderedAccessViewRHI; }

	/**
	* Render target resource should be sampled in linear color space
	*
	* @return display gamma expected for rendering to this render target
	*/
	float GetDisplayGamma() const override;

	// UE_DEPRECATED(5.4, "This using is there temporarily until the 2 deprecated ReadPixels 'overrides' are removed : they were hiding FRenderTarget's virtual functions")
	using FRenderTarget::ReadPixels;

	/**
	* Copy the texels of a single depth slice of the 2d array into an array.
	* @param OutImageData - float16 values will be stored in this array.
	* @param InDepthSlice - which depth slice to read.
	* @param InRect - Rectangle of texels to copy.
	* @return true if the read succeeded.
	*/
	UE_DEPRECATED(5.4, "Use FRenderTarget's ReadPixels, which is functionally equivalent")
	ENGINE_API bool ReadPixels(TArray<FColor>& OutImageData, int32 InDepthSlice, FIntRect InRect = FIntRect(0, 0, 0, 0));

	/**
	* Copy the texels of a single depth slice of the cube into an array.
	* @param OutImageData - float16 values will be stored in this array.
	* @param InDepthSlice - which depth slice to read.
	* @param InRect - Rectangle of texels to copy.
	* @return true if the read succeeded.
	*/
	UE_DEPRECATED(5.4, "Use FRenderTarget's ReadFloat16Pixels, which is functionally equivalent")
	ENGINE_API bool ReadPixels(TArray<FFloat16Color>& OutImageData, int32 InDepthSlice, FIntRect InRect = FIntRect(0, 0, 0, 0));

protected:
	/**
	* Updates (resolves) the render target texture.
	* Optionally clears each face of the render target to green.
	* This is only called by the rendering thread.
	*/
	virtual void UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget = true) override;

private:
	/** The UTextureRenderTarget2DArray which this resource represents. */
	const class UTextureRenderTarget2DArray* Owner;

	/** Represents the current render target (from one of the slices)*/
	UE_DEPRECATED(5.1, "RenderTarget2DArrayRHI is deprecated. Use TextureRHI instead.")
	FTextureRHIRef RenderTarget2DArrayRHI;
	/** Texture resource used for rendering with and resolving to */
	UE_DEPRECATED(5.1, "Texture2DArrayRHI is deprecated. Use TextureRHI instead.")
	FTextureRHIRef Texture2DArrayRHI;
	/** Optional Unordered Access View for the resource, automatically created if bCanCreateUAV is true */
	FUnorderedAccessViewRHIRef UnorderedAccessViewRHI;
};
