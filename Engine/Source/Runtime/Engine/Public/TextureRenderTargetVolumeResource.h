// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TextureResource.h"

/**
 * FTextureRenderTargetVolumeResource type for Volume render target textures.
 */
class FTextureRenderTargetVolumeResource : public FTextureRenderTargetResource
{
	friend class UTextureRenderTargetVolume;
public:

	/**
	 * Constructor
	 * @param InOwner - Volume texture object to create a resource for
	 */
	FTextureRenderTargetVolumeResource(const class UTextureRenderTargetVolume* InOwner)
		: Owner(InOwner)
	{
	}

	/**
	 * Volume texture RT resource interface
	 */
	virtual class FTextureRenderTargetVolumeResource* GetTextureRenderTargetVolumeResource()
	{
		return this;
	}

	/**
	 * Initializes the dynamic RHI resource and/or RHI render target used by this resource.
	 * Called when the resource is initialized, or when reseting all RHI resources.
	 * Resources that need to initialize after a D3D device reset must implement this function.
	 * This is only called by the rendering thread.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/**
	 * Releases the dynamic RHI resource and/or RHI render target resources used by this resource.
	 * Called when the resource is released, or when reseting all RHI resources.
	 * Resources that need to release before a D3D device reset must implement this function.
	 * This is only called by the rendering thread.
	 */
	virtual void ReleaseRHI() override;

	// FRenderTarget interface.

	// FTexture interface :
	virtual uint32 GetSizeX() const override;
	virtual uint32 GetSizeY() const override;
	virtual uint32 GetSizeZ() const override;

	// FRenderTarget interface:	
	virtual FIntPoint GetSizeXY() const override;

	/**
	 * @return UnorderedAccessView for rendering
	 */
	FUnorderedAccessViewRHIRef GetUnorderedAccessViewRHI() { return UnorderedAccessViewRHI; }

	/**
	* Render target resource should be sampled in linear color space
	*
	* @return display gamma expected for rendering to this render target
	*/
	float GetDisplayGamma() const override;

	virtual bool ReadPixels(TArray<FColor>& OutImageData, FReadSurfaceDataFlags InFlags = FReadSurfaceDataFlags(RCM_UNorm, CubeFace_MAX), FIntRect InSrcRect = FIntRect(0, 0, 0, 0)) override;

	virtual bool ReadFloat16Pixels(TArray<FFloat16Color>& OutImageData, FReadSurfaceDataFlags InFlags = FReadSurfaceDataFlags(RCM_UNorm, CubeFace_MAX), FIntRect InSrcRect = FIntRect(0, 0, 0, 0)) override;

	virtual bool ReadLinearColorPixels(TArray<FLinearColor>& OutImageData, FReadSurfaceDataFlags InFlags = FReadSurfaceDataFlags(RCM_UNorm, CubeFace_MAX), FIntRect InSrcRect = FIntRect(0, 0, 0, 0)) override;

	/**
	* Copy the texels of a single depth slice of the volume into an array.
	* @param OutImageData - float16 values will be stored in this array.
	* @param InDepthSlice - which depth slice to read.
	* @param InRect - Rectangle of texels to copy.
	* @return true if the read succeeded.
	*/
	UE_DEPRECATED(5.4, "Use FRenderTarget's ReadPixels, which is functionally equivalent")
	ENGINE_API bool ReadPixels(TArray<FColor>& OutImageData, int32 InDepthSlice, FIntRect InRect = FIntRect(0, 0, 0, 0));

	/**
	* Copy the texels of a single depth slice of the cube into an array.
	* @param OutImageData - float16 values will be stored in this array.
	* @param InDepthSlice - which depth slice to read.
	* @param InRect - Rectangle of texels to copy.
	* @return true if the read succeeded.
	*/
	UE_DEPRECATED(5.4, "Use FRenderTarget's ReadFloat16Pixels, which is functionally equivalent")
	ENGINE_API bool ReadPixels(TArray<FFloat16Color>& OutImageData, int32 InDepthSlice, FIntRect InRect = FIntRect(0, 0, 0, 0));

protected:
	/**
	* Updates (resolves) the render target texture.
	* Optionally clears each face of the render target to green.
	* This is only called by the rendering thread.
	*/
	virtual void UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget = true) override;

private:
	/** The UTextureRenderTargetVolume which this resource represents. */
	const class UTextureRenderTargetVolume* Owner;

	UE_DEPRECATED(5.1, "RenderTargetVolumeRHI has been deprecated. Use RenderTargetTextureRHI instead.")
	FTextureRHIRef RenderTargetVolumeRHI;
	UE_DEPRECATED(5.1, "TextureVolumeRHI has been deprecated. Use TextureRHI instead.")
	FTextureRHIRef TextureVolumeRHI;

	/** Optional Unordered Access View for the resource, automatically created if bCanCreateUAV is true */
	FUnorderedAccessViewRHIRef UnorderedAccessViewRHI;
};
