// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	Texture.h: Unreal texture related classes.
=============================================================================*/

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/IndirectArray.h"
#include "Stats/Stats.h"
#include "Containers/List.h"
#include "Async/AsyncWork.h"
#include "Async/AsyncFileHandle.h"
#include "RHI.h"
#include "RenderResource.h"
#include "Serialization/BulkData.h"
#include "Serialization/DerivedData.h"
#include "Engine/TextureDefines.h"
#include "UnrealClient.h"
#include "Templates/UniquePtr.h"
#include "VirtualTexturing.h"

class FTexture2DResourceMem;
class UTexture;
class UTexture2D;
class UTexture2DArray;
class IVirtualTexture;
struct FTexturePlatformData;
class FStreamableTextureResource;
class FTexture2DResource;
class FTexture3DResource;
class FTexture2DArrayResource;
class FVirtualTexture2DResource;
struct IPooledRenderTarget;
enum class EVTProducerPriority : uint8;

/** Maximum number of slices in texture source art. */
#define MAX_TEXTURE_SOURCE_SLICES 6

/**
 * A 2D texture mip-map.
 */
struct FTexture2DMipMap
{
	/** Reference to the data for the mip if it can be streamed. */
	UE::FDerivedData DerivedData;
	/** Stores the data for the mip when it is loaded. */
	FByteBulkData BulkData;

	/** Width of the mip-map. */
	uint16 SizeX = 0;
	/** Height of the mip-map. */
	uint16 SizeY = 0;
	/** Depth of the mip-map. This also holds array size. It's thunked through to FStreamableTextureResource::SizeZ and is not used for cubemap arrays or cubemaps */
	uint16 SizeZ = 0;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FTexture2DMipMap() = default;
	FTexture2DMipMap(uint32 InSizeX, uint32 InSizeY, uint32 InSizeZ = 0)
		: SizeX((uint16)InSizeX), SizeY((uint16)InSizeY), SizeZ((uint16)InSizeZ)
	{
		if (InSizeX > 0xFFFF || InSizeY > 0xFFFF || InSizeZ > 0xFFFF)
		{
			LowLevelFatalError(TEXT("texture mip size doesn't fit in 16 bits! %ux%ux%u"), InSizeX, InSizeY, InSizeZ);
		}
	}
	FTexture2DMipMap(FTexture2DMipMap&&) = default;
	FTexture2DMipMap(const FTexture2DMipMap&) = default;
	FTexture2DMipMap& operator=(FTexture2DMipMap&&) = default;
	FTexture2DMipMap& operator=(const FTexture2DMipMap&) = default;
	~FTexture2DMipMap() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Serialization. */
	ENGINE_API void Serialize(FArchive& Ar, UObject* Owner, int32 MipIndex, bool bSerializeMipData);

#if WITH_EDITORONLY_DATA
	/** The file region type appropriate for the pixel format of this mip-map. */
	EFileRegionType FileRegionType = EFileRegionType::None;

	UE_DEPRECATED(5.1, "Use DerivedData.HasData().")
	bool bPagedToDerivedData = false;

	/** Whether this mip-map is stored in the derived data cache. */
	inline bool IsPagedToDerivedData() const { return DerivedData.HasData(); }

	UE_DEPRECATED(5.1, "Setting DerivedData is sufficient to control this state.")
	inline void SetPagedToDerivedData(bool InValue)
	{
	}

	/** Place mip-map data in the derived data cache associated with the provided key. */
	int64 StoreInDerivedDataCache(FStringView Key, FStringView Name, bool bReplaceExisting);
#endif // #if WITH_EDITORONLY_DATA
};

/** 
 * The rendering resource which represents a texture.
 */
class FTextureResource : public FTexture
{
public:

	FTextureResource() {}
	virtual ~FTextureResource() = default;

	/**
	* Returns true if the resource is proxying another one.
	*/
	virtual bool IsProxy() const { return false; }

	// Dynamic cast methods.
	virtual FTexture2DResource* GetTexture2DResource() { return nullptr; }
	virtual FTexture3DResource* GetTexture3DResource() { return nullptr; }
	virtual FTexture2DArrayResource* GetTexture2DArrayResource() { return nullptr; }
	virtual FStreamableTextureResource* GetStreamableTextureResource() { return nullptr; }
	virtual FVirtualTexture2DResource* GetVirtualTexture2DResource() { return nullptr; }
	// Dynamic cast methods (const).
	virtual const FTexture2DResource* GetTexture2DResource() const { return nullptr; }
	virtual const FTexture3DResource* GetTexture3DResource() const { return nullptr; }
	virtual const FTexture2DArrayResource* GetTexture2DArrayResource() const { return nullptr; }
	virtual const FStreamableTextureResource* GetStreamableTextureResource() const { return nullptr; }
	virtual const FVirtualTexture2DResource* GetVirtualTexture2DResource() const { return nullptr; }

	// Current mip count. We use "current" to specify that it is not computed from SizeX() which is the size when fully streamed in.
	inline int32 GetCurrentMipCount() const
	{
		return TextureRHI.IsValid() ? TextureRHI->GetNumMips() : 0;
	}

	inline bool IsTextureRHIPartiallyResident() const
	{
		return TextureRHI.IsValid() && !!(TextureRHI->GetFlags() & TexCreate_Virtual);
	}

	inline FRHITexture* GetTexture2DRHI() const
	{
		return TextureRHI.IsValid() ? TextureRHI->GetTexture2D() : nullptr;
	}

	inline FRHITexture* GetTexture3DRHI() const
	{
		return TextureRHI.IsValid() ? TextureRHI->GetTexture3D() : nullptr;
	}

	inline FRHITexture* GetTexture2DArrayRHI() const
	{
		return TextureRHI.IsValid() ? TextureRHI->GetTexture2DArray() : nullptr;
	}

	void SetTextureReference(FRHITextureReference* TextureReference)
	{
		TextureReferenceRHI = TextureReference;
	}

	FRHITextureReference* GetTextureReference() const
	{
		return TextureReferenceRHI;
	}

#if STATS
	/* The Stat_ FName corresponding to each TEXTUREGROUP */
	static FName TextureGroupStatFNames[TEXTUREGROUP_MAX];
#endif

protected :
	// A FRHITextureReference to update whenever the FTexture::TextureRHI changes.
	// It allows to prevent dereferencing the UAsset pointers when updating a texture resource.
	FTextureReferenceRHIRef TextureReferenceRHI;
};

class FVirtualTexture2DResource : public FTextureResource
{
public:
	ENGINE_API FVirtualTexture2DResource();
	ENGINE_API FVirtualTexture2DResource(const UTexture2D* InOwner, struct FVirtualTextureBuiltData* InVTData, int32 FirstMipToUse);
	ENGINE_API virtual ~FVirtualTexture2DResource();

	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;

	// Dynamic cast methods.
	virtual FVirtualTexture2DResource* GetVirtualTexture2DResource() { return this; }
	// Dynamic cast methods (const).
	virtual const FVirtualTexture2DResource* GetVirtualTexture2DResource() const { return this; }

#if WITH_EDITOR
	ENGINE_API virtual void InitializeEditorResources(FRHICommandListBase& RHICmdList, IVirtualTexture* InVirtualTexture);
#endif

	ENGINE_API virtual uint32 GetSizeX() const override;
	ENGINE_API virtual uint32 GetSizeY() const override;

	const FVirtualTextureProducerHandle& GetProducerHandle() const { return ProducerHandle; }

	/**
	 * FVirtualTexture2DResource may have an AllocatedVT, which represents a page table allocation for the virtual texture.
	 * VTs used by materials generally don't need their own allocation, since the material has its own page table allocation for each VT stack.
	 * VTs used as lightmaps need their own allocation.  Also VTs open in texture editor will have a temporary allocation.
	 * GetAllocatedVT() will return the current allocation if one exists.
	 * AcquireAllocatedVT() will make a new allocation if needed, and return it.
	 * ReleaseAllocatedVT() will free any current allocation.
	 */
	class IAllocatedVirtualTexture* GetAllocatedVT() const { return AllocatedVT; }
	ENGINE_API class IAllocatedVirtualTexture* AcquireAllocatedVT();
	ENGINE_API void ReleaseAllocatedVT();

	ENGINE_API virtual EPixelFormat GetFormat(uint32 LayerIndex) const;
	ENGINE_API virtual FIntPoint GetSizeInBlocks() const;
	ENGINE_API virtual uint32 GetNumTilesX() const;
	ENGINE_API virtual uint32 GetNumTilesY() const;
	ENGINE_API virtual uint32 GetNumMips() const;
	ENGINE_API virtual uint32 GetNumLayers() const;
	ENGINE_API virtual uint32 GetTileSize() const; //no borders
	ENGINE_API virtual uint32 GetBorderSize() const;
	uint32 GetAllocatedvAddress() const;

	ENGINE_API virtual FIntPoint GetPhysicalTextureSize(uint32 LayerIndex) const;

protected:
	/** The FName of the texture asset */
	FName TextureName;
	/** The FName of the texture package for stats */
	FName PackageName;
	/** A hash of the texture asset name */
	uint32 FullNameHash = 0;
	/** Cached sampler config */
	TEnumAsByte<ESamplerFilter> Filter = SF_Point;
	TEnumAsByte<ESamplerAddressMode> AddressU = AM_Wrap;
	TEnumAsByte<ESamplerAddressMode> AddressV = AM_Wrap;
	/** Cached flags for texture creation. */
	ETextureCreateFlags TexCreateFlags = ETextureCreateFlags::None;
	/** Cached owner settings */
	bool bSinglePhysicalSpace = false;
	bool bRequiresSinglePhysicalPool = false;
	EVTProducerPriority VirtualTextureStreamingPriority;
	/** Mip offset */
	int32 FirstMipToUse = 0;
	/** Built data owned by texture asset */
	struct FVirtualTextureBuiltData* VTData = nullptr;
	/** Local allocated VT objects used for editor views etc. */
	class IAllocatedVirtualTexture* AllocatedVT = nullptr;
	FVirtualTextureProducerHandle ProducerHandle;
};

/** A dynamic 2D texture resource. */
class FTexture2DDynamicResource : public FTextureResource
{
public:
	/** Initialization constructor. */
	ENGINE_API FTexture2DDynamicResource(class UTexture2DDynamic* InOwner);

	/** Returns the width of the texture in pixels. */
	ENGINE_API virtual uint32 GetSizeX() const override;

	/** Returns the height of the texture in pixels. */
	ENGINE_API virtual uint32 GetSizeY() const override;

	/** Called when the resource is initialized. This is only called by the rendering thread. */
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Called when the resource is released. This is only called by the rendering thread. */
	ENGINE_API virtual void ReleaseRHI() override;

	/** Returns the Texture2DRHI, which can be used for locking/unlocking the mips. */
	ENGINE_API FTextureRHIRef GetTexture2DRHI();

#if !UE_SERVER
	ENGINE_API void WriteRawToTexture_RenderThread(TArrayView64<const uint8> RawData);
#endif

private:
	/** The owner of this resource. */
	class UTexture2DDynamic* Owner;
	/** Texture2D reference, used for locking/unlocking the mips. */
	FTextureRHIRef Texture2DRHI;
};

/**
 * FDeferredUpdateResource for resources that need to be updated after scene rendering has begun
 * (should only be used on the rendering thread)
 */
class FDeferredUpdateResource
{
public:

	/**
	 * Constructor, initializing UpdateListLink.
	 */
	FDeferredUpdateResource()
		:	UpdateListLink(NULL)
		,	bOnlyUpdateOnce(false)
	{ }

public:

	/**
	 * Iterate over the global list of resources that need to
	 * be updated and call UpdateResource on each one.
	 */
	ENGINE_API static void UpdateResources( FRHICommandListImmediate& RHICmdList );

	/**
	 * Performs a deferred resource update on this resource if it exists in the UpdateList.
	 */
	ENGINE_API void FlushDeferredResourceUpdate( FRHICommandListImmediate& RHICmdList );

	/** 
	 * This is reset after all viewports have been rendered
	 */
	static void ResetNeedsUpdate()
	{
		bNeedsUpdate = true;
	}

	static bool IsUpdateNeeded()
	{
		return bNeedsUpdate;
	}

	ENGINE_API void ResetSceneTextureExtentsHistory();
protected:

	/**
	 * Updates (resolves) the render target texture.
	 * Optionally clears the contents of the render target to green.
	 * This is only called by the rendering thread.
	 */
	virtual void UpdateDeferredResource( FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget=true ) = 0;

	/**
	 * Add this resource to deferred update list
	 * @param OnlyUpdateOnce - flag this resource for a single update if true
	 */
	ENGINE_API void AddToDeferredUpdateList( bool OnlyUpdateOnce );

	/**
	 * Remove this resource from deferred update list
	 */
	ENGINE_API void RemoveFromDeferredUpdateList();

private:

	/** 
	 * Resources can be added to this list if they need a deferred update during scene rendering.
	 * @return global list of resource that need to be updated. 
	 */
	static TLinkedList<FDeferredUpdateResource*>*& GetUpdateList();
	/** This resource's link in the global list of resources needing clears. */
	TLinkedList<FDeferredUpdateResource*> UpdateListLink;
	/** if true then UpdateResources needs to be called */
	ENGINE_API static bool bNeedsUpdate;
	/** if true then remove this resource from the update list after a single update */
	bool bOnlyUpdateOnce;
};

/**
 * FTextureResource type for render target textures.
 */
class FTextureRenderTargetResource : public FTextureResource, public FRenderTarget, public FDeferredUpdateResource
{
public:
	/**
	 * Constructor, initializing ClearLink.
	 */
	FTextureRenderTargetResource()
	{}

	/** 
	 * Return true if a render target of the given format is allowed
	 * for creation
	 */
	ENGINE_API static bool IsSupportedFormat( EPixelFormat Format );

	// FRenderTarget implementation
	virtual const FTextureRHIRef& GetShaderResourceTexture() const override;

	// FTextureRenderTargetResource interface
	
	virtual class FTextureRenderTarget2DResource* GetTextureRenderTarget2DResource() { return nullptr; }
	virtual class FTextureRenderTarget2DArrayResource* GetTextureRenderTarget2DArrayResource() { return nullptr; }
	virtual class FTextureRenderTargetVolumeResource* GetTextureRenderTargetVolumeResource() { return nullptr; }
	virtual class FTextureRenderTargetCubeResource* GetTextureRenderTargetCubeResource() { return nullptr; }

	virtual void ClampSize(int32 SizeX,int32 SizeY) {}

	// GetSizeX/GetSizeY are from "FTexture" interface.
	//   the FTexture implementations return zero; force them to be implemented:
	virtual uint32 GetSizeX() const override = 0;
	virtual uint32 GetSizeY() const override = 0;
	// also GetSizeZ()
	// GetSizeXY from "FRenderTarget"
	//virtual FIntPoint GetSizeXY() const override = 0;

	/** 
	 * Render target resource should be sampled in linear color space
	 *
	 * @return display gamma expected for rendering to this render target 
	 */
	virtual float GetDisplayGamma() const override;

	virtual FRHIGPUMask GetGPUMask(FRHICommandListImmediate& RHICmdList) const final override
	{
		return GPUMask & ActiveGPUMask;
	}

	// Changes the GPUMask used when updating the texture with multi-GPU.
	void SetActiveGPUMask(FRHIGPUMask InGPUMask)
	{
		check(IsInRenderingThread());
		check(GPUMask.ContainsAll(InGPUMask));
		ActiveGPUMask = InGPUMask;
	}

protected:
	void SetGPUMask(FRHIGPUMask InGPUMask)
	{
		check(IsInRenderingThread());
		ActiveGPUMask = GPUMask = InGPUMask;
	}

private:
	FRHIGPUMask GPUMask;
	FRHIGPUMask ActiveGPUMask;  // GPU mask copied from parent render target for multi-GPU
};

/**
 * FTextureResource type for 2D render target textures.
 */
class FTextureRenderTarget2DResource : public FTextureRenderTargetResource
{
public:
	
	/** 
	 * Constructor
	 * @param InOwner - 2d texture object to create a resource for
	 */
	FTextureRenderTarget2DResource(const class UTextureRenderTarget2D* InOwner);
	virtual ~FTextureRenderTarget2DResource();

	inline FLinearColor GetClearColor()
	{
		return ClearColor;
	}

	inline EPixelFormat GetFormat() const
	{
		return Format;
	}

	ETextureCreateFlags GetCreateFlags();

	// FTextureRenderTargetResource interface

	/** 
	 * 2D texture RT resource interface 
	 */
	virtual class FTextureRenderTarget2DResource* GetTextureRenderTarget2DResource() override
	{
		return this;
	}

	/**
	 * Clamp size of the render target resource to max values
	 *
	 * @param MaxSizeX max allowed width
	 * @param MaxSizeY max allowed height
	 */
	virtual void ClampSize(int32 SizeX,int32 SizeY) override;
	
	// FRenderResource interface.

	/**
	 * Initializes the dynamic RHI resource and/or RHI render target used by this resource.
	 * Called when the resource is initialized, or when reseting all RHI resources.
	 * Resources that need to initialize after a D3D device reset must implement this function.
	 * This is only called by the rendering thread.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/**
	 * Releases the dynamic RHI resource and/or RHI render target resources used by this resource.
	 * Called when the resource is released, or when reseting all RHI resources.
	 * Resources that need to release before a D3D device reset must implement this function.
	 * This is only called by the rendering thread.
	 */
	virtual void ReleaseRHI() override;

	// FDeferredClearResource interface

	// FRenderTarget interface.
	/** 
	 * @return width of the target
	 */
	virtual uint32 GetSizeX() const override;

	/** 
	 * @return height of the target
	 */
	virtual uint32 GetSizeY() const override;

	/**
	 * @return dimensions of the target
	 */
	virtual FIntPoint GetSizeXY() const override;

	/** 
	 * Render target resource should be sampled in linear color space
	 *
	 * @return display gamma expected for rendering to this render target 
	 */
	virtual float GetDisplayGamma() const override;

	/**
	 * @return UnorderedAccessView for rendering
	 */
	FUnorderedAccessViewRHIRef GetUnorderedAccessViewRHI() { return UnorderedAccessViewRHI; }

protected:
	/**
	 * Updates (resolves) the render target texture.
	 * Optionally clears the contents of the render target to green.
	 * This is only called by the rendering thread.
	 */
	friend class UTextureRenderTarget2D;
	virtual void UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget=true) override;
	void Resize(int32 NewSizeX, int32 NewSizeY, int32 NewNumMips);

	/** Utility function used for resizing of scene texture sized scene capture render targets in the render thread */
	friend class FScene;
	friend class FSceneCapturePass;
	inline void Resize(FRHICommandListBase& RHICmdList, int32 NewSizeX, int32 NewSizeY, bool bAutoGenerateMips)
	{
		int32 NewNumMips = bAutoGenerateMips ? FMath::FloorLog2(FMath::Max(NewSizeX, NewSizeY)) + 1 : 1;
		if (TargetSizeX != NewSizeX || TargetSizeY != NewSizeY || TargetNumMips != NewNumMips)
		{
			TargetSizeX = NewSizeX;
			TargetSizeY = NewSizeY;
			TargetNumMips = NewNumMips;
			UpdateRHI(RHICmdList);
		}
	}

private:
	/** The UTextureRenderTarget2D which this resource represents. */
	const class UTextureRenderTarget2D* Owner;
	/** Texture resource used for rendering with and resolving to */
	UE_DEPRECATED(5.1, "Texture2DRHI is deprecated. Use TextureRHI instead.")
	FTextureRHIRef Texture2DRHI;
	/** Optional Unordered Access View for the resource, automatically created if bCanCreateUAV is true */
	FUnorderedAccessViewRHIRef UnorderedAccessViewRHI;
	/** the color the texture is cleared to */
	FLinearColor ClearColor;
	EPixelFormat Format;
	int32 TargetSizeX,TargetSizeY;
	int32 TargetNumMips;
	TRefCountPtr<IPooledRenderTarget> MipGenerationCache;
};

/**
 * FTextureResource type for cube render target textures.
 */
class FTextureRenderTargetCubeResource : public FTextureRenderTargetResource
{
public:

	/** 
	 * Constructor
	 * @param InOwner - cube texture object to create a resource for
	 */
	FTextureRenderTargetCubeResource(const class UTextureRenderTargetCube* InOwner)
		:	Owner(InOwner)
	{
	}

	/** 
	 * Cube texture RT resource interface 
	 */
	virtual class FTextureRenderTargetCubeResource* GetTextureRenderTargetCubeResource()
	{
		return this;
	}

	/**
	 * Initializes the dynamic RHI resource and/or RHI render target used by this resource.
	 * Called when the resource is initialized, or when reseting all RHI resources.
	 * Resources that need to initialize after a D3D device reset must implement this function.
	 * This is only called by the rendering thread.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/**
	 * Releases the dynamic RHI resource and/or RHI render target resources used by this resource.
	 * Called when the resource is released, or when reseting all RHI resources.
	 * Resources that need to release before a D3D device reset must implement this function.
	 * This is only called by the rendering thread.
	 */
	virtual void ReleaseRHI() override;

	// FRenderTarget interface.

	/** 
	 * @return width of the target
	 */
	virtual uint32 GetSizeX() const override;

	/** 
	 * @return height of the target
	 */
	virtual uint32 GetSizeY() const override;

	/**
	 * @return dimensions of the target
	 */
	virtual FIntPoint GetSizeXY() const override;

	/**
	 * @return UnorderedAccessView for rendering
	 */
	FUnorderedAccessViewRHIRef GetUnorderedAccessViewRHI() { return UnorderedAccessViewRHI; }

	/** 
	* Render target resource should be sampled in linear color space
	*
	* @return display gamma expected for rendering to this render target 
	*/
	float GetDisplayGamma() const override;

	/**
	* Copy the texels of a single face of the cube into an array.
	* @param OutImageData - float16 values will be stored in this array.
	* @param InFlags - read flags. ensure cubeface member has been set.
	* @param InRect - Rectangle of texels to copy.
	* @return true if the read succeeded.
	*/
	UE_DEPRECATED(5.4, "Use FRenderTarget's ReadPixels, which is functionally equivalent")
	ENGINE_API bool ReadPixels(TArray< FColor >& OutImageData, FReadSurfaceDataFlags InFlags, FIntRect InRect = FIntRect(0, 0, 0, 0));

	/**
	* Copy the texels of a single face of the cube into an array.
	* @param OutImageData - float16 values will be stored in this array.
	* @param InFlags - read flags. ensure cubeface member has been set.
	* @param InRect - Rectangle of texels to copy.
	* @return true if the read succeeded.
	*/
	UE_DEPRECATED(5.4, "Use FRenderTarget's ReadFloat16Pixels, which is functionally equivalent")
	ENGINE_API bool ReadPixels(TArray<FFloat16Color>& OutImageData, FReadSurfaceDataFlags InFlags, FIntRect InRect = FIntRect(0, 0, 0, 0));

protected:
	/**
	* Updates (resolves) the render target texture.
	* Optionally clears each face of the render target to green.
	* This is only called by the rendering thread.
	*/
	friend class UTextureRenderTargetCube;
	virtual void UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget=true) override;

private:
	/** The UTextureRenderTargetCube which this resource represents. */
	const class UTextureRenderTargetCube* Owner;

	UE_DEPRECATED(5.1, "TextureCubeRHI is deprecated. Use TextureRHI instead.")
	FTextureRHIRef TextureCubeRHI;

	UE_DEPRECATED(5.1, "CubeFaceSurfaceRHI is deprecated. Use TextureRHI instead.")
	FTextureRHIRef CubeFaceSurfaceRHI;

	UE_DEPRECATED(5.1, "RenderTargetCubeRHI is deprecated. Use TextureRHI instead.")
	FTextureRHIRef RenderTargetCubeRHI;

	/** Optional Unordered Access View for the resource, automatically created if bCanCreateUAV is true */
	FUnorderedAccessViewRHIRef UnorderedAccessViewRHI;

	/** Face currently used for target surface */
	ECubeFace CurrentTargetFace;
};

/**
 * Do not call these (GetDefaultTextureFormatName) directly, use GetPlatformTextureFormatNamesWithPrefix instead
 * this should only be called by TargetPlatform::GetTextureFormats()
 */

/** Gets the name of a format for the given LayerIndex */
ENGINE_API FName GetDefaultTextureFormatName( const class ITargetPlatform* TargetPlatform, const class UTexture* Texture, int32 LayerIndex, bool bSupportCompressedVolumeTexture, int32 Unused_BlockSize, bool bSupportFilteredFloat32Textures);
ENGINE_API FName GetDefaultTextureFormatName( const class ITargetPlatformSettings* TargetPlatformSettings, const class ITargetPlatformControls* TargetPlatformControls, const class UTexture* Texture, int32 LayerIndex, bool bSupportCompressedVolumeTexture, int32 Unused_BlockSize, bool bSupportFilteredFloat32Textures);

/** Gets an array of format names for each layer in the texture */
ENGINE_API void GetDefaultTextureFormatNamePerLayer(TArray<FName>& OutFormatNames, const class ITargetPlatform* TargetPlatform, const class UTexture* Texture, bool bSupportCompressedVolumeTexture, int32 Unused_BlockSize, bool bSupportFilteredFloat32Textures);
ENGINE_API void GetDefaultTextureFormatNamePerLayer(TArray<FName>& OutFormatNames, const class ITargetPlatformSettings* TargetPlatformSettings, const class ITargetPlatformControls* TargetPlatformControls, const class UTexture* Texture, bool bSupportCompressedVolumeTexture, int32 Unused_BlockSize, bool bSupportFilteredFloat32Textures);

UE_DEPRECATED(5.9, "GetAllDefaultTextureFormats is deprecated.")
ENGINE_API void GetAllDefaultTextureFormats( const class ITargetPlatform* TargetPlatform, TArray<FName>& OutFormats);

UE_DEPRECATED(5.9, "GetAllDefaultTextureFormats is deprecated.")
ENGINE_API void GetAllDefaultTextureFormats( const class ITargetPlatformSettings* TargetPlatformSettings, TArray<FName>& OutFormats);
