// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Engine/EngineBaseTypes.h"

class FTickTaskLevel;
class ULevel;

DECLARE_STATS_GROUP(TEXT("TickGroups"), STATGROUP_TickGroups, STATCAT_Advanced);

/** 
 * Interface for the tick task manager, which is used to queue and execute FTickFunctions.
 * These functions are called for a world to handle the "tick frame" that runs normal gameplay operations.
 * The frame is split into multiple tick groups to allow broad coordination between different engine systems.
 */
class FTickTaskManagerInterface
{
public:
	virtual ~FTickTaskManagerInterface() = default;

	/** Allocate a new ticking structure to track registered ticks for a level. */
	virtual FTickTaskLevel* AllocateTickTaskLevel() = 0;

	/** Free a ticking structure used for tracking registered ticks. */
	virtual void FreeTickTaskLevel(FTickTaskLevel* TickTaskLevel) = 0;

	/**
	 * Queue all of the ticks for one tick frame.
	 * This initializes execution of all tick functions for the frame and handles prerequisite scheduling.
	 * @param World	- World currently ticking
	 * @param DeltaSeconds - time in seconds since last tick
	 * @param TickType - type of tick (viewports only, time only, etc)
	 */
	virtual void StartFrame(UWorld* InWorld, float DeltaSeconds, ELevelTick TickType, const TArray<ULevel*>& LevelsToTick) = 0;

	/**
	 * Run all of the ticks for a pause frame synchronously on the game thread.
	 * The capability of pause ticks are very limited. There are no dependencies or ordering or tick groups.
	 * @param World	- World currently ticking
	 * @param DeltaSeconds - time in seconds since last tick
	 * @param TickType - type of tick (viewports only, time only, etc)
	 */
	virtual void RunPauseFrame(UWorld* InWorld, float DeltaSeconds, ELevelTick TickType, const TArray<ULevel*>& LevelsToTick) = 0;

	/**
	 * Run a tick group, ticking all actors and components registered to execute in that tick group.
	 * @param Group - Ticking group to run
	 * @param bBlockTillComplete - if true, do not return until all ticks with the corresponding EndTickGroup have completed
	 */
	virtual void RunTickGroup(ETickingGroup Group, bool bBlockTillComplete ) = 0;

	/** Finish a frame of ticks for all worlds that called StartFrame. */
	virtual void EndFrame() = 0;

	/** Dumps all registered tick functions to output device. */
	virtual void DumpAllTickFunctions(FOutputDevice& Ar, UWorld* InWorld, bool bEnabled, bool bDisabled, bool bGrouped) = 0;

	/** Returns a map of enabled ticks, grouped by 'diagnostic context' string, along with count of enabled ticks */
	virtual void GetEnabledTickFunctionCounts(UWorld* InWorld, TSortedMap<FName, int32, FDefaultAllocator, FNameFastLess>& TickContextToCountMap, int32& EnabledCount, bool bDetailed, bool bFilterCoolingDown=false) = 0;

	/** Accessor for the singleton global tick task manager. */
	static ENGINE_API FTickTaskManagerInterface& Get();
};
