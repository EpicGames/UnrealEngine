// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Tickable.h: Interface for tickable objects.

=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Engine/EngineBaseTypes.h"

/**
 * Enum used to determine the current ticking rules for an object, this can change after creation.
 */
enum class ETickableTickType : uint8
{
	/** Use IsTickable to determine whether to tick */
	Conditional,

	/** Always tick the object */
	Always,

	/** Never tick the object, do not add to tickables array */
	Never,

	/** Unknown state, true for newly registered objects that have not yet called GetTickableTickType */
	NewObject
};

/**
 * Base class for tickable objects
 */
class FTickableObjectBase
{
protected:	
	/** Implementation struct for an individual tickable object */
	struct FTickableObjectEntry
	{
		FTickableObjectBase* TickableObject = nullptr;
		ETickableTickType TickType = ETickableTickType::NewObject;

		bool operator==(FTickableObjectBase* OtherObject) const { return TickableObject == OtherObject; }
	};

	/** Implementation struct for internals of ticking, there should be one instance of this for each direct subclass */
	struct FTickableStatics
	{
		/** This critical section should be locked during entire tick process */
		FTransactionallySafeCriticalSection TickableObjectsCritical;

		/** List of objects that are fully ticking */
		TArray<FTickableObjectBase::FTickableObjectEntry> TickableObjects;

		/** Lock for modifying new list, this is automatically acquired by functions below */
		FTransactionallySafeCriticalSection NewTickableObjectsCritical;

		/** Set of objects that have not yet been queried for tick type */
		TMap<FTickableObjectBase*, ETickableTickType> NewTickableObjects;

		/** True if any of the tick arrays are being iterated */
		bool bIsTickingObjects = false;

		/** True if any objects were deleted and this needs cleanup after tick */
		bool bNeedsCleanup = false;


		/** Call from anywhere to lock and add to the new objects list */
		ENGINE_API void QueueTickableObjectForAdd(FTickableObjectBase* InTickable, ETickableTickType NewTickType = ETickableTickType::NewObject);

		/** Call from anywhere to lock critical sections and set the tick type for an object which could modify both arrays */
		ENGINE_API void SetTickTypeForTickableObject(FTickableObjectBase* TickableObject, ETickableTickType NewTickType);


		/** Call after locking TickableObjectsCritical to process the NewTickableObjects array and prepare for tick */
		ENGINE_API void StartTicking();

		/** Finishes ticking and handle cleanup for entries that were invalidated */
		ENGINE_API void FinishTicking();
	};

	/** Perform a simple tick using a class-specific statics struct and a function ref */
	static ENGINE_API void SimpleTickObjects(FTickableStatics& Statics, TFunctionRef<void(FTickableObjectBase*)> TickFunc);

public:
	/**
	 * Pure virtual that must be overloaded by the inheriting class. It will
	 * be called at different times in the frame depending on the subclass.
	 *
	 * @param DeltaTime	Game time passed since the last call.
	 */
	virtual void Tick( float DeltaTime ) = 0;

	/**
	 * Virtual that can be overloaded by the inheriting class and is called before first tick.
	 * It is used to determine whether an object can possibly tick, and if not,
	 * it will not get added to the tickable objects array. If the tickable tick type
	 * is Conditional then the virtual IsTickable will be called to determine whether
	 * to tick the object on each given frame.
	 *
	 * @return an enum defining the rules for ticking this object.
	 */
	virtual ETickableTickType GetTickableTickType() const { return ETickableTickType::Conditional; }

	/**
	 * Virtual that can be overloaded by the inheriting class. It is
	 * used to determine whether an object should be conditionally ticked.
	 *
	 * @return	true if object is ready to be ticked, false otherwise.
	 */
	virtual bool IsTickable() const { return true; }

	/**
	 * Function called before IsTickable, used to change rules without breaking existing API.
	 * This was used to allow checking for tickable safety before calling GetTickableGameObjectWorld.
	 * @return	true if object is allowed to be ticked, false otherwise.
	 */
	UE_DEPRECATED(5.5, "Use IsTickable for conditional ticks and SetTickableTickType to change the tick type after initial registration")
	virtual bool IsAllowedToTick() const { return true; }

	/** Return the stat id used to track the performance of this object */
	virtual TStatId GetStatId() const = 0;
};

/**
 * This class provides an easy way to register gamethread-only tickable objects. It is an
 * abstract base class requiring you to implement the Tick() and GetStatId() methods.
 * It is intended to be used for simple objects that cannot easily use other methods for ticking.
 * It can optionally also be ticked in the Editor, allowing for an object that both ticks
 * during edit time and at runtime.
 * 
 * NOTE: Do not use this class with UObjects unless you add explicit code to start and stop ticking.
 * To do this, construct with ETickableTickType::Never and call SetTickableTickType to enable ticking later.
 * Ticking cannot be safely started until after object construction finishes, and should be set back to Never in BeginDestroy or earlier.
 * UObjects are often created and destroyed on background threads so it is not safe to control ticking from the constructor/destructor.
 */
class FTickableGameObject : public FTickableObjectBase
{
	/** Returns the tracking struct for this type */
	static ENGINE_API FTickableStatics& GetStatics();

public:
	/** Tickable objects cannot be copied safely due to the auto registration */
	UE_NONCOPYABLE(FTickableGameObject);

	/**
	 * Registers this instance with the static array of tickable objects, if it can ever tick.
	 * By default this function can only be called on the game thread, to make sure it doesn't call functions before initialization is complete.
	 * With the default NewObject tick type, it will call GetTickableTickType at the start of the next tick to determine the desired type.
	 * NOTE: This is not threadsafe with the default tick type! 
	 * For UObjects and other classes that can be constructed on worker threads, you must construct with ETickableTickType::Never and enable ticking later.
	 * 
	 * @param StartingTickType The initial ticking state of this object, override it to start disabled or skip the call to GetTickableTickType
	 */
	ENGINE_API FTickableGameObject(ETickableTickType StartingTickType = ETickableTickType::NewObject);

	/**
	 * Removes this instance from the static array of tickable objects.
	 */
	ENGINE_API virtual ~FTickableGameObject();

	/**
	 * Used to determine if an object should be ticked when the game is paused.
	 * Defaults to false, as that mimics old behavior.
	 *
	 * @return true if it should be ticked when paused, false otherwise
	 */
	virtual bool IsTickableWhenPaused() const
	{
		return false;
	}

	/**
	 * Used to determine whether the object should be ticked in the editor when there is no gameplay world.
	 * Objects will still be ticked in Play in Editor if they are associated with a PIE world.
	 *
	 * @return true if this tickable object should always be ticked in the editor, even if there is no gameplay world
	 */
	virtual bool IsTickableInEditor() const
	{
		return false;
	}

	/**
	 * Used to determine the specific world this object is associated with.
	 * If this returns a valid world it will tick during that world's level tick.
	 * If this returns null, it will tick during the general engine tick after all world ticks.
	 *
	 * @return the world this object is associated with, or nullptr
	 */
	virtual UWorld* GetTickableGameObjectWorld() const 
	{ 
		return nullptr;
	}

	/**
	 * Call to modify the tickable type of this instance. 
	 * This can be used to enable or disable tick even if GetTickableTickType has already been called.
	 * If this is a UObject that could be destroyed on a different thread, call this with Never during BeginDestroy (or earlier).
	 * 
	 * @param NewTickType	The new tick type of this instance
	 */
	ENGINE_API void SetTickableTickType(ETickableTickType NewTickType);

	/**
	 * Tick all FTickableGameObject instances that match the parameters.
	 *
	 * @param World			Specific world that is ticking, must match GetTickableGameObjectWorld to tick.
	 * @param LevelTickType	The type of tick where LEVELTICK_All is treated like a gameplay tick.
	 * @param bIsPaused		True if the gameplay world is paused, if this is false IsTickableWhenPaused must return true to tick.
	 * @param DeltaTime		Game time passed since the last call.
	 */
	static ENGINE_API void TickObjects(UWorld* World, ELevelTick LevelTickType, bool bIsPaused, float DeltaSeconds);
};
