// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TimerManager.h: Global gameplay timer facility
=============================================================================*/

#pragma once

#include "AutoRTFM.h"
#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/Object.h"
#include "Engine/TimerHandle.h"
#include "Misc/TVariant.h"
#include "Templates/Function.h"

class UGameInstance;
enum class ELevelCollectionType : uint8;
struct FTimerSourceList;

// using "not checked" user policy (means race detection is disabled) because this delegate is stored in a TSparseArray and causes its reallocation
// from inside delegate's execution. This is incompatible with race detection that needs to access the delegate instance after its execution
using FTimerDelegate = TDelegate<void(), FNotThreadSafeNotCheckedDelegateUserPolicy>;
using FTimerFunction = TFunction<void(void)>;
using FTimerDelegateVariant = TVariant<TYPE_OF_NULLPTR, FTimerDelegate, FTimerDynamicDelegate, FTimerFunction>;

#ifndef UE_ENABLE_TRACKING_TIMER_SOURCES
#define UE_ENABLE_TRACKING_TIMER_SOURCES !UE_BUILD_SHIPPING
#endif

/** Simple interface to wrap a timer delegate that can be either native or dynamic. */
struct FTimerUnifiedDelegate
{
	/** Holds the delegate to call. */
	FTimerDelegateVariant VariantDelegate;

	FTimerUnifiedDelegate() {};

	FTimerUnifiedDelegate(FTimerDelegate const& D) : VariantDelegate(TInPlaceType<FTimerDelegate>(), D) {}
	FTimerUnifiedDelegate(FTimerDynamicDelegate const& D) : VariantDelegate(TInPlaceType<FTimerDynamicDelegate>(), D) {}
	FTimerUnifiedDelegate(FTimerFunction&& Callback) : VariantDelegate(TInPlaceType<FTimerFunction>(), MoveTemp(Callback)) {}
	
	void Execute() const;

	bool IsBound() const;

	const void* GetBoundObject() const;

	void Unbind()
	{
		VariantDelegate.Set<TYPE_OF_NULLPTR>(nullptr);
	}

	/** Utility to output info about delegate as a string. */
	FString ToString() const;

	// Movable only
	FTimerUnifiedDelegate(FTimerUnifiedDelegate&&) = default;
	FTimerUnifiedDelegate(const FTimerUnifiedDelegate&) = delete;
	FTimerUnifiedDelegate& operator=(FTimerUnifiedDelegate&&) = default;
	FTimerUnifiedDelegate& operator=(const FTimerUnifiedDelegate&) = delete;
};

enum class ETimerStatus : uint8
{
	Pending,
	Active,
	Paused,
	Executing,
	ActivePendingRemoval,
	ReservedForTransaction,
};

struct FTimerData
{
	/** If true, this timer will loop indefinitely.  Otherwise, it will be destroyed when it expires. */
	uint8 bLoop : 1;

	/** If true for a looping timer, it will run a maximum of once per frame when it expires. Otherwise, it will run as many times as can fit in
	 *  the current frame's delta time (rounding down).
	 */
	uint8 bMaxOncePerFrame : 1;

	/** If true, this timer was created with a delegate to call (which means if the delegate becomes invalid, we should invalidate the timer too). */
	uint8 bRequiresDelegate : 1;

	/** Timer Status */
	ETimerStatus Status;

	/** The level collection that was active when this timer was created. Used to set the correct context before executing the timer's delegate. */
	ELevelCollectionType LevelCollection;
	
	/** Time between set and fire, or repeat frequency if looping. */
	float Rate;

	/** 
	 * Time (on the FTimerManager's clock) that this timer should expire and fire its delegate. 
	 * Note when a timer is paused, we re-base ExpireTime to be relative to 0 instead of the running clock, 
	 * meaning ExpireTime contains the remaining time until fire.
	 */
	double ExpireTime;

	/** Holds the delegate to call. */
	FTimerUnifiedDelegate TimerDelegate;

	/** Handle representing this timer */
	FTimerHandle Handle;

	/** This is the key to the TimerIndicesByObject map - this is kept so that we can look up even if the referenced object is expired */
	const void* TimerIndicesByObjectKey = nullptr;

	ENGINE_API FTimerData();

	// Movable only
	FTimerData(FTimerData&&) = default;
	FTimerData(const FTimerData&) = delete;
	FTimerData& operator=(FTimerData&&) = default;
	FTimerData& operator=(const FTimerData&) = delete;
};

/**
 * Structure to support specialized looping behavior.
 */
struct FTimerManagerTimerParameters
{
	bool bLoop = false;
	bool bMaxOncePerFrame = false;
	float FirstDelay = -1.f;
};

/** 
 * Class to globally manage timers.
 *
 * FTimerManager supports transactional usage from both the open and closed.
 * Open modifications may be clobbered with transactional changes on abort.
 */
class FTimerManager : public FNoncopyable
{
public:

	ENGINE_API void Tick(float DeltaTime);
	ENGINE_API TStatId GetStatId() const;

	// ----------------------------------
	// Timer API

	ENGINE_API explicit FTimerManager(UGameInstance* GameInstance = nullptr);
	ENGINE_API virtual ~FTimerManager();

	/**
	 * Called from crash handler to provide more debug information.
	 */
	ENGINE_API virtual void OnCrash();

	/**
	 * Sets a timer to call the given native function at a set interval.  If a timer is already set
	 * for this handle, it will replace the current timer.
	 *
	 * @param InOutHandle			If the passed-in handle refers to an existing timer, it will be cleared before the new timer is added. A new handle to the new timer is returned in either case.
	 * @param InObj					Object to call the timer function on.
	 * @param InTimerMethod			Method to call when timer fires.
	 * @param InRate				The amount of time (in seconds) between set and firing.  If <= 0.f, clears existing timers.
	 * @param InbLoop				true to keep firing at Rate intervals, false to fire only once.
	 * @param InFirstDelay			The time (in seconds) for the first iteration of a looping timer. If < 0.f InRate will be used.
	 */
	template< class UserClass >
	inline void SetTimer(FTimerHandle& InOutHandle, UserClass* InObj, typename FTimerDelegate::TMethodPtr< UserClass > InTimerMethod, float InRate, bool InbLoop = false, float InFirstDelay = -1.f)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate( FTimerDelegate::CreateUObject(InObj, InTimerMethod) ), InRate, InbLoop, InFirstDelay);
	}
	template< class UserClass >
	inline void SetTimer(FTimerHandle& InOutHandle, UserClass* InObj, typename FTimerDelegate::TConstMethodPtr< UserClass > InTimerMethod, float InRate, bool InbLoop = false, float InFirstDelay = -1.f)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate( FTimerDelegate::CreateUObject(InObj, InTimerMethod) ), InRate, InbLoop, InFirstDelay);
	}

	/** Version that takes any generic delegate. */
	inline void SetTimer(FTimerHandle& InOutHandle, FTimerDelegate const& InDelegate, float InRate, bool InbLoop, float InFirstDelay = -1.f)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate(InDelegate), InRate, InbLoop, InFirstDelay);
	}
	/** Version that takes a dynamic delegate (e.g. for UFunctions). */
	inline void SetTimer(FTimerHandle& InOutHandle, FTimerDynamicDelegate const& InDynDelegate, float InRate, bool InbLoop, float InFirstDelay = -1.f)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate(InDynDelegate), InRate, InbLoop, InFirstDelay);
	}
	/*** Version that doesn't take a delegate */
	inline void SetTimer(FTimerHandle& InOutHandle, float InRate, bool InbLoop, float InFirstDelay = -1.f)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate(), InRate, InbLoop, InFirstDelay);
	}
	/** Version that takes a TFunction */
	inline void SetTimer(FTimerHandle& InOutHandle, TFunction<void(void)>&& Callback, float InRate, bool InbLoop, float InFirstDelay = -1.f )
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate(MoveTemp(Callback)), InRate, InbLoop, InFirstDelay);
	}

	/* Preferred versions of the above methods that now take a more configurable structure that specifies timer looping and delay behavior. */

	/**
	 * Sets a timer to call the given native function at a set interval.  If a timer is already set
	 * for this handle, it will replace the current timer.
	 *
	 * @param InOutHandle			If the passed-in handle refers to an existing timer, it will be cleared before the new timer is added. A new handle to the new timer is returned in either case.
	 * @param InObj					Object to call the timer function on.
	 * @param InTimerMethod			Method to call when timer fires.
	 * @param InRate				The amount of time (in seconds) between set and firing.  If <= 0.f, clears existing timers.
	 * @param InTimerParameters		The structure with additional timer looping and delay parameters.
	 */
	template <class UserClass>
	inline void SetTimer(FTimerHandle& InOutHandle, UserClass* InObj, typename FTimerDelegate::TMethodPtr<UserClass> InTimerMethod, float InRate, const FTimerManagerTimerParameters& InTimerParameters)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate(FTimerDelegate::CreateUObject(InObj, InTimerMethod)), InRate, InTimerParameters);
	}
	template< class UserClass >
	inline void SetTimer(FTimerHandle& InOutHandle, UserClass* InObj, typename FTimerDelegate::TConstMethodPtr<UserClass> InTimerMethod, float InRate, const FTimerManagerTimerParameters& InTimerParameters)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate(FTimerDelegate::CreateUObject(InObj, InTimerMethod)), InRate, InTimerParameters);
	}

	/** Version that takes any generic delegate. */
	inline void SetTimer(FTimerHandle& InOutHandle, FTimerDelegate const& InDelegate, float InRate, const FTimerManagerTimerParameters& InTimerParameters)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate(InDelegate), InRate, InTimerParameters);
	}
	/** Version that takes a dynamic delegate (e.g. for UFunctions). */
	inline void SetTimer(FTimerHandle& InOutHandle, FTimerDynamicDelegate const& InDynDelegate, float InRate, const FTimerManagerTimerParameters& InTimerParameters)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate(InDynDelegate), InRate, InTimerParameters);
	}
	/*** Version that doesn't take a delegate */
	inline void SetTimer(FTimerHandle& InOutHandle, float InRate, const FTimerManagerTimerParameters& InTimerParameters)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate(), InRate, InTimerParameters);
	}
	/** Version that takes a TFunction */
	inline void SetTimer(FTimerHandle& InOutHandle, TFunction<void(void)>&& Callback, float InRate, const FTimerManagerTimerParameters& InTimerParameters)
	{
		InternalSetTimer(InOutHandle, FTimerUnifiedDelegate(MoveTemp(Callback)), InRate, InTimerParameters);
	}

	/**
	 * Sets a timer to call the given native function on the next tick.
	 *
	 * @param inObj					Object to call the timer function on.
	 * @param inTimerMethod			Method to call when timer fires.
	 */
	template< class UserClass >
	inline FTimerHandle SetTimerForNextTick(UserClass* inObj, typename FTimerDelegate::TMethodPtr< UserClass > inTimerMethod)
	{
		return InternalSetTimerForNextTick(FTimerUnifiedDelegate(FTimerDelegate::CreateUObject(inObj, inTimerMethod)));
	}
	template< class UserClass >
	inline FTimerHandle SetTimerForNextTick(UserClass* inObj, typename FTimerDelegate::TConstMethodPtr< UserClass > inTimerMethod)
	{
		return InternalSetTimerForNextTick(FTimerUnifiedDelegate(FTimerDelegate::CreateUObject(inObj, inTimerMethod)));
	}

	/** Version that takes any generic delegate. */
	inline FTimerHandle SetTimerForNextTick(FTimerDelegate const& InDelegate)
	{
		return InternalSetTimerForNextTick(FTimerUnifiedDelegate(InDelegate));
	}
	/** Version that takes a dynamic delegate (e.g. for UFunctions). */
	inline FTimerHandle SetTimerForNextTick(FTimerDynamicDelegate const& InDynDelegate)
	{
		return InternalSetTimerForNextTick(FTimerUnifiedDelegate(InDynDelegate));
	}
	/** Version that takes a TFunction */
	inline FTimerHandle SetTimerForNextTick(TFunction<void(void)>&& Callback)
	{
		return InternalSetTimerForNextTick(FTimerUnifiedDelegate(MoveTemp(Callback)));
	}

	/**
	* Clears a previously set timer, identical to calling SetTimer() with a <= 0.f rate.
	* Invalidates the timer handle as it should no longer be used.
	*
	* @param InHandle The handle of the timer to clear.
	*/
	inline void ClearTimer(FTimerHandle& InHandle)
	{
		if (const FTimerData* TimerData = FindTimer(InHandle))
		{
			InternalClearTimer(InHandle);
		}
		InHandle.Invalidate();
	}

	/** Clears all timers that are bound to functions on the given object. */
	inline void ClearAllTimersForObject(void const* Object)
	{
		if (Object)
		{
			InternalClearAllTimers( Object );
		}
	}

	/**
	 * Pauses a previously set timer.
	 *
	 * @param InHandle The handle of the timer to pause.
	 */
	ENGINE_API void PauseTimer(FTimerHandle InHandle);

	/**
	 * Unpauses a previously set timer
	 *
	 * @param InHandle The handle of the timer to unpause.
	 */
	ENGINE_API void UnPauseTimer(FTimerHandle InHandle);

	/**
	 * Gets the current rate (time between activations) for the specified timer.
	 *
	 * @param InHandle The handle of the timer to return the rate of.
	 * @return The current rate or -1.f if timer does not exist.
	 */
	inline float GetTimerRate(FTimerHandle InHandle) const
	{
		FTimerData const* const TimerData = FindTimer(InHandle);
		return InternalGetTimerRate(TimerData);
	}

	/**
	 * Returns true if the specified timer exists and is not paused
	 *
	 * @param InHandle The handle of the timer to check for being active.
	 * @return true if the timer exists and is active, false otherwise.
	 */
	inline bool IsTimerActive(FTimerHandle InHandle) const
	{
#if UE_AUTORTFM
		if (AutoRTFM::IsClosed())
		{
			if (const FTransactionalTimer* Timer = TransactionalTimers.Find(InHandle))
			{
				return Timer->Status == FTransactionalTimer::Active ||
					Timer->Status == FTransactionalTimer::Pending;
			}
		}
#endif  // UE_AUTORTFM
		
		FTimerData const* const TimerData = FindTimer( InHandle );
		return TimerData && TimerData->Status != ETimerStatus::Paused;
	}

	/**
	* Returns true if the specified timer exists and is paused
	*
	* @param InHandle The handle of the timer to check for being paused.
	* @return true if the timer exists and is paused, false otherwise.
	*/
	inline bool IsTimerPaused(FTimerHandle InHandle) const
	{
#if UE_AUTORTFM
		if (AutoRTFM::IsClosed())
		{
			if (const FTransactionalTimer* Timer = TransactionalTimers.Find(InHandle))
			{
				return Timer->Status == FTransactionalTimer::Paused;
			}
		}
#endif  // UE_AUTORTFM
		
		FTimerData const* const TimerData = FindTimer(InHandle);
		return TimerData && TimerData->Status == ETimerStatus::Paused;
	}

	/**
	* Returns true if the specified timer exists and is pending
	*
	* @param InHandle The handle of the timer to check for being pending.
	* @return true if the timer exists and is pending, false otherwise.
	*/
	inline bool IsTimerPending(FTimerHandle InHandle) const
	{
#if UE_AUTORTFM
		if (AutoRTFM::IsClosed())
		{
			if (const FTransactionalTimer* Timer = TransactionalTimers.Find(InHandle))
			{
				return Timer->Status == FTransactionalTimer::Pending;
			}
		}
#endif  // UE_AUTORTFM

		FTimerData const* const TimerData = FindTimer(InHandle);
		return TimerData && TimerData->Status == ETimerStatus::Pending;
	}

	/**
	* Returns true if the specified timer exists
	*
	* @param InHandle The handle of the timer to check for existence.
	* @return true if the timer exists, false otherwise.
	*/
	inline bool TimerExists(FTimerHandle InHandle) const
	{
#if UE_AUTORTFM
		if (AutoRTFM::IsClosed())
		{
			if (const FTransactionalTimer* Timer = TransactionalTimers.Find(InHandle))
			{
				return Timer->Status != FTransactionalTimer::Canceled;
			}
		}
#endif  // UE_AUTORTFM

		return FindTimer(InHandle) != nullptr;
	}

	/**
	 * Gets the current elapsed time for the specified timer.
	 *
	 * @param InHandle The handle of the timer to check the elapsed time of.
	 * @return The current time elapsed or -1.f if the timer does not exist.
	 */
	inline float GetTimerElapsed(FTimerHandle InHandle) const
	{
#if UE_AUTORTFM
		if (AutoRTFM::IsClosed())
		{
			if (const FTransactionalTimer* Timer = TransactionalTimers.Find(InHandle))
			{
				// A transactionally created timer cannot be executed until the
				// transaction commits, so will have an elapsed time of 0,
				// unless it was cancelled, in which case we return -1 (same as
				// InternalGetTimerElapsed()).
				return Timer->Status == FTransactionalTimer::Canceled ? -1.0f : 0.0f;
			}
		}
#endif  // UE_AUTORTFM
		FTimerData const* const TimerData = FindTimer(InHandle);
		return InternalGetTimerElapsed(TimerData);
	}

	/**
	 * Gets the time remaining before the specified timer is called
	 *
	 * @param InHandle The handle of the timer to check the remaining time of.
	 * @return	 The current time remaining, or -1.f if timer does not exist
	 */
	inline float GetTimerRemaining(FTimerHandle InHandle) const
	{
#if UE_AUTORTFM
		if (AutoRTFM::IsClosed())
		{
			if (const FTransactionalTimer* Timer = TransactionalTimers.Find(InHandle))
			{
				if (Timer->NewData.IsSet())
				{
					return static_cast<float>(Timer->NewData->ExpireTime);
				}
				else if (Timer->Status == FTransactionalTimer::Canceled)
				{
					return -1.0f;
				}
			}
		}
#endif  // UE_AUTORTFM
		FTimerData const* const TimerData = FindTimer(InHandle);
		return InternalGetTimerRemaining(TimerData);
	}

	bool inline HasBeenTickedThisFrame() const
	{
		return (LastTickedFrame == GFrameCounter);
	}

	/**
	 * Finds a handle to a timer bound to a particular dynamic delegate.
	 * This function is intended to be used only by the K2 system.
	 *
	 * @param  InDynamicDelegate  The dynamic delegate to search for.
	 *
	 * @return A handle to the found timer - !IsValid() if no such timer was found.
	 */
	ENGINE_API FTimerHandle K2_FindDynamicTimerHandle(FTimerDynamicDelegate InDynamicDelegate) const;

	/** Debug command to output info on all timers currently set to the log. */
	ENGINE_API void ListTimers() const;

private:
	ENGINE_API void SetGameInstance(UGameInstance* InGameInstance);

// This should be private, but needs to be public for testing.
public:
	/** Generates a handle for a timer at a given index */
	/* AUTORTFM_DISABLE */ ENGINE_API FTimerHandle GenerateHandle(int32 Index);

// These should be private, but need to be protected so IMPLEMENT_GET_PROTECTED_FUNC works for testing.
protected:
	/** Will find a timer in the active, paused, or pending list. */
	inline FTimerData const* FindTimer(FTimerHandle const& InHandle) const
	{
		return const_cast<FTimerManager*>(this)->FindTimer(InHandle);
	}
	ENGINE_API FTimerData* FindTimer( FTimerHandle const& InHandle );

private:
	ENGINE_API void InternalSetTimer( FTimerHandle& InOutHandle, FTimerUnifiedDelegate&& InDelegate, float InRate, bool bInLoop, float InFirstDelay );
	ENGINE_API void InternalSetTimer( FTimerHandle& InOutHandle, FTimerUnifiedDelegate&& InDelegate, float InRate, const FTimerManagerTimerParameters& InTimerParameters );
	ENGINE_API FTimerHandle InternalSetTimerForNextTick( FTimerUnifiedDelegate&& InDelegate );
	ENGINE_API void InternalClearTimer( FTimerHandle InDelegate );
	ENGINE_API void InternalClearAllTimers( void const* Object );
	ENGINE_API float InternalGetTimerRate( FTimerData const* const TimerData ) const;
	ENGINE_API float InternalGetTimerElapsed( FTimerData const* const TimerData ) const;
	ENGINE_API float InternalGetTimerRemaining( FTimerData const* const TimerData ) const;

	/** Will get a timer in the active, paused, or pending list.  Expected to be given a valid, non-stale handle */
	inline const FTimerData& GetTimer(const FTimerHandle& InHandle) const
	{
		return const_cast<FTimerManager*>(this)->GetTimer(InHandle);
	}
	ENGINE_API FTimerData& GetTimer(FTimerHandle const& InHandle);

	/** Adds a timer from the Timers list, also updating the TimerIndicesByObject map.  Returns the insertion index. */
	/* AUTORTFM_DISABLE */ ENGINE_API FTimerHandle AddTimer(FTimerData&& TimerData);
	/** Adds TimerHandle to the handle set of ObjectToTimers with the key TimerIndicesByObjectKey */
	/* AUTORTFM_DISABLE */ ENGINE_API void AddObjectToTimers(FTimerHandle TimerHandle, const void* TimerIndicesByObjectKey);
	
	/** Removes a timer from the Timers list at the given index, also cleaning up the TimerIndicesByObject map */
	/* AUTORTFM_DISABLE */ ENGINE_API void RemoveTimer(FTimerHandle Handle);
	ENGINE_API bool WillRemoveTimerAssert(FTimerHandle Handle) const;

	/** Check for expired timers that are pending destruction, and destroy them */
	/* AUTORTFM_DISABLE */ ENGINE_API void RemoveExpiredTimers();

#if UE_AUTORTFM
	/**
	 * Registers an on-commit and on-abort handler to transactionally commit or
	 * revert the transactional timers in TransactionalTimers, if these handlers
	 * have not already been registered.
	 */
	ENGINE_API void MaybeRegisterTransactionalCallbacks();
	/** Adds a new FTimerData with the ReservedForTransaction status, returning its unique handle */
	AUTORTFM_OPEN ENGINE_API FTimerHandle GenerateReservedHandleForTransaction();
#endif  // UE_AUTORTFM

	/** The array of timers - all other arrays will index into this */
	TSparseArray<FTimerData> Timers;
	/** Heap of actively running timers. */
	TArray<FTimerHandle> ActiveTimerHeap;
	/** Set of paused timers. */
	TSet<FTimerHandle> PausedTimerSet;
	/** Set of timers added this frame, to be added after timer has been ticked */
	TSet<FTimerHandle> PendingTimerSet;
	/** A map of object pointers to timers with delegates bound to those objects, for quick lookup */
	TMap<const void*, TSet<FTimerHandle>> ObjectToTimers;

#if UE_AUTORTFM
	/** FTransactionalTimer holds the state of a timer modified in the closed of a transaction */
	struct FTransactionalTimer
	{
		// The transactional timer state.
		enum EStatus
		{
			// On commit of the transaction, the timer will be put into the 
			// Pending state.
			Pending,
			// On commit of the transaction, the timer will be put into the 
			// Active state.
			Active,
			// On commit of the transaction, the timer will be put into the
			// Paused state.
			Paused,
			// On commit of the transaction, the timer will be removed.
			Canceled,
		};
		// Timer data, if the timer was transactionally created.
		// Note that FTransactionalTimer::Status is used instead of
		// FTimerData::Status, and NewData->Status is ignored.
		TOptional<FTimerData> NewData;
		// The current status of the transactional timer.
		EStatus Status = Pending;
	};

	/**
	 * A map of timer handle to transactional timers, which will be merged into
	 * Timers on transaction commit, or discarded on transaction abort.
	 * The timer handle key indexes to a slot in Timers which must be preserved
	 * for the duration of the transaction.
	 */
	TMap<FTimerHandle, FTransactionalTimer> TransactionalTimers;

	/** True if on-commit and on-abort handlers have been registered for this transaction nest. */
	bool bRegisteredTransactionalCallbacks = false; 
#endif  // UE_AUTORTFM

	/** An internally consistent clock, independent of World.  Advances during ticking. */
	double InternalTime;

	/** Index to the timer delegate currently being executed, or INDEX_NONE if none are executing.  Used to handle "timer delegates that manipulate timers" cases. */
	FTimerHandle CurrentlyExecutingTimer;

	/** Set this to GFrameCounter when Timer is ticked. To figure out if Timer has been already ticked or not this frame. */
	uint64 LastTickedFrame;

	/** The last serial number we assigned from this timer manager */
	static ENGINE_API uint64 LastAssignedSerialNumber;

	/** The game instance that created this timer manager. May be null if this timer manager wasn't created by a game instance. */
	UGameInstance* OwningGameInstance;

#if UE_ENABLE_TRACKING_TIMER_SOURCES
	/** Debugging/tracking information used when TimerManager.BuildTimerSourceList is set */
	TUniquePtr<FTimerSourceList> TimerSourceList;
#endif
};

