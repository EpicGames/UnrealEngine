// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "Animation/AnimInstance.h"
#include "Components/ActorComponent.h"
#include "Components/SkeletalMeshComponent.h" // Needed because AnimInstance::GetOwningComponent. (this only fails on clang compilers)
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Templates/EnableIf.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"

#define TRACE_FILTERING_ENABLED (UE_TRACE_FORCE_ENABLED || UE_TRACE_MINIMAL_ENABLED || (UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING))

#if TRACE_FILTERING_ENABLED

struct FTraceFilter
{
	struct TObjectFilter
	{
		template<typename T>
		static TEnableIf<TPointerIsConvertibleFromTo<typename TRemovePointer<T>::Type, UWorld>::Value, bool>::Type FORCEINLINE CanTrace(const T* Object)
		{
			const UWorld* World = (const UWorld*)Object;
			return IsObjectTraceable(World);
		}

		template<typename T>
		static TEnableIf<TPointerIsConvertibleFromTo<typename TRemovePointer<T>::Type, AActor>::Value, bool>::Type FORCEINLINE CanTrace(const T* Object)
		{
			/** For an individual Actor, we expect it and the owning world to both be marked traceable */
			const AActor* Actor = (const AActor*)Object;
			return (IsObjectTraceable(Cast<UObject>(Actor)) && (Actor && CanTrace(Actor->GetWorld())));
		}

		template<typename T>
		static TEnableIf<TPointerIsConvertibleFromTo<typename TRemovePointer<T>::Type, UActorComponent>::Value, bool>::Type FORCEINLINE CanTrace(const T* Object)
		{
			/** For an individual Actor Component, we expect it or the owning actor to be marked traceable */
			const UActorComponent* ActorComponent = (const UActorComponent*)Object;
			return IsObjectTraceable(ActorComponent) || (ActorComponent && CanTrace(ActorComponent->GetOwner()));
		}

		template<typename T>
		static TEnableIf<TPointerIsConvertibleFromTo<typename TRemovePointer<T>::Type, UAnimInstance>::Value, bool>::Type FORCEINLINE CanTrace(const T* Object)
		{
			/** For an AnimInstance object, we expect it or the owning component to be marked traceable */
			const UAnimInstance* AnimInstance = (const UAnimInstance*)Object;
			return IsObjectTraceable(AnimInstance) || (AnimInstance && CanTrace(AnimInstance->GetOwningComponent()));
		}

		template<typename T>
		static TEnableIf<(!TPointerIsConvertibleFromTo<typename TRemovePointer<T>::Type, UActorComponent>::Value
			&& !TPointerIsConvertibleFromTo<typename TRemovePointer<T>::Type, AActor>::Value
			&& !TPointerIsConvertibleFromTo<typename TRemovePointer<T>::Type, UWorld>::Value
			&& !TPointerIsConvertibleFromTo<typename TRemovePointer<T>::Type, UAnimInstance>::Value
			&& TPointerIsConvertibleFromTo<typename TRemovePointer<T>::Type, UObject>::Value), bool>::Type FORCEINLINE CanTrace(const T* Object)
		{
			/** For an individual UObject, we expect it or the owning world to be marked traceable */
			const UObject* BaseObject = (const UObject*)Object;
			return FTraceFilter::IsObjectTraceable(Object) || (BaseObject && IsObjectTraceable(BaseObject->GetWorld()));
		}
	};

	/** Set whether an Object is Traceable, and eligible for outputting Trace Data */
	template<bool bForceThreadSafe = true>
	static void SetObjectIsTraceable(const UObject* InObject, bool bIsTraceable);

	/** Mark an Object as Traceable, and eligible for outputting Trace Data */
	template<bool bForceThreadSafe = true>
	static void MarkObjectTraceable(const UObject* InObject);

	/** Initializes any of the Filters which are part of the Engine */
	static ENGINE_API void Init();

	/** Destroys any of the Filters which are part of the Engine */
	static ENGINE_API void Destroy();

	/** Returns whether an object is eligible to be outputted (output trace data) */
	template<bool bForceThreadSafe = true>
	static bool IsObjectTraceable(const UObject* InObject);

	static ENGINE_API void Lock();
	static ENGINE_API void Unlock();
};

extern template ENGINE_API void FTraceFilter::SetObjectIsTraceable<true>(const UObject* InObject, bool bIsTraceable);
extern template ENGINE_API void FTraceFilter::SetObjectIsTraceable<false>(const UObject* InObject, bool bIsTraceable);
extern template ENGINE_API void FTraceFilter::MarkObjectTraceable<true>(const UObject* InObject);
extern template ENGINE_API void FTraceFilter::MarkObjectTraceable<false>(const UObject* InObject);
extern template ENGINE_API bool FTraceFilter::IsObjectTraceable<true>(const UObject* InObject);
extern template ENGINE_API bool FTraceFilter::IsObjectTraceable<false>(const UObject* InObject);

#define CAN_TRACE_OBJECT(Object) \
	FTraceFilter::TObjectFilter::CanTrace(Object)

#define CANNOT_TRACE_OBJECT(Object) \
	(!FTraceFilter::TObjectFilter::CanTrace(Object))

#define MARK_OBJECT_TRACEABLE(Object) \
	FTraceFilter::MarkObjectTraceable(Object)

#define SET_OBJECT_TRACEABLE(Object, bIsTraceable) \
	FTraceFilter::SetObjectIsTraceable(Object, bIsTraceable)

#define GET_TRACE_OBJECT_VALUE(Object) \
	FTraceFilter::IsObjectTraceable(Object)
#else

#define CAN_TRACE_OBJECT(Object) \
	true
#define CANNOT_TRACE_OBJECT(Object) \
	false

#define MARK_OBJECT_TRACEABLE(Object)

#define SET_OBJECT_TRACEABLE(Object, bIsTraceable)

#define GET_TRACE_OBJECT_VALUE(Object) \
	false

#endif // TRACE_FILTERING_ENABLED
