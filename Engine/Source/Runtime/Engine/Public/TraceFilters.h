// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceFilter.h"

#if TRACE_FILTERING_ENABLED

/** AActor specific Trace filter, marks individual instances to not be traced out */
struct FTraceActorFilter
{
	static ENGINE_API void Initialize();
	static ENGINE_API void Destroy();
protected:
	static ENGINE_API FDelegateHandle WorldPreInitHandle;
	static ENGINE_API FDelegateHandle WorldPostInitHandle;
	static ENGINE_API FDelegateHandle OnActorSpawnedHandle;
	static ENGINE_API void OnPreWorldInit(UWorld* World, const UWorld::InitializationValues IVS);
	static ENGINE_API void OnPostWorldInit(UWorld* World, const UWorld::InitializationValues IVS);
	static ENGINE_API void OnActorSpawned(AActor* Actor);
};

/** UWorld specific Trace filter, marks individual instances to not be traced out */
struct FTraceWorldFilter
{
	static ENGINE_API void Initialize();
	static ENGINE_API void Destroy();
protected:
	static ENGINE_API void OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS);	
	static ENGINE_API FDelegateHandle WorldInitHandle;	
};

#define DISABLE_ENGINE_ACTOR_TRACE_FILTERING() \
	FTraceActorFilter::Destroy()

#define DISABLE_ENGINE_WORLD_TRACE_FILTERING() \
	FTraceWorldFilter::Destroy()

#else

#define DISABLE_ENGINE_ACTOR_TRACE_FILTERING()	
#define DISABLE_ENGINE_WORLD_TRACE_FILTERING() 

#endif // TRACE_FILTERING_ENABLED
