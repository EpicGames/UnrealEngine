// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "TraceControlMessages.h"

class IMessageBus;
class IMessageContext;
class FMessageEndpoint;

class FTraceService
{
public:
	ENGINE_API FTraceService();
	ENGINE_API FTraceService(const TSharedPtr<IMessageBus>&);
	virtual ~FTraceService() = default;
	
private:
	void OnStatusPing(const FTraceControlStatusPing& Message, const TSharedRef<IMessageContext>& Context);
	void OnChannelsPing(const FTraceControlChannelsPing& Message, const TSharedRef<IMessageContext>& Context);
	void OnSettingsPing(const FTraceControlSettingsPing& Message, const TSharedRef<IMessageContext>& Context);
	void OnDiscoveryPing(const FTraceControlDiscoveryPing& Message, const TSharedRef<IMessageContext>& Context);
	void OnStop(const FTraceControlStop& Message, const TSharedRef<IMessageContext>& Context);
	void OnSend(const FTraceControlSend& Message, const TSharedRef<IMessageContext>& Context);
	void OnChannelSet(const FTraceControlChannelsSet& Message, const TSharedRef<IMessageContext>& Context);
	void OnFile(const FTraceControlFile& Message, const TSharedRef<IMessageContext>& Context);
	void OnSnapshotSend(const FTraceControlSnapshotSend& Message, const TSharedRef<IMessageContext>& Context);
	void OnSnapshotFile(const FTraceControlSnapshotFile& Message, const TSharedRef<IMessageContext>& Context);
	void OnPause(const FTraceControlPause& Message, const TSharedRef<IMessageContext>& Context);
	void OnResume(const FTraceControlResume& Message, const TSharedRef<IMessageContext>& Context);
	void OnBookmark(const FTraceControlBookmark& Message, const TSharedRef<IMessageContext>& Context);
	void OnScreenshot(const FTraceControlScreenshot& Message, const TSharedRef<IMessageContext>& Context);
	void OnSetStatNamedEvents(const FTraceControlSetStatNamedEvents& Message, const TSharedRef<IMessageContext>& Context);
	void OnSecure(const FTraceControlSecure& Message, const TSharedRef<IMessageContext>& Context);

protected:
	/**
	 * Allows for overriding handling of how to connect when a send message is received. Default implementation
	 * assumes uri is a valid host name and tries to establish a regular socket connection.
	 * @param Message Request that was received
	 */
	ENGINE_API virtual void HandleSendUri(const FTraceControlSend& Message);

private:
	
	static void FillTraceStatusMessage(FTraceControlStatus* Message);

	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
	FGuid SessionId;
	FGuid InstanceId;
};
