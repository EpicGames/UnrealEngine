// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Set.h"
#include "Templates/OverrideVoidReturnInvoker.h"

namespace UE
{

/**
 * Wraps a generic tree traversal with invocation of enter and exit functions for top-down or bottom-up parsing (or both).
 * The passed functions can continue or break at any time, or return void for complete parsing. Supports rooted and DAG trees.
 */
class FTreeParser
{
public:
	enum class EParseAction
	{
		Continue,	// Continue parsing into child nodes
		Skip,		// Skip child nodes and exit function
		Break		// Break execution completely
	};

private:
	template <typename TNodeType, class TGetNumChildren, class TGetChild, class TEnterFunc, class TExitFunc>
	static EParseAction Parse(TNodeType InNode, TGetNumChildren GetNumChildren, TGetChild GetChild, TEnterFunc EnterFunc, TExitFunc ExitFunc)
	{
		TOverrideVoidReturnInvoker EnterFuncInvoker(EParseAction::Continue, EnterFunc);
		const EParseAction EnterParseAction = EnterFuncInvoker(InNode);

		if (EnterParseAction == EParseAction::Break)
		{
			return EParseAction::Break;
		}
		
		if (EnterParseAction == EParseAction::Continue)
		{
			for (int32 ChildIndex = 0; ChildIndex < GetNumChildren(InNode); ChildIndex++)
			{
				TNodeType ChildNode = GetChild(InNode, ChildIndex);
				if (Parse(ChildNode, GetNumChildren, GetChild, EnterFunc, ExitFunc) == EParseAction::Break)
				{
					return EParseAction::Break;
				}
			}

			TOverrideVoidReturnInvoker ExitFuncInvoker(EParseAction::Continue, ExitFunc);
			return ExitFuncInvoker(InNode);
		}

		return EParseAction::Continue;
	}

public:
	/**
	 * Parses a list of nodes from a hierarchical rooted tree structure.
	 * This function processes a graph defined by nodes and directed edges, assuming it represents a valid rooted tree.
	 * A valid rooted tree must satisfy three conditions:
	 *  1. It is connected (all nodes are reachable from the root).
	 *  2. It contains no cycles.
	 *  3. Every node, except for a single "root" node, has exactly one incoming edge (one parent).
	 * It cannot visit nodes more than once.
	 */
	template <typename TNodeType, class TGetNumChildren, class TGetChild, class TEnterFunc, class TExitFunc>
	static EParseAction ParseRooted(TNodeType InRootNode, TGetNumChildren GetNumChildren, TGetChild GetChild, TEnterFunc EnterFunc, TExitFunc ExitFunc)
	{
		return Parse<TNodeType>(InRootNode, GetNumChildren, GetChild, EnterFunc, ExitFunc);
	}

	template <typename TProfile, class TEnterFunc, class TExitFunc>
	static EParseAction ParseRooted(TProfile::TNodeType InRootNode, TProfile& InProfile, TEnterFunc EnterFunc, TExitFunc ExitFunc)
	{
		return ParseRooted(
			InRootNode, 
			[&InProfile](TProfile::TNodeType InNodeIndex)
			{
				return InProfile.GetNumChildren(InNodeIndex);
			},
			[&InProfile](TProfile::TNodeType InNodeIndex, TProfile::TNodeType InChildNodeIndex)
			{
				return InProfile.GetChild(InNodeIndex, InChildNodeIndex);
			},
			EnterFunc,
			ExitFunc);
	}

	/**
	 * Parses a list of nodes and edges from a a Directed Acyclic Graph (DAG).
	 * This function is designed for a more general graph structure where a node can have multiple parents, which is common 
	 * for modeling dependencies, etc. 
	 * A valid DAG tree must satisfy three conditions:
	 *   1. Edges are directed.
	 *   2. It contains no cycles.
	 *   3. A node can have any number of incoming edges (0, 1, or more).
	 * Already visited nodes will be skipped.
	 */
	template <typename TNodeType, class TGetNumChildren, class TGetChild, class TEnterFunc, class TExitFunc>
	static EParseAction ParseDirectedAcyclic(TNodeType InRootNode, TGetNumChildren GetNumChildren, TGetChild GetChild, TEnterFunc EnterFunc, TExitFunc ExitFunc)
	{
		TSet<TNodeType> VisitedNodes;
		return ParseRooted<TNodeType>(
            InRootNode, GetNumChildren, GetChild,
			[&EnterFunc, &VisitedNodes](TNodeType Node)
			{
				bool bWasAlreadyInSet;
				VisitedNodes.Add(Node, &bWasAlreadyInSet);

				if (!bWasAlreadyInSet)
				{
					EnterFunc(Node);
					return EParseAction::Continue;
				}

				return EParseAction::Skip;				
			}, ExitFunc);
	}

	template <typename TProfile, class TEnterFunc, class TExitFunc>
	static EParseAction ParseDirectedAcyclic(TProfile::TNodeType InRootNode, TProfile& InProfile, TEnterFunc EnterFunc, TExitFunc ExitFunc)
	{
		return ParseDirectedAcyclic(
			InRootNode, 
			[&InProfile](TProfile::TNodeType InNodeIndex)
			{
				return InProfile.GetNumChildren(InNodeIndex);
			},
			[&InProfile](TProfile::TNodeType InNodeIndex, TProfile::TNodeType InChildNodeIndex)
			{
				return InProfile.GetChild(InNodeIndex, InChildNodeIndex);
			},
			EnterFunc,
			ExitFunc);
	}
};

}