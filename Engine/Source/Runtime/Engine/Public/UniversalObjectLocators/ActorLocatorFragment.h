// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorFwd.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "ActorLocatorFragment.generated.h"


/**
 * 32 Bytes (40 in-editor).
 */
USTRUCT()
struct FActorLocatorFragmentResolveParameter
{
	GENERATED_BODY()

	ENGINE_API static UE::UniversalObjectLocator::TParameterTypeHandle<FActorLocatorFragmentResolveParameter> ParameterType;

	UPROPERTY()
	TObjectPtr<UWorld> StreamingWorld;

	UPROPERTY()
	FActorContainerID ContainerID;

	UPROPERTY()
	FTopLevelAssetPath SourceAssetPath;
};

/**
 * 32 Bytes (40 in-editor).
 */
USTRUCT()
struct FActorLocatorFragment
{
	GENERATED_BODY()

	UPROPERTY()
	FSoftObjectPath Path;

	ENGINE_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FActorLocatorFragment> FragmentType;

	friend uint32 GetTypeHash(const FActorLocatorFragment& A)
	{
		return GetTypeHash(A.Path);
	}

	friend bool operator==(const FActorLocatorFragment& A, const FActorLocatorFragment& B)
	{
		return A.Path == B.Path;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);

	void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);

	static uint32 ComputePriority(const UObject* Object, const UObject* Context);
};

template<>
struct TStructOpsTypeTraits<FActorLocatorFragment> : public TStructOpsTypeTraitsBase2<FActorLocatorFragment>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};