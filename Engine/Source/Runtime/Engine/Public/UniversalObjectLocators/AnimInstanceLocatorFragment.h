// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorFwd.h"
#include "AnimInstanceLocatorFragment.generated.h"

UENUM()
enum class EAnimInstanceLocatorFragmentType
{
	AnimInstance,
	PostProcessAnimInstance
};

/**
 * 32 Bytes (40 in-editor).
 */
USTRUCT()
struct FAnimInstanceLocatorFragment
{
	GENERATED_BODY()

	UPROPERTY()
	EAnimInstanceLocatorFragmentType Type;

	FAnimInstanceLocatorFragment()
		: Type(EAnimInstanceLocatorFragmentType::AnimInstance)
	{
	}

	explicit FAnimInstanceLocatorFragment(EAnimInstanceLocatorFragmentType InType)
		: Type(InType)
	{
	}

	ENGINE_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimInstanceLocatorFragment> FragmentType;

	friend uint32 GetTypeHash(const FAnimInstanceLocatorFragment& A)
	{
		return GetTypeHash(A.Type);
	}

	friend bool operator==(const FAnimInstanceLocatorFragment& A, const FAnimInstanceLocatorFragment& B)
	{
		return A.Type == B.Type;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);

	void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);

	static uint32 ComputePriority(const UObject* Object, const UObject* Context);
};

template<>
struct TStructOpsTypeTraits<FAnimInstanceLocatorFragment> : public TStructOpsTypeTraitsBase2<FAnimInstanceLocatorFragment>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};