// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorFwd.h"
#include "AssetLocatorFragment.generated.h"

struct FAssetData;

/**
 * 32 Bytes (40 in-editor).
 */
USTRUCT()
struct FAssetLocatorFragment
{
	GENERATED_BODY()

	FAssetLocatorFragment() = default;

	ENGINE_API explicit FAssetLocatorFragment(const FAssetData& InAssetData);

	ENGINE_API explicit FAssetLocatorFragment(const FTopLevelAssetPath& InPath);

	UPROPERTY()
	FTopLevelAssetPath Path;

	friend uint32 GetTypeHash(const FAssetLocatorFragment& A)
	{
		return GetTypeHash(A.Path);
	}

	friend bool operator==(const FAssetLocatorFragment& A, const FAssetLocatorFragment& B)
	{
		return A.Path == B.Path;
	}

	ENGINE_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FAssetLocatorFragment> FragmentType;

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);

	void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);

	static uint32 ComputePriority(const UObject* Object, const UObject* Context);
};

template<>
struct TStructOpsTypeTraits<FAssetLocatorFragment> : public TStructOpsTypeTraitsBase2<FAssetLocatorFragment>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};
