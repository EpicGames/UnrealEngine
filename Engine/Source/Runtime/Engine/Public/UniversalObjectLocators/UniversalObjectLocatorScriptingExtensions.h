// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/LatentActionManager.h"

#include "UniversalObjectLocatorScriptingExtensions.generated.h"

struct FUniversalObjectLocator;

/**
 * Function library containing methods that should be hoisted onto FUniversalObjectLocators for scripting
 */
UCLASS(MinimalAPI)
class UUniversalObjectLocatorScriptingExtensions : public UBlueprintFunctionLibrary
{
public:
	GENERATED_BODY()

	/**
	 * Construct a new universal object locator
	 */
	UFUNCTION(BlueprintPure, Category="Universal Object Locators")
	static FUniversalObjectLocator MakeUniversalObjectLocator(UObject* Object, UObject* Context);

	/**
	 * Construct a new universal object locator from a string
	 */
	UFUNCTION(BlueprintPure, Category="Universal Object Locators")
	static FUniversalObjectLocator UniversalObjectLocatorFromString(const FString& InString);

	/**
	 * Check whether the specified locator is empty; not equivalent to Resolve() != None.
	 * An empty locator will never resolve to a valid object.
	 */
	UFUNCTION(BlueprintPure, Category="Universal Object Locators", meta=(ScriptMethod))
	static bool IsEmpty(const FUniversalObjectLocator& Locator);

	/**
	 * Convert the specified locator to its string representation
	 */
	UFUNCTION(BlueprintPure, Category="Universal Object Locators", meta=(ScriptMethod))
	static FString ToString(const FUniversalObjectLocator& Locator);

	/**
	 * Attempt to resolve the object locator by finding the object. If it is not currently loaded or created, 
	 * 
	 * @param Context    (Optional) Context object to use for resolving the object. This should usually be the object that owns or created the locator.
	 * @return The resolve object pointer, or null if it was not found.
	 */
	UFUNCTION(BlueprintPure, Category="Universal Object Locators", meta=(ScriptMethod))
	static UObject* SyncFind(const FUniversalObjectLocator& Locator, UObject* Context = nullptr);

	/**
	 * Attempt to resolve the object locator by finding or loading the object.
	 * 
	 * @param Context    (Optional) Context object to use for resolving the object. This should usually be the object that owns or created the locator.
	 * @return The resolve object pointer, or null if it was not found.
	 */
	UFUNCTION(BlueprintPure, Category="Universal Object Locators", meta=(ScriptMethod))
	static UObject* SyncLoad(const FUniversalObjectLocator& Locator, UObject* Context = nullptr);

	/**
	 * Attempt to resolve the object locator by unloading the object if possible.
	 * 
	 * @param Context    (Optional) Context object to use for resolving the object. This should usually be the object that owns or created the locator.
	 * @return The resolve object pointer, or null if it was not found.
	 */
	UFUNCTION(BlueprintCallable, Category="Universal Object Locators", meta=(ScriptMethod))
	static void SyncUnload(const FUniversalObjectLocator& Locator, UObject* Context = nullptr);
};