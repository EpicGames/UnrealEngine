// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SubclassOf.h"

class AActor;
class UWorld;

namespace UE::UniversalObjectLocator
{
	// Spawns a new actor of the given actor class and name. Optionally you can provide a template actor.
	// Takes care of any special setup that may be required for the locator case, for example for editor preview actors.
	// Intended to be called from custom Locator code during a Load operation.
	ENGINE_API AActor* SpawnActorForLocator(UWorld* World, TSubclassOf<AActor> ActorClass, FName ActorName, AActor* TemplateActor = nullptr);

	// Destroys an Actor that was created for a locator and is no longer needed.
	// Intended to be called from custom Locator code during an Unload operation.
	ENGINE_API void DestroyActorForLocator(AActor* Actor);

} // namespace UE::UniversalObjectLocator