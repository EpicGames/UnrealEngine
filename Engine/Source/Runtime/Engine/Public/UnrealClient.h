// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnrealClient.h: Interface definition for platform specific client code.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Concepts/PointerConvertibleTo.h"
#include "InputKeyEventArgs.h"
#include "ColorManagement/ColorSpace.h"
#include "Engine/EngineBaseTypes.h"
#include "UObject/GCObject.h"
#include "RenderResource.h"
#include "HitProxies.h"
#include "RenderGraphDefinitions.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "GenericPlatform/GenericWindow.h"
#include "DynamicRenderScaling.h"
#include "RHITypes.h"
#include "Templates/PointerVariants.h"

class FCanvas;
class FRDGBuilder;
class FSceneView;
class FViewport;
class FViewportClient;
class UModel;
class UWorld;

/**
 * A render target.
 */
class FRenderTarget
{
public:

	/**
	 * Default constructor
	 */
	ENGINE_API FRenderTarget();

	/**
	 * Destructor
	 */
	ENGINE_API virtual ~FRenderTarget();

	/**
	* Accessor for the surface RHI when setting this render target
	* @return render target surface RHI resource
	*/
	ENGINE_API virtual const FTextureRHIRef& GetRenderTargetTexture() const;
	ENGINE_API virtual FUnorderedAccessViewRHIRef GetRenderTargetUAV() const;

	/**
	 * Returns a valid RDG texture for this render target.
	 */
	ENGINE_API virtual FRDGTextureRef GetRenderTargetTexture(FRDGBuilder& GraphBuilder) const;

	/**
	* Accessor for the surface RHI to use when reading the reading target (may differ from GetRenderTargetTexture() for some implementations like cubemaps)
	* @return surface RHI resource
	*/
	ENGINE_API virtual const FTextureRHIRef& GetShaderResourceTexture() const;

	// Properties.
	virtual FIntPoint GetSizeXY() const = 0;

	/**
	* @return display gamma expected for rendering to this render target
	*/
	ENGINE_API virtual float GetDisplayGamma() const;

	// return global Engine default gamma (GetDisplayGamma returns this if not overriden)
	static float GetEngineDisplayGamma();

	virtual EDisplayColorGamut GetDisplayColorGamut() const { return EDisplayColorGamut::sRGB_D65; }
	virtual EDisplayOutputFormat GetDisplayOutputFormat() const { return EDisplayOutputFormat::SDR_sRGB; }
	virtual bool GetSceneHDREnabled() const { return false; }
	virtual float GetMinimumLuminanceInNits() const { return 0.f; }
	virtual float GetMaximumLuminanceInNits() const { return 100.f; }
	virtual float GetMaximumFullFrameLuminanceInNits() const { return 100.f; }
	virtual float GetHDRPaperWhiteInNits() const { return 203.f; }
	virtual const UE::Color::FColorSpace& GetLimitingColorSpace() const { return UE::Color::FColorSpace::GetSRGB(); }

	/**
	* Handles freezing/unfreezing of rendering
	*/
	virtual void ProcessToggleFreezeCommand() {};

	/**
	 * Returns if there is a command to toggle freezerendering
	 */
	virtual bool HasToggleFreezeCommand() { return false; };

	/**
	* Reads the render target's displayed pixels into a preallocated color buffer.
	* @param OutImageData - RGBA8 values will be stored in this buffer
	* @param InFlags - Additional information about how to to read the surface data (cube face, slice index, etc.)
	* @param InSrcRect - InSrcRect not specified means the whole rect
	* @return True if the read succeeded.
	*
	* This will convert whatever the pixel format is to FColor
	* Prefer using FImageUtils::GetRenderTargetImage rather than calling this directly.
	*
	* The default value for InFlags specifies RCM_UNorm which will cause values to be scaled into [0,1] ; use RCM_MinMax to retrieve values without change.
	*
	* If the RenderTarget surface is float linear, it will converted to SRGB FColor, if InFlags.bLinearToGamma is set (which is on by default).
	* If the RenderTarget surface is U8, then the SRGB/not state is unchanged, the U8 values are retrieved unchanged in either Linear or SRGB.
	* Gamma is handled correctly automatically by FImageUtils::GetRenderTargetImage
	*/
	ENGINE_API virtual bool ReadPixels(TArray<FColor>& OutImageData, FReadSurfaceDataFlags InFlags = FReadSurfaceDataFlags(RCM_UNorm, CubeFace_MAX), FIntRect InSrcRect = FIntRect(0, 0, 0, 0));

	/**
	* Reads the render target's displayed pixels into a preallocated color buffer.
	* @param OutImageBytes - RGBA8 values will be stored in this buffer.  Buffer must be preallocated with the correct size!
	* @param InFlags - Additional information about how to to read the surface data (cube face, slice index, etc.)
	* @param InSrcRect - InSrcRect not specified means the whole rect
	* @return True if the read succeeded.
	*
	* Ptr variant of this API just does an extra memcpy; prefer the TArray variant.
	* Prefer using FImageUtils::GetRenderTargetImage rather than calling this directly.
	*/
	ENGINE_API bool ReadPixelsPtr(FColor* OutImageBytes, FReadSurfaceDataFlags InFlags = FReadSurfaceDataFlags(RCM_UNorm, CubeFace_MAX), FIntRect InSrcRect = FIntRect(0, 0, 0, 0));

	/**
	 * Reads the render target's displayed pixels into the given color buffer.
	 * @param OutImageData - RGBA16F values will be stored in this buffer
	 * @param CubeFace - optional cube face for when reading from a cube render target
	 * @return True if the read succeeded.
	 */
	UE_DEPRECATED(5.4, "Use the other ReadFloat16Pixels variant (ECubeFace can be set in FReadSurfaceDataFlags)")
	ENGINE_API bool ReadFloat16Pixels(TArray<FFloat16Color>& OutImageData, ECubeFace CubeFace);

	/**
	 * Reads the render target's displayed pixels into the given color buffer.
	 * @param OutImageData - RGBA16F values will be stored in this buffer
	 * @param InFlags - Additional information about how to to read the surface data (cube face, slice index, etc.)
	 * @param InSrcRect - InSrcRect not specified means the whole rect
	 * @return True if the read succeeded.
	 *
	 * The default value for InFlags specifies RCM_UNorm which will cause values to be scaled into [0,1] ; use RCM_MinMax to retrieve values without change.
	 *
	 * Unlike other RenderTarget Read functions, this only works if surface is PF_FloatRGBA exactly ; it does not convert.
	 * Prefer using FImageUtils::GetRenderTargetImage rather than calling this directly.
	 */
	ENGINE_API virtual bool ReadFloat16Pixels(TArray<FFloat16Color>& OutImageData, FReadSurfaceDataFlags InFlags = FReadSurfaceDataFlags(RCM_UNorm, CubeFace_MAX), FIntRect InSrcRect = FIntRect(0, 0, 0, 0));

	/**
	 * Reads the render target's displayed pixels into the given color buffer.
	 * @param OutImageData - Linear color array to store the value
	 * @param InFlags - Additional information about how to to read the surface data (cube face, slice index, etc.)
	 * @param InSrcRect - InSrcRect not specified means the whole rect
	 * @return True if the read succeeded.
	 *
	 * The default value for InFlags specifies RCM_UNorm which will cause values to be scaled into [0,1] ; use RCM_MinMax to retrieve values without change.
	 *
	 * This will convert whatever the pixel format is to FLinearColor (if supported).
	 * Prefer using FImageUtils::GetRenderTargetImage rather than calling this directly.
	 */
	ENGINE_API virtual bool ReadLinearColorPixels(TArray<FLinearColor>& OutImageData, FReadSurfaceDataFlags InFlags = FReadSurfaceDataFlags(RCM_MinMax, CubeFace_MAX), FIntRect InSrcRect = FIntRect(0, 0, 0, 0));

	/**
	 * Reads the render target's displayed pixels into the given color buffer.
	 * @param OutImageBytes - Linear color array will be stored in this buffer.  Buffer must be preallocated with the correct size!
	 * @param InFlags - Additional information about how to to read the surface data (cube face, slice index, etc.)
	 * @param InSrcRect - InSrcRect not specified means the whole rect
	 * @return True if the read succeeded.
	 *
	 * Ptr variant of this API just does an extra memcpy; prefer the TArray variant.
	 * Prefer using FImageUtils::GetRenderTargetImage rather than calling this directly.
	 */
	ENGINE_API bool ReadLinearColorPixelsPtr(FLinearColor* OutImageBytes, FReadSurfaceDataFlags InFlags = FReadSurfaceDataFlags(RCM_MinMax, CubeFace_MAX), FIntRect InSrcRect = FIntRect(0, 0, 0, 0));

	/**
	 * Returns the GPU nodes on which to render this render target.
	 **/
	virtual FRHIGPUMask GetGPUMask(FRHICommandListImmediate& RHICmdList) const { return FRHIGPUMask::GPU0(); }

protected:

	FTextureRHIRef RenderTargetTextureRHI;
};

using FViewportFrame UE_DEPRECATED(5.8, "FViewportFrame is deprecated. Use FSceneViewport instead.") = class FSceneViewport;

/**
* The maximum size that the hit proxy kernel is allowed to be set to
*/
#define MAX_HITPROXYSIZE 200

DECLARE_MULTICAST_DELEGATE(FOnScreenshotRequestProcessed);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnScreenshotCaptured, int32 /*Width*/, int32 /*Height*/, const TArray<FColor>& /*Colors*/);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnHDRScreenshotCaptured, int32 /*Width*/, int32 /*Height*/, const TArray<FLinearColor>& /*Colors*/);


struct FScreenshotRequest
{
	/**
	 * Requests a new screenshot.  Screenshot can be read from memory by subscribing
	 * to the ViewPort's OnScreenshotCaptured delegate.
	 *
	 * @param bInShowUI				Whether or not to show Slate UI
	 * @param bInRestrictToGameViewport If true then the capture will be restricted to the game viewport, so will not incldue the entire editor in PIE for example
	 */
	static ENGINE_API void RequestScreenshot(bool bInShowUI, const bool bInRestrictToGameViewport = false);

	/**
	 * Requests a new screenshot with a specific filename
	 *
	 * @param InFilename			The filename to use
	 * @param bInShowUI				Whether or not to show Slate UI
	 * @param bAddFilenameSuffix	Whether an auto-generated unique suffix should be added to the supplied filename
	 * @param bHdrScreenshot		If true, the screenshot is saved as .exr rather than .png
	 * @param InViewRect			If non-empty, within the viewport extent, and bInShowUI=false, only this view rect will be included in the screenshot
	 * @param bInRestrictToGameViewport If true then the capture will be restricted to the game viewport, so will not incldue the entire editor in PIE for example
	 */
	static ENGINE_API void RequestScreenshot(const FString& InFilename, bool bInShowUI, bool bAddFilenameSuffix, bool bHdrScreenshot = false, const FIntRect& InViewRect = FIntRect(), const bool bInRestrictToGameViewport = false);

	/**
	 * Resets a screenshot request
	 */
	static ENGINE_API void Reset();

	/**
	 * @return The filename of the next screenshot
	 */
	static const FString& GetFilename() { return Filename; }

	/**
	 * @return True if a screenshot is requested
	 */
	static bool IsScreenshotRequested() { return bIsScreenshotRequested; }

	/**
	 * @return True if UI should be shown in the screenshot
	 */
	static bool ShouldShowUI() { return bShowUI; }

	/**
	 * @return True if capture should be restricted to the game viewport
	 */
	static bool ShouldRestrictToGameViewport() { return bRestrictToGameViewport; }

	/**
	 * @return The view rect for the next screenshot
	 */
	static FIntRect GetViewRect() { return ViewRect; }

	/**
	 * Creates a new screenshot filename from the passed in filename template
	 */
	static ENGINE_API void CreateViewportScreenShotFilename( FString& InOutFilename );

	/**
	 * Access a temporary color array for storing the pixel colors for the highres screenshot mask
	 */
	static ENGINE_API TArray<FColor>* GetHighresScreenshotMaskColorArray();

	/**
	 * Extents of array returned by function above
	 */
	static ENGINE_API FIntPoint& GetHighresScreenshotMaskExtents();

	static FOnScreenshotRequestProcessed& OnScreenshotRequestProcessed()
	{
		return ScreenshotProcessedDelegate;
	}

	static FOnScreenshotCaptured& OnScreenshotCaptured()
	{
		return ScreenshotCapturedDelegate;
	}

	static FOnHDRScreenshotCaptured& OnHDRScreenshotCaptured()
	{
		return HDRScreenshotCapturedDelegate;
	}

private:
	static ENGINE_API FOnScreenshotRequestProcessed ScreenshotProcessedDelegate;
	static ENGINE_API FOnScreenshotCaptured ScreenshotCapturedDelegate;
	static ENGINE_API FOnHDRScreenshotCaptured HDRScreenshotCapturedDelegate;
	static ENGINE_API bool bIsScreenshotRequested;
	static ENGINE_API FString NextScreenshotName;
	static ENGINE_API FString Filename;
	static ENGINE_API bool bShowUI;
	static ENGINE_API bool bRestrictToGameViewport;
	static ENGINE_API TArray<FColor> HighresScreenshotMaskColorArray;
	static ENGINE_API FIntPoint HighresScreenshotMaskExtents;
	static ENGINE_API FIntRect ViewRect;
};

// @param bAutoType true: automatically choose GB/MB/KB/... false: always use MB for easier comparisons
ENGINE_API FString GetMemoryString( const double Value, const bool bAutoType = true );

/** Data needed to display perframe stat tracking when STAT UNIT is enabled */
struct FStatUnitData
{
	/** Unit frame times filtered with a simple running average */
	float RenderThreadTime;
	float RenderThreadTimeCriticalPath;
	float GameThreadTime;
	float GameThreadTimeCriticalPath;
	float GPUFrameTime[MAX_NUM_GPUS];
	float GPUClockFraction[MAX_NUM_GPUS];
	float GPUUsageFraction[MAX_NUM_GPUS];
	uint64 GPUMemoryUsage[MAX_NUM_GPUS];
	float GPUExternalUsageFraction[MAX_NUM_GPUS];
	uint64 GPUExternalMemoryUsage[MAX_NUM_GPUS];
	float FrameTime;
	float RHITTime;
	float InputLatencyTime;

	/** Raw equivalents of the above variables */
	float RawRenderThreadTime;
	float RawRenderThreadTimeCriticalPath;
	float RawGameThreadTime;
	float RawGameThreadTimeCriticalPath;
	float RawGPUFrameTime[MAX_NUM_GPUS];
	float RawGPUClockFraction[MAX_NUM_GPUS];
	float RawGPUUsageFraction[MAX_NUM_GPUS];
	uint64 RawGPUMemoryUsage[MAX_NUM_GPUS];
	float RawGPUExternalUsageFraction[MAX_NUM_GPUS];
	uint64 RawGPUExternalMemoryUsage[MAX_NUM_GPUS];
	float RawFrameTime;
	float RawRHITTime;
	float RawInputLatencyTime;

	/** Time that has transpired since the last draw call */
	double LastTime;

#if !UE_BUILD_SHIPPING
	static const int32 NumberOfSamples = 200;

	int32 CurrentIndex;
	TArray<float> RenderThreadTimes;
	TArray<float> GameThreadTimes;
	TArray<float> GPUFrameTimes[MAX_NUM_GPUS];
	TArray<float> FrameTimes;
	TArray<float> RHITTimes;
	TArray<float> InputLatencyTimes;
	DynamicRenderScaling::TMap<TArray<float>> ResolutionFractions;
#endif //!UE_BUILD_SHIPPING

	FStatUnitData()
		: RenderThreadTime(0.0f)
		, RenderThreadTimeCriticalPath(0.0f)
		, GameThreadTime(0.0f)
		, GameThreadTimeCriticalPath(0.0f)
		, GPUFrameTime{ 0.0f }
		, GPUClockFraction{ 0.0f }
		, GPUUsageFraction{ 0.0f }
		, GPUExternalUsageFraction{ 0.0f }
		, FrameTime(0.0f)
		, RHITTime(0.0f)
		, InputLatencyTime(0.0f)
		, RawRenderThreadTime(0.0f)
		, RawRenderThreadTimeCriticalPath(0.0f)
		, RawGameThreadTime(0.0f)
		, RawGameThreadTimeCriticalPath(0.0f)
		, RawGPUFrameTime{ 0.0f }
		, RawGPUClockFraction{ 0.0f }
		, RawGPUUsageFraction{ 0.0f }
		, RawGPUExternalUsageFraction{ 0.0f }
		, RawFrameTime(0.0f)
		, RawRHITTime(0.0f)
		, RawInputLatencyTime(0.0f)
		, LastTime(0.0)
	{
#if !UE_BUILD_SHIPPING
		CurrentIndex = 0;
		RenderThreadTimes.AddZeroed(NumberOfSamples);
		GameThreadTimes.AddZeroed(NumberOfSamples);
		for (auto& GPUFrameTimesArray : GPUFrameTimes)
		{
			GPUFrameTimesArray.AddZeroed(NumberOfSamples);
		}
		FrameTimes.AddZeroed(NumberOfSamples);
		RHITTimes.AddZeroed(NumberOfSamples);
		InputLatencyTimes.AddZeroed(NumberOfSamples);
		for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
		{
			const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
			ResolutionFractions[Budget].Reserve(NumberOfSamples);
			for (int32 i = 0; i < NumberOfSamples; i++)
			{
				ResolutionFractions[Budget].Add(-1.0f);
			}
		}
#endif //!UE_BUILD_SHIPPING
	}

	/** Render function to display the stat */
	int32 DrawStat(FViewport* InViewport, FCanvas* InCanvas, int32 InX, int32 InY);
};

struct FStatThermalsData
{
	FStatThermalsData() = default;

	/** Render function to display the stat */
	int32 DrawStat(FViewport* InViewport, FCanvas* InCanvas, int32 InX, int32 InY);
};

/** Data needed to display perframe stat tracking when STAT HITCHES is enabled */
struct FStatHitchesData
{
	double LastTime;

	static const int32 NumHitches = 20;
	TArray<float> Hitches;
	TArray<double> When;
	int32 OverwriteIndex;
	int32 Count;

	FStatHitchesData()
		: LastTime(0.0)
		, OverwriteIndex(0)
		, Count(0)
	{
		Hitches.AddZeroed(NumHitches);
		When.AddZeroed(NumHitches);
	}

	/** Render function to display the stat */
	int32 DrawStat(FViewport* InViewport, FCanvas* InCanvas, int32 InX, int32 InY);
};

class UScriptViewportClient;

/**
 * Encapsulates the I/O of a viewport.
 * The viewport display is implemented using the platform independent RHI.
 */
class FViewport : public FRenderTarget, protected FRenderResource
{
public:
	/** delegate type for viewport resize events ( Params: FViewport* Viewport, uint32 ) */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnViewportResized, FViewport*, uint32);
	/** Send when a viewport is resized */
	ENGINE_API static FOnViewportResized ViewportResizedEvent;

	// Constructor.
	ENGINE_API FViewport();

	UE_DEPRECATED(5.8, "Use FViewport() + SetViewportClient or static Create(TStrongPtrVariant<FViewportClient>) instead")
	ENGINE_API FViewport(FViewportClient* const InClient);

	// Destructor
	ENGINE_API virtual ~FViewport();

	//~ Begin FViewport Interface.
	virtual void* GetWindow() = 0;
	virtual void MoveWindow(int32 NewPosX, int32 NewPosY, int32 NewSizeX, int32 NewSizeY) = 0;

	virtual void Destroy() = 0;

	// New MouseCapture/MouseLock API
	virtual bool HasMouseCapture() const				{ return false; }
	virtual bool HasFocus() const					{ return true; }
	virtual bool IsForegroundWindow() const			{ return true; }
	virtual void CaptureMouse( bool bCapture )		{ }
	virtual void LockMouseToViewport( bool bLock )	{ }
	virtual void ShowCursor( bool bVisible)			{ }
	virtual bool UpdateMouseCursor(bool bSetCursor)	{ return true; }

	virtual void ShowSoftwareCursor( bool bVisible )		{ }
	virtual void SetSoftwareCursorPosition( FVector2D Position ) { }
	virtual bool IsSoftwareCursorVisible() const { return false; }

	/**
	 * Returns true if the mouse cursor is currently visible
	 *
	 * @return True if the mouse cursor is currently visible, otherwise false.
	 */
	virtual bool IsCursorVisible() const { return true; }

	virtual bool SetUserFocus(bool bFocus) = 0;
	virtual bool KeyState(FKey Key) const = 0;
	virtual int32 GetMouseX() const = 0;
	virtual int32 GetMouseY() const = 0;
	virtual void GetMousePos(FIntPoint& MousePosition, const bool bLocalPosition = true) = 0;
	virtual void GetUnboundedMousePos(FIntPoint& OutUnboundedMousePosition, const bool bLocalPosition = true) {}
	virtual float GetTabletPressure() { return 0.f; }
	virtual bool IsPenActive() { return false; }
	virtual void SetMouse(int32 x, int32 y) = 0;
	virtual bool IsFullscreen()	const { return WindowMode == EWindowMode::Fullscreen || WindowMode == EWindowMode::WindowedFullscreen; }
	virtual int32 GetCurrentMonitorIndex() const { return -1; }
	virtual bool IsExclusiveFullscreenSupported() const { return true; }
	virtual bool IsExclusiveFullscreen() const { return WindowMode == EWindowMode::Fullscreen; }
	virtual EWindowMode::Type GetWindowMode()	const { return WindowMode; }
	virtual void ProcessInput( float DeltaTime ) = 0;

	/**
	 * Transforms a virtual desktop pixel (the origin is in the primary screen's top left corner) to the local space of this viewport
	 *
	 * @param VirtualDesktopPointPx   Coordinate on the virtual desktop in pixel units. Desktop is considered virtual because multiple monitors are supported.
	 *
	 * @return The transformed pixel. It is normalized to the range [0, 1]
	 */
	virtual FVector2D VirtualDesktopPixelToViewport(FIntPoint VirtualDesktopPointPx) const = 0;

	/**
	 * Transforms a coordinate in the local space of this viewport into a virtual desktop pixel.
	 *
	 * @param ViewportCoordiante    Normalized coordniate in the rate [0..1]; (0,0) is upper left and (1,1) is lower right.
	 *
	 * @return the transformed coordinate. It is in virtual desktop pixels.
	 */
	virtual FIntPoint ViewportToVirtualDesktopPixel(FVector2D ViewportCoordinate) const = 0;

	/**
	 * @return A canvas that can be used while this viewport is being drawn to render debug elements on top of everything else
	 */
	virtual FCanvas* GetDebugCanvas() { return NULL; };

	/**
	 * Indicate that the viewport should be block for vsync.
	 */
	virtual void SetRequiresVsync(bool bShouldVsync) {}

	/**
	 * Indicate that this frame is a camera cut.
	*/
	virtual void SetCameraCut(bool bInCameraCut) {}


	/**
	 * Sets PreCapture coordinates from the current position of the slate cursor.
	 */
	virtual void SetPreCaptureMousePosFromSlateCursor() {}

	/**
	 * Starts a new rendering frame. Called from the game thread thread.
	 * @param bShouldPresent Whether the frame will be presented to the screen
	 */
	virtual void EnqueueBeginRenderFrame(const bool bShouldPresent) {}

	/**
	 *	Ends a rendering frame. Called from the game thread.
	 *	@param bPresent		Whether the frame should be presented to the screen
	 */
	virtual void EnqueueEndRenderFrame(const bool bLockToVsync, const bool bShouldPresent) {}

	/**
	 * Returns the GPU nodes on which to render this viewport.
	 **/
	ENGINE_API virtual FRHIGPUMask GetGPUMask(FRHICommandListImmediate& RHICmdList) const override;

	/**
	 * @return whether or not this Controller has a keyboard available to be used
	 **/
	virtual bool IsKeyboardAvailable( int32 ControllerID ) const { return true; }

	/**
	 * @return whether or not this Controller has a mouse available to be used
	 **/
	virtual bool IsMouseAvailable( int32 ControllerID ) const { return true; }


	/**
	 * @return aspect ratio that this viewport should be rendered at
	 */
	virtual float GetDesiredAspectRatio() const
	{
		FIntPoint Size = GetSizeXY();
		return (float)Size.X / (float)Size.Y;
	}

	/**
	 * Invalidates the viewport's displayed pixels.
	 */
	virtual void InvalidateDisplay() = 0;

	/**
	 * Updates the viewport's displayed pixels with the results of calling ViewportClient->Draw.
	 *
	 * @param	bShouldPresent	Whether we want this frame to be presented
	 */
	ENGINE_API void Draw( bool bShouldPresent = true );

	/**
	 * Invalidates the viewport's cached hit proxies at the end of the frame.
	 */
	ENGINE_API virtual void DeferInvalidateHitProxy();

	/**
	 * Invalidates cached hit proxies
	 */
	ENGINE_API void InvalidateHitProxy();

	/**
	 * Invalidates cached hit proxies and the display.
	 */
	ENGINE_API void Invalidate();

	ENGINE_API const TArray<FColor>& GetRawHitProxyData(FIntRect InRect);

	/**
	 * Copies the hit proxies from an area of the screen into a buffer.
	 * InRect must be entirely within the viewport's client area.
	 * If the hit proxies are not cached, this will call ViewportClient->Draw with a hit-testing canvas.
	 */
	ENGINE_API void GetHitProxyMap(FIntRect InRect, TArray<HHitProxy*>& OutMap);

	/**
	 * Returns the dominant hit proxy at a given point.  If X,Y are outside the client area of the viewport, returns NULL.
	 * Caution is required as calling Invalidate after this will free the returned HHitProxy.
	 */
	ENGINE_API HHitProxy* GetHitProxy(int32 X,int32 Y);

	/**
	 * Returns all actors and models found in the hit proxy within a specified region.
	 * InRect must be entirely within the viewport's client area.
	 * If the hit proxies are not cached, this will call ViewportClient->Draw with a hit-testing canvas.
	 */
	ENGINE_API void GetActorsAndModelsInHitProxy(FIntRect InRect, TSet<AActor*>& OutActors, TSet<UModel*>& OutModels);

	/**
	 * Returns the dominant element handle at a given point.  If X,Y are outside the client area of the viewport, returns an invalid handle.
	 */
	ENGINE_API FTypedElementHandle GetElementHandleAtPoint(int32 X, int32 Y);

	/**
	 * Returns all element handles found within a specified region.
	 * InRect must be entirely within the viewport's client area.
	 * If the hit proxies are not cached, this will call ViewportClient->Draw with a hit-testing canvas.
	 */
	ENGINE_API void GetElementHandlesInRect(FIntRect InRect, FTypedElementListRef OutElementHandles);

	/**
	 * Enumerate all valid hit proxies found within a specified region.
	 * InRect must be entirely within the viewport's client area.
	 * If the hit proxies are not cached, this will call ViewportClient->Draw with a hit-testing canvas.
	 */
	ENGINE_API void EnumerateHitProxiesInRect(FIntRect InRect, TFunctionRef<bool(HHitProxy*)> InCallback);
	
	/**
	 * Retrieves the interface to the viewport's frame, if it has one.
	 * @return The viewport's frame interface.
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "FViewport::GetViewportFrame() and FViewportFrame are deprecated. Use FSceneViewport instead.")
	virtual FViewportFrame* GetViewportFrame()
	{
		return nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Calculates the view inside the viewport when the aspect ratio is locked.
	 * Used for creating cinematic bars.
	 * @param Aspect [in] ratio to lock to
	 * @param ViewRect [in] unconstrained view rectangle
	 * @return	constrained view rectangle
	 */
	ENGINE_API FIntRect CalculateViewExtents(float AspectRatio, const FIntRect& ViewRect);
	
	/**
	* Calculates the view inside the viewport when the aspect ratio is locked.
	* Used for creating cinematic bars.
	* @param AspectRatio [in] ratio to lock to
	* @param DesiredAspectRatio [in] the aspect ratio of the viewport
	* @param ViewRect [in] unconstrained view rectangle
	* @param DestSize [in] the size of the the viewport
	* @return constrained view rectangle
	*/
	ENGINE_API static FIntRect CalculateViewExtents(float AspectRatio, float DesiredAspectRatio, const FIntRect& ViewRect, const FIntPoint& DestSize);

	/**
	 *	Sets the client this viewport is associated with.  AddAssociation/RemoveAssociation will be called on the new/old clients.  nullptr to clear.
	 */
	ENGINE_API virtual void SetViewportClient(const TStrongPtrVariant<FViewportClient>& InViewportClient);

	UE_REWRITE void SetViewportClient(TYPE_OF_NULLPTR InViewportClient)
	{
		SetViewportClient(TStrongPtrVariant<FViewportClient>(nullptr));
	}

	template<UE::CPointerConvertibleTo<FViewportClient> ClientType>
	requires (!UE::CUObject<ClientType>)
	UE_DEPRECATED(5.8, "Use SetViewportClient(const TStrongPtrVariant<FViewportClient>&) instead")
	void SetViewportClient(ClientType* InViewportClient)
	{
		Client.Reset(); // No way to set this safely. If caller is using deprecated setter, safe to assume they're still only using raw pointer getter
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ViewportClient = InViewportClient;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	//~ Begin FRenderTarget Interface.
	virtual FIntPoint GetSizeXY() const override { return FIntPoint(SizeX, SizeY); }
	FIntPoint GetInitialPositionXY() const { return FIntPoint(InitialPositionX, InitialPositionY); }

	const TStrongPtrVariant<FViewportClient>& GetClientPtr() const
	{
		return Client;
	}

	ENGINE_API FViewportClient& GetClientChecked() const;

	/** Returns a raw pointer to the client.
	 * Use when you know the Viewport will stay alive throughout the scope in which you access the returned pointer copy.
	 * Do not Viewport->GetClient()->... — either test the result, or use GetClientChecked() instead.
	 */
	FViewportClient* GetClient() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ViewportClient;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Globally enables/disables rendering
	 *
	 * @param bIsEnabled true if drawing should occur
	 * @param PresentAndStopMovieDelay Number of frames to delay before enabling bPresent in RHIEndDrawingViewport, and before stopping the movie
	 */
	ENGINE_API static void SetGameRenderingEnabled(bool bIsEnabled, int32 PresentAndStopMovieDelay=0);

	/**
	 * Returns whether rendering is globally enabled or disabled.
	 * @return	true if rendering is globally enabled, otherwise false.
	 **/
	static bool IsGameRenderingEnabled() { return bIsGameRenderingEnabled; }

	/**
	 * Handles freezing/unfreezing of rendering
	 */
	ENGINE_API virtual void ProcessToggleFreezeCommand() override;

	/**
	 * Returns if there is a command to freeze
	 */
	ENGINE_API virtual bool HasToggleFreezeCommand() override;

	/**
	 * First chance for viewports to render custom stats text
	 * @param InCanvas - Canvas for rendering
	 * @param InX - Starting X for drawing
	 * @param InY - Starting Y for drawing
	 * @return - Y for next stat drawing
	 */
	virtual int32 DrawStatsHUD (FCanvas* InCanvas, const int32 InX, const int32 InY)
	{
		return InY;
	}

	/**
	 * Sets the initial size of this viewport.  Will do nothing if the viewport has already been sized
	 *
	 * @param InitialSizeXY	The initial size of the viewport
	 */
	virtual void SetInitialSize( FIntPoint InitialSizeXY ) = 0;

	/** Returns true if the viewport is for play in editor */
	bool IsPlayInEditorViewport() const
	{
		return bIsPlayInEditorViewport;
	}

	/** Sets this viewport as a play in editor viewport */
	void SetPlayInEditorViewport( bool bInPlayInEditorViewport )
	{
		bIsPlayInEditorViewport = bInPlayInEditorViewport;
	}

	/** Returns true if this is an FSlateSceneViewport */
	bool IsSlateViewport() const { return bIsSlateViewport; }

	/** Returns true if this viewport should be rendered in HDR */
	bool IsHDRViewport() const { return bIsHDR; }

	/** Sets HDR Status of Viewport */
	void SetHDRMode( bool bHDR) { bIsHDR = bHDR; }

	/** The current version of the running instance */
	FString AppVersionString;

	/** Trigger a high res screenshot. Returns true if the screenshot can be taken, and false if it can't. The screenshot
      * can fail if the requested multiplier makes the screen too big for the GPU to cope with
  	 **/
	ENGINE_API bool TakeHighResScreenShot();

	/** Should return true, if stereo rendering is allowed in this viewport */
	virtual bool IsStereoRenderingAllowed() const { return false; }

	/** Returns dimensions of RenderTarget texture. Can be called on a game thread. */
	virtual FIntPoint GetRenderTargetTextureSizeXY() const { return GetSizeXY(); }

	/** Casts this instance to an FSceneViewport, or returns nullptr if this is not a scene viewport. */
	virtual class FSceneViewport* AsSceneViewport() { return nullptr; }

	TUniquePtr<FViewport> Snapshot();

protected:

	/** The viewport's client. */
	TStrongPtrVariant<FViewportClient> Client;

	UE_DEPRECATED(5.8, "Use Client instead")
	FViewportClient* ViewportClient;

	/**
	 * Take a high-resolution screenshot and save to disk.
	 */
	void HighResScreenshot();

	/** A map from 2D coordinates to cached hit proxies. */
	class FHitProxyMap : public FHitProxyConsumer, public FRenderTarget, public FGCObject
	{
	public:

		/** Constructor */
		FHitProxyMap();

		/** Destructor */
		ENGINE_API virtual ~FHitProxyMap();

		/** Initializes the hit proxy map with the given dimensions. */
		void Init(FRHICommandListBase& RHICmdList, uint32 NewSizeX, uint32 NewSizeY);

		UE_DEPRECATED(5.8, "RHI command list is now a required argument")
		void Init(uint32 NewSizeX,uint32 NewSizeY);

		/** Releases the hit proxy resources. */
		void Release();

		/** Invalidates the cached hit proxy map. */
		void Invalidate();

		//~ Begin FHitProxyConsumer Interface.
		virtual void AddHitProxy(HHitProxy* HitProxy) override;

		//~ Begin FRenderTarget Interface.
		virtual FIntPoint GetSizeXY() const override { return FIntPoint(SizeX, SizeY); }

		/** FGCObject interface */
		virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
		virtual FString GetReferencerName() const override;

		const FTextureRHIRef& GetHitProxyTexture(void) const		{ return RenderTargetTextureRHI; }
		const FTextureRHIRef& GetHitProxyCPUTexture(void) const		{ return HitProxyCPUTexture; }

	private:

		/** The width of the hit proxy map. */
		uint32 SizeX;

		/** The height of the hit proxy map. */
		uint32 SizeY;

		/** References to the hit proxies cached by the hit proxy map. */
		TArray<TRefCountPtr<HHitProxy> > HitProxies;

		FTextureRHIRef HitProxyCPUTexture;
	};

	/** The viewport's hit proxy map. */
	FHitProxyMap HitProxyMap;

	/** Cached hit proxy data. */
	TArray<FColor> CachedHitProxyData;

	/** The initial position of the viewport. */
	uint32 InitialPositionX = 0;

	/** The initial position of the viewport. */
	uint32 InitialPositionY = 0;

	/** The width of the viewport. */
	uint32 SizeX = 0;

	/** The height of the viewport. */
	uint32 SizeY = 0;

	/** The size of the region to check hit proxies */
	uint32 HitProxySize;

	/** What is the current window mode. */
	EWindowMode::Type WindowMode;

	/** True if the viewport client requires hit proxy storage. */
	uint32 bRequiresHitProxyStorage : 1;

	/** True if the hit proxy buffer buffer has up to date hit proxies for this viewport. */
	uint32 bHitProxiesCached : 1;

	/** If a toggle freeze request has been made */
	uint32 bHasRequestedToggleFreeze : 1;

	/** if true  this viewport is for play in editor */
	uint32 bIsPlayInEditorViewport : 1;

	/** If true this viewport is an FSlateSceneViewport */
	uint32 bIsSlateViewport : 1;

	/** If true this viewport is being displayed on a HDR monitor */
	uint32 bIsHDR : 1;

	/** true if we should draw game viewports (has no effect on Editor viewports) */
	ENGINE_API static bool bIsGameRenderingEnabled;

	/** Delay in frames to disable present (but still render scene) and stopping of a movie. This is useful to keep playing a movie while driver caches things on the first frame, which can be slow. */
	static int32 PresentAndStopMovieDelay;

	/** Triggers the taking of a high res screen shot for this viewport. */
	bool bTakeHighResScreenShot;

	//~ Begin FRenderResource Interface.
	ENGINE_API virtual void ReleaseRHI() override;
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

// Shortcuts for checking the state of both left&right variations of control keys.
extern ENGINE_API bool IsCtrlDown(FViewport* Viewport);
extern ENGINE_API bool IsShiftDown(FViewport* Viewport);
extern ENGINE_API bool IsAltDown(FViewport* Viewport);

extern ENGINE_API bool GetViewportScreenShot(FViewport* Viewport, TArray<FColor>& Bitmap, const FIntRect& ViewRect = FIntRect(), const FReadSurfaceDataFlags& ReadSurfaceFlags = FReadSurfaceDataFlags());
extern ENGINE_API bool GetViewportScreenShotHDR(FViewport* Viewport, TArray<FLinearColor>& Bitmap, const FIntRect& ViewRect = FIntRect(), const FReadSurfaceDataFlags& ReadSurfaceFlags = FReadSurfaceDataFlags(RCM_MinMax));
extern ENGINE_API bool GetHighResScreenShotInput(const TCHAR* Cmd, FOutputDevice& Ar, uint32& OutXRes, uint32& OutYRes, float& OutResMult, FIntRect& OutCaptureRegion, bool& OutShouldEnableMask, bool& OutDumpBufferVisualizationTargets, bool& OutCaptureHDR, FString& OutFilenameOverride, bool& OutUseDateTimeAsFileName);

/** Informs a world (and the streaming manager) that a view has been rendered within it during the current frame.
 * Must be called for asset streaming, and any other features that need to know "where is the scene being drawn from?"
 */
ENGINE_API void AddStreamingViewInfo(UWorld& World, const FSceneView& View);

/** Tracks the viewport client that should process the stat command, can be NULL */
extern ENGINE_API class FCommonViewportClient* GStatProcessingViewportClient;
