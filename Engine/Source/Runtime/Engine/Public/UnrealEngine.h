// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnrealEngine.h: Unreal engine helper definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderCommandFence.h"
#include "GenericPlatform/GenericWindow.h"
#include "Engine/Engine.h"
#include "SceneTypes.h"

class FViewportClient;
class UFont;
class ULocalPlayer;

extern ENGINE_API int32 GShowMaterialDrawEvents;
extern ENGINE_API int32 GDoAsyncLoadingWhileWaitingForVSync;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogEngine, Log, All);
//
//	FLocalPlayerIterator - Iterates over local players in the game.
//	There are no advantages to using this over GEngine->GetLocalPlayerIterator(GetWorld());
//	
//	Example:
//	for (FLocalPlayerIterator It(GEngine, GetWorld()); It; ++It)
//	{
//			// Do Stuff
//	}	
//


class FLocalPlayerIterator
{
protected:
	TArray<class ULocalPlayer*>::TConstIterator Iter;

	void GetCurrent()
	{		
		// advance iter while not finished iterating, and while current elemnt in NULL
		while( Iter && *Iter == NULL )
			++Iter;
	}

public:
	FLocalPlayerIterator(UEngine* InEngine, class UWorld* InWorld)
		: Iter(InEngine->GetLocalPlayerIterator(InWorld))
	{
		GetCurrent();
	}

	void operator++()
	{
		++Iter;
		GetCurrent();
	}
	ULocalPlayer* operator*() const
	{
		return *Iter;
	}
	ULocalPlayer* operator->() const
	{
		return *Iter;
	}
	explicit operator bool() const
	{
		return (bool)Iter;
	}
};

//	PlayerControllerIterators
//	This is the safest, most efficient way to iterate over player controllers.
//
//	Examples:
//
//	for (TPlayerControllerIterator<AQAPlayerController>::LocalOnly It(GetWorld()); It; ++It)
//	{
//		// PC is a locally controlled AQAPlayerController.
//		AQAPlayerController * PC = *It;	
//
//		// This PC will always be locally controlled!
//		check(PC->IsLocalController();
//	}
//
//
//
//
//	for (TPlayerControllerIterator<AQAPlayerController>::ServerAll It(GetWorld()); It; ++It)
//	{
//		// PC is a AQAPlayerController. It may local or remotely controlled.
//		AQAPlayerController * PC = *It;	
//	
//		// This can only be done on the server! 
//		// Only the server has player controllers for everyone!
//		check(GetWorld()->GetNetMode() != NM_Client);
//	}



template< class T, bool LocalOnly > class TBasePlayerControllerIterator
{
public:
	TBasePlayerControllerIterator(class UWorld* InWorld)
		: Iter(InWorld->GetPlayerControllerIterator())
	{
		check(LocalOnly || InWorld->GetNetMode() != NM_Client);	// You should only iterate on non local player controllers if you are the server
		AdvanceCurrent();
	}

	void operator++()
	{
		Next();
	}
	T* operator*() const
	{
		return Current;
	}
	T* operator->() const
	{
		return Current;
	}
	explicit operator bool() const
	{
		return (Current!=NULL);
	}

protected:

	FConstPlayerControllerIterator Iter;
	T* Current;

	void AdvanceCurrent()
	{
		// Look at current Iter
		Current = Iter ? Cast<T>(*Iter) : nullptr;

		// Advance if we have to
		while(Iter && (!Current || (LocalOnly && !Current->IsLocalController())))
		{
			++Iter;
			Current = Iter ? Cast<T>(*Iter) : nullptr;
		}
	}

	void Next()
	{
		// Advance one
		++Iter;

		// Update Current
		AdvanceCurrent();
	}
};

template<class T>
struct TPlayerControllerIterator
{
	typedef TBasePlayerControllerIterator<T, true>	LocalOnly;	// Only iterates locally controlled player controllers - can be used on client or server
	typedef TBasePlayerControllerIterator<T, false> ServerAll;	// Iterates all player controllers - local or remote - only can be used on server
};


/*-----------------------------------------------------------------------------
	Tick/ update stats helper for profiling.
-----------------------------------------------------------------------------*/

/**
 * Helper structure encapsulating all information gathered.
 */
struct FTickStats
{
	/** Object associated with instances. We keep the name because the object might be gone.*/
	FString ObjectPathName;
	/** Result of GetDetailedInfo() on the above.  */
	FString ObjectDetailedInfo;
	/** Result of GetDetailedInfo() on the above.  */
	FName ObjectClassFName;
	/** Index of GC run when the validity of the UObject Pointer was last checked.  */
	int32 GCIndex;
	/** Total accumulative time captured. */
	float TotalTime;
	/** Number of captures this frame. */
	int32 Count;
	/** bForSummary is used for the logging code to know if this should be used for a summary or not **/
	bool bForSummary;

	/** Compare helper for Sort */
	inline bool operator()( const FTickStats& A, const FTickStats& B	) const
	{
		return (B.TotalTime < A.TotalTime);
	}
};

/**
 * Helper struct for gathering detailed per object tick stats.
 */
struct FDetailedTickStats
{
	/** Constructor, initializing all members. */
	FDetailedTickStats( int32 InNumClassesToReport, float InTimeBetweenLogDumps, float InMinTimeBetweenLogDumps, float InTimesToReport, const TCHAR* InOperationPerformed );

	/** Destructor */
	virtual ~FDetailedTickStats();

	/**
	 * Starts tracking an object and returns whether it's a recursive call or not. If it is recursive
	 * the function will return false and EndObject should not be called on the object.
	 *
	 * @param	Object		Object to track
	 * @return	false if object is already tracked and EndObject should NOT be called, true otherwise
	 */
	bool BeginObject( UObject* Object );

	/**
	 * Finishes tracking the object and updates the time spent.
	 *
	 * @param	Object		Object to track
	 * @param	DeltaTime	Time we've been tracking it
	 * @param   bForSummary Object should be used for high level summary
	 */
	void EndObject( UObject* Object, float DeltaTime, bool bForSummary );

	/**
	 * Reset stats to clean slate.
	 */
	void Reset();

	/**
	 * Dump gathered stats informatoin to the log.
	 */
	void DumpStats();

	/** 
	 * Delegate handler for pre garbage collect event
	 **/
	void OnPreGarbageCollect()
	{
		GCIndex++;
		check(ObjectsInFlight.Num() == 0); // probably not required, but we shouldn't have anything in flight when we GC
	}

private:
	/** This is the collection of stats; some refer to objects that are long gone. */
	TArray<FTickStats> AllStats;
	/** Mapping from class to an index into the AllStats array. */
	TMap<const UObject*,int32> ObjectToStatsMap;
	/** Set of objects currently being tracked. Needed to correctly handle recursion. */
	TSet<const UObject*> ObjectsInFlight;

	/** Index of GC run. This is used to invalidate UObject pointers to make the whole system GC safe.  */
	int32 GCIndex;
	/** The GC callback cannot usually be registered at construction because this comes from a static data structure  */
	bool GCCallBackRegistered;
	/** Number of objects to report. Top X */
	int32	NumObjectsToReport;
	/** Time between dumping to the log in seconds. */
	float TimeBetweenLogDumps;
	/** Minimum time between log dumps, used for e.g. slow frames dumping. */
	float MinTimeBetweenLogDumps;
	/** Last time stats where dumped to the log. */
	double LastTimeOfLogDump;
	/** Tick time in ms to report if above. */
	float TimesToReport;
	/** Name of operation performed that is being tracked. */
	FString OperationPerformed;
	/** Handle to the registered OnPreGarbageCollect delegate */
	FDelegateHandle OnPreGarbageCollectDelegateHandle;
};

/** Scoped helper structure for capturing tick time. */
struct FScopedDetailTickStats
{
	/**
	 * Constructor, keeping track of object and start time.
	 */
	FScopedDetailTickStats( FDetailedTickStats& InDetailedTickStats, UObject* ObjectToTrack );

	/**
	 * Destructor, calculating delta time and updating global helper.
	 */
	~FScopedDetailTickStats();


private:
	/** Detailed tick stats to update. */
	FDetailedTickStats& DetailedTickStats;
	/**
	 * Object to track.
	 * Not GC safe, but we won't have anything in-flight during GC so that should be moot.
	 */
	UObject* Object;
	/** Tick start time. */
	uint32 StartCycles;
	/** Whether object should be tracked. false e.g. when recursion is involved. */
	bool bShouldTrackObject;
	/** Whether object class should be tracked. false e.g. when recursion is involved. */
	bool bShouldTrackObjectClass;
};


/** Delegate for switching between PIE and Editor worlds without having access to them */
DECLARE_DELEGATE_TwoParams( FOnSwitchWorldForPIE, bool, UWorld* );

#if WITH_EDITOR
/**
 * When created, switches global context to a PIE world
 * When destroyed, resets the GWorld back to what it was before
 */
class FScopedConditionalWorldSwitcher
{
public:
	/** Use the viewport to figure out what world to set temporarily */
	ENGINE_API FScopedConditionalWorldSwitcher( class FViewportClient* InViewportClient );

	/** Explicitly set to the specific world */
	ENGINE_API FScopedConditionalWorldSwitcher( UWorld* InWorld );

	/** Resets back to initial world */
	ENGINE_API ~FScopedConditionalWorldSwitcher();

	/** Delegate to call to switch worlds for PIE viewports.  Is not called when simulating (non-gameviewportclient) */
	static ENGINE_API FOnSwitchWorldForPIE SwitchWorldForPIEDelegate;
private:
	/** Called by constructors to do actual switch */
	void ConditionalSwitchWorld( class FViewportClient* InViewportClient, UWorld* InWorld );

	/** Viewport client used to set the world */
	FViewportClient* ViewportClient;

	/** World to reset when this is destroyed.  Can be null if nothing needs to be reset */
	UWorld* OldWorld;
};
#else
// does nothing outside of the editor
class FScopedConditionalWorldSwitcher
{
public:
	FScopedConditionalWorldSwitcher( class FViewportClient* InViewportClient ) {}
	FScopedConditionalWorldSwitcher( UWorld* InWorld ) {}
	~FScopedConditionalWorldSwitcher() = default;
};

#endif

/**
 * This function will look at the given command line and see if we have passed in a map to load.
 * If we have then use that.
 * If we have not then use the DefaultMap which is stored in the Engine.ini
 * 
 * @see UGameEngine::Init() for this method of getting the correct start up map
 *
 * @param CommandLine Commandline to use to get startup map (NULL or "" will return default startup map)
 *
 * @return Name of the startup map without an extension (so can be used as a package name)
 */
ENGINE_API FString appGetStartupMap(const TCHAR* CommandLine);

// Calculate the average frame time by using the stats system.
ENGINE_API void CalculateFPSTimings();


/** @return The font to use for rendering stats display. */
extern ENGINE_API UFont* GetStatsFont();

/*-----------------------------------------------------------------------------
	Frame end sync object implementation.
-----------------------------------------------------------------------------*/

namespace FFrameEndSync
{
	/**
	 * Syncs the game thread based on progress throughout the rendering pipeline
	 * @param bFullSync - when true, blocks the caller until all rendering work is completed,
	 * otherwise only blocks until the N - m frame has completed, where m is driven by various config.
	 */
	UE_DEPRECATED(5.6, "Use FFrameEndSync::Sync defined in RenderingThread.h, which takes an enum.")
	ENGINE_API void Sync(bool bFullSync = false);
};

/** Public interface to FEngineLoop so we can call it from editor or editor code */
class IEngineLoop
{
public:

	virtual int32 Init() = 0;

	virtual void Tick() = 0;

	/** Removes references to any objects pending cleanup by deleting them. */
	virtual void ClearPendingCleanupObjects() = 0;
};

/**
 * Cache some of the scalability CVars to avoid some virtual function calls (no longer the case with the new console variable system, but we only have 1 render thread)
 * and to detect changes and act accordingly if needed.
 * read by rendering thread[s], written by main thread, uses FlushRenderingCommands() to avoid conflict
 */
struct FCachedSystemScalabilityCVars
{
	bool bInitialized;
	int32 DetailMode;
	int32 CookOutUnusedDetailModeComponents;
	EMaterialQualityLevel::Type MaterialQualityLevel;
	int32 MaxShadowResolution;
	int32 MaxCSMShadowResolution;
	float ViewDistanceScale;
	float ViewDistanceScaleSquared;
	int32 FieldOfViewAffectsHLOD;
	float StaticMeshLODDistanceScale;
	float SkeletalMeshOverlayDistanceScale;

	float CalculateFieldOfViewDistanceScale(const float FieldOfView) const
	{
		if (!FMath::IsNearlyEqual(FieldOfViewMaxAngleScale, FieldOfViewMinAngleScale))
		{
			float ClampedFoV = FMath::Clamp(FieldOfView, FieldOfViewMinAngle, FieldOfViewMaxAngle);
			ClampedFoV = (ClampedFoV - FieldOfViewMinAngle) / (FieldOfViewMaxAngle - FieldOfViewMinAngle);
			return FMath::Lerp<float>(FieldOfViewMinAngleScale, FieldOfViewMaxAngleScale, ClampedFoV);
		}
		
		return FieldOfViewMaxAngleScale;
	}

	FCachedSystemScalabilityCVars();

	bool operator==(const FCachedSystemScalabilityCVars& Other) const;

protected:
	// This isn't public as it's only used to detect the change. Use ComputeAnisotropyRT()
	int32 MaxAnisotropy;

	// Scaling view distance with active camera field of view
	float FieldOfViewMinAngle;
	float FieldOfViewMinAngleScale;
	float FieldOfViewMaxAngle;
	float FieldOfViewMaxAngleScale;

	friend void ScalabilityCVarsSinkCallback();
};

ENGINE_API bool AllowHighQualityLightmaps(const FStaticFeatureLevel FeatureLevel);

ENGINE_API const FCachedSystemScalabilityCVars& GetCachedScalabilityCVars();

struct FSystemResolution
{
	int32 ResX = 0;
	int32 ResY = 0;
	EWindowMode::Type WindowMode = EWindowMode::Windowed;
	int32 MonitorIndex = -1;
	bool bForceRefresh = false;

	static ENGINE_API void SetMonitorIndex(int32 MonitorIndex);

	// Helper function for changing system resolution via the r.setres console command
	// This function will set r.setres, which will trigger a resolution change later on
	// when the console variable sinks are called
	static ENGINE_API void RequestResolutionChange(int32 InResX, int32 InResY, EWindowMode::Type InWindowMode);

	ENGINE_API void ForceRefresh();
};

ENGINE_API extern FSystemResolution GSystemResolution;
ENGINE_API extern int32 GUnbuiltHLODCount;

#ifndef UE_ENABLE_LOG_STACK_ON_FORCE_GC
#define UE_ENABLE_LOG_STACK_ON_FORCE_GC 0
#endif
#if UE_ENABLE_LOG_STACK_ON_FORCE_GC
ENGINE_API extern bool GLogStackOnForceGC;
#endif

// Update the debugging aid GPlayInEditorContextString based on the current world context (does nothing in WITH_EDITOR=0 builds)
ENGINE_API void UpdatePlayInEditorWorldDebugString(const FWorldContext* WorldContext);

// Returns the Debug string for a given world (Standalone, Listen Server, Client #, etc)
ENGINE_API FString GetDebugStringForWorld(const UWorld* World);

// Used to temporarily override GPlayInEditorID, correctly updating the debug string and other state as necessary
struct FTemporaryPlayInEditorIDOverride
{
public:
	ENGINE_API FTemporaryPlayInEditorIDOverride(int32 NewOverrideID);
	ENGINE_API ~FTemporaryPlayInEditorIDOverride();

private:
	int32 PreviousID;

	void SetID(int32 NewID);
};
