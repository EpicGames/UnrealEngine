// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnrealExporter.h: Exporter class definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

class AActor;

/**
 * Encapsulates a map from objects to their direct inners, used by UExporter::ExportObjectInner when exporting objects.
 * Should be recreated before new objects are created within objects that are to be exported!
 *
 * Typical usage pattern is:
 *
 *     const FExportObjectInnerContext Context;
 *     foreach( Object in a set of objects)
 *         Exporter->ExportObjectInner( Context, Object ); 
 */
class FExportObjectInnerContext
{
public:
	/**
	 * Creates the map from objects to their direct inners.
	 */
	ENGINE_API FExportObjectInnerContext();

	/**
	 * Creates the map from objects to their direct inners.
	 * @param	ObjsToIgnore	An array of objects that should NOT be put in the list
	 */
	ENGINE_API FExportObjectInnerContext(const TArray<UObject*>& ObjsToIgnore);

	/**Empty Constructor for derived export contexts */
	FExportObjectInnerContext(const bool bIgnoredValue) {};

	virtual ~FExportObjectInnerContext() = default;
	virtual int32 GetObjectNumber() const { return 0; }

protected:
	friend class UExporter;

	/** Data structure used to map an object to its inners. */
	typedef TArray<UObject*>			InnerList;
	typedef TMap<UObject*,InnerList>	ObjectToInnerMapType;

	ObjectToInnerMapType				ObjectToInnerMap;

	ENGINE_API void AddObjectToInnerMap(UObject* InObject);

public:
	/** 
	 * Get the inner list of an object
	 * @param InObj - Object
	 * @return Object Inners
	 */
	const TArray<UObject*>* GetObjectInners(UObject* InObj) const
	{
		return ObjectToInnerMap.Find(InObj);
	}

	/**
	 * Should the given object be considered selected by the current export?
	 */
	ENGINE_API virtual bool IsObjectSelected(const UObject* InObj) const;
};

#if WITH_EDITOR
class FSelectedActorExportObjectInnerContext : public FExportObjectInnerContext
{
public:
	ENGINE_API FSelectedActorExportObjectInnerContext();
	ENGINE_API explicit FSelectedActorExportObjectInnerContext(const TArray<AActor*> InSelectedActors);

	ENGINE_API virtual bool IsObjectSelected(const UObject* InObj) const override;
	virtual int32 GetObjectNumber() const override { return SelectedActors.Num(); }

private:
	ENGINE_API void AddSelectedActor(const AActor* InActor);

	TSet<const AActor*> SelectedActors;
};
#endif

/**
 * Exports the property values for the specified object as text to the output device.
 *
 * @param	Context			Context from which the set of 'inner' objects is extracted.  If NULL, an object iterator will be used.
 * @param	Out				the output device to send the exported text to
 * @param	ObjectClass		the class of the object to dump properties for
 * @param	Object			the address of the object to dump properties for
 * @param	Indent			number of spaces to prepend to each line of output
 * @param	DiffClass		the class to use for comparing property values when delta export is desired.
 * @param	Diff			the address of the object to use for determining whether a property value should be exported.  If the value in Object matches the corresponding
 *							value in Diff, it is not exported.  Specify NULL to export all properties.
 * @param	Parent			the UObject corresponding to Object
 * @param	PortFlags		flags used for modifying the output and/or behavior of the export
 * @param	ExportRootScope	The scope to create relative paths from, if the PPF_ExportsNotFullyQualified flag is passed in.  If NULL, the package containing the object will be used instead.
 */
ENGINE_API void ExportProperties( const class FExportObjectInnerContext* Context, FOutputDevice& Out, UClass* ObjectClass, uint8* Object, int32 Indent, UClass* DiffClass, uint8* Diff, UObject* Parent, uint32 PortFlags=0, UObject* ExportRootScope = NULL );

/**
 * Debug spew for components
 * @param Object object to dump component spew for
 */
ENGINE_API void DumpComponents(UObject *Object);

/** Debug spew for object's components, returned as an FString. */
ENGINE_API FString DumpComponentsToString(UObject *Object);

/**
 * Debug spew for an object
 * @param Label header line for spew
 * @param Object object to dump component spew for
 */
ENGINE_API void DumpObject(const TCHAR *Label, UObject* Object);

/** Debug spew for an object, returned as an FString. */
ENGINE_API FString DumpObjectToString(UObject* Object);
