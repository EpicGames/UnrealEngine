// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderCompilerCore.h"

enum class EUpdateTextureValueType
{
	Float,
	Int32,
	Uint32
};

class FUpdateTextureShaderBase : public FGlobalShader
{
public:
	FUpdateTextureShaderBase() {}
	FUpdateTextureShaderBase(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		if (FDataDrivenShaderPlatformInfo::GetRequiresBindfulUtilityShaders(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_ForceBindful);
		}
	}
};

class FUpdateTexture2DSubresourceCS : public FUpdateTextureShaderBase
{
	DECLARE_EXPORTED_SHADER_TYPE(FUpdateTexture2DSubresourceCS, Global, ENGINE_API);
public:
	FUpdateTexture2DSubresourceCS() {}
	FUpdateTexture2DSubresourceCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FUpdateTextureShaderBase( Initializer )
	{
		SrcPosPitchParameter.Bind(Initializer.ParameterMap, TEXT("TSrcPosPitch"), SPF_Mandatory);
		SrcBuffer.Bind(Initializer.ParameterMap, TEXT("TSrcBuffer"), SPF_Mandatory);
		DestPosSizeParameter.Bind(Initializer.ParameterMap, TEXT("TDestPosSize"), SPF_Mandatory);
		DestTexture.Bind(Initializer.ParameterMap, TEXT("TDestTexture"), SPF_Mandatory);
	}

	static const TCHAR* GetSourceFilename() { return TEXT("/Engine/Private/UpdateTextureShaders.usf"); }
	static const TCHAR* GetFunctionName() { return TEXT("TUpdateTexture2DSubresourceCS"); }

	using FUpdateTextureShaderBase::ModifyCompilationEnvironment;

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FUpdateTextureRegion2D& UpdateRegionInBlocks, uint32 SrcElementPitch)
	{
		const uint32 DstPosSize[4] =
		{
			UpdateRegionInBlocks.DestX,
			UpdateRegionInBlocks.DestY,
			UpdateRegionInBlocks.Width,
			UpdateRegionInBlocks.Height
		};
		FIntVector SrcPosPitch(UpdateRegionInBlocks.SrcX, UpdateRegionInBlocks.SrcY, SrcElementPitch);

		SetShaderValue(BatchedParameters, DestPosSizeParameter, DstPosSize);
		SetShaderValue(BatchedParameters, SrcPosPitchParameter, SrcPosPitch);
	}

	LAYOUT_FIELD(FShaderParameter, SrcPosPitchParameter);
	LAYOUT_FIELD(FShaderResourceParameter, SrcBuffer);
	LAYOUT_FIELD(FShaderParameter, DestPosSizeParameter);
	LAYOUT_FIELD(FShaderResourceParameter, DestTexture);

	static inline TShaderRef<FUpdateTexture2DSubresourceCS> SelectShader(FGlobalShaderMap* GlobalShaderMap, EUpdateTextureValueType ValueType);

protected:
	template<typename ShaderType>
	static inline TShaderRef<FUpdateTexture2DSubresourceCS> SelectShader(FGlobalShaderMap* GlobalShaderMap);
};

template<EUpdateTextureValueType ValueType>
class TUpdateTexture2DSubresourceCS : public FUpdateTexture2DSubresourceCS
{
	DECLARE_EXPORTED_SHADER_TYPE(TUpdateTexture2DSubresourceCS, Global, ENGINE_API);
public:
	TUpdateTexture2DSubresourceCS() {}
	TUpdateTexture2DSubresourceCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FUpdateTexture2DSubresourceCS(Initializer)
	{}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FUpdateTexture2DSubresourceCS::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		const TCHAR* Type = nullptr;
		switch (ValueType)
		{
		default: checkNoEntry();
		case EUpdateTextureValueType::Float: OutEnvironment.SetDefine(TEXT("COMPONENT_TYPE"), TEXT("float4")); break;
		case EUpdateTextureValueType::Int32: OutEnvironment.SetDefine(TEXT("COMPONENT_TYPE"), TEXT("int4")); break;
		case EUpdateTextureValueType::Uint32: OutEnvironment.SetDefine(TEXT("COMPONENT_TYPE"), TEXT("uint4")); break;
		}
	}
};

template <typename ShaderType>
inline TShaderRef<FUpdateTexture2DSubresourceCS> FUpdateTexture2DSubresourceCS::SelectShader(FGlobalShaderMap* GlobalShaderMap)
{
	TShaderRef<ShaderType> Shader = GlobalShaderMap->GetShader<ShaderType>();
	return Shader;
}

inline TShaderRef<FUpdateTexture2DSubresourceCS> FUpdateTexture2DSubresourceCS::SelectShader(FGlobalShaderMap* GlobalShaderMap, EUpdateTextureValueType ValueType)
{
	switch (ValueType)
	{
	default: checkNoEntry();
	case EUpdateTextureValueType::Float: return FUpdateTexture2DSubresourceCS::SelectShader<TUpdateTexture2DSubresourceCS<EUpdateTextureValueType::Float>>(GlobalShaderMap);
	case EUpdateTextureValueType::Int32: return FUpdateTexture2DSubresourceCS::SelectShader<TUpdateTexture2DSubresourceCS<EUpdateTextureValueType::Int32>>(GlobalShaderMap);
	case EUpdateTextureValueType::Uint32: return FUpdateTexture2DSubresourceCS::SelectShader<TUpdateTexture2DSubresourceCS<EUpdateTextureValueType::Uint32>>(GlobalShaderMap);
	}
}

class FUpdateTexture3DSubresourceCS : public FUpdateTextureShaderBase
{
	DECLARE_EXPORTED_SHADER_TYPE(FUpdateTexture3DSubresourceCS, Global, ENGINE_API);
public:
	FUpdateTexture3DSubresourceCS() {}
	FUpdateTexture3DSubresourceCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FUpdateTextureShaderBase(Initializer)
	{
		SrcPitchParameter.Bind(Initializer.ParameterMap, TEXT("SrcPitch"), SPF_Mandatory);
		SrcDepthPitchParameter.Bind(Initializer.ParameterMap, TEXT("SrcDepthPitch"), SPF_Mandatory);

		SrcBuffer.Bind(Initializer.ParameterMap, TEXT("SrcBuffer"), SPF_Mandatory);

		DestPosParameter.Bind(Initializer.ParameterMap, TEXT("DestPos"), SPF_Mandatory);
		DestSizeParameter.Bind(Initializer.ParameterMap, TEXT("DestSize"), SPF_Mandatory);

		DestTexture3D.Bind(Initializer.ParameterMap, TEXT("DestTexture3D"), SPF_Mandatory);
	}

	static const TCHAR* GetSourceFilename() { return TEXT("/Engine/Private/UpdateTextureShaders.usf"); }
	
	static const TCHAR* GetFunctionName() { return TEXT("UpdateTexture3DSubresourceCS"); };

	LAYOUT_FIELD(FShaderParameter, SrcPitchParameter);
	LAYOUT_FIELD(FShaderParameter, SrcDepthPitchParameter);
	LAYOUT_FIELD(FShaderResourceParameter, SrcBuffer);

	LAYOUT_FIELD(FShaderParameter, DestPosParameter);
	LAYOUT_FIELD(FShaderParameter, DestSizeParameter);

	LAYOUT_FIELD(FShaderResourceParameter, DestTexture3D)
};

template<uint32 ElementsPerThread>
class TCopyDataCS : public FUpdateTextureShaderBase
{
	DECLARE_EXPORTED_SHADER_TYPE(TCopyDataCS, Global, ENGINE_API);
public:
	TCopyDataCS() {}
	TCopyDataCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FUpdateTextureShaderBase(Initializer)
	{
		SrcBuffer.Bind(Initializer.ParameterMap, TEXT("SrcCopyBuffer"), SPF_Mandatory);
		DestBuffer.Bind(Initializer.ParameterMap, TEXT("DestBuffer"), SPF_Mandatory);		
	}

	static const TCHAR* GetSourceFilename() { return TEXT("/Engine/Private/UpdateTextureShaders.usf"); }

	static const TCHAR* GetFunctionName() 
	{
		switch (ElementsPerThread)
		{
			case 1u: return TEXT("CopyData1CS"); break;
			case 2u: return TEXT("CopyData2CS"); break;
		}
		return nullptr;
	}

	LAYOUT_FIELD(FShaderResourceParameter, SrcBuffer);
	LAYOUT_FIELD(FShaderResourceParameter, DestBuffer);
};
