// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

class FUserActivityTracking : FNoncopyable
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnActivityChanged, const FUserActivity&);

	static ENGINE_API void SetContextFilter(EUserActivityContext InContext);
	static ENGINE_API void SetActivity(const FUserActivity& InUserActivity);
	static const FUserActivity& GetUserActivity() { return UserActivity; }

	// Called when the UserActivity changes
	static ENGINE_API FOnActivityChanged OnActivityChanged;

private:
	static ENGINE_API FUserActivity UserActivity;
	static ENGINE_API EUserActivityContext ContextFilter;
};

