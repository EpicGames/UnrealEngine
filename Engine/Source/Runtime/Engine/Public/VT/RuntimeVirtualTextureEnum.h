// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumRange.h"
#include "RuntimeVirtualTextureEnum.generated.h"

/** Maximum number of texture layers we will have in a runtime virtual texture. Increase if we add a ERuntimeVirtualTextureMaterialType with more layers. */
namespace RuntimeVirtualTexture { enum { MaxTextureLayers = 3 }; }

/** 
 * Enumeration of all runtime virtual texture material attributes.
 * These can be combined to form full ERuntimeVirtualTextureMaterialType layouts.
 */
enum class ERuntimeVirtualTextureAttributeType : uint8
{
	BaseColor,
	Normal,
	Roughness,
	Specular,
	Mask,
	Mask4,
	WorldHeight,
	Displacement,

	Count
};

static_assert((uint32)ERuntimeVirtualTextureAttributeType::Count <= 8u, "ERuntimeVirtualTextureAttributeType can no longer be used to create 8bit masks.");

/** 
 * Enumeration of virtual texture stack layouts to support. 
 * Extend this enumeration with other layouts as required. For example we will probably want to add a displacement texture option.
 * This "fixed function" approach will probably break down if we end up needing to support some complex set of attribute combinations but it is OK to begin with.
 */
UENUM()
enum class ERuntimeVirtualTextureMaterialType : uint8
{
	BaseColor UMETA(DisplayName = "Base Color"),
	Mask4 UMETA(DisplayName = "Mask4", ToolTip = "4 channel mask texture."),
	BaseColor_Normal_Roughness UMETA(DisplayName = "Base Color, Normal, Roughness", ToolTip = "Local space Normal. Requires less memory than 'Base Color, Normal, Roughness, Specular'. Supports LQ compression."),
	BaseColor_Normal_Specular UMETA(DisplayName = "Base Color, Normal, Roughness, Specular"),
	BaseColor_Normal_Specular_YCoCg UMETA(DisplayName = "YCoCg Base Color, Normal, Roughness, Specular", ToolTip = "Base Color is stored in YCoCg space. This requires more memory but may provide better quality."),
	BaseColor_Normal_Specular_Mask_YCoCg UMETA(DisplayName = "YCoCg Base Color, Normal, Roughness, Specular, Mask", ToolTip="Base Color is stored in YCoCg space. This requires more memory but may provide better quality."),
	WorldHeight UMETA(DisplayName = "World Height"),
	Displacement UMETA(DisplayName = "Displacement"),
	Count UMETA(Hidden),
};

namespace RuntimeVirtualTexture { enum { MaterialType_NumBits = 3 }; }
static_assert((uint32)ERuntimeVirtualTextureMaterialType::Count <= (1 << (uint32)RuntimeVirtualTexture::MaterialType_NumBits), "NumBits is too small");

ENUM_RANGE_BY_COUNT(ERuntimeVirtualTextureMaterialType, ERuntimeVirtualTextureMaterialType::Count);

/** Enumeration of main pass behaviors when rendering to a runtime virtual texture. */
UENUM()
enum class ERuntimeVirtualTextureMainPassType : uint8
{
	/** 
	 * Never render to the main pass. 
	 * Use this for primitives that only render to Runtime Virtual Texture and can be missing if there is no virtual texture support. 
	 */
	Never UMETA(DisplayName = "Never"),
	/** 
	 * Render to the main pass if no associated Runtime Virtual Texture Volumes are set to 'Hide Primitives'.
	 * This will render to the main pass if there is no matching Runtime Virtual Texture Volume placed in the scene. 
	 */
	Exclusive UMETA(DisplayName = "From Virtual Texture"),
	/** 
	 * Always render to the main pass. 
	 * Use this for items that both read from and write to a Runtime Virtual Texture.
	 */
	Always UMETA(DisplayName = "Always"),
};

/** Enumeration of runtime virtual texture shader uniforms. */
enum ERuntimeVirtualTextureShaderUniform
{
	ERuntimeVirtualTextureShaderUniform_WorldToUVTransform0,
	ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1,
	ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2,
	ERuntimeVirtualTextureShaderUniform_WorldHeightUnpack,
	ERuntimeVirtualTextureShaderUniform_Count,
};
