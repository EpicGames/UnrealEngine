// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace VirtualTextureScalability
{
	/** Get max upload rate to virtual textures. */
	ENGINE_API int32 GetMaxUploadsPerFrame();
	/** Get max upload rate to streaming virtual textures. May return 0 which means SVTs aren't budgeted separately. */
	ENGINE_API int32 GetMaxUploadsPerFrameForStreamingVT();
	/** Get max produce rate to virtual textures. */
	ENGINE_API int32 GetMaxPagesProducedPerFrame();
	/** Get max update rate of already mapped virtual texture pages. */
	ENGINE_API int32 GetMaxContinuousUpdatesPerFrame();
	/** Get max time in seconds to spend in a single time slice of the gather page requests task. */
	ENGINE_API float GetMaxGatherTimePerFrame();
	/** Get max time in seconds to spend in a single time slice of the submit page requests task. */
	ENGINE_API float GetMaxSubmitTimePerFrame();
	/** Get max allocated virtual textures to release per frame. */
	ENGINE_API int32 GetMaxAllocatedVTReleasedPerFrame();
	/** Get the size of the virtual texture feedback tiles. */
	ENGINE_API uint32 GetVirtualTextureFeedbackFactor();
	/** Get the number of frames a page must be unused, before it's considered free */
	ENGINE_API uint32 GetPageFreeThreshold();
	/** Get the number of frames a page must be unused, before it is unmapped instead of updated during RVT page invalidation. */
	ENGINE_API uint32 GetKeepDirtyPageMappedFrameThreshold();
	/** Get resolution bias for runtime virtual textures. */
	ENGINE_API int32 GetRuntimeVirtualTextureSizeBias();
	/** Get resolution bias for runtime virtual textures. */
	ENGINE_API int32 GetRuntimeVirtualTextureSizeBias(uint32 GroupIndex);
	/** Is HW Anisotropic filtering enabled for VT */
	ENGINE_API bool IsAnisotropicFilteringEnabled();
	/** Get maximum anisotropy when virtual texture sampling. This is also clamped per virtual texture according to the tile border size. */
	ENGINE_API int32 GetMaxAnisotropy();
}
