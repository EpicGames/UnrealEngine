// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Stats/Stats.h"

DECLARE_MULTICAST_DELEGATE_TwoParams(FVideoRecordingFinalized, bool /* bSucceeded */, const FString& /* FilePath */);

DECLARE_STATS_GROUP(TEXT("VideoRecordingSystem"), STATGROUP_VideoRecordingSystem, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("EnableRecording"), STAT_VideoRecordingSystem_EnableRecording, STATGROUP_VideoRecordingSystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("NewRecording"), STAT_VideoRecordingSystem_NewRecording, STATGROUP_VideoRecordingSystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("StartRecording"), STAT_VideoRecordingSystem_StartRecording, STATGROUP_VideoRecordingSystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PauseRecording"), STAT_VideoRecordingSystem_PauseRecording, STATGROUP_VideoRecordingSystem, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("FinalizeRecording"), STAT_VideoRecordingSystem_FinalizeRecording, STATGROUP_VideoRecordingSystem, );

#define DEFINE_VIDEOSYSTEMRECORDING_STATS \
	DEFINE_STAT(STAT_VideoRecordingSystem_EnableRecording); \
	DEFINE_STAT(STAT_VideoRecordingSystem_NewRecording); \
	DEFINE_STAT(STAT_VideoRecordingSystem_StartRecording); \
	DEFINE_STAT(STAT_VideoRecordingSystem_PauseRecording); \
	DEFINE_STAT(STAT_VideoRecordingSystem_FinalizeRecording);

class Error;

/** Represents the state of the video recorder */
enum class EVideoRecordingState
{
	None,
	Starting,
	Recording,
	Pausing,
	Paused,
	Finalizing,
	Error
};

struct FVideoRecordingParameters
{
	// If greater than 0 sets the length of the ring buffer (maximum limited by platform).
	// NewRecording will return false if the number here is unsupported.
	// Use GetMinimumRecordingSeconds/GetMaximumRecordingSeconds to find the range for the current platform.
	uint64 RecordingLengthSeconds;
	// If true, video recording will automatically start.
	bool bAutoStart;
	// If true, a new video will be created based on the original file name and automatically start.
	bool bAutoContinue;
	// If true, the saved video will be exported to the platform's video library. If false it will remain as a temporary file only.
	bool bExportToLibrary;
	// Set the user current recording belongs to.
	FPlatformUserId UserId;

	FVideoRecordingParameters()
		: RecordingLengthSeconds(0)
		, bAutoStart(false)
		, bAutoContinue(false)
		, bExportToLibrary(true)
		, UserId(PLATFORMUSERID_NONE)
	{}

	FVideoRecordingParameters(uint64 InRecordingLengthSeconds, bool bInAutoStart, bool bInAutoContinue, bool bInExportToLibrary, FPlatformUserId InUserId)
		: RecordingLengthSeconds(InRecordingLengthSeconds)
		, bAutoStart(bInAutoStart)
		, bAutoContinue(bInAutoContinue)
		, bExportToLibrary(bInExportToLibrary)
		, UserId(InUserId)
	{}
};

/**
 * Interface for platform video recording functionality
 */
class IVideoRecordingSystem
{
public:
	virtual ~IVideoRecordingSystem() = default;

	/**
	 * Enables or disables recording if the platform supports it. Useful to prevent users from sharing spoilers.\
	 *
	 * @param bEnableRecording If true, video recording will be allowed. If false, videos will not be recorded.
	 */
	virtual void EnableRecording(const bool bEnableRecording) = 0;

	/** Returns whether recording is currently enabled. */
	virtual bool IsEnabled() const = 0;

	/**
	 * Initializes a new video recording.
	 *
	 *  By default does not start capturing, will not start again after saving and will export to the library.
	 *  Use GetOnVideoRecordingFinalizedDelegate to retrieve the final filepath.
	 *	
	 * @param DestinationFileName The base name of the resulting video, without a path or extension.
	 * @param Parameters the parameters for this recording
	 * @return True if opening the recording succeeded, false otherwise.
	 */
	virtual bool NewRecording(const TCHAR* DestinationFileName, FVideoRecordingParameters Parameters = FVideoRecordingParameters()) = 0;

	/** Begins capturing video after a call to NewRecording or PauseRecording. */
	virtual void StartRecording() = 0;

	/** Pauses video recording after a call to StartRecording. Call StartRecording again to resume. */
	virtual void PauseRecording() = 0;

	/** The minimum and maximum time available for recording on this platform. */
	virtual uint64 GetMinimumRecordingSeconds() const = 0;
	virtual uint64 GetMaximumRecordingSeconds() const = 0;

	/** Returns the current length of time the of the recording, up to RecordingLengthSeconds. */
	virtual float GetCurrentRecordingSeconds() const = 0;

	/**
	 * Stops recording and prepares the final video file for use.
	 *
	 * @param bSaveRecording If true, the recording will be saved. If false, the recording will be discarded.
	 * @param bStopAutoContinue If true, a auto-continue recording will be stopped.
	 * @param Title The title to use for the final video
	 * @param Comment A comment to store with the final video
	 */
	virtual void FinalizeRecording(const bool bSaveRecording, const FText& Title, const FText& Comment, const bool bStopAutoContinue = true) = 0;

	/** Returns the current state of video recording. */
	virtual EVideoRecordingState GetRecordingState() const = 0;

	FVideoRecordingFinalized& GetOnVideoRecordingFinalizedDelegate()
	{
		return OnVideoRecordingFinalized;
	}

protected:
	FVideoRecordingFinalized OnVideoRecordingFinalized;
};

/** A generic implementation of the video recording system, that doesn't support recording */
class FGenericVideoRecordingSystem : public IVideoRecordingSystem
{
public:
	virtual void EnableRecording(const bool bEnableRecording) override {}
	virtual bool IsEnabled() const override { return false; }
	virtual bool NewRecording(const TCHAR* DestinationFileName, FVideoRecordingParameters Parameters = FVideoRecordingParameters()) override { return false; }
	virtual void StartRecording() override {}
	virtual void PauseRecording() override {}
	virtual uint64 GetMinimumRecordingSeconds() const override { return 0; }
	virtual uint64 GetMaximumRecordingSeconds() const override { return 0; }
	virtual float GetCurrentRecordingSeconds() const override { return 0.0f; }
	virtual void FinalizeRecording(const bool bSaveRecording, const FText& Title, const FText& Comment, const bool bStopAutoContinue) override {}

	virtual EVideoRecordingState GetRecordingState() const override { return EVideoRecordingState::None; }
};
