// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "GenericPlatform/ICursor.h"
#include "InputCoreTypes.h"

class FCanvas;
class FCursorReply;
class FViewport;
class FPopupMethodReply;
class FWindowActivateEvent;
class SWidget;
enum class EFocusCause : uint8;
enum class EGestureEvent : uint8;
enum class EGesturePhase : uint8;
struct FInputKeyEventArgs;
struct FKey;
struct FStatHitchesData;
struct FStatUnitData;
struct FStatThermalsData;

/**
 * An abstract interface to a viewport's client.
 * The viewport's client processes input received by the viewport, and draws the viewport.
 */
class FViewportClient
{
public:
	virtual ~FViewportClient()= default;
	virtual void Precache() {}
	ENGINE_API virtual void RedrawRequested(FViewport* Viewport);
	ENGINE_API virtual void RequestInvalidateHitProxy(FViewport* Viewport);
	virtual void Draw(FViewport* Viewport,FCanvas* Canvas) {}
	virtual bool ProcessScreenShots(FViewport* Viewport) { return false; }
	virtual UWorld* GetWorld() const { return NULL; }
	virtual struct FEngineShowFlags* GetEngineShowFlags() { return NULL; }

	/**
	 * Allow the viewport-client to process a key event received by the viewport.
	 * If the viewport-client uses the event, it should return true.  If true is returned, the viewport should not bubble the event.
	 * 
	 * @param	EventArgs - The Input event args.
	 * @return	True to consume the key event, false to pass it on.
	 */
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) { return false; }

	/**
	 * Allow the viewport-client to process an axis event received by the viewport.
	 * If the viewport-client uses the event, it should return true.  If true is returned, the viewport should not bubble the event.
	 * 
	 * @param Args The input event arguments for this axis movement.
	 * @return True to consume the axis movement, false to pass it on.
	 */
	virtual bool InputAxis(const FInputKeyEventArgs& Args) { return false; }
	
	/**
	 * Allow the viewport-client to process a character event received by the viewport.
	 * If the viewport-client uses the event, it should return true.  If true is returned, the viewport should not bubble the event.
	 * 
	 * @param	Viewport - The viewport that received the event (is focused by the owner of the axis).
	 * @param	ControllerId - The controller from which the character event was emitted.
	 * @param	Character - The character.
	 * @return	True to consume the character, false to pass it on.
	 */
	virtual bool InputChar(FViewport* Viewport,int32 ControllerId,TCHAR Character) { return false; }

	/**
	 * Check a key event received by the viewport.
	 * If the viewport client uses the event, it should return true to consume it.
	 * @param	Viewport - The viewport which the event is from.
	 * @param	DeviceId - The input device id from which this event is from
	 * @param	Handle - Identifier unique to this touch event
	 * @param	Type - What kind of touch event this is (see ETouchType)
	 * @param	TouchLocation - Screen position of the touch
	 * @param	Force - How hard the touch is
	 * @param	TouchpadIndex - For devices with multiple touchpads, this is the index of which one
	 * @param	Timestamp - The timestamp of when this input event was originally polled in the form of CPU cycles.
	 * @return	True to consume the key event, false to pass it on.
	 */
	UE_DEPRECATED(5.8, "Use the overload that uses FTouchId instead")
	virtual bool InputTouch(
		FViewport* const Viewport,
		const FInputDeviceId DeviceId,
		const uint32 Handle,
		const ETouchType::Type Type,
		const FVector2D& TouchLocation,
		const float Force,
		const uint32 TouchpadIndex,
		const uint64 Timestamp) final
	{
		return InputTouch(Viewport, FTouchId(DeviceId, static_cast<ETouchIndex::Type>(Handle)), Type, TouchLocation, Force, Timestamp);
	}

	/**
	 * Allow the viewport client to process a touch event received by the viewport.
	 * If the viewport client uses the event, it should return true.  If true is returned, the viewport should not bubble the event.
	 * 
	 * @param	Viewport - The viewport that was touched.
	 * @param	TouchId - Identifier unique to this ongoing touch (i.e. device + finger).
	 * @param	Type - What kind of touch event this is (see ETouchType).
	 * @param	TouchLocation - Viewport-relative position of the touch.
	 * @param	Force - How hard the touch is.
	 * @param	Timestamp - The timestamp of when this input event was originally polled in the form of CPU cycles.
	 * @return	True to consume the event, false to pass it on.
	 */
	virtual bool InputTouch(
		FViewport* const Viewport,
		const FTouchId TouchId,
		const ETouchType::Type Type,
		const FVector2D& TouchLocation,
		const float Force,
		const uint64 Timestamp)
	{
		return false;
	}

	/**
	 * Allow the viewport-client to process a gesture event received by the viewport.
	 * If the viewport-client uses the event, it should return true.  If true is returned, the viewport should not bubble the event.
	 * 
	 * @param	Viewport - The viewport on which the gesture was performed.
	 * @param	DeviceId - The input device on which the gesture was performed.
	 * @param	GestureType - The type of the gesture.
	 * @param	GesturePhase - The phase of the gesture.
	 * @param	GestureDelta - Delta values of this gesture from the previous instance of it.
	 * @param	Timestamp - The timestamp of when this input event was originally polled in the form of CPU cycles.
	 * @return	True to consume the gesture event, false to pass it on.
	 */
	virtual bool InputGesture(
		FViewport* Viewport,
		const FInputDeviceId DeviceId,
		EGestureEvent GestureType,
		EGesturePhase GesturePhase,
		const FVector2D& GestureDelta,
		bool bIsDirectionInvertedFromDevice,
		const uint64 Timestamp)
	{
		return false;
	}

	/**
	 * Allow the viewport-client to process a motion event received by the viewport.
	 * If the viewport-client uses the event, it should return true.  If true is returned, the viewport should not bubble the event.
	 *
	 * @param Viewport - The viewport that received the event (is focused by the owner of the accelerometer).
	 * @param DeviceId - The input device that was moved.
	 * @param Tilt - The current orientation of the device.
	 * @param RotationRate - How fast the tilt is changing.
	 * @param Gravity - Describes the current gravity of the device.
	 * @param Acceleration - Describes the acceleration of the device.
	 * @param Timestamp - The timestamp of when this input event was originally polled in the form of CPU cycles.
	 * @return	True to consume the motion event, false to pass it on.
	 */
	virtual bool InputMotion(
		FViewport* Viewport,
		const FInputDeviceId DeviceId,
		const FVector& Tilt,
		const FVector& RotationRate,
		const FVector& Gravity,
		const FVector& Acceleration,
		const uint64 Timestamp)
	{
		return false;
	}

	virtual void SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport) { };

	virtual void MouseEnter( FViewport* Viewport,int32 x, int32 y ) {}

	virtual void MouseLeave( FViewport* Viewport ) {}

	virtual void MouseMove(FViewport* Viewport,int32 X,int32 Y) {}

	/**
	 * Called when the mouse is moved while a window input capture is in effect
	 *
	 * @param	InViewport	Viewport that captured the mouse input
	 * @param	InMouseX	New mouse cursor X coordinate
	 * @param	InMouseY	New mouse cursor Y coordinate
	 */
	virtual void CapturedMouseMove( FViewport* InViewport, int32 InMouseX, int32 InMouseY ) { }

	/**
	 * Called from slate when input is finished for this frame, and we should process any queued mouse moves.
	 */
	virtual void ProcessAccumulatedPointerInput(FViewport* InViewport) {};

	/**
	 * Retrieves the cursor that should be displayed by the OS
	 *
	 * @param	Viewport	the viewport that contains the cursor
	 * @param	X			the x position of the cursor
	 * @param	Y			the Y position of the cursor
	 *
	 * @return	the cursor that the OS should display
	 */
	virtual EMouseCursor::Type GetCursor(FViewport* Viewport, int32 X,int32 Y) { return EMouseCursor::Default; }

	/**
	 * Called to map a cursor reply to an actual widget to render.
	 *
	 * @return	the widget that should be rendered for this cursor, return TOptional<TSharedRef<SWidget>>() if no mapping.
	 */
	ENGINE_API virtual TOptional<TSharedRef<SWidget>> MapCursor(FViewport* Viewport, const FCursorReply& CursorReply);

	/**
	 * Called to determine if we should render the focus brush.
	 *
	 * @param InFocusCause	The cause of focus
	 */
	virtual TOptional<bool> QueryShowFocus(const EFocusCause InFocusCause) const { return TOptional<bool>(); }

	virtual void LostFocus(FViewport* Viewport) {}
	virtual void ReceivedFocus(FViewport* Viewport) {}
	virtual bool IsFocused(FViewport* Viewport) { return true; }

	virtual void Activated(FViewport* Viewport, const FWindowActivateEvent& InActivateEvent) {}
	virtual void Deactivated(FViewport* Viewport, const FWindowActivateEvent& InActivateEvent) {}

	virtual bool IsInPermanentCapture()
	{ 
		return  !GIsEditor && ((GetMouseCaptureMode() == EMouseCaptureMode::CapturePermanently) ||
			(GetMouseCaptureMode() == EMouseCaptureMode::CapturePermanently_IncludingInitialMouseDown));
	}

	/**
	 * Called when the top level window associated with the viewport has been requested to close.
	 * At this point, the viewport has not been closed and the operation may be canceled.
	 * This may not called from PIE, Editor Windows, on consoles, or before the game ends
	 * from other methods.
	 * This is only when the platform specific window is closed.
	 *
	 * @return True if the viewport may be closed, false otherwise.
	 */
	virtual bool WindowCloseRequested() { return true; }

	virtual void CloseRequested(FViewport* Viewport) {}

	virtual bool RequiresHitProxyStorage() { return true; }

	/**
	 * Determines whether this viewport client should receive calls to InputAxis() if the game's window is not currently capturing the mouse.
	 * Used by the UI system to easily receive calls to InputAxis while the viewport's mouse capture is disabled.
	 */
	virtual bool RequiresUncapturedAxisInput() const { return false; }

	/**
	* Determine if the viewport client is going to need any keyboard input
	* @return true if keyboard input is needed
	*/
	virtual bool RequiresKeyboardInput() const { return true; }

	/**
	 * Returns true if this viewport is orthogonal.
	 * If hit proxies are ever used in-game, this will need to be
	 * overridden correctly in GameViewportClient.
	 */
	virtual bool IsOrtho() const { return false; }

	/**
	 * Returns true if this viewport is excluding non-game elements from its display
	 */
	virtual bool IsInGameView() const { return false; }

	/**
	 * Sets GWorld to the appropriate world for this client
	 *
	 * @return the previous GWorld
	 */
	virtual class UWorld* ConditionalSetWorld() { return NULL; }

	/**
	 * Restores GWorld to InWorld
	 *
	 * @param InWorld	The world to restore
	 */
	virtual void ConditionalRestoreWorld( class UWorld* InWorld ) {}

	/**
	 * Allow viewport client to override the current capture region
	 *
	 * @param OutCaptureRegion    Ref to rectangle where we will write the overridden region
	 * @return true if capture region has been overridden, false otherwise
	 */
	virtual bool OverrideHighResScreenshotCaptureRegion(FIntRect& OutCaptureRegion) { return false; }

	/**
	 * Get a ptr to the stat unit data for this viewport
	 */
	virtual FStatUnitData* GetStatUnitData() const { return NULL; }

	/**
	 * Get a ptr to the stat thermals data for this viewport
	 */
	virtual FStatThermalsData* GetStatThermalsData() const { return NULL; }

	/**
	* Get a ptr to the stat unit data for this viewport
	*/
	virtual FStatHitchesData* GetStatHitchesData() const { return NULL; }

	/**
	 * Get a ptr to the enabled stats list
	 */
	virtual const TArray<FString>* GetEnabledStats() const { return NULL; }

	/**
	 * Sets all the stats that should be enabled for the viewport
	 */
	virtual void SetEnabledStats(const TArray<FString>& InEnabledStats) {}

	/**
	 * Check whether a specific stat is enabled for this viewport
	 */
	virtual bool IsStatEnabled(const FString& InName) const { return false; }

	/**
	* Sets whether stats should be visible for the viewport
	*/
	virtual void SetShowStats(bool bWantStats) { }

	/**
	 * Check whether we should ignore input.
	 */
	virtual bool IgnoreInput() { return false; }

	/**
	 * Gets the mouse capture behavior when the viewport is clicked
	 */
	virtual EMouseCaptureMode GetMouseCaptureMode() const { return EMouseCaptureMode::CapturePermanently; }

	/**
	 * Gets whether or not the viewport captures the Mouse on launch of the application
	 * Technically this controls capture on the first window activate, so in situations
	 * where the application is launched but isn't activated the effect is delayed until
	 * activation.
	 */
	virtual bool CaptureMouseOnLaunch() { return true; }

	/**
	 * Gets whether or not the cursor is locked to the viewport when the viewport captures the mouse
	 */
	virtual bool LockDuringCapture() { return true; }

	/**
	 * Gets whether or not the cursor should always be locked to the viewport.
	 */
	virtual bool ShouldAlwaysLockMouse() { return false; }

	/**
	 * Gets whether or not the cursor is hidden when the viewport captures the mouse
	 */
	virtual bool HideCursorDuringCapture() const { return false; }

	/** 
	 * Should we make new windows for popups or create an overlay in the current window.
	 */
	ENGINE_API virtual FPopupMethodReply OnQueryPopupMethod() const;

	/**
	 * Optionally do custom handling of a navigation. 
	 */
	virtual bool HandleNavigation(const uint32 InUserIndex, TSharedPtr<SWidget> InDestination) { return false; }

	/**
	 * @return Whether or not the scene canvas should be scaled.  Note: The debug canvas is always scaled 
	 */
	virtual bool ShouldDPIScaleSceneCanvas() const { return true; }

	/**
	 * @return The DPI Scale of this viewport
	 */
	virtual float GetDPIScale() const { return 1.0f; }

	/**
	 * Notifies this client that it has gained an association with a viewport.
	 * Client-to-viewport is a one-to-many relationship; this is called once per associated viewport.
	 * Override to install per-association resources (e.g. for slate-backed viewports, overlay widgets on the SViewport).
	 */
	virtual void AddAssociation(FViewport& InViewport) {}

	/**
	 * Notifies this client that an association established via AddAssociation is being torn down.
	 * Override to undo any per-association setup performed in AddAssociation.
	 */
	virtual void RemoveAssociation(FViewport& InViewport) {}
};

/**
 * Common functionality for game and editor viewport clients
 */

class FCommonViewportClient : public FViewportClient
{
public:
	FCommonViewportClient()
		: CachedDPIScale(1.0f)
		, bShouldUpdateDPIScale(true)
	{}

	ENGINE_API virtual ~FCommonViewportClient() override;

	/** Tells this viewport to update editor dpi scale when needed */
	ENGINE_API void RequestUpdateDPIScale();

	/** @return the current resolution fraction to be used for scene rendering in this client. */
	ENGINE_API float GetDPIDerivedResolutionFraction() const;

	/**
	 * @return The DPI Scale of this viewport
	 */
	ENGINE_API virtual float GetDPIScale() const override;

	ENGINE_API void DrawHighResScreenshotCaptureRegion(FCanvas& Canvas);

protected:
	/** @return the DPI scale of the window that the viewport client is in */
	virtual float UpdateViewportClientWindowDPIScale() const { return 1.0; }

private:
	mutable float CachedDPIScale;
	mutable bool bShouldUpdateDPIScale;
};

