// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"

class FVirtualShadowMapVisualizationData
{
public:
	enum class FModeType : uint8
	{
		Standard,
		Advanced,
	};

	/** Describes a single available visualization mode. */
	struct FModeRecord
	{
		FString        ModeString;
		FName          ModeName;
		FText          ModeText;
		FText          ModeDesc;
		FModeType      ModeType;
		int32          ModeID;
		EViewModeIndex ViewModeIndex;
	};

	/** Mapping of FName to a visualization mode record. */
	typedef TMap<FName, FModeRecord> TModeMap;

public:
	FVirtualShadowMapVisualizationData()
	: bIsInitialized(false)
	{
	}

	/** Initialize the system. */
	void Initialize();

	/** Check if system was initialized. */
	inline bool IsInitialized() const { return bIsInitialized; }

	/** Check if visualization is active. */
	ENGINE_API bool IsActive() const;

	/** Update state and check if visualization is active. */
	ENGINE_API bool Update(const FName& InViewMode);

	/** Get the display name of a named mode from the available mode map. **/
	ENGINE_API FText GetModeDisplayName(const FName& InModeName) const;

	ENGINE_API EViewModeIndex GetViewModeIndexForVisualization(const FName& InModeName) const;

	ENGINE_API bool IsVSMViewMode(EViewModeIndex ViewModeIndex) const;

	ENGINE_API int32 GetModeID(const FName& InModeName) const;

	inline const TModeMap& GetModeMap() const
	{
		return ModeMap;
	}

	inline int32 GetActiveModeID() const
	{
		return ActiveVisualizationModeID;
	}

	inline const FName& GetActiveModeName() const
	{
		return ActiveVisualizationModeName;
	}

	/** Return the console command name for enabling single mode visualization. */
	static const TCHAR* GetVisualizeConsoleCommandName()
	{
		return TEXT("r.Shadow.Virtual.Visualize");
	}

	int SelectAdjacentVisualizeLight = 0;

private:
	/** Internal helper function for creating the VirtualShadowMap visualization system console commands. */
	void ConfigureConsoleCommand();

	void AddVisualizationMode(
		const TCHAR* ModeString,
		const FText& ModeText,
		const FText& ModeDesc,
		const FModeType ModeType,
		int32 ModeID,
		EViewModeIndex ViewModeIndex = VMI_VisualizeVirtualShadowMap
	);

	void SetActiveMode(int32 ModeID, const FName& ModeName);

private:
	/** The name->mode mapping table */
	TModeMap ModeMap;

	int32 ActiveVisualizationModeID = INDEX_NONE;
	FName ActiveVisualizationModeName = NAME_None;

	/** Storage for console variable documentation strings. **/
	FString ConsoleDocumentationVisualizationMode;

	/** Flag indicating if system is initialized. **/
	bool bIsInitialized;
};

ENGINE_API FVirtualShadowMapVisualizationData& GetVirtualShadowMapVisualizationData();
