// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Misc/ScopeRWLock.h"
#include "Stats/Stats.h"
#include "UObject/Object.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "Containers/Ticker.h"
#include "EngineDefines.h"
#include "Logging/LogMacros.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "UObject/ObjectKey.h"
#include "VisualLogger/VisualLoggerTypes.h"

#define UE_API ENGINE_API

// helper macros
#define TEXT_EMPTY TEXT("")
#define TEXT_NULL TEXT("NULL")
#define TEXT_TRUE TEXT("TRUE")
#define TEXT_FALSE TEXT("FALSE")
#define TEXT_CONDITION(Condition) ((Condition) ? TEXT_TRUE : TEXT_FALSE)

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogVisual, Display, All);

#define CONNECT_WITH_VLOG(Dest) UE_DEPRECATED_MACRO(5.8, "CONNECT_WITH_VLOG is no longer used")
#define CONNECT_OBJECT_WITH_VLOG(Src, Dest) UE_DEPRECATED_MACRO(5.8, "CONNECT_OBJECT_WITH_VLOG is no longer used")

#if UE_DEBUG_RECORDING_ENABLED

#define REDIRECT_TO_VLOG(Dest) FVisualLogger::Redirect(this, Dest)
#define REDIRECT_OBJECT_TO_VLOG(Src, Dest) FVisualLogger::Redirect(Src, Dest)

// Text, regular log
#define UE_VLOG(LogOwner, CategoryName, Verbosity, Format, ...) if( FVisualLogger::IsRecording() ) FVisualLogger::CategorizedLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Format, ##__VA_ARGS__)
#define UE_CVLOG(Condition, LogOwner, CategoryName, Verbosity, Format, ...)  if(FVisualLogger::IsRecording() && Condition) {UE_VLOG(LogOwner, CategoryName, Verbosity, Format, ##__VA_ARGS__);} 
// Text, log with output to regular unreal logs too.
#define UE_VLOG_UELOG(LogOwner, CategoryName, Verbosity, Format, ...) { if(FVisualLogger::IsRecording()) FVisualLogger::CategorizedLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Format, ##__VA_ARGS__); UE_LOG(CategoryName, Verbosity, Format, ##__VA_ARGS__); }

// Segment shape
#define UE_VLOG_SEGMENT(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::SegmentLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, SegmentStart, SegmentEnd, Color, 0, Format, ##__VA_ARGS__)
#define UE_CVLOG_SEGMENT(Condition, LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_SEGMENT(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Format, ##__VA_ARGS__);}
// Segment shape
#define UE_VLOG_SEGMENT_THICK(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Thickness, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::SegmentLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, SegmentStart, SegmentEnd, Color, Thickness, Format, ##__VA_ARGS__)
#define UE_CVLOG_SEGMENT_THICK(Condition, LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Thickness, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_SEGMENT_THICK(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Thickness, Format, ##__VA_ARGS__);} 
// Localization as sphere shape
#define UE_VLOG_LOCATION(LogOwner, CategoryName, Verbosity, Location, Thickness, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::LocationLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Location, Thickness, Color, Format, ##__VA_ARGS__)
#define UE_CVLOG_LOCATION(Condition, LogOwner, CategoryName, Verbosity, Location, Thickness, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_LOCATION(LogOwner, CategoryName, Verbosity, Location, Thickness, Color, Format, ##__VA_ARGS__);} 
// Sphere shape
#define UE_VLOG_SPHERE(LogOwner, CategoryName, Verbosity, Location, Radius, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::SphereLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Location, Radius, Color, /*bWireframe = */false, Format, ##__VA_ARGS__)
#define UE_CVLOG_SPHERE(Condition, LogOwner, CategoryName, Verbosity, Location, Radius, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_SPHERE(LogOwner, CategoryName, Verbosity, Location, Radius, Color, Format, ##__VA_ARGS__);} 
// Wire sphere shape
#define UE_VLOG_WIRESPHERE(LogOwner, CategoryName, Verbosity, Location, Radius, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::SphereLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Location, Radius, Color, /*bWireframe = */true, Format, ##__VA_ARGS__)
#define UE_CVLOG_WIRESPHERE(Condition, LogOwner, CategoryName, Verbosity, Location, Radius, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_WIRESPHERE(LogOwner, CategoryName, Verbosity, Location, Radius, Color, Format, ##__VA_ARGS__);} 
// Box shape
#define UE_VLOG_BOX(LogOwner, CategoryName, Verbosity, Box, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::BoxLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Box, FMatrix::Identity, Color, /*bWireframe = */false, Format, ##__VA_ARGS__)
#define UE_CVLOG_BOX(Condition, LogOwner, CategoryName, Verbosity, Box, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_BOX(LogOwner, CategoryName, Verbosity, Box, Color, Format, ##__VA_ARGS__);} 
// Wire box shape
#define UE_VLOG_WIREBOX(LogOwner, CategoryName, Verbosity, Box, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::BoxLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Box, FMatrix::Identity, Color, /*bWireframe = */true, Format, ##__VA_ARGS__)
#define UE_CVLOG_WIREBOX(Condition, LogOwner, CategoryName, Verbosity, Box, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_WIREBOX(LogOwner, CategoryName, Verbosity, Box, Color, Format, ##__VA_ARGS__);} 
// Oriented box shape
#define UE_VLOG_OBOX(LogOwner, CategoryName, Verbosity, Box, Matrix, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::BoxLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Box, Matrix, Color, /*bWireframe = */false, Format, ##__VA_ARGS__)
#define UE_CVLOG_OBOX(Condition, LogOwner, CategoryName, Verbosity, Box, Matrix, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_OBOX(LogOwner, CategoryName, Verbosity, Box, Matrix, Color, Format, ##__VA_ARGS__);} 
// Wire oriented box shape
#define UE_VLOG_WIREOBOX(LogOwner, CategoryName, Verbosity, Box, Matrix, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::BoxLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Box, Matrix, Color, /*bWireframe = */true, Format, ##__VA_ARGS__)
#define UE_CVLOG_WIREOBOX(Condition, LogOwner, CategoryName, Verbosity, Box, Matrix, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_WIREOBOX(LogOwner, CategoryName, Verbosity, Box, Matrix, Color, Format, ##__VA_ARGS__);} 
// Cone shape
#define UE_VLOG_CONE(LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::ConeLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Origin, Direction, Length, Angle, Color, /*bWireframe = */false, Format, ##__VA_ARGS__)
#define UE_CVLOG_CONE(Condition, LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_CONE(LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, Format, ##__VA_ARGS__);} 
// Wire cone shape
#define UE_VLOG_WIRECONE(LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::ConeLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Origin, Direction, Length, Angle, Color, /*bWireframe = */true, Format, ##__VA_ARGS__)
#define UE_CVLOG_WIRECONE(Condition, LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_WIRECONE(LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, Format, ##__VA_ARGS__);} 
// Cylinder shape
#define UE_VLOG_CYLINDER(LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::CylinderLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Start, End, Radius, Color, /*bWireframe = */false, Format, ##__VA_ARGS__)
#define UE_CVLOG_CYLINDER(Condition, LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_CYLINDER(LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, Format, ##__VA_ARGS__);} 
// Wire cylinder shape
#define UE_VLOG_WIRECYLINDER(LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::CylinderLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Start, End, Radius, Color, /*bWireframe = */true, Format, ##__VA_ARGS__)
#define UE_CVLOG_WIRECYLINDER(Condition, LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_WIRECYLINDER(LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, Format, ##__VA_ARGS__);} 
// Capsule shape
#define UE_VLOG_CAPSULE(LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::CapsuleLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Base, HalfHeight, Radius, Rotation, Color, /*bWireframe = */false, Format, ##__VA_ARGS__)
#define UE_CVLOG_CAPSULE(Condition, LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_CAPSULE(LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, Format, ##__VA_ARGS__);} 
// Wire capsule shape
#define UE_VLOG_WIRECAPSULE(LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::CapsuleLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Base, HalfHeight, Radius, Rotation, Color, /*bWireframe = */true, Format, ##__VA_ARGS__)
#define UE_CVLOG_WIRECAPSULE(Condition, LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_WIRECAPSULE(LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, Format, ##__VA_ARGS__);} 
// Histogram data for 2d graphs 
#define UE_VLOG_HISTOGRAM(LogOwner, CategoryName, Verbosity, GraphName, DataName, Data) if(FVisualLogger::IsRecording()) FVisualLogger::HistogramDataLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, GraphName, DataName, Data, FColor::White, TEXT(""))
#define UE_CVLOG_HISTOGRAM(Condition, LogOwner, CategoryName, Verbosity, GraphName, DataName, Data) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_HISTOGRAM(LogOwner, CategoryName, Verbosity, GraphName, DataName, Data);} 
// NavArea or vertically pulled convex shape
#define UE_VLOG_PULLEDCONVEX(LogOwner, CategoryName, Verbosity, ConvexPoints, MinZ, MaxZ, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::PulledConvexLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, ConvexPoints, MinZ, MaxZ, Color, Format, ##__VA_ARGS__)
#define UE_CVLOG_PULLEDCONVEX(Condition, LogOwner, CategoryName, Verbosity, ConvexPoints, MinZ, MaxZ, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_PULLEDCONVEX(LogOwner, CategoryName, Verbosity, ConvexPoints, MinZ, MaxZ, Color, Format, ##__VA_ARGS__);}
// regular 3d mesh shape to log
#define UE_VLOG_MESH(LogOwner, CategoryName, Verbosity, Vertices, Indices, Color, Format, ...) if (FVisualLogger::IsRecording()) FVisualLogger::MeshLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Vertices, Indices, Color, Format, ##__VA_ARGS__)
#define UE_CVLOG_MESH(Condition, LogOwner, CategoryName, Verbosity, Vertices, Indices, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_MESH(LogOwner, CategoryName, Verbosity, Vertices, Indices, Color, Format, ##__VA_ARGS__);}
// 2d convex poly shape
#define UE_VLOG_CONVEXPOLY(LogOwner, CategoryName, Verbosity, Points, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::ConvexLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Points, Color, Format, ##__VA_ARGS__)
#define UE_CVLOG_CONVEXPOLY(Condition, LogOwner, CategoryName, Verbosity, Points, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_CONVEXPOLY(LogOwner, CategoryName, Verbosity, Points, Color, Format, ##__VA_ARGS__);}
// Segment with an arrowhead
#define UE_VLOG_ARROW(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::ArrowLineLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, SegmentStart, SegmentEnd, Color, 0, Format, ##__VA_ARGS__)
#define UE_CVLOG_ARROW(Condition, LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_ARROW(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Format, ##__VA_ARGS__);} 
// Segment with an arrowhead with a custom arrow size
#define UE_VLOG_ARROW_MAG(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Mag, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::ArrowLineLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, SegmentStart, SegmentEnd, Color, Mag, Format, ##__VA_ARGS__)
#define UE_CVLOG_ARROW_MAG(Condition, LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Mag, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_ARROW_MAG(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Mag, Format, ##__VA_ARGS__);} 
// Circle shape
#define UE_VLOG_CIRCLE(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::DiscLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Center, UpAxis, Radius, Color, 0, /*bWireframe = */false, Format, ##__VA_ARGS__)
#define UE_CVLOG_CIRCLE(Condition, LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_CIRCLE(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Format, ##__VA_ARGS__);} 
// Circle shape
#define UE_VLOG_CIRCLE_THICK(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::DiscLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Center, UpAxis, Radius, Color, Thickness, /*bWireframe = */false, Format, ##__VA_ARGS__)
#define UE_CVLOG_CIRCLE_THICK(Condition, LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_CIRCLE_THICK(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, Format, ##__VA_ARGS__);} 
// Wire circle shape
#define UE_VLOG_WIRECIRCLE(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::DiscLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Center, UpAxis, Radius, Color, 0, /*bWireframe = */true, Format, ##__VA_ARGS__)
#define UE_CVLOG_WIRECIRCLE(Condition, LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_WIRECIRCLE(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Format, ##__VA_ARGS__);} 
// Wire circle shape
#define UE_VLOG_WIRECIRCLE_THICK(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::DiscLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, Center, UpAxis, Radius, Color, Thickness, /*bWireframe = */true, Format, ##__VA_ARGS__)
#define UE_CVLOG_WIRECIRCLE_THICK(Condition, LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_WIRECIRCLE_THICK(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, Format, ##__VA_ARGS__);} 
// Coordinate system
#define UE_VLOG_COORDINATESYSTEM(LogOwner, CategoryName, Verbosity, AxisLoc, AxisRot, Scale, Color, Thickness, Format, ...) if(FVisualLogger::IsRecording()) FVisualLogger::CoordinateSystemLogf(LogOwner, CategoryName, ELogVerbosity::Verbosity, AxisLoc, AxisRot, Scale, Color, Thickness, Format, ##__VA_ARGS__)
#define UE_CVLOG_COORDINATESYSTEM(Condition, LogOwner, CategoryName, Verbosity, AxisLoc, AxisRot, Scale, Color, Thickness, Format, ...) if(FVisualLogger::IsRecording() && Condition) {UE_VLOG_COORDINATESYSTEM(LogOwner, CategoryName, Verbosity, AxisLoc, AxisRot, Scale, Color, Thickness, Format, ##__VA_ARGS__);}

#else // UE_DEBUG_RECORDING_ENABLED

#define REDIRECT_TO_VLOG(Dest)
#define REDIRECT_OBJECT_TO_VLOG(Src, Dest)

#define UE_VLOG(LogOwner, CategoryName, Verbosity, Format, ...)
#define UE_CVLOG(Condition, LogOwner, CategoryName, Verbosity, Format, ...)
// Make sure to UE_LOG when Visual Logging is disabled.
#define UE_VLOG_UELOG(LogOwner, CategoryName, Verbosity, Format, ...) { UE_LOG(CategoryName, Verbosity, Format, ##__VA_ARGS__); }

#define UE_VLOG_SEGMENT(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, DescriptionFormat, ...)
#define UE_CVLOG_SEGMENT(Condition, LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, DescriptionFormat, ...)
#define UE_VLOG_SEGMENT_THICK(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Thickness, DescriptionFormat, ...)
#define UE_CVLOG_SEGMENT_THICK(Condition, LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Thickness, DescriptionFormat, ...)
#define UE_VLOG_LOCATION(LogOwner, CategoryName, Verbosity, Location, Thickness, Color, DescriptionFormat, ...)
#define UE_CVLOG_LOCATION(Condition, LogOwner, CategoryName, Verbosity, Location, Thickness, Color, DescriptionFormat, ...)
#define UE_VLOG_SPHERE(LogOwner, CategoryName, Verbosity, Location, Radius, Color, DescriptionFormat, ...)
#define UE_CVLOG_SPHERE(Condition, LogOwner, CategoryName, Verbosity, Location, Radius, Color, DescriptionFormat, ...)
#define UE_VLOG_WIRESPHERE(LogOwner, CategoryName, Verbosity, Location, Radius, Color, DescriptionFormat, ...)
#define UE_CVLOG_WIRESPHERE(Condition, LogOwner, CategoryName, Verbosity, Location, Radius, Color, DescriptionFormat, ...)
#define UE_VLOG_BOX(LogOwner, CategoryName, Verbosity, Box, Color, DescriptionFormat, ...) 
#define UE_CVLOG_BOX(Condition, LogOwner, CategoryName, Verbosity, Box, Color, DescriptionFormat, ...) 
#define UE_VLOG_WIREBOX(LogOwner, CategoryName, Verbosity, Box, Color, DescriptionFormat, ...) 
#define UE_CVLOG_WIREBOX(Condition, LogOwner, CategoryName, Verbosity, Box, Color, DescriptionFormat, ...) 
#define UE_VLOG_OBOX(LogOwner, CategoryName, Verbosity, Box, Matrix, Color, Format, ...)  
#define UE_CVLOG_OBOX(Condition, LogOwner, CategoryName, Verbosity, Box, Matrix, Color, Format, ...) 
#define UE_VLOG_WIREOBOX(LogOwner, CategoryName, Verbosity, Box, Matrix, Color, Format, ...)  
#define UE_CVLOG_WIREOBOX(Condition, LogOwner, CategoryName, Verbosity, Box, Matrix, Color, Format, ...) 
#define UE_VLOG_CONE(LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, DescriptionFormat, ...)
#define UE_CVLOG_CONE(Condition, LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, DescriptionFormat, ...)
#define UE_VLOG_WIRECONE(LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, DescriptionFormat, ...)
#define UE_CVLOG_WIRECONE(Condition, LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, DescriptionFormat, ...)
#define UE_VLOG_CYLINDER(LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, DescriptionFormat, ...)
#define UE_CVLOG_CYLINDER(Condition, LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, DescriptionFormat, ...)
#define UE_VLOG_WIRECYLINDER(LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, DescriptionFormat, ...)
#define UE_CVLOG_WIRECYLINDER(Condition, LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, DescriptionFormat, ...)
#define UE_VLOG_CAPSULE(LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, DescriptionFormat, ...)
#define UE_CVLOG_CAPSULE(Condition, LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, DescriptionFormat, ...)
#define UE_VLOG_WIRECAPSULE(LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, DescriptionFormat, ...)
#define UE_CVLOG_WIRECAPSULE(Condition, LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, DescriptionFormat, ...)
#define UE_VLOG_HISTOGRAM(LogOwner, CategoryName, Verbosity, GraphName, DataName, Data)
#define UE_CVLOG_HISTOGRAM(Condition, LogOwner, CategoryName, Verbosity, GraphName, DataName, Data)
#define UE_VLOG_PULLEDCONVEX(LogOwner, CategoryName, Verbosity, ConvexPoints, MinZ, MaxZ, Color, Format, ...)
#define UE_CVLOG_PULLEDCONVEX(Condition, LogOwner, CategoryName, Verbosity, ConvexPoints, MinZ, MaxZ, Color, Format, ...)
#define UE_VLOG_MESH(LogOwner, CategoryName, Verbosity, Vertices, Indexes, Color, Format, ...) 
#define UE_CVLOG_MESH(Condition, LogOwner, CategoryName, Verbosity, Vertices, Indexes, Color, Format, ...) 
#define UE_VLOG_CONVEXPOLY(LogOwner, CategoryName, Verbosity, Points, Color, Format, ...) 
#define UE_CVLOG_CONVEXPOLY(Condition, LogOwner, CategoryName, Verbosity, Points, Color, Format, ...)
#define UE_VLOG_ARROW(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Format, ...) 
#define UE_CVLOG_ARROW(Condition, LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Format, ...) 
#define UE_VLOG_ARROW_MAG(LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Mag, Format, ...) 
#define UE_CVLOG_ARROW_MAG(Condition, LogOwner, CategoryName, Verbosity, SegmentStart, SegmentEnd, Color, Mag, Format, ...) 
#define UE_VLOG_CIRCLE(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Format, ...)
#define UE_CVLOG_CIRCLE(Condition, LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Format, ...)
#define UE_VLOG_CIRCLE_THICK(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, Format, ...)
#define UE_CVLOG_CIRCLE_THICK(Condition, LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, Format, ...)
#define UE_VLOG_WIRECIRCLE(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Format, ...)
#define UE_CVLOG_WIRECIRCLE(Condition, LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Format, ...)
#define UE_VLOG_WIRECIRCLE_THICK(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, Format, ...)
#define UE_CVLOG_WIRECIRCLE_THICK(Condition, LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, Format, ...)
#define UE_VLOG_COORDINATESYSTEM(LogOwner, CategoryName, Verbosity, AxisLoc, AxisRot, Scale, Color, Thickness, Format, ...)
#define UE_CVLOG_COORDINATESYSTEM(Condition, LogOwner, CategoryName, Verbosity, AxisLoc, AxisRot, Scale, Color, Thickness, Format, ...)

#endif // UE_DEBUG_RECORDING_ENABLED

// Definition is the same with/without UE_DEBUG_RECORDING_ENABLED, because it's dependent on UE_VLOG_UELOG which handles the difference.
// Text, log with output to regular logs if the condition is met... regular log will still happen if condition is true
// even when the compiler switch (UE_DEBUG_RECORDING_ENABLED) is off.
#define UE_CVLOG_UELOG(Condition, LogOwner, CategoryName, Verbosity, Format, ...) if (Condition) { UE_VLOG_UELOG(LogOwner, CategoryName, Verbosity, Format, ##__VA_ARGS__); }

#if UE_DEBUG_RECORDING_USING_VLOG

#define DECLARE_VLOG_EVENT(EventName) extern FVisualLogEventBase EventName;
#define DEFINE_VLOG_EVENT(EventName, Verbosity, UserFriendlyDesc) FVisualLogEventBase EventName(TEXT(#EventName), TEXT(UserFriendlyDesc), ELogVerbosity::Verbosity); 
#define UE_VLOG_EVENTS(LogOwner, TagNameToLog, ...) if (FVisualLogger::IsRecording()) FVisualLogger::EventLog(LogOwner, TagNameToLog, ##__VA_ARGS__)
#define UE_CVLOG_EVENTS(Condition, LogOwner, TagNameToLog, ...) if (FVisualLogger::IsRecording() && Condition) {UE_VLOG_EVENTS(LogOwner, TagNameToLog, ##__VA_ARGS__);}
#define UE_VLOG_EVENT_WITH_DATA(LogOwner, LogEvent, ...) if (FVisualLogger::IsRecording()) FVisualLogger::EventLog(LogOwner, LogEvent, ##__VA_ARGS__)
#define UE_CVLOG_EVENT_WITH_DATA(Condition, LogOwner, LogEvent, ...) if (FVisualLogger::IsRecording() && Condition) {UE_VLOG_EVENT_WITH_DATA(LogOwner, LogEvent, ##__VA_ARGS__);}
#define UE_IFVLOG(__code_block__) if (FVisualLogger::IsRecording()) { __code_block__; }

#else // UE_DEBUG_RECORDING_USING_VLOG

#define DECLARE_VLOG_EVENT(EventName)
#define DEFINE_VLOG_EVENT(EventName, Verbosity, UserFriendlyDesc)
#define UE_VLOG_EVENTS(LogOwner, TagNameToLog, ...) 
#define UE_CVLOG_EVENTS(Condition, LogOwner, TagNameToLog, ...)
#define UE_VLOG_EVENT_WITH_DATA(LogOwner, LogEvent, ...)
#define UE_CVLOG_EVENT_WITH_DATA(Condition, LogOwner, LogEvent, ...)
#define UE_IFVLOG(__code_block__)

#endif // UE_DEBUG_RECORDING_USING_VLOG

#if UE_DEBUG_RECORDING_ENABLED

DECLARE_DELEGATE_RetVal(FString, FVisualLogFilenameGetterDelegate);

class FVisualLogger : public FOutputDevice
{
	static UE_API void CategorizedLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...);
	static UE_API void SegmentLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const FColor& Color, const uint16 Thickness, const TCHAR* Fmt, ...);
	static UE_API void LocationLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Center, uint16 Thickness, const FColor& Color, const TCHAR* Fmt, ...);
	static UE_API void SphereLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Center, float Radius, const FColor& Color, bool bWireframe, const TCHAR* Fmt, ...);
	static UE_API void BoxLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FBox& Box, const FMatrix& Matrix, const FColor& Color, bool bWireframe, const TCHAR* Fmt, ...);
	static UE_API void ConeLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Origin, const FVector& Direction, const float Length, const float Angle, const FColor& Color, bool bWireframe, const TCHAR* Fmt, ...);
	static UE_API void CylinderLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const float Radius, const FColor& Color, bool bWireframe, const TCHAR* Fmt, ...);
	static UE_API void CapsuleLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Base, float HalfHeight, float Radius, const FQuat& Rotation, const FColor& Color, bool bWireframe, const TCHAR* Fmt, ...);
	static UE_API void PulledConvexLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TArray<FVector>& ConvexPoints, float MinZ, float MaxZ, const FColor& Color, const TCHAR* Fmt, ...);
	static UE_API void MeshLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TArray<FVector>& Vertices, const TArray<int32>& Indices, const FColor& Color, const TCHAR* Fmt, ...);
	static UE_API void ConvexLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TArray<FVector>& Points, const FColor& Color, const TCHAR* Fmt, ...);
	static UE_API void HistogramDataLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, FName GraphName, FName DataName, const FVector2D& Data, const FColor& Color, const TCHAR* Fmt, ...);
	static UE_API void ArrowLineLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const FColor& Color, const uint16 Mag, const TCHAR* Fmt, ...);
	static UE_API void DiscLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Center, const FVector& UpAxis, const float Radius, const FColor& Color, const uint16 Thickness, bool bWireframe, const TCHAR* Fmt, ...);
	static UE_API void CoordinateSystemLogfImpl(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& AxisLoc, const FRotator& AxisRot, const float Scale, const FColor& Color, const uint16 Thickness, const TCHAR* Fmt, ...);

	static bool IsFilteredOut(const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity)
	{
		return bUseVerbosityFilterWhenRecording
			&& ((Verbosity & ELogVerbosity::VerbosityMask) > Category.GetCompileTimeVerbosity()
				|| Category.IsSuppressed(Verbosity));
	}

public:
	UE_API static bool bUseVerbosityFilterWhenRecording;

	// Regular text log
	template <typename FmtType, typename... Types>
	static void CategorizedLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::CategorizedLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		CategorizedLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void CategorizedLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::CategorizedLogf");

		CategorizedLogfImpl(LogOwner, CategoryName, Verbosity, (const TCHAR*)Fmt, Args...);
	}

	// Segment log
	template <typename FmtType, typename... Types>
	static void SegmentLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const FColor& Color, const uint16 Thickness, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::SegmentLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		SegmentLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Start, End, Color, Thickness, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void SegmentLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const FColor& Color, const uint16 Thickness, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::SegmentLogf");

		SegmentLogfImpl(LogOwner, CategoryName, Verbosity, Start, End, Color, Thickness, (const TCHAR*)Fmt, Args...);
	}

	// Arrow
	template <typename FmtType, typename... Types>
	static void ArrowLineLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const FColor& Color, const uint16 Mag, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::ArrowLineLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		ArrowLineLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Start, End, Color, Mag, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void ArrowLineLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const FColor& Color, const uint16 Mag, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::ArrowLineLogf");

		ArrowLineLogfImpl(LogOwner, CategoryName, Verbosity, Start, End, Color, Mag, (const TCHAR*)Fmt, Args...);
	}

	// Disc/circle log
	template <typename FmtType, typename... Types>
	static void DiscLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FVector& Center, const FVector& UpAxis, const float Radius, const FColor& Color, const uint16 Thickness, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::DiscLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		DiscLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Center, UpAxis, Radius, Color, Thickness, bWireframe, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void DiscLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Center, const FVector& UpAxis, const float Radius, const FColor& Color, const uint16 Thickness, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::DiscLogf");

		DiscLogfImpl(LogOwner, CategoryName, Verbosity, Center, UpAxis, Radius, Color, Thickness, bWireframe, (const TCHAR*)Fmt, Args...);
	}

	// Location log
	template <typename FmtType, typename... Types>
	static void LocationLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FVector& Location, const uint16 Thickness, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::LocationLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		LocationLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Location, Thickness, Color, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void LocationLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Location, const uint16 Thickness, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::LocationLogf");

		LocationLogfImpl(LogOwner, CategoryName, Verbosity, Location, Thickness, Color, (const TCHAR*)Fmt, Args...);
	}

	// Sphere log
	template <typename FmtType, typename... Types>
	static void SphereLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FVector& Location, float Radius, const FColor& Color, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert(TAnd<TIsValidVariadicFunctionArg<Types>...>::Value, "Invalid argument(s) passed to FVisualLogger::SphereLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		SphereLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Location, Radius, Color, bWireframe, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void SphereLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Location, float Radius, const FColor& Color, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert(TAnd<TIsValidVariadicFunctionArg<Types>...>::Value, "Invalid argument(s) passed to FVisualLogger::SphereLogf");

		SphereLogfImpl(LogOwner, CategoryName, Verbosity, Location, Radius, Color, bWireframe, (const TCHAR*)Fmt, Args...);
	}

	// Box log
	template <typename FmtType, typename... Types>
	static void BoxLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FBox& Box, const FMatrix& Matrix, const FColor& Color, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::BoxLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		BoxLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Box, Matrix, Color, bWireframe, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void BoxLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FBox& Box, const FMatrix& Matrix, const FColor& Color, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::BoxLogf");

		BoxLogfImpl(LogOwner, CategoryName, Verbosity, Box, Matrix, Color, bWireframe, (const TCHAR*)Fmt, Args...);
	}

	// Cone log
	template <typename FmtType, typename... Types>
	static void ConeLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FVector& Origin, const FVector& Direction, const float Length, const float Angle, const FColor& Color, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::ConeLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		ConeLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Origin, Direction, Length, Angle, Color, bWireframe, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void ConeLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Origin, const FVector& Direction, const float Length, const float Angle, const FColor& Color, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::ConeLogf");

		ConeLogfImpl(LogOwner, CategoryName, Verbosity, Origin, Direction, Length, Angle, Color, bWireframe, (const TCHAR*)Fmt, Args...);
	}

	// Cylinder log
	template <typename FmtType, typename... Types>
	static void CylinderLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const float Radius, const FColor& Color, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::CylinderLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		CylinderLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Start, End, Radius, Color, bWireframe, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void CylinderLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const float Radius, const FColor& Color, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::CylinderLogf");

		CylinderLogfImpl(LogOwner, CategoryName, Verbosity, Start, End, Radius, Color, bWireframe, (const TCHAR*)Fmt, Args...);
	}

	// Capsule log
	template <typename FmtType, typename... Types>
	static void CapsuleLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FVector& Base, float HalfHeight, float Radius, const FQuat& Rotation, const FColor& Color, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::CapsuleLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		CapsuleLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Base, HalfHeight, Radius, Rotation, Color, bWireframe, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void CapsuleLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Base, float HalfHeight, float Radius, const FQuat& Rotation, const FColor& Color, bool bWireframe, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::CapsuleLogf");

		CapsuleLogfImpl(LogOwner, CategoryName, Verbosity, Base, HalfHeight, Radius, Rotation, Color, bWireframe, (const TCHAR*)Fmt, Args...);
	}

	// NavArea/Extruded convex log
	template <typename FmtType, typename... Types>
	static void PulledConvexLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const TArray<FVector>& ConvexPoints, float MinZ, float MaxZ, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::PulledConvexLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		PulledConvexLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, ConvexPoints, MinZ, MaxZ, Color, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void PulledConvexLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TArray<FVector>& ConvexPoints, float MinZ, float MaxZ, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::PulledConvexLogf");

		PulledConvexLogfImpl(LogOwner, CategoryName, Verbosity, ConvexPoints, MinZ, MaxZ, Color, (const TCHAR*)Fmt, Args...);
	}

	// 3d Mesh log
	template <typename FmtType, typename... Types>
	static void MeshLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const TArray<FVector>& Vertices, const TArray<int32>& Indices, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::MeshLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		MeshLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Vertices, Indices, Color, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void MeshLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TArray<FVector>& Vertices, const TArray<int32>& Indices, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::MeshLogf");

		MeshLogfImpl(LogOwner, CategoryName, Verbosity, Vertices, Indices, Color, (const TCHAR*)Fmt, Args...);
	}

	// 2d Convex shape
	template <typename FmtType, typename... Types>
	static void ConvexLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const TArray<FVector>& Points, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::ConvexLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		ConvexLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, Points, Color, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void ConvexLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TArray<FVector>& Points, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::ConvexLogf");

		ConvexLogfImpl(LogOwner, CategoryName, Verbosity, Points, Color, (const TCHAR*)Fmt, Args...);
	}

	//Histogram data
	template <typename FmtType, typename... Types>
	static void HistogramDataLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, FName GraphName, FName DataName, const FVector2D& Data, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::HistogramDataLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		HistogramDataLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, GraphName, DataName, Data, Color, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void HistogramDataLogf(const UObject* LogOwner, const FName& CategoryName, ELogVerbosity::Type Verbosity, FName GraphName, FName DataName, const FVector2D& Data, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::HistogramDataLogf");

		HistogramDataLogfImpl(LogOwner, CategoryName, Verbosity, GraphName, DataName, Data, Color, (const TCHAR*)Fmt, Args...);
	}

	// Coordinate System
	template <typename FmtType, typename... Types>
	static void CoordinateSystemLogf(const UObject* LogOwner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const FVector& AxisLoc, const FRotator& AxisRot, const float Scale, const FColor& Color, const uint16 Thickness, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::CoordinateSystemDataLogf");
		if (IsFilteredOut(Category, Verbosity))
		{
			return;
		}
		CoordinateSystemLogfImpl(LogOwner, Category.GetCategoryName(), Verbosity, AxisLoc, AxisRot, Scale, Color, Thickness, (const TCHAR*)Fmt, Args...);
	}
	template <typename FmtType, typename... Types>
	static void CoordinateSystemLogf(const UObject* LogOwner, const FName& Category, ELogVerbosity::Type Verbosity, const FVector& AxisLoc, const FRotator& AxisRot, const float Scale, const FColor& Color, const uint16 Thickness, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FVisualLogger::CoordinateSystemDataLogf");

		CoordinateSystemLogfImpl(LogOwner, Category, Verbosity, AxisLoc, AxisRot, Scale, Color, Thickness, (const TCHAR*)Fmt, Args...);
	}

	// static getter
	static UE_API FVisualLogger& Get();

	UE_API FVisualLogger();
	virtual ~FVisualLogger()
	{
	}

	/** return information is vlog recording is enabled or not */
	static bool IsRecording()
	{
		return !!bIsRecording;
	}

	/** Starts visual log collecting and recording */
	UE_API void SetIsRecording(const bool bInIsRecording);

	/** Starts visual log collecting and recording to insights traces (for Rewind Debugger)*/
	UE_API void SetIsRecordingToTrace(const bool bInIsRecording);

	/** Set log owner redirection from one object to another, to combine logs */
	static UE_API void Redirect(const UObject* FromObject, const UObject* ToObject);

	struct FObjectNames
	{
		FName Name;
		FName DisplayName;
		FName ClassName;
	};

	/** Sets function to call to get a timestamp instead of the default implementation (e.g. using a network synchronized clock instead of the local world time) */
	UE_API void SetGetTimeStampFunc(TFunction<double(const UObject*)> Function);

	/** Returns a current time stamp to associate with a recorded event that occurred on Object (used for ordering events on a timeline) */
	UE_API double GetTimeStampForObject(const UObject* Object) const;

	/** Configure whether VisLog should be using decorated, unique names */
	UE_API void SetUseUniqueNames(const bool bEnable);

private:
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override { ensureMsgf(0, TEXT("Regular serialize is forbidden for visual logs")); }

	/** Redirect internal implementation, not thread safe */
	UE_API UObject* RedirectInternal(const UObject* FromObject, const UObject* ToObject);

	/** Find redirects internal implementation, not thread safe */
	UE_API UObject* FindRedirectionInternal(const UObject* Object) const;

	/** Populates the struct of names associated to a given object */
	FObjectNames CreateObjectNames(TNotNull<const UObject*> LogOwner);

	/** Flushes entries recorded in the frame and clean invalid redirects if any. */
	UE_API void Tick(float DeltaTime);

	/** Cleanup invalid redirects */
	UE_API void CleanupRedirects();

public:
	typedef TMap<FObjectKey, TWeakObjectPtr<const UObject>> FChildToOwnerRedirectionMap;

	DECLARE_MULTICAST_DELEGATE_SixParams(FNavigationDataDump, const UObject* /*Object*/, const FName& /*CategoryName*/, const ELogVerbosity::Type /*Verbosity*/, const FBox& /*Box*/, const UWorld& /*World*/, FVisualLogEntry& /*CurrentEntry*/);
	static UE_API FNavigationDataDump NavigationDataDumpDelegate;

#if UE_DEBUG_RECORDING_USING_VLOG

	/** Navigation data debug snapshot */
	static UE_API void NavigationDataDump(const UObject* LogOwner, const FLogCategoryBase& Category, const ELogVerbosity::Type Verbosity, const FBox& Box);
	static UE_API void NavigationDataDump(const UObject* LogOwner, const FName& CategoryName, const ELogVerbosity::Type Verbosity, const FBox& Box);

	/** Log events */
	static UE_API void EventLog(const UObject* LogOwner, const FName EventTag1, const FVisualLogEventBase& Event1, const FName EventTag2 = NAME_None, const FName EventTag3 = NAME_None, const FName EventTag4 = NAME_None, const FName EventTag5 = NAME_None, const FName EventTag6 = NAME_None);
	static UE_API void EventLog(const UObject* LogOwner, const FName EventTag1, const FVisualLogEventBase& Event1, const FVisualLogEventBase& Event2);
	static UE_API void EventLog(const UObject* LogOwner, const FName EventTag1, const FVisualLogEventBase& Event1, const FVisualLogEventBase& Event2, const FVisualLogEventBase& Event3);
	static UE_API void EventLog(const UObject* LogOwner, const FName EventTag1, const FVisualLogEventBase& Event1, const FVisualLogEventBase& Event2, const FVisualLogEventBase& Event3, const FVisualLogEventBase& Event4);
	static UE_API void EventLog(const UObject* LogOwner, const FName EventTag1, const FVisualLogEventBase& Event1, const FVisualLogEventBase& Event2, const FVisualLogEventBase& Event3, const FVisualLogEventBase& Event4, const FVisualLogEventBase& Event5);
	static UE_API void EventLog(const UObject* LogOwner, const FName EventTag1, const FVisualLogEventBase& Event1, const FVisualLogEventBase& Event2, const FVisualLogEventBase& Event3, const FVisualLogEventBase& Event4, const FVisualLogEventBase& Event5, const FVisualLogEventBase& Event6);
	static UE_API void EventLog(const UObject* LogOwner, const FVisualLogEventBase& Event1, const FName EventTag1 = NAME_None, const FName EventTag2 = NAME_None, const FName EventTag3 = NAME_None, const FName EventTag4 = NAME_None, const FName EventTag5 = NAME_None, const FName EventTag6 = NAME_None);

	// called on engine shutdown to flush all, etc.
	UE_API virtual void TearDown() override;

	// Removes all logged data
	UE_API void Cleanup(UWorld* OldWorld, bool bReleaseMemory = false);

	// Use when a visual logger device has discarded all of its data, waiting for new data
	UE_API void OnDataReset();

	/** find and return redirection object for given object*/
	static UE_API UObject* FindRedirection(const UObject* Object);

	/** blocks all categories from logging. It can be bypassed with the category allow list */
	void BlockAllCategories(const bool bInBlock)
	{
		bBlockedAllCategories = bInBlock;
	}

	/** checks if all categories are blocked */
	bool IsBlockedForAllCategories() const
	{
		return !!bBlockedAllCategories;
	}

	/** Returns category allow list for logging */
	const TArray<FName>& GetCategoryAllowList() const
	{
		return CategoryAllowList;
	}

	bool IsCategoryAllowed(const FName& Name) const
	{
		return CategoryAllowList.Find(Name) != INDEX_NONE;
	}

	void AddCategoryToAllowList(FName Category)
	{
		CategoryAllowList.AddUnique(Category);
	}

	void ClearCategoryAllowList()
	{
		CategoryAllowList.Reset();
	}

	/** Generates and returns Id unique for given timestamp - used to connect different logs between (ex. text log with geometry shape) */
	UE_API int32 GetUniqueId(double Timestamp);

	/** Starts visual log collecting and recording */
	UE_API void SetIsRecordingToFile(bool InIsRecording);

	/** return information is vlog recording is enabled or not */
	bool IsRecordingToFile() const
	{
		return !!bIsRecordingToFile;
	}

	/** disables recording to file and discards all data without saving it to file */
	UE_API void DiscardRecordingToFile();

	void SetIsRecordingOnServer(const bool bInIsRecording)
	{
		bIsRecordingOnServer = bInIsRecording;
	}

	bool IsRecordingOnServer() const
	{
		return !!bIsRecordingOnServer;
	}

	/** Add visual logger output device */
	void AddDevice(FVisualLogDevice* InDevice)
	{
		OutputDevices.AddUnique(InDevice);
	}

	/** Remove visual logger output device */
	void RemoveDevice(FVisualLogDevice* InDevice)
	{
		OutputDevices.RemoveSwap(InDevice);
	}

	/** Remove visual logger output device */
	const TArray<FVisualLogDevice*>& GetDevices() const
	{
		return OutputDevices;
	}

	/** Check if log category can be recorded, verify before using GetEntryToWrite! */
	UE_API bool IsCategoryLogged(const FLogCategoryBase& Category) const;

	/** Returns current entry for given TimeStamp or creates another one, but first it serialize previous 
	 *	entry as completed to vislog devices. Use VisualLogger::DontCreate to get current entry without serialization
	 *	@note this function can return null */
	UE_DEPRECATED_FORGAME(5.4, "Use the static GetEntryToWrite instead because this TimeStamp is inconsistent across multiple instances (or threads in Editor).  This function will be made private/protected.")
	UE_API FVisualLogEntry* GetEntryToWrite(const UObject* Object, double TimeStamp, ECreateIfNeeded ShouldCreate = ECreateIfNeeded::Create);

	/** Returns  the current (or new) entry for the given object; alternatively nullptr if we aren't allowed to vlog with the given parameters.
	 * @param LogOwner - The UObject (typically an AActor) we are going to write log entries about.  This becomes the row in the Visual Logger timeline.
	 * @param LogCategory - The LogCategory we are logging about.  This function will only return a valid log entry if the visual logging is enabled for the category.
	 */
	[[nodiscard]] static UE_API FVisualLogEntry* GetEntryToWrite(const UObject* LogOwner, const FLogCategoryBase& LogCategory);

	/**
	 * Executes the provided function using the last used entry for given UObject if any.
	 * Method provides validation for thread safe access when adding information to an existing LogEntry.
	 * @param Object The object to find a LogEntry for.
	 * @param Function Function to execute if a valid view can be created.
	 * @return Whether a LogEntry was found and the function was called
	 */
	UE_API bool ExecuteOnLastEntryForObject(const UObject* Object, TFunctionRef<void(FVisualLogEntry&)> Function);

	/** flush and serialize data if timestamp allows it */
	UE_API virtual void Flush() override;

	/** Moves all threads entries into the global entry map */
	UE_API void FlushThreadsEntries();

	/** FileName getter to set project specific file name for vlogs - highly encouraged to use FVisualLogFilenameGetterDelegate::CreateUObject with this */
	void SetLogFileNameGetter(const FVisualLogFilenameGetterDelegate& InLogFileNameGetter)
	{
		LogFileNameGetter = InLogFileNameGetter;
	}

	/** Register extension to use by LogVisualizer  */
	void RegisterExtension(FName TagName, FVisualLogExtensionInterface* ExtensionInterface)
	{
		check(AllExtensions.Contains(TagName) == false); AllExtensions.Add(TagName, ExtensionInterface);
	}

	/**  Removes previously registered extension */
	void UnregisterExtension(FName TagName, FVisualLogExtensionInterface* ExtensionInterface)
	{
		AllExtensions.Remove(TagName);
	}

	/** returns extension identified by given tag */
	FVisualLogExtensionInterface* GetExtensionForTag(const FName TagName) const
	{
		return AllExtensions.Contains(TagName) ? AllExtensions[TagName] : nullptr;
	}

	/** Returns reference to map with all registered extension */
	const TMap<FName, FVisualLogExtensionInterface*>& GetAllExtensions() const
	{
		return AllExtensions;
	}

	/** internal check for each usage of visual logger */
	static UE_API bool CheckVisualLogInputInternal(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, UWorld** OutWorld, FVisualLogEntry** OutCurrentEntry);

	typedef TMap<FObjectKey, TArray<TWeakObjectPtr<const UObject>>> FOwnerToChildrenRedirectionMap;
	static UE_API FOwnerToChildrenRedirectionMap& GetRedirectionMap(const UObject* InObject);

	FChildToOwnerRedirectionMap& GetChildToOwnerRedirectionMap()
	{
		return ChildToOwnerMap;
	}

	typedef TMap<FObjectKey, TWeakObjectPtr<const UWorld>> FObjectToWorldMapType;
	FObjectToWorldMapType& GetObjectToWorldMap()
	{
		return ObjectToWorldMap;
	}

	UE_API void AddClassToAllowList(UClass& InClass);
	UE_API bool IsClassAllowed(const UClass& InClass) const;

	UE_API void AddObjectToAllowList(const UObject& InObject);
	UE_API void ClearObjectAllowList();
	UE_API bool IsObjectAllowed(const UObject* InObject) const;

	struct FVisualLoggerObjectEntryMap : TMap<FObjectKey, FVisualLogEntry>
	{
		// Multithread access detector for map entries (adding/removing to/from the map or modifying an entry)
		UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(EntriesAccessDetector);
	};

private:

	UE_API FVisualLoggerObjectEntryMap& GetThreadCurrentEntryMap();

	/** Retrieves last used entry for given UObject
	 * @note this function can return null
	 */
	UE_API FVisualLogEntry* GetLastEntryForObject(const UObject* Object);

	/** Get global entry to write where all logs are combined, not thread safe */
	UE_API FVisualLogEntry* GetEntryToWriteInternal(const UObject* Object, double TimeStamp, ECreateIfNeeded ShouldCreate);

	/** Figure out all conditions if this entry is allowed to log */
	UE_API void CalculateEntryAllowLogging(FVisualLogEntry* CurrentEntry, const UObject* LogOwner, const UObject* Object) const;

protected:

	/**
	 * Serializes a single entry and resets it.
	 * Method expects an initialized entry and will ensure otherwise. 
	 */
	UE_API void FlushEntry(FVisualLogEntry& Entry, const FObjectKey& ObjectKey);

	/** Handle to the registered ticker to flush entries */
	FTSTicker::FDelegateHandle TickerHandle;

	/** Array of output devices to redirect to */
	TArray<FVisualLogDevice*> OutputDevices;

	/** Map for inter-objects redirections */
	static UE_API TMap<const UWorld*, FOwnerToChildrenRedirectionMap> WorldToRedirectionMap;

	/** allowed classes - only instances of these classes will be logged.
	 * if ClassAllowList is empty (default) everything will log
	 */
	TArray<UClass*> ClassAllowList;

	/** allowed objects - takes priority over class allow list and should be used to create exceptions in it
	 * if ObjectAllowList is empty (default) everything will log
	 * do NOT read from those pointers, they can be invalid!
	 */
	TSet<FObjectKey> ObjectAllowList;

	/** list of categories that are still allowed to be logged when logging is blocking */
	TArray<FName> CategoryAllowList;

	/** Visual Logger extensions map */
	TMap<FName, FVisualLogExtensionInterface*> AllExtensions;

	/** last generated unique id for given times tamp */
	TMap<double, int32> LastUniqueIds;

	/** Current entry with all data */
	FVisualLoggerObjectEntryMap CurrentEntryPerObject;

	/** Threads current entry maps */
	TArray<FVisualLoggerObjectEntryMap*> ThreadCurrentEntryMaps;

	/** Cached map to world information because it's just raw pointer and not real object */
	FObjectToWorldMapType ObjectToWorldMap;

	/** Delegate to set project specific file name for vlogs */
	FVisualLogFilenameGetterDelegate LogFileNameGetter;

	/** Read Write lock protecting object entries */
	mutable FTransactionallySafeRWLock EntryRWLock;

	/** start recording time */
	double StartRecordingToFileTime;

#if WITH_EDITOR
	/** Handle for registering with PIEStarted to reset the EditorBaseTimeStamp */
	FDelegateHandle PIEStartedHandle;
#endif // WITH_EDITOR

	/** Multithread access detector for the list of thread local FVisualLoggerObjectEntryMap. Each map has its own MT detector for its entries. */
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(ThreadLocalMapListAccessDetector);

	/** if set all categories are blocked from logging */
	bool bBlockedAllCategories : 1;

	/** if set we are recording to file */
	bool bIsRecordingToFile : 1;

	/** variable set (from cheat manager) when logging is active on server */
	bool bIsRecordingOnServer : 1;

	/** Indicates that entries were added/updated and that a flush is required */
	bool bIsFlushRequired : 1;

#else // ^^^ UE_DEBUG_RECORDING_USING_VLOG ^^^ / vvv INSIGHT TRACES ONLY vvv

	/**
	 * Not used when UE_DEBUG_RECORDING_USING_VLOG is not enabled; debug macros use CreateEntryInternal instead.
	 * @note Method is still implemented to allow existing code to compile but will no longer produce logging.
	 */
	static bool CheckVisualLogInputInternal(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, UWorld** OutWorld, FVisualLogEntry** OutCurrentEntry)
	{
		return false;
	}

	/** Creates and sets up a LogEntry intended to be traced synchronously */
	static UE_API bool CreateEntryInternal(const UObject* Object, FVisualLogEntry& OutEntry, FVisualLogger::FObjectNames& Names);

	/** Starts visual log collecting and recording */
	void SetIsRecordingToFile(bool)
	{
	}

	/** return information is vlog recording is enabled or not */
	bool IsRecordingToFile() const
	{
		return false;
	}

	/** disables recording to file and discards all data without saving it to file */
	void DiscardRecordingToFile()
	{
	}

	void SetIsRecordingOnServer(const bool)
	{
	}

	bool IsRecordingOnServer() const
	{
		return false;
	}

	void AddClassToAllowList(UClass&)
	{
	}

	/** Register extension to use by LogVisualizer  */
	void RegisterExtension(FName TagName, FVisualLogExtensionInterface*)
	{
	}

	/**  Removes previously registered extension */
	void UnregisterExtension(FName TagName, FVisualLogExtensionInterface*)
	{
	}

	const TMap<FName, FVisualLogExtensionInterface*>& GetAllExtensions() const
	{
		// Not supported when only recording traces
		static TMap<FName, FVisualLogExtensionInterface*> EmptyExtensionList;
		return EmptyExtensionList;
	}

	bool IsCategoryLogged(const FLogCategoryBase&) const
	{
		return IsRecording();
	}

	bool IsCategoryAllowed(const FName&) const
	{
		return true;
	}

	void AddCategoryToAllowList(FName)
	{
	}

	void ClearCategoryAllowList()
	{
	}

	void BlockAllCategories(const bool)
	{
	}

	static FVisualLogEntry* GetEntryToWrite(const UObject*, const FLogCategoryBase&)
	{
		return nullptr;
	}

	void Cleanup(UWorld*, bool bReleaseMemory = false)
	{
	}

	/** Add visual logger output device */
	void AddDevice(FVisualLogDevice* InDevice)
	{
	}

	/** Remove visual logger output device */
	void RemoveDevice(FVisualLogDevice*)
	{
	}

	const TArray<FVisualLogDevice*>& GetDevices() const
	{
		static TArray<FVisualLogDevice*> EmptyDeviceList;
		return EmptyDeviceList;
	}

	bool ExecuteOnLastEntryForObject(const UObject*, TFunctionRef<void(FVisualLogEntry&)>)
	{
		return false;
	}

	void SetLogFileNameGetter(const FVisualLogFilenameGetterDelegate& InLogFileNameGetter)
	{
	}

#endif // UE_DEBUG_RECORDING_USING_VLOG

private:
	/** controls how we generate log names. When set to TRUE there's a lower
	 * chance of name conflict, but it's more expensive
	 */
	bool bForceUniqueLogNames : 1;

	/** Indicates there are entries in the redirection map that are invalid */
	mutable bool bContainsInvalidRedirects : 1;

	/** if set we are recording to insights trace */
	bool bIsRecordingToTrace : 1;

	/** Read Write lock protecting redirection maps (ChildToOwnerMap and ObjectToWorldMap) */
	mutable FTransactionallySafeRWLock RedirectRWLock;

	/** for any object that has requested redirection this map holds where we should
	 * redirect the traffic to
	 */
	FChildToOwnerRedirectionMap ChildToOwnerMap;

	/** Map to contain various names for Objects (they can be destroyed after while) */
	TMap<FObjectKey, FObjectNames> ObjectToNamesMap;

	/** Function to call when recording the absolute time stamp of an event. Useful for manually aligning events across multiple instances (e.g. using FPlatformTime::Seconds() rather than WorldTime) */
	TFunction<double(const UObject*)> GetTimeStampFunc;

	/** if set we are recording and collecting all vlog data */
	static UE_API int32 bIsRecording;
};

#endif //UE_DEBUG_RECORDING_ENABLED
#undef UE_API