// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "Misc/Guid.h"

// version for vlog binary file format
namespace EVisualLoggerVersion
{
	enum Type
	{
		Initial = 0,
		HistogramGraphsSerialization = 1,
		AddedOwnerClassName = 2,
		StatusCategoryWithChildren = 3,
		TransformationForShapes = 4,
		LargeWorldCoordinatesAndLocationValidityFlag = 5,
		// -----<new versions can be added before this line>-------------------------------------------------
		// - this needs to be the last line (see note below)
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	extern ENGINE_API const FGuid GUID;
}