// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "VisualLoggerDebugSnapshotInterface.generated.h"

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UVisualLoggerDebugSnapshotInterface : public UInterface
{
	GENERATED_BODY()
};

class IVisualLoggerDebugSnapshotInterface
{
	GENERATED_BODY()
public:
	virtual void GrabDebugSnapshot(struct FVisualLogEntry* Snapshot) const {}
};

