// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EngineDefines.h"
#include "Misc/Build.h"
#include "Trace/Config.h"

/** Indicates whether the UE_DEBUG_* macros will record the element (VisualLogger and/or insight traces) */
#ifndef UE_DEBUG_RECORDING_ENABLED
#define UE_DEBUG_RECORDING_ENABLED ((ENABLE_VISUAL_LOG) || (UE_TRACE_MINIMAL_ENABLED && !NO_LOGGING))
#endif

/**
 * Indicates whether the UE_DEBUG_* macros will use the VisualLogger recording process
 * to store entries and write to all registered output devices or use insight traces directly
 */
#ifndef UE_DEBUG_RECORDING_USING_VLOG
#define UE_DEBUG_RECORDING_USING_VLOG ((UE_DEBUG_RECORDING_ENABLED) && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))
#endif

/** Indicates whether the UE_DEBUG_* macros will use immediate debug draw visualization */
#ifndef UE_DEBUG_RENDERING_ENABLED
#define UE_DEBUG_RENDERING_ENABLED ((ENABLE_VISUAL_LOG) || (UE_ENABLE_DEBUG_DRAWING && !NO_LOGGING))
#endif

/** Indicates whether the analysis of the recorded data is active and dedicated UI and tools should be compiled */
#ifndef UE_DEBUG_VISUALIZER_TOOL_ENABLED
#define UE_DEBUG_VISUALIZER_TOOL_ENABLED ((ENABLE_VISUAL_LOG) || ((PLATFORM_DESKTOP) && (WITH_EDITOR)))
#endif