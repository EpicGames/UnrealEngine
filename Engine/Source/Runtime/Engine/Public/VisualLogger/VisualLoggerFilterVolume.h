// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Volume.h"
#include "VisualLoggerFilterVolume.generated.h"

/**
 * A volume to be placed in the level while browsing visual logger output.
 */
UCLASS(MinimalAPI)
class AVisualLoggerFilterVolume : public AVolume
{
	GENERATED_BODY()
public:
	ENGINE_API AVisualLoggerFilterVolume(const FObjectInitializer& ObjectInitializer);
	
};
