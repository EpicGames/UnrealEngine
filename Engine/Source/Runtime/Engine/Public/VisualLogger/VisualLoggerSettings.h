// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "VisualLoggerDefines.h"
#include "VisualLoggerSettings.generated.h"

#define UE_API ENGINE_API

UCLASS(MinimalAPI, config = VisualLogger, defaultconfig, DisplayName = "VisualLogger")
class UVisualLoggerSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	UVisualLoggerSettings() = default;

	/**
	 * Whether logging functions based on LogCategories should record an entry based on the category active verbosity level.
	 * This reduces the amount of recorded entries which directly improves performance when using VisualLogger in an active session (i.e., PIE)
	 * This also implies that the verbosity of the desired categories needs to be set accordingly before, or during, the recording.
	 * Default behavior (false) is to record all entries and the verbosity based filtering is only used inside the tool when analysing the recording.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Recording")
	bool bUseVerbosityFilterWhenRecording = false;

#if UE_DEBUG_RECORDING_ENABLED
	UE_API void SetUseVerbosityFilterWhenRecording(bool bEnable);

protected:
	UE_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
#endif // UE_DEBUG_RECORDING_ENABLED
};

#undef UE_API
