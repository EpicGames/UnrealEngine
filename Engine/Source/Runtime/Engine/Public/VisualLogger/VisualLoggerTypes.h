// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#include "EngineDefines.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "VisualLoggerDefines.h"

#define DECLARE_ENUM_TO_STRING(EnumType) FString EnumToString(const EnumType Value)
#define DEFINE_ENUM_TO_STRING(EnumType, EnumPackage) FString EnumToString(const EnumType Value) \
{ \
	static const UEnum* TypeEnum = FindObject<UEnum>(nullptr, TEXT(EnumPackage) TEXT(".") TEXT(#EnumType)); \
	return TypeEnum->GetNameStringByIndex(static_cast<int32>(Value)); \
}

#if (UE_DEBUG_RECORDING_ENABLED) || (UE_DEBUG_VISUALIZER_TOOL_ENABLED)

#include "Containers/Map.h"
#include "Logging/LogVerbosity.h"
#include "Math/Color.h"
#include "Math/Matrix.h"
#include "UObject/NameTypes.h"

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
#include "VisualLoggerTypes.h"
class AActor;
class UCanvas;
struct FLogEntryItem;
#else
class FVisualLogExtensionInterface;
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED

namespace UE::VisualLogger
{
	ENGINE_API extern const FName NAME_UnnamedCategory;
}

enum class ECreateIfNeeded : int8
{
	Invalid = -1,
	DontCreate = 0,
	Create = 1,
};

#if UE_DEBUG_RECORDING_ENABLED
/** flags describing VisualLogger device's features */
namespace EVisualLoggerDeviceFlags
{
	enum Type
	{
		NoFlags = 0,
		CanSaveToFile = 1,
		StoreLogsLocally = 2,
	};
}
#endif // UE_DEBUG_RECORDING_ENABLED

/** types of shape elements */
enum class EVisualLoggerShapeElement : uint8
{
	Invalid = 0,
	SinglePoint, // individual points, rendered as plain spheres
	Sphere, 
	WireSphere,
	Segment, // pairs of points 
	Path,	// sequence of point
	Box,
	WireBox,
	Cone,
	WireCone,
	Cylinder,
	WireCylinder,
	Capsule,
	WireCapsule,
	Polygon,
	Mesh,
	NavAreaMesh, // convex based mesh with min and max Z values
	Arrow,
	Circle,
	WireCircle,
	CoordinateSystem,
	// note that in order to remain backward compatibility in terms of log
	// serialization new enum values need to be added at the end
};

struct FVisualLogEventBase
{
	const FString Name;
	const FString FriendlyDesc;
	const ELogVerbosity::Type Verbosity;

	FVisualLogEventBase(const FString& InName, const FString& InFriendlyDesc, ELogVerbosity::Type InVerbosity)
		: Name(InName), FriendlyDesc(InFriendlyDesc), Verbosity(InVerbosity)
	{
	}
};

struct FVisualLogEvent
{
	FString Name;
	FString UserFriendlyDesc;
	TEnumAsByte<ELogVerbosity::Type> Verbosity = ELogVerbosity::NoLogging;
	TMap<FName, int32>	 EventTags;
	int32 Counter = 1;
	int64 UserData = 0;
	FName TagName;

	FVisualLogEvent() = default;
	FVisualLogEvent(const FVisualLogEventBase& Event);
	FVisualLogEvent& operator=(const FVisualLogEventBase& Event);

	friend inline bool operator==(const FVisualLogEvent& Left, const FVisualLogEvent& Right) 
	{ 
		return Left.Name == Right.Name; 
	}
};

struct FVisualLogLine
{
	FString Line;
	FName Category;
	TEnumAsByte<ELogVerbosity::Type> Verbosity = ELogVerbosity::NoLogging;
	int32 UniqueId = INDEX_NONE;
	int64 UserData = 0;
	FName TagName;
	FColor Color = FColor::White;
	bool bMonospace = false;
	FVisualLogLine() = default;
	FVisualLogLine(FVisualLogLine&& Other) = default;
	FVisualLogLine(const FVisualLogLine& Other) = default;
	FVisualLogLine(const FName& InCategory, ELogVerbosity::Type InVerbosity, const FString& InLine);
	FVisualLogLine(const FName& InCategory, ELogVerbosity::Type InVerbosity, const FString& InLine, int64 InUserData);
	FVisualLogLine(const FName& InCategory, ELogVerbosity::Type InVerbosity, const FString& InLine, const FColor& InColor, bool bInMonospace);
};

struct FVisualLogStatusCategory
{
	TArray<FString> Data;
	FString Category = UE::VisualLogger::NAME_UnnamedCategory.ToString();
	int32 UniqueId = INDEX_NONE;
	TArray<FVisualLogStatusCategory> Children;

	FVisualLogStatusCategory() = default;
	explicit FVisualLogStatusCategory(const FString& InCategory/* = TEXT("")*/)
		: Category(InCategory)
	{
	}

	void Add(const FString& Key, const FString& Value);
	ENGINE_API bool GetDesc(int32 Index, FString& Key, FString& Value) const;
	void AddChild(const FVisualLogStatusCategory& Child);
};

struct FVisualLogShapeElement
{
	FVisualLogShapeElement(FVisualLogShapeElement&& Other)
		:	Description(MoveTemp(Other.Description)),
			Category(Other.Category),
			Verbosity(Other.Verbosity),
			Points(MoveTemp(Other.Points)),
			TransformationMatrix(Other.TransformationMatrix),
			UniqueId(Other.UniqueId),
			Type(Other.Type),
			Thickness(Other.Thickness)
	{
	}

	FVisualLogShapeElement(const FVisualLogShapeElement& Other) = default;
	FVisualLogShapeElement& operator=(const FVisualLogShapeElement& Other) = default;

	ENGINE_API explicit FVisualLogShapeElement(EVisualLoggerShapeElement InType = EVisualLoggerShapeElement::Invalid);
	FVisualLogShapeElement(const FString& InDescription, const FColor& InColor, uint16 InThickness, const FName& InCategory);

	FString Description;
	FName Category = UE::VisualLogger::NAME_UnnamedCategory;
	TEnumAsByte<ELogVerbosity::Type> Verbosity = ELogVerbosity::All;
	TArray<FVector> Points;
	FMatrix TransformationMatrix = FMatrix::Identity;
	int32 UniqueId = INDEX_NONE;
	EVisualLoggerShapeElement Type = EVisualLoggerShapeElement::Invalid;
	uint8 Color = 0xff;
	union
	{
		uint16 Thickness = 0;
		uint16 Radius;
		uint16 Mag;
	};

	void SetColor(const FColor& InColor);
	EVisualLoggerShapeElement GetType() const;
	void SetType(EVisualLoggerShapeElement InType);
	FColor GetFColor() const;
};

struct FVisualLogHistogramSample
{
	FName Category;
	TEnumAsByte<ELogVerbosity::Type> Verbosity = ELogVerbosity::NoLogging;
	FName GraphName;
	FName DataName;
	FVector2D SampleValue;
	int32 UniqueId = INDEX_NONE;
};

struct FVisualLogDataBlock
{
	FName TagName;
	FName Category;
	TEnumAsByte<ELogVerbosity::Type> Verbosity = ELogVerbosity::NoLogging;
	TArray<uint8> Data;
	int32 UniqueId = INDEX_NONE;
};

struct FVisualLogEntry final
{
	/** For absolute position of events along a timeline (can involve multiple worlds/game instances such as clients and server) */
	double TimeStamp = -1.0;
	/** The time of the event according to its UWorld (can vary widely between game instances such as clients and server) */
	double WorldTimeStamp = -1.0;

	FVector Location = FVector::ZeroVector;
	uint8 bPassedClassAllowList : 1 = false;
	uint8 bPassedObjectAllowList : 1 = false;
	uint8 bIsAllowedToLog : 1 = false;
	uint8 bIsLocationValid : 1 = false;
	uint8 bIsInitialized : 1 = false;

	TArray<FVisualLogEvent> Events;
	TArray<FVisualLogLine> LogLines;
	TArray<FVisualLogStatusCategory> Status;
	TArray<FVisualLogShapeElement> ElementsToDraw;
	TArray<FVisualLogHistogramSample> HistogramSamples;
	TArray<FVisualLogDataBlock> DataBlocks;

	FVisualLogEntry() = default;

	bool ShouldLog(const ECreateIfNeeded ShouldCreate) const
	{
		// We serialize and reinitialize entries only when allowed to log and parameter
		// indicates that new entry can be created.
		return bIsAllowedToLog && ShouldCreate == ECreateIfNeeded::Create;
	}

	bool ShouldFlush(double InTimeStamp) const
	{
		//Same LogOwner can be used for logs at different time in the frame so need to flush entry right away
		return bIsInitialized && InTimeStamp > TimeStamp;
	}

	ENGINE_API void InitializeEntry( const double InTimeStamp );
	ENGINE_API void Reset();
	ENGINE_API void SetPassedObjectAllowList(const bool bPassed);
	ENGINE_API void UpdateAllowedToLog();

	ENGINE_API void AddText(const FString& TextLine, const FName& CategoryName, ELogVerbosity::Type Verbosity);
	// path
	ENGINE_API void AddPath(const TArray<FVector>& Points, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""), uint16 Thickness = 0);
	// location
	ENGINE_API void AddLocation(const FVector& Point, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""), uint16 Thickness = 0);	// location
	// sphere
	ENGINE_API void AddSphere(const FVector& Center, float Radius, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""), bool bInUseWires = false);
	// segment
	ENGINE_API void AddSegment(const FVector& Start, const FVector& End, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""), uint16 Thickness = 0);
	// box
	ENGINE_API void AddBox(const FBox& Box, const FMatrix& Matrix, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""), uint16 Thickness = 0, bool bInUseWires = false);
	// cone
	ENGINE_API void AddCone(const FVector& Origin, const FVector& Direction, float Length, float AngleWidth, float AngleHeight, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""), uint16 Thickness = 0, bool bInUseWires = false);
	// cylinder
	ENGINE_API void AddCylinder(const FVector& Start, const FVector& End, float Radius, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""), uint16 Thickness = 0, bool bInUseWires = false);
	// capsule
	ENGINE_API void AddCapsule(const FVector& Base, float HalfHeight, float Radius, const FQuat & Rotation, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""), bool bInUseWires = false);
	// custom element
	ENGINE_API void AddElement(const FVisualLogShapeElement& Element);
	// NavArea or vertically pulled convex shape
	ENGINE_API void AddPulledConvex(const TArray<FVector>& ConvexPoints, FVector::FReal MinZ, FVector::FReal MaxZ, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""));
	// 3d Mesh
	ENGINE_API void AddMesh(const TArray<FVector>& Vertices, const TArray<int32>& Indices, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""));
	// 2d convex
	ENGINE_API void AddConvexElement(const TArray<FVector>& Points, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""));
	// histogram sample
	ENGINE_API void AddHistogramData(const FVector2D& DataSample, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FName& GraphName, const FName& DataName);
	// arrow
	ENGINE_API void AddArrow(const FVector& Start, const FVector& End, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White, const FString& Description = TEXT(""), const uint16 Mag = 0);
	// boxes
	ENGINE_API void AddBoxes(const TArray<FBox>& Boxes, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color = FColor::White);
	// circle
	ENGINE_API void AddCircle(const FVector& Center, const FVector& UpAxis, const float Radius, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color, const FString& Description = TEXT(""), uint16 Thickness = 0, bool bInUseWires = false);
	// coordinate system
	ENGINE_API void AddCoordinateSystem(const FVector& AxisLoc, const FRotator& AxisRot, const float Scale, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FColor& Color, const FString& Description = TEXT(""), uint16 Thickness = 0);

	// Custom data block
	ENGINE_API FVisualLogDataBlock& AddDataBlock(const FString& TagName, const TArray<uint8>& BlobDataArray, const FName& CategoryName, ELogVerbosity::Type Verbosity);
	// Event
	ENGINE_API int32 AddEvent(const FVisualLogEventBase& Event);
	// find index of status category
	int32 FindStatusIndex(const FString& CategoryName);

	// Moves all content to provided entry and reseting our content.
	ENGINE_API void MoveTo(FVisualLogEntry& Other);
};

/**
 * Interface for Visual Logger Device
 */
class FVisualLogDevice
{
public:
	struct FVisualLogEntryItem
	{
		FVisualLogEntryItem() {}
		FVisualLogEntryItem(const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName, const FVisualLogEntry& InLogEntry) 
			: OwnerName(InOwnerName)
			, OwnerDisplayName(InOwnerDisplayName)
			, OwnerClassName(InOwnerClassName)
			, Entry(InLogEntry)
		{}

		FName OwnerName;
		FName OwnerDisplayName;
		FName OwnerClassName;
		FVisualLogEntry Entry;
	};

	virtual ~FVisualLogDevice() = default;
	virtual void Serialize(const UObject* LogOwner, const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName, const FVisualLogEntry& InLogEntry) = 0;
	virtual void Cleanup(bool bReleaseMemory = false) { /* Empty */ }
	virtual void StartRecordingToFile(double TimeStamp) { /* Empty */ }
	virtual void StopRecordingToFile(double TimeStamp) { /* Empty */ }
	virtual void DiscardRecordingToFile() { /* Empty */ }
	virtual void SetFileName(const FString& InFileName) { /* Empty */ }
	virtual void GetRecordedLogs(TArray<FVisualLogEntryItem>& OutLogs) const { /* Empty */ }
	virtual bool HasFlags(int32 InFlags) const { return false; }
	FGuid GetSessionGUID() const { return SessionGUID; }
	uint32 GetShortSessionID() const { return SessionGUID[0]; }
protected:
	FGuid SessionGUID;
};

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED || UE_DEBUG_RECORDING_USING_VLOG
namespace FVisualLoggerHelpers
{
	extern ENGINE_API FArchive& Serialize(FArchive& Ar, TArray<FVisualLogDevice::FVisualLogEntryItem>& RecordedLogs);
};
#endif

#if UE_DEBUG_VISUALIZER_TOOL_ENABLED
struct FVisualLoggerCategoryVerbosityPair
{
	FVisualLoggerCategoryVerbosityPair(FName Category, ELogVerbosity::Type InVerbosity) : CategoryName(Category), Verbosity(InVerbosity) {}

	FName CategoryName;
	ELogVerbosity::Type Verbosity;

	friend inline bool operator==(const FVisualLoggerCategoryVerbosityPair& A, const FVisualLoggerCategoryVerbosityPair& B)
	{
		return A.CategoryName == B.CategoryName
			&& A.Verbosity == B.Verbosity;
	}
};

namespace FVisualLoggerHelpers
{
	extern ENGINE_API void GetCategories(const FVisualLogEntry& RecordedLogs, TArray<FVisualLoggerCategoryVerbosityPair>& OutCategories);
	extern ENGINE_API void GetHistogramCategories(const FVisualLogEntry& RecordedLogs, TMap<FString, TArray<FString>>& OutCategories);
};

struct IVisualLoggerEditorInterface
{
	virtual const FName& GetRowClassName(FName RowName) const = 0;
	virtual int32 GetSelectedItemIndex(FName RowName) const = 0;
	virtual const TArray<FVisualLogDevice::FVisualLogEntryItem>& GetRowItems(FName RowName) = 0;
	virtual const FVisualLogDevice::FVisualLogEntryItem& GetSelectedItem(FName RowName) const = 0;

	virtual const TArray<FName>& GetSelectedRows() const = 0;
	virtual bool IsRowVisible(FName RowName) const = 0;
	virtual bool IsItemVisible(FName RowName, int32 ItemIndex) const = 0;
	virtual UWorld* GetWorld() const = 0;
	virtual AActor* GetHelperActor(UWorld* InWorld = nullptr) const = 0;

	virtual bool MatchCategoryFilters(const FString& String, ELogVerbosity::Type Verbosity = ELogVerbosity::All) = 0;
};

class FVisualLogExtensionInterface
{
public:
	virtual ~FVisualLogExtensionInterface() = default;

	virtual void ResetData(IVisualLoggerEditorInterface* EdInterface) = 0;
	virtual void DrawData(IVisualLoggerEditorInterface* EdInterface, UCanvas* Canvas) = 0;

	virtual void OnItemsSelectionChanged(IVisualLoggerEditorInterface* EdInterface) {};
	virtual void OnLogLineSelectionChanged(IVisualLoggerEditorInterface* EdInterface, TSharedPtr<FLogEntryItem> SelectedItem, int64 UserData) {};
	virtual void OnScrubPositionChanged(IVisualLoggerEditorInterface* EdInterface, double NewScrubPosition, bool bScrubbing) {}
};

ENGINE_API FArchive& operator<<(FArchive& Ar, FVisualLogDevice::FVisualLogEntryItem& FrameCacheItem);
#endif // UE_DEBUG_VISUALIZER_TOOL_ENABLED

#if UE_DEBUG_RECORDING_ENABLED
namespace FVisualLoggerHelpers
{
	extern ENGINE_API FArchive& Serialize(FArchive& Ar, FName& Name);

#if UE_DEBUG_RECORDING_USING_VLOG
	extern ENGINE_API FString GenerateTemporaryFilename(const FString& FileExt);
	extern ENGINE_API FString GenerateFilename(const FString& TempFileName, const FString& Prefix, double StartRecordingTime, double EndTimeStamp);
#endif // UE_DEBUG_RECORDING_USING_VLOG

}

ENGINE_API FArchive& operator<<(FArchive& Ar, FVisualLogDataBlock& Data);
ENGINE_API FArchive& operator<<(FArchive& Ar, FVisualLogHistogramSample& Sample);
ENGINE_API FArchive& operator<<(FArchive& Ar, FVisualLogShapeElement& Element);
ENGINE_API FArchive& operator<<(FArchive& Ar, FVisualLogEvent& Event);
ENGINE_API FArchive& operator<<(FArchive& Ar, FVisualLogLine& LogLine);
ENGINE_API FArchive& operator<<(FArchive& Ar, FVisualLogStatusCategory& Status);
ENGINE_API FArchive& operator<<(FArchive& Ar, FVisualLogEntry& LogEntry);

#endif // UE_DEBUG_RECORDING_ENABLED

inline
FVisualLogEvent::FVisualLogEvent(const FVisualLogEventBase& Event)
: Name(Event.Name)
, UserFriendlyDesc(Event.FriendlyDesc)
, Verbosity(Event.Verbosity)
{
}

inline
FVisualLogEvent& FVisualLogEvent::operator= (const FVisualLogEventBase& Event)
{
	Name = Event.Name;
	UserFriendlyDesc = Event.FriendlyDesc;
	Verbosity = Event.Verbosity;
	return *this;
}

inline
FVisualLogLine::FVisualLogLine(const FName& InCategory, ELogVerbosity::Type InVerbosity, const FString& InLine)
: Line(InLine)
, Category(InCategory)
, Verbosity(InVerbosity)
{

}

inline
FVisualLogLine::FVisualLogLine(const FName& InCategory, ELogVerbosity::Type InVerbosity, const FString& InLine, int64 InUserData)
: Line(InLine)
, Category(InCategory)
, Verbosity(InVerbosity)
, UserData(InUserData)
{

}

inline
FVisualLogLine::FVisualLogLine(const FName& InCategory, ELogVerbosity::Type InVerbosity, const FString& InLine, const FColor& InColor, bool bInMonospace)
	: Line(InLine)
	, Category(InCategory)
	, Verbosity(InVerbosity)
	, Color(InColor)
	, bMonospace(bInMonospace)
{

}

inline
void FVisualLogStatusCategory::Add(const FString& Key, const FString& Value)
{
	Data.Add(FString(Key).AppendChar(TEXT('|')) + Value);
}

inline
void FVisualLogStatusCategory::AddChild(const FVisualLogStatusCategory& Child)
{
	Children.Add(Child);
}

inline
FVisualLogShapeElement::FVisualLogShapeElement(const FString& InDescription, const FColor& InColor, uint16 InThickness, const FName& InCategory)
: Description(InDescription)
, Category(InCategory)
, Thickness(InThickness)
{
	SetColor(InColor);
}

inline
void FVisualLogShapeElement::SetColor(const FColor& InColor)
{
	Color = (uint8)(((InColor.DWColor() >> 30) << 6)	| (((InColor.DWColor() & 0x00ff0000) >> 22) << 4)	| (((InColor.DWColor() & 0x0000ff00) >> 14) << 2)	| ((InColor.DWColor() & 0x000000ff) >> 6));
}

inline
EVisualLoggerShapeElement FVisualLogShapeElement::GetType() const
{
	return Type;
}

inline
void FVisualLogShapeElement::SetType(EVisualLoggerShapeElement InType)
{
	Type = InType;
}

inline
FColor FVisualLogShapeElement::GetFColor() const
{
	FColor RetColor(((Color & 0xc0) << 24) | ((Color & 0x30) << 18) | ((Color & 0x0c) << 12) | ((Color & 0x03) << 6));
	RetColor.A = (RetColor.A * 255) / 192; // convert alpha to 0-255 range
	return RetColor;
}

inline
int32 FVisualLogEntry::FindStatusIndex(const FString& CategoryName)
{
	for (int32 TestCategoryIndex = 0; TestCategoryIndex < Status.Num(); TestCategoryIndex++)
	{
		if (Status[TestCategoryIndex].Category == CategoryName)
		{
			return TestCategoryIndex;
		}
	}

	return INDEX_NONE;
}

#endif // UE_DEBUG_RECORDING_ENABLED || UE_DEBUG_VISUALIZER_TOOL_ENABLED