// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FVisualizationDataLegendEntry
{
	FText			Label;
	FLinearColor	Color;
};
