// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricCloudSceneProxy.h: FVolumetricCloudSceneProxy definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"



class UVolumetricCloudComponent;
class FVolumetricCloudRenderSceneInfo;
class UMaterialInterface;



/** Represents a UVolumetricCloudComponent to the rendering thread, created game side from the component. */
class FVolumetricCloudSceneProxy
{
public:

	// Initialization constructor.
	ENGINE_API FVolumetricCloudSceneProxy(const UVolumetricCloudComponent* InComponent);
	ENGINE_API ~FVolumetricCloudSceneProxy();

	UMaterialInterface* GetCloudVolumeMaterial() const { return CloudVolumeMaterial; }

	FVolumetricCloudRenderSceneInfo* RenderSceneInfo;

	float LayerBottomAltitudeKm;
	float LayerHeightKm;

	float TracingStartMaxDistance;
	float TracingStartDistanceFromCamera;
	uint8 TracingMaxDistanceMode;
	float TracingMaxDistance;

	float PlanetRadiusKm;
	FColor GroundAlbedo;
	bool bUsePerSampleAtmosphericLightTransmittance;
	bool bHoldout;
	bool bRenderInMainPass;
	bool bVisibleInRealTimeSkyCaptures;
	float SkyLightCloudBottomOcclusion;
	
	float ViewSampleCountScale;
	float ReflectionViewSampleCountScale;
	float ShadowViewSampleCountScale;
	float ShadowReflectionViewSampleCountScale;
	float ShadowTracingDistance;
	float StopTracingTransmittanceThreshold;

	float AerialPespectiveMieScatteringStartDistance;
	float AerialPespectiveMieScatteringFadeDistance;
	float AerialPespectiveRayleighScatteringStartDistance;
	float AerialPespectiveRayleighScatteringFadeDistance;
private:

	UMaterialInterface* CloudVolumeMaterial = nullptr;
};


