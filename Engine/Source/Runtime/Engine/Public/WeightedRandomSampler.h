// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
WeightedRandomSampler.h: 
Helper class for randomly sampling from N entries with non uniform weighting/probability.
Useful for constant data that is sampled many times / sampling is performance critical.

Init time: O(n)
Memory use: O(n)
Sampling time: O(1)

As discussed here:
http://www.keithschwarz.com/darts-dice-coins/

FWeightedRandomSampler is the base class which build the probability and alias tables.
To use, inherit a class from this which implements the GetWeights() function.
GetWeights() should return N positive floating point weights.
Call initialize before sampling.
To sample, call GetEntryIndex().
GetEntryIndex takes two random floating values. This does not generate randoms for you as it's up to the user how to generate their randoms.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Serialization/MemoryImage.h"
#include "Serialization/MemoryLayout.h"

struct FWeightedRandomSampler
{
	DECLARE_TYPE_LAYOUT(FWeightedRandomSampler, Abstract);
public:
	ENGINE_API FWeightedRandomSampler();
	ENGINE_API virtual ~FWeightedRandomSampler();

	/**
	Takes two random values (0...1) and returns the corresponding element index.
	Returns INDEX_NONE if there are no entries.
	*/
	inline int32 GetEntryIndex(float R0, float R1)const
	{
		if(Prob.Num() > 0)
		{			
			int32 Idx = FMath::Clamp((int32)(R0 * float(Prob.Num())), 0, Prob.Num() - 1);
			return R1 < Prob[Idx] ? Idx : Alias[Idx];
		}
		return INDEX_NONE;
	}

	inline float GetTotalWeight()const { return TotalWeight; }

	ENGINE_API virtual void Initialize();

	ENGINE_API virtual void Serialize(FArchive& Ar);

	int32 GetNumEntries() const { return Prob.Num(); }

	TArrayView<const float> GetProb() const { return Prob; }
	TArrayView<const int32> GetAlias() const { return Alias; }

protected:

	/** Gets the weight of all elements and returns their sum. */
	virtual float GetWeights(TArray<float>& OutWeights) = 0;

	LAYOUT_FIELD(TMemoryImageArray<float>, Prob);
	LAYOUT_FIELD(TMemoryImageArray<int32>, Alias);
	LAYOUT_FIELD(float, TotalWeight);
};
