// Copyright Epic Games, Inc. All Rights Reserved.

// Structs that are used for Async Trace functionality
// Mostly used by a batch of traces that you don't need a result right away

#pragma once 

#include "Async/TaskGraphFwd.h"
#include "CollisionQueryParams.h"
#include "CollisionShape.h"

struct FOverlapDatum;
struct FTraceDatum;
struct FHitResult;
struct FOverlapResult;
typedef TArray<FGraphEventRef, TInlineAllocator<4> > FGraphEventArray;

/** Trace Data Structs that are used for Async Trace */

/** Trace Handle - unique ID struct that is returned once trace is requested for tracking purpose **/
struct FTraceHandle
{
	/** Handle is created by FrameNumber + Index of the request */
	union
	{
		uint64	_Handle;
		struct  
		{
			uint32 FrameNumber;
			uint32 Index;
		} _Data;
	};
	/** Denotes that this trace was in an auto-rtfm transactional context */
	bool bTransactional = false;

	FTraceHandle() : _Handle(0) {};
	FTraceHandle(uint32 InFrameNumber, uint32 InIndex, bool bInTransactional = false)
	{
		_Data.FrameNumber = InFrameNumber;
		_Data.Index = InIndex;
		bTransactional = bInTransactional;
	}

	friend inline uint32 GetTypeHash(const FTraceHandle& Handle)
	{
		return GetTypeHash(Handle._Handle);
	}

	bool operator==(FTraceHandle const& Other) const
	{
		return Other._Handle == _Handle;
	}

	bool operator!=(FTraceHandle const& Other) const
	{
		return Other._Handle != _Handle;
	}

	bool IsValid() const
	{
		return _Handle != 0;
	}

	void Invalidate()
	{
		_Handle = 0;
	}

	bool IsTransactional() const
	{
		return bTransactional;
	}
};

/** 
 * Sets of Collision Parameters to run the async trace
 * 
 * It includes basic Query Parameter, Response Parameter, Object Query Parameter as well as Shape of collision testing 
 *
 */
struct FCollisionParameters
{
	/** Collision Trace Parameters **/
	struct FCollisionQueryParams		CollisionQueryParam;
	struct FCollisionResponseParams		ResponseParam;
	struct FCollisionObjectQueryParams	ObjectQueryParam;

	/** Contains Collision Shape data including dimension of the shape **/
	struct FCollisionShape CollisionShape;

	FCollisionParameters()
		: CollisionQueryParam(NAME_None, TStatId())
	{
	}
};

/** 
 * Base Async Trace Data Struct for both overlap and trace 
 * 
 * Contains basic data that will need for handling trace 
 * such as World, Collision parameters and so on. 
 */
struct FBaseTraceDatum
{
	/** Physics World this trace will run in ***/
	TWeakObjectPtr<UWorld> PhysWorld;

	/** Collection of collision parameters */
	FCollisionParameters	CollisionParams;

	/** Collsion Trace Channel that this trace is running*/
	ECollisionChannel TraceChannel;

	/** Framecount when requested is made*/
	uint32	FrameNumber; 

	/** User data */
	uint32	UserData;

	FBaseTraceDatum() {};

	/** Set functions for each Shape type */
	void Set(UWorld * World, const FCollisionShape& InCollisionShape, const FCollisionQueryParams& Param, const struct FCollisionResponseParams &InResponseParam, const struct FCollisionObjectQueryParams& InObjectQueryParam, 
		ECollisionChannel Channel, uint32 InUserData, int32 FrameCounter);
};

struct FTraceDatum;
struct FOverlapDatum;

/**
 * This is Trace/Sweep Delegate that can be used if you'd like to get notified whenever available
 * Otherwise, you'll have to query manually using your TraceHandle 
 * 
 * @param	FTraceHandle	TraceHandle that is returned when requested
 * @param	FTraceDatum		TraceDatum that includes input/output
 */
DECLARE_DELEGATE_TwoParams( FTraceDelegate, const FTraceHandle&, FTraceDatum &);
/**
 * This is Overlap Delegate that can be used if you'd like to get notified whenever available
 * Otherwise, you'll have to query manually using your TraceHandle
 * 
 * @param	FTHandle		TraceHandle that is returned when requestsed
 * @param	FOverlapDatum	OverlapDatum that includes input/output
 */
DECLARE_DELEGATE_TwoParams( FOverlapDelegate, const FTraceHandle&, FOverlapDatum &);

/** Enum to indicate type of test to perfom */
enum class EAsyncTraceType : uint8
{
	/** Return whether the trace succeeds or fails (using bBlockingHit flag on FHitResult), but gives no info about what you hit or where. Good for fast occlusion tests. */
	Test,
	/** Returns a single blocking hit */
	Single,
	/** Returns a single blocking hit, plus any overlapping hits up to that point */
	Multi
};

/**
 * Trace/Sweep Data structure for async trace
 * 
 * This saves request information by main thread and result will be filled up by worker thread
 */
struct FTraceDatum : public FBaseTraceDatum
{
	/** 
	 * Input of the trace/sweep request. Filled up by main thread 
	 * 
	 * Start/End of the trace
	 * The Shape is defined in FBaseTraceDatum
	 */
	FVector Start;
	FVector End;
	FQuat	Rot;
	/** Delegate to be set if you want Delegate to be called when the output is available. Filled up by requester (main thread) **/
	FTraceDelegate Delegate;

	/** Output of the overlap request. Filled up by worker thread */
	TArray<struct FHitResult> OutHits;

	/** Whether to do test, single or multi test */
	EAsyncTraceType TraceType;

	ENGINE_API FTraceDatum();

	/** Set Trace Datum for each shape type **/
	ENGINE_API FTraceDatum(UWorld* World, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Param, const struct FCollisionResponseParams& InResponseParam, const struct FCollisionObjectQueryParams& InObjectQueryParam,
		ECollisionChannel Channel, uint32 InUserData, EAsyncTraceType InTraceType, const FVector& InStart, const FVector& InEnd, const FQuat& InRot, const FTraceDelegate* InDelegate, int32 FrameCounter);
};

/**
 * Overlap Data structure for async trace
 * 
 * This saves request information by main thread and result will be filled up by worker thread
 */
struct FOverlapDatum : FBaseTraceDatum
{
	/** 
	 * Input of the overlap request. Filled up by main thread 
	 *
	 * Position/Rotation data of overlap request. 
	 * The Shape is defined in FBaseTraceDatum
	 */
	FVector Pos;
	FQuat	Rot;
	/** Delegate to be set if you want Delegate to be called when the output is available. Filled up by requester (main thread) **/
	FOverlapDelegate Delegate;

	/** Output of the overlap request. Filled up by worker thread */
	TArray<struct FOverlapResult> OutOverlaps;

	ENGINE_API FOverlapDatum();

	ENGINE_API FOverlapDatum(UWorld * World, const FCollisionShape& CollisionShape, const FCollisionQueryParams& Param, const struct FCollisionResponseParams &InResponseParam, const struct FCollisionObjectQueryParams& InObjectQueryParam, 
	              ECollisionChannel Channel, uint32 InUserData,
	              const FVector& InPos, const FQuat& InRot, const FOverlapDelegate* InDelegate, int32 FrameCounter);
};

#define ASYNC_TRACE_BUFFER_SIZE 64

/**
 * Trace Data that one Thread can handle per type. For now ASYNC_TRACE_BUFFER_SIZE is 64. 
 */
template <typename T>
struct TTraceThreadData
{
	T Buffer[ASYNC_TRACE_BUFFER_SIZE];
};

template <typename DatumType>
struct FTransactionalAsyncTraceBuffer
{
	TArray<TUniquePtr<TTraceThreadData<DatumType>>> Data;
	int32 NumQueued;
};

/** 
 * Contains all Async Trace Result for one frame. 
 * 
 * We use double buffer for trace data pool. FrameNumber % 2 = is going to be the one collecting NEW data 
 * Check FWorldAsyncTraceState to see how this is used. For now it is only double buffer, but it can be more. 
 */
struct AsyncTraceData : FNoncopyable
{
	/**
	 * Data Buffer for each trace type - one for Trace/Sweep and one for Overlap
	 *
	 * FTraceThreadData is one atomic data size for thread
	 * so once that's filled up, this will be sent to thread
	 * if you need more, it will allocate new FTraceThreadData and add to TArray
	 *
	 * however when we calculate index of buffer, we calculate continuously, 
	 * meaning if TraceData(1).Buffer[50] will have 1*ASYNC_TRACE_BUFFER_SIZE + 50 as index
	 * in order to give UNIQUE INDEX to every data in this buffer
	 */
	TArray<TUniquePtr<TTraceThreadData<FTraceDatum>>>			TraceData;
	TArray<TUniquePtr<TTraceThreadData<FOverlapDatum>>>			OverlapData;

	/** Datum entries in TraceData are persistent for efficiency. This is the number of them that are actually in use (rather than TraceData.Num()). */
	int32 NumQueuedTraceData;
	/** Datum entries in OverlapData are persistent for efficiency. This is the number of them that are actually in use (rather than OverlapData.Num()). */
	int32 NumQueuedOverlapData;

	/**
	 * Data for Transactional async queries. This is effectively the same as above,
	 * however it must be stored separately as we cannot run any async queries until a transaction is closed.
	 */
	FTransactionalAsyncTraceBuffer<FTraceDatum> TransactionalTraceData;
	FTransactionalAsyncTraceBuffer<FOverlapDatum> TransactionalOverlapData;

	/**
	 * if Execution is all done, set this to be true
	 * 
	 * when reinitialize bAsyncAllowed is true, once execution is done, this will set to be false
	 * this is to find cases where execution is already done but another request is made again within the same frame. 
	 */
	bool					bAsyncAllowed; 

	bool					bAsyncTasksCompleted;

	/**  Thread completion event for batch **/
	FGraphEventArray		AsyncTraceCompletionEvent;

	ENGINE_API AsyncTraceData();
	ENGINE_API ~AsyncTraceData();
};


extern ECollisionChannel DefaultCollisionChannel;
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogCollision, Warning, All);
