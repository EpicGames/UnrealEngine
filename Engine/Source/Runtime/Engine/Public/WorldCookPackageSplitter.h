// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CookPackageSplitter.h"
#include "Templates/OverrideVoidReturnInvoker.h"
#include "Engine/World.h"
#include "Templates/Function.h"
#endif

#define UE_API ENGINE_API

#if WITH_EDITOR
class FWorldCookPackageSplitter : public FGCObject, ICookPackageSplitter
{
	DEFINE_COOKPACKAGE_SPLITTER(FWorldCookPackageSplitter, UWorld);

public:
	~FWorldCookPackageSplitter();

	struct ISubSplitter
	{
		virtual ~ISubSplitter()	= default;
		virtual TArray<ICookPackageSplitter::FGeneratedPackage> GetGenerateList(const UPackage* OwnerPackage) = 0;
		virtual TOptional<bool> RequiresEmptyPackageBeforePopulate(const UPackage* Package) = 0;
		virtual bool PopulateGeneratedPackage(ICookPackageSplitter::FPopulateContext& PopulateContext) = 0;
		virtual bool PopulateGeneratorPackage(ICookPackageSplitter::FPopulateContext& PopulateContext) = 0;
		/**
		 * This can return true to disable the save of the generator package, for fine-tuned output control. Return unset by default.
		 * All splitters should return the same value, or leave unset (as only one sub-splitter should be responsible for each package), otherwise the resulting mismatch will log an error.
		 */
		virtual TOptional<bool> SkipSaveGeneratorPackage(const UPackage* OwnerPackage) = 0;
		virtual void Teardown(ICookPackageSplitter::ETeardown Status) = 0;
	};

	struct FSubSplitterFactory
	{
		TFunction<bool(TNotNull<const UWorld*>)> ShouldSplit;
		TFunction<ISubSplitter*(TNotNull<const UWorld*>)> MakeInstance;
		TFunction<void(TNotNull<const UWorld*>, TNotNull<ISubSplitter*>)> ReleaseInstance;
	};

	static UE_API void RegisterCookPackageSubSplitterFactory(UClass* Class, FSubSplitterFactory Factory);
	static UE_API void UnregisterCookPackageSubSplitterFactory(UClass* Class);

protected:
	static TMap<UClass*, FSubSplitterFactory> RegisteredCookPackageSubSplitterFactories;

	template<typename FuncType>
	static bool ForEachRegisteredCookPackageSubSplitterFactories(FuncType Func)
	{
		TOverrideVoidReturnInvoker Invoker(true, Func);
		for (const TPair<UClass*, FSubSplitterFactory>& RegisteredCookPackageSubSplitterFactory : RegisteredCookPackageSubSplitterFactories)
		{
			if (!Invoker(RegisteredCookPackageSubSplitterFactory.Value))
			{
				return false;
			}
		}
		return true;
	}

	//~Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	//~End FGCObject interface

	//~ Begin of ICookPackageSplitter
	static bool ShouldSplit(UObject* SplitData);
	static FString GetSplitterDebugName() { return TEXT("FWorldCookPackageSplitter"); }

	virtual bool UseInternalReferenceToAvoidGarbageCollect() override;
	virtual EGeneratedRequiresGenerator DoesGeneratedRequireGenerator() override;
	virtual bool RequiresGeneratorPackageDestructBeforeResplit() override;
	virtual bool RequiresEmptyPackageBeforePopulate(const UPackage* Package) override;
	virtual ICookPackageSplitter::FGenerationManifest ReportGenerationManifest(
		const UPackage* OwnerPackage, const UObject* OwnerObject) override;
	virtual bool PopulateGeneratedPackage(FPopulateContext& PopulateContext) override;
	virtual bool PopulateGeneratorPackage(FPopulateContext& PopulateContext) override;
	virtual void OnOwnerReloaded(UPackage* OwnerPackage, UObject* OwnerObject) override;
	virtual void Teardown(ETeardown Status) override;
	//~ End of ICookPackageSplitter

	TArray<TPair<const FWorldCookPackageSplitter::FSubSplitterFactory*, ISubSplitter*>> CookPackageSubSplitters;
	TMap<TPair<FName, FName>, ISubSplitter*> SplittersGenerateListMap;

	template<typename FuncType>
	bool ForEachCookPackageSubSplitters(FuncType Func)
	{
		TOverrideVoidReturnInvoker Invoker(true, Func);
		for (const TPair<const FWorldCookPackageSplitter::FSubSplitterFactory*, ISubSplitter*>& CookPackageSubSplitterPair : CookPackageSubSplitters)
		{
			if (!Invoker(CookPackageSubSplitterPair.Value))
			{
				return false;
			}
		}
		return true;
	}

	void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources);

	TObjectPtr<UWorld> ReferencedWorld = nullptr;
	bool bForceInitializedWorld = false;
	bool bInitializedPhysicsSceneForSave = false;
};
#endif // WITH_EDITOR

#undef UE_API
