// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Engine/PostProcessVolume.h"

#if WITH_EDITOR

class UWorld;
class UMaterial;
class UMaterialInstanceConstant;
class APostProcessVolume;

struct FWorldGridPreviewer : public FGCObject
{
public:
	ENGINE_API FWorldGridPreviewer();
	ENGINE_API FWorldGridPreviewer(UWorld* InWorld, bool bInIs2D);
	ENGINE_API ~FWorldGridPreviewer();

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FWorldGridPreviewer"); }
	//~ End FGCObject interface

	ENGINE_API void Update();

	FName Owner;
	int32 CellSize;
	int32 LoadingRange;
	FLinearColor GridColor;
	FVector GridOffset;

private:
	TObjectPtr<UWorld> World;
	TObjectPtr<UMaterial> Material;
	TObjectPtr<UMaterialInstanceConstant> MaterialInstance;
	TWeakObjectPtr<APostProcessVolume> PostProcessVolume;
};

#endif
