// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/UObjectAnnotation.h"
#include "WorldPartition/ActorDescList.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "ActorDescContainerInitParams.h"
#include "AssetRegistry/AssetData.h"
#include "ActorDescContainer.generated.h"

class FLinkerInstancingContext;
class UDeletedObjectPlaceholder;
class UWorldPartition;

#if WITH_EDITOR
struct FDeletedObjectPlaceholderAnnotation
{
public:
	FDeletedObjectPlaceholderAnnotation(const UDeletedObjectPlaceholder* InDeletedObjectPlaceholder = nullptr, const FString& InActorDescContainerName = FString());
	bool IsDefault() const { return DeletedObjectPlaceholder.IsExplicitlyNull() && ActorDescContainerName.IsEmpty(); }
	bool IsValid() const { return DeletedObjectPlaceholder.IsValid() && !ActorDescContainerName.IsEmpty(); }
	const UDeletedObjectPlaceholder* GetDeletedObjectPlaceholder() const { return DeletedObjectPlaceholder.Get(); }
	UActorDescContainer* GetActorDescContainer() const;

private:
	TWeakObjectPtr<const UDeletedObjectPlaceholder> DeletedObjectPlaceholder;
	// We store the container name instead of keeping a WeakObjectPtr to properly handle the case where the container 
	// is unregistered/re-registered between usage of annotation (this can happen if a plugin is unregistered/re-registered).
	FString ActorDescContainerName;
};
#endif


UCLASS(MinimalAPI)
class UActorDescContainer : public UObject, public FActorDescList
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	friend struct FWorldPartitionHandleUtils;
	friend class FWorldPartitionActorDesc;
	friend class UActorDescContainerInstance;

	using FNameActorDescMap = TMap<FName, TUniquePtr<FWorldPartitionActorDesc>*>;

public:
	/* Struct of parameters passed to Initialize function. */
	using FInitializeParams = FActorDescContainerInitParams;

	ENGINE_API virtual void Initialize(const FInitializeParams& InitParams);
	ENGINE_API virtual void Uninitialize();

	bool IsInitialized() const { return bContainerInitialized; }

	ENGINE_API void OnObjectPreSave(UObject* Object, FObjectPreSaveContext SaveContext);
	ENGINE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewObjectMap);
	ENGINE_API void OnPackageDeleted(UPackage* Package);
	ENGINE_API void OnEditorActorReplaced(AActor* InOldActor, AActor* InNewActor);
	ENGINE_API void OnClassDescriptorUpdated(const FWorldPartitionActorDesc* InClassDesc);

	virtual FString GetContainerName() const { return ContainerPackageName.ToString(); }
	FName GetContainerPackage() const { return ContainerPackageName; }
	void SetContainerPackage(const FName& InContainerPackageName) { ContainerPackageName = InContainerPackageName; }

	const UExternalDataLayerAsset* GetExternalDataLayerAsset() const { return ExternalDataLayerAsset; }
	bool HasExternalContent() const;

	FGuid GetContentBundleGuid() const { return ContentBundleGuid; }

	ENGINE_API FString GetExternalActorPath() const;
	ENGINE_API FString GetExternalObjectPath() const;

	/** Removes an actor desc without the need to load a package */
	ENGINE_API bool RemoveActor(const FGuid& ActorGuid);

	ENGINE_API bool IsActorDescHandled(const AActor* Actor) const;

	DECLARE_EVENT_OneParam(UActorDescContainer, FActorDescAddedEvent, FWorldPartitionActorDesc*);
	FActorDescAddedEvent OnActorDescAddedEvent;
	
	DECLARE_EVENT_OneParam(UActorDescContainer, FActorDescRemovedEvent, FWorldPartitionActorDesc*);
	FActorDescRemovedEvent OnActorDescRemovedEvent;

	// Pre update event, before the ActorDesc gets updated
	DECLARE_EVENT_OneParam(UActorDescContainer, FActorDescUpdatingEvent, FWorldPartitionActorDesc*);
	FActorDescUpdatingEvent OnActorDescPreUpdateEvent;

	// Update event, to update to the new pointer
	DECLARE_EVENT_OneParam(UActorDescContainer, FOnActorDescUpdatingEvent, FWorldPartitionActorDesc*);
	FOnActorDescUpdatingEvent OnActorDescUpdatingEvent;

	// Post update event, when the new pointer is now safe to use across instance containers
	DECLARE_EVENT_OneParam(UActorDescContainer, FActorDescUpdatedEvent, FWorldPartitionActorDesc*);
	FActorDescUpdatedEvent OnActorDescUpdatedEvent;

	// On objects replaced event, for container instances
	using FOnObjectsReplacedEventType = const TMap<UObject*, UObject*>&;
	DECLARE_EVENT_OneParam(UActorDescContainer, FOnObjectsReplacedEvent, FOnObjectsReplacedEventType);
	FOnObjectsReplacedEvent OnObjectsReplacedEvent;

	// On editor object replaced event, for container instances
	DECLARE_EVENT_TwoParams(UActorDescContainer, FOnEditorActorReplaced, AActor*, AActor*);
	FOnEditorActorReplaced OnEditorActorReplacedEvent;

	DECLARE_MULTICAST_DELEGATE_OneParam(FActorDescContainerInitializeDelegate, UActorDescContainer*);
	static ENGINE_API FActorDescContainerInitializeDelegate OnActorDescContainerInitialized;
	
	bool HasInvalidActors() const { return InvalidActors.Num() > 0; }
	const TArray<FAssetData>& GetInvalidActors() const { return InvalidActors; }
	void ClearInvalidActors() { InvalidActors.Empty(); }

	ENGINE_API void RegisterActorDescriptor(FWorldPartitionActorDesc* InActorDesc);
	ENGINE_API void UnregisterActorDescriptor(FWorldPartitionActorDesc* InActorDesc);

	ENGINE_API void OnActorDescAdded(FWorldPartitionActorDesc* NewActorDesc);
	ENGINE_API void OnActorDescRemoved(FWorldPartitionActorDesc* ActorDesc);
	ENGINE_API void OnActorDescPreUpdate(FWorldPartitionActorDesc* ActorDesc);
	ENGINE_API void OnActorDescUpdating(FWorldPartitionActorDesc* ActorDesc);
	ENGINE_API void OnActorDescUpdated(FWorldPartitionActorDesc* ActorDesc);

	ENGINE_API bool ShouldHandleActorEvent(const AActor* Actor);

	virtual ENGINE_API const FWorldPartitionActorDesc* GetActorDescByPath(const FString& ActorPath) const;
	virtual ENGINE_API const FWorldPartitionActorDesc* GetActorDescByPath(const FSoftObjectPath& ActorPath) const;
	virtual ENGINE_API const FWorldPartitionActorDesc* GetActorDescByName(FName ActorName) const;

	bool bContainerInitialized;
	bool bRegisteredDelegates;

	FName ContainerPackageName;
	FGuid ContentBundleGuid;

	TArray<FAssetData> InvalidActors;

	void ForEachChildActorDesc(const FGuid& InActorGuid, TFunctionRef<void(const FGuid&)> InFunc);

protected:
	FNameActorDescMap ActorsByName;

	//~ Begin UObject Interface
	ENGINE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

	ENGINE_API virtual bool ShouldRegisterDelegates() const;

	ENGINE_API bool ShouldHandleActorEvent(const AActor* Actor, bool bInUseLoadedPath) const;
	ENGINE_API bool IsActorDescHandled(const AActor* InActor, bool bInUseLoadedPath) const;

	// Updater for the ActorDesc which allows UpdateAction to change the pointer
	void ReassignActorDesc(TUniquePtr<FWorldPartitionActorDesc>& DescToUpdate, TFunctionRef<void(TUniquePtr<FWorldPartitionActorDesc>& ActorDesc)> UpdateAction);

	// Updater for the ActorDesc which only allows updating the contents of the descriptor
	void UpdateActorDesc(FWorldPartitionActorDesc* DescToUpdate, TFunctionRef<void(FWorldPartitionActorDesc* ActorDesc)> UpdateAction);

private:
	// GetWorld() should never be called on an ActorDescContainer to avoid any confusion as it can be used as a template
	UWorld* GetWorld() const override { return nullptr; }

	bool ShouldHandleDeletedObjectPlaceholderEvent(const UDeletedObjectPlaceholder* InDeletedObjectPlaceholder) const;
	void OnDeletedObjectPlaceholderCreated(const UDeletedObjectPlaceholder* InDeletedObjectPlaceholder);

	ENGINE_API void RegisterEditorDelegates();
	ENGINE_API void UnregisterEditorDelegates();

	TMap<FGuid, TSet<FGuid>> ParentActorToChildrenMap;

	void AddChildActorToParentMap(FWorldPartitionActorDesc* ActorDesc);
	void RemoveChildActorFromParentMap(FWorldPartitionActorDesc* ActorDesc);

	void PropagateActorToWorldUpdate(FWorldPartitionActorDesc* ActorDesc);

	static FUObjectAnnotationSparse<FDeletedObjectPlaceholderAnnotation, true> DeletedObjectPlaceholdersAnnotation;

protected:
	TObjectPtr<const UExternalDataLayerAsset> ExternalDataLayerAsset;
#endif
};
