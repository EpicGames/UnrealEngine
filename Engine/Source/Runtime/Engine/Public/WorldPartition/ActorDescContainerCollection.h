// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "WorldPartition/ActorDescContainer.h"
#include <type_traits>

class FWorldPartitionActorDesc;
class FReferenceCollector;

template<class ActorDescContPtrType>
class TActorDescContainerCollection
{
#if WITH_EDITOR
public:
	template<class> friend class TActorDescContainerCollection;

	TActorDescContainerCollection() = default;

	template<class U>
	TActorDescContainerCollection(std::initializer_list<U> ActorDescContainerArray);

	template<class U>
	TActorDescContainerCollection(const TArray<U>& ActorDescContainers);

	virtual ~TActorDescContainerCollection();

	void AddContainer(ActorDescContPtrType Container);
	bool RemoveContainer(ActorDescContPtrType Container);
	bool Contains(const FName& ContainerPackageName) const;
	ActorDescContPtrType FindContainer(const FName& ContainerPackageName) const;

	template<class U>
	void Append(const TActorDescContainerCollection<U>& OtherCollection);
	
	void Empty();
	bool IsEmpty() const { return ActorDescContainerCollection.IsEmpty(); }
	uint32 GetActorDescContainerCount() const { return ActorDescContainerCollection.Num(); }

	FWorldPartitionActorDesc* GetActorDesc(const FGuid& Guid);
	const FWorldPartitionActorDesc* GetActorDesc(const FGuid& Guid) const;

	FWorldPartitionActorDesc& GetActorDescChecked(const FGuid& Guid);
	const FWorldPartitionActorDesc& GetActorDescChecked(const FGuid& Guid) const;

	const FWorldPartitionActorDesc* GetActorDescByPath(const FString& ActorPath) const;
	const FWorldPartitionActorDesc* GetActorDescByPath(const FSoftObjectPath& ActorPath) const;
	const FWorldPartitionActorDesc* GetActorDescByName(FName ActorName) const;

	bool RemoveActor(const FGuid& ActorGuid)
		requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>);

	void OnPackageDeleted(UPackage* Package)
		requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>);

	bool IsActorDescHandled(const AActor* Actor) const
		requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>);

	void LoadAllActors(TArray<FWorldPartitionReference>& OutReferences)
		requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>);

	DECLARE_EVENT_OneParam(UWorldPartition, FActorDescAddedEvent, FWorldPartitionActorDesc*);
	FActorDescAddedEvent OnActorDescAddedEvent;

	DECLARE_EVENT_OneParam(UWorldPartition, FActorDescRemovedEvent, FWorldPartitionActorDesc*);
	FActorDescRemovedEvent OnActorDescRemovedEvent;
	
	ActorDescContPtrType GetActorDescContainer(const FGuid& ActorGuid) const;
	
	ActorDescContPtrType FindHandlingContainer(const AActor* Actor) const;
	
	void ForEachActorDescContainerBreakable(TFunctionRef<bool(ActorDescContPtrType)> Func);
	void ForEachActorDescContainerBreakable(TFunctionRef<bool(ActorDescContPtrType)> Func) const;
	
	void ForEachActorDescContainer(TFunctionRef<void(ActorDescContPtrType)> Func);
	void ForEachActorDescContainer(TFunctionRef<void(ActorDescContPtrType)> Func) const;

protected:
	virtual void OnCollectionChanged() {};

	TArray<ActorDescContPtrType> ActorDescContainerCollection;

private:
	void RegisterDelegates(ActorDescContPtrType Container);
	void UnregisterDelegates(ActorDescContPtrType Container);

	void OnActorDescAdded(FWorldPartitionActorDesc* ActorDesc);
	void OnActorDescRemoved(FWorldPartitionActorDesc* ActorDesc);

public:
	template<bool bConst, class ActorType>
	class TBaseIterator
	{
		static_assert(TIsDerivedFrom<ActorType, AActor>::IsDerived, "Type is not derived from AActor.");

	protected:
		typedef UActorDescContainer ContainerType;
		typedef TArray<ActorDescContPtrType> ContainerCollectionType;
		typedef std::conditional_t<bConst, typename ContainerCollectionType::TConstIterator, typename ContainerCollectionType::TIterator> ContainerIteratorType;
		typedef std::conditional_t<bConst, typename ContainerType::TConstIterator<ActorType>, typename ContainerType::TIterator<ActorType>> ActDescIteratorType;

		typedef typename FWorldPartitionActorDescType<ActorType>::Type ValueType;
		typedef std::conditional_t<bConst, const ValueType*, ValueType*> ReturnType;

	public:
		template<class T>
		TBaseIterator(T* Collection)
			: ContainerIterator(Collection->ActorDescContainerCollection)
		{
			if (ContainerIterator)
			{
				ActorsIterator = MakeUnique<ActDescIteratorType>(*ContainerIterator);
			}
		}

		/**
		 * Iterates to next suitable actor desc.
		 */
		void operator++()
		{
			++(*ActorsIterator);

			if (!*ActorsIterator)
			{
				AdvanceToRelevantActorInNextContainer();
			}
		}

		/**
		 * Returns the current suitable actor desc pointed at by the Iterator
		 *
		 * @return	Current suitable actor desc
		 */
		inline ReturnType operator*() const
		{
			return StaticCast<ReturnType>(**ActorsIterator);
		}

		/**
		 * Returns the current suitable actor desc pointed at by the Iterator
		 *
		 * @return	Current suitable actor desc
		 */
		inline ReturnType operator->() const
		{
			return StaticCast<ReturnType>(**ActorsIterator);
		}
		/**
		 * Returns whether the iterator has reached the end and no longer points
		 * to a suitable actor desc.
		 *
		 * @return true if iterator points to a suitable actor desc, false if it has reached the end
		 */
		inline explicit operator bool() const
		{
			return ActorsIterator ? (bool)*ActorsIterator : false;
		}

	protected:
		inline void AdvanceToRelevantActorInNextContainer()
		{
			while (!(*ActorsIterator) && ContainerIterator)
			{
				++ContainerIterator;
				if (ContainerIterator)
				{
					ActorsIterator = MakeUnique<ActDescIteratorType>(*ContainerIterator);
				}
			}
		}

		ContainerIteratorType ContainerIterator;
		TUniquePtr<ActDescIteratorType> ActorsIterator;
	};

	template <class ActorType = AActor>
	class TIterator : public TBaseIterator<false, ActorType>
	{
		typedef TBaseIterator<false, ActorType> BaseType;

	public:
		template<class T>
		TIterator(T* Collection)
			: BaseType(Collection)
		{}
	};

	template <class ActorType = AActor>
	class TConstIterator : public TBaseIterator<true, ActorType>
	{
		typedef TBaseIterator<true, ActorType> BaseType;

	public:
		template<class T>
		TConstIterator(T* Collection)
			: BaseType(Collection)
		{}
	};
#endif
};

#if WITH_EDITOR

template<class ActorDescContPtrType>
template<class U>
TActorDescContainerCollection<ActorDescContPtrType>::TActorDescContainerCollection(std::initializer_list<U> ActorDescContainerArray)
	: ActorDescContainerCollection(ActorDescContainerArray)
{
	ForEachActorDescContainer([this](ActorDescContPtrType ActorDescContainer)
	{
		RegisterDelegates(ActorDescContainer);
	});
}

template<class ActorDescContPtrType>
template<class U>
TActorDescContainerCollection<ActorDescContPtrType>::TActorDescContainerCollection(const TArray<U>& ActorDescContainers)
	: ActorDescContainerCollection(ActorDescContainers)
{
	ForEachActorDescContainer([this](ActorDescContPtrType ActorDescContainer)
	{
		RegisterDelegates(ActorDescContainer);
	});
}

template<class ActorDescContPtrType>
TActorDescContainerCollection<ActorDescContPtrType>::~TActorDescContainerCollection()
{
	Empty();
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::AddContainer(ActorDescContPtrType Container)
{
	ActorDescContainerCollection.Add(Container);
	RegisterDelegates(Container);
	OnCollectionChanged();
}

template<class ActorDescContPtrType>
bool TActorDescContainerCollection<ActorDescContPtrType>::RemoveContainer(ActorDescContPtrType Container)
{
	if (ActorDescContainerCollection.RemoveSwap(Container) > 0)
	{
		UnregisterDelegates(Container);
		OnCollectionChanged();
		return true;
	}
	return false;
}

template<class ActorDescContPtrType>
template<class U>
void TActorDescContainerCollection<ActorDescContPtrType>::Append(const TActorDescContainerCollection<U>& OtherCollection)
{
	if (OtherCollection.IsEmpty())
	{
		return;
	}

	ActorDescContainerCollection.Append(OtherCollection.ActorDescContainerCollection);
	int32 AppendedContainerIndex = ActorDescContainerCollection.Num() - OtherCollection.ActorDescContainerCollection.Num();
	for (; AppendedContainerIndex < ActorDescContainerCollection.Num(); ++AppendedContainerIndex)
	{
		RegisterDelegates(ActorDescContainerCollection[AppendedContainerIndex]);
	}

	OnCollectionChanged();
}

template<class ActorDescContPtrType>
bool TActorDescContainerCollection<ActorDescContPtrType>::Contains(const FName& ContainerPackageName) const
{
	return FindContainer(ContainerPackageName) != nullptr;
}

template<class ActorDescContPtrType>
ActorDescContPtrType TActorDescContainerCollection<ActorDescContPtrType>::FindContainer(const FName& ContainerPackageName) const
{
	auto ContainerPtr = ActorDescContainerCollection.FindByPredicate([&ContainerPackageName](ActorDescContPtrType ActorDescContainer) { return ActorDescContainer->GetContainerPackage() == ContainerPackageName; });
	return ContainerPtr != nullptr ? *ContainerPtr : nullptr;
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::Empty()
{
	ForEachActorDescContainer([this](ActorDescContPtrType ActorDescContainer)
	{		
		UnregisterDelegates(ActorDescContainer);
	});

	ActorDescContainerCollection.Empty();
	OnCollectionChanged();
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::RegisterDelegates(ActorDescContPtrType Container)
{
	ConstCast(Container)->OnActorDescAddedEvent.AddRaw(this, &TActorDescContainerCollection<ActorDescContPtrType>::OnActorDescAdded);
	ConstCast(Container)->OnActorDescRemovedEvent.AddRaw(this, &TActorDescContainerCollection::OnActorDescRemoved);
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::UnregisterDelegates(ActorDescContPtrType Container)
{
	ConstCast(Container)->OnActorDescAddedEvent.RemoveAll(this);
	ConstCast(Container)->OnActorDescRemovedEvent.RemoveAll(this);
}

template<class ActorDescContPtrType>
const FWorldPartitionActorDesc* TActorDescContainerCollection<ActorDescContPtrType>::GetActorDesc(const FGuid& Guid) const
{
	const FWorldPartitionActorDesc* ActorDesc = nullptr;
	ForEachActorDescContainerBreakable([&Guid, &ActorDesc](ActorDescContPtrType ActorDescContainer)
	{
		ActorDesc = ActorDescContainer->GetActorDesc(Guid);
		return ActorDesc == nullptr;
	});

	return ActorDesc;
}

template<class ActorDescContPtrType>
FWorldPartitionActorDesc* TActorDescContainerCollection<ActorDescContPtrType>::GetActorDesc(const FGuid& Guid)
{
	return const_cast<FWorldPartitionActorDesc*>(const_cast<const TActorDescContainerCollection*>(this)->GetActorDesc(Guid));
}

template<class ActorDescContPtrType>
const FWorldPartitionActorDesc& TActorDescContainerCollection<ActorDescContPtrType>::GetActorDescChecked(const FGuid& Guid) const
{
	const FWorldPartitionActorDesc* ActorDesc = GetActorDesc(Guid);
	check(ActorDesc != nullptr);

	static FWorldPartitionActorDesc EmptyDescriptor;
	return ActorDesc != nullptr ? *ActorDesc : EmptyDescriptor;
}

template<class ActorDescContPtrType>
FWorldPartitionActorDesc& TActorDescContainerCollection<ActorDescContPtrType>::GetActorDescChecked(const FGuid& Guid)
{
	return const_cast<FWorldPartitionActorDesc&>(const_cast<const TActorDescContainerCollection*>(this)->GetActorDescChecked(Guid));
}

template<class ActorDescContPtrType>
const FWorldPartitionActorDesc* TActorDescContainerCollection<ActorDescContPtrType>::GetActorDescByPath(const FString& ActorPath) const
{
	const FWorldPartitionActorDesc* ActorDesc = nullptr;
	ForEachActorDescContainerBreakable([&ActorPath, &ActorDesc](ActorDescContPtrType ActorDescContainer)
	{
		ActorDesc = ActorDescContainer->GetActorDescByPath(ActorPath);
		return ActorDesc == nullptr;
	});

	return ActorDesc;
}

template<class ActorDescContPtrType>
const FWorldPartitionActorDesc* TActorDescContainerCollection<ActorDescContPtrType>::GetActorDescByPath(const FSoftObjectPath& ActorPath) const
{
	const FWorldPartitionActorDesc* ActorDesc = nullptr;
	ForEachActorDescContainerBreakable([&ActorPath, &ActorDesc](ActorDescContPtrType ActorDescContainer)
	{
		ActorDesc = ActorDescContainer->GetActorDescByPath(ActorPath);
		return ActorDesc == nullptr;
	});

	return ActorDesc;
}

template<class ActorDescContPtrType>
const FWorldPartitionActorDesc* TActorDescContainerCollection<ActorDescContPtrType>::GetActorDescByName(FName ActorName) const
{
	const FWorldPartitionActorDesc* ActorDesc = nullptr;
	ForEachActorDescContainerBreakable([&ActorName, &ActorDesc](ActorDescContPtrType ActorDescContainer)
	{
		ActorDesc = ActorDescContainer->GetActorDescByName(ActorName);
		return ActorDesc == nullptr;
	});

	return ActorDesc;
}

template<class ActorDescContPtrType>
bool TActorDescContainerCollection<ActorDescContPtrType>::RemoveActor(const FGuid& ActorGuid)
	requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>)
{
	bool bRemoved = false;
	ForEachActorDescContainerBreakable([&ActorGuid, &bRemoved](ActorDescContPtrType ActorDescContainer)
	{
		bRemoved = ActorDescContainer->RemoveActor(ActorGuid);
		return !bRemoved;
	});

	return bRemoved;
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::OnPackageDeleted(UPackage* Package)
	requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>)
{
	ForEachActorDescContainer([Package](ActorDescContPtrType ActorDescContainer)
	{
		ActorDescContainer->OnPackageDeleted(Package);
	});
}

template<class ActorDescContPtrType>
bool TActorDescContainerCollection<ActorDescContPtrType>::IsActorDescHandled(const AActor* Actor) const
	requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>)
{
	bool bIsHandled = false;
	ForEachActorDescContainerBreakable([Actor, &bIsHandled](ActorDescContPtrType ActorDescContainer)
	{
		bIsHandled = ActorDescContainer->IsActorDescHandled(Actor);
		return !bIsHandled;
	});

	return bIsHandled;
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::LoadAllActors(TArray<FWorldPartitionReference>& OutReferences)
	requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>)
{
	ForEachActorDescContainer([&OutReferences](ActorDescContPtrType ActorDescContainer)
	{
		ActorDescContainer->LoadAllActors(OutReferences);
	});
}

template<class ActorDescContPtrType>
ActorDescContPtrType TActorDescContainerCollection<ActorDescContPtrType>::GetActorDescContainer(const FGuid& ActorGuid) const
{
	ActorDescContPtrType ActorDescContainer = nullptr;
	ForEachActorDescContainerBreakable([&ActorGuid, &ActorDescContainer](ActorDescContPtrType InActorDescContainer)
	{
		if (InActorDescContainer->GetActorDesc(ActorGuid) != nullptr)
		{
			ActorDescContainer = InActorDescContainer;
		}
		return ActorDescContainer == nullptr;
	});

	return ActorDescContainer;
}

template<class ActorDescContPtrType>
ActorDescContPtrType TActorDescContainerCollection<ActorDescContPtrType>::FindHandlingContainer(const AActor* Actor) const
{
	// Actor is already in a container
	if (ActorDescContPtrType ActorDescContainer = GetActorDescContainer(Actor->GetActorGuid()))
	{
		return ActorDescContainer;
	}

	// Actor is not yet stored in a container. Find which one should handle it.
	ActorDescContPtrType ActorDescContainer = nullptr;
	ForEachActorDescContainerBreakable([&ActorDescContainer, &Actor](ActorDescContPtrType InActorDescContainer)
	{
		if (InActorDescContainer->IsActorDescHandled(Actor))
		{
			ActorDescContainer = InActorDescContainer;
		}
		return ActorDescContainer == nullptr;
	});

	return ActorDescContainer;
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::ForEachActorDescContainerBreakable(TFunctionRef<bool(ActorDescContPtrType)> Func) const
{
	for (ActorDescContPtrType ActorDescContainer : ActorDescContainerCollection)
	{
		if (!Func(ActorDescContainer))
		{
			break;
		}
	}
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::ForEachActorDescContainerBreakable(TFunctionRef<bool(ActorDescContPtrType)> Func)
{
	const_cast<const TActorDescContainerCollection*>(this)->ForEachActorDescContainerBreakable(Func);
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::ForEachActorDescContainer(TFunctionRef<void(ActorDescContPtrType)> Func) const
{
	for (ActorDescContPtrType ActorDescContainer : ActorDescContainerCollection)
	{
		Func(ActorDescContainer);
	}
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::ForEachActorDescContainer(TFunctionRef<void(ActorDescContPtrType)> Func)
{
	const_cast<const TActorDescContainerCollection*>(this)->ForEachActorDescContainer(Func);
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::OnActorDescAdded(FWorldPartitionActorDesc* ActorDesc)
{
	OnActorDescAddedEvent.Broadcast(ActorDesc);
}

template<class ActorDescContPtrType>
void TActorDescContainerCollection<ActorDescContPtrType>::OnActorDescRemoved(FWorldPartitionActorDesc* ActorDesc)
{
	OnActorDescRemovedEvent.Broadcast(ActorDesc);
}

#endif 

using FActorDescContainerCollection = TActorDescContainerCollection<TObjectPtr<UActorDescContainer>>;