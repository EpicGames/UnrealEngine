// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Guid.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

class UExternalDataLayerAsset;
class UActorDescContainer;
class FWorldPartitionActorDesc;

struct FActorDescContainerInitParams
{
	FActorDescContainerInitParams(FName InPackageName)
		: PackageName(InPackageName)
		, ContainerName(InPackageName.ToString())
	{}

	FActorDescContainerInitParams(const FString& InContainerName, FName InPackageName)
		: PackageName(InPackageName)
		, ContainerName(InContainerName)
	{}

	/* The long package name of the container package on disk. */
	FName PackageName;

	/* The unique name for the container : defaults to PackageName */
	FString ContainerName;

	/** The associated Content Bundle Guid */
	FGuid ContentBundleGuid;

	/** If the container should bind to editor events */
	bool bShouldRegisterEditorDeletages = true;
	
	/** Whether non-external actors should be gathered */
	bool bShouldGatherInternalActors = true;

	/** The associated External Data Layer Asset */
	const UExternalDataLayerAsset* ExternalDataLayerAsset = nullptr;

	/* Custom pre-init function that is called before calling Initialize on the new container */
	TUniqueFunction<void(UActorDescContainer*)> PreInitialize;

	/* Custom filter function used to filter actors descriptors. */
	TUniqueFunction<bool(const FWorldPartitionActorDesc*)> FilterActorDesc;
};
