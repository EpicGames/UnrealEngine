// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Optional.h"
#include "UObject/LinkerInstancingContext.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/ActorDescList.h"
#include "ActorDescContainerInstance.generated.h"

class UActorDescContainer;
class FWorldPartitionActorDesc;
class UWorldPartition;
class UExternalDataLayerAsset;
struct FWorldPartitionRuntimeCellPropertyOverride;

class FActorDescInstanceList : public TActorDescList<FWorldPartitionActorDescInstance> { };

UCLASS(MinimalAPI)
class UActorDescContainerInstance : public UObject, public FActorDescInstanceList
{
	GENERATED_BODY()

protected:
	UActorDescContainerInstance()
#if WITH_EDITORONLY_DATA
		: bIsInitialized(false) 
		, bRegisteredDelegates(false)
		, bCreateChildContainerHierarchy(false)
#endif
	{}

#if WITH_EDITOR
	friend FWorldPartitionActorDescInstance;
	friend UWorldPartition;
		
public:
	struct FInitializeParams
	{
		FInitializeParams(FName InContainerPackageName, bool bInCreateContainerInstanceHierarchy = false)
			: ContainerPackageName(InContainerPackageName)
			, bCreateContainerInstanceHierarchy(bInCreateContainerInstanceHierarchy)
		{}
				
		FInitializeParams& SetParent(const UActorDescContainerInstance* InParentContainerInstance, const FGuid& InContainerActorGuid)
		{
			check(InContainerActorGuid.IsValid());
			ParentContainerInstance = InParentContainerInstance;
			ContainerActorGuid = InContainerActorGuid;
			return *this;
		}

		FInitializeParams& SetTransform(const FTransform& InTransform)
		{
			Transform = InTransform;
			return *this;
		}

		FInitializeParams& SetShouldRegisterEditorDeletages(bool bInShouldRegisterEditorDeletages)
		{
			bShouldRegisterEditorDeletages = bInShouldRegisterEditorDeletages;
			return *this;
		}

		FName ContainerPackageName;

		FGuid ContainerActorGuid;

		/** The associated Content Bundle Guid */
		FGuid ContentBundleGuid;

		/** The associated External Data Layer Asset */
		const UExternalDataLayerAsset* ExternalDataLayerAsset = nullptr;
		
		const UActorDescContainerInstance* ParentContainerInstance = nullptr;

		TOptional<FTransform> Transform;

		bool bCreateContainerInstanceHierarchy;
		bool bShouldRegisterEditorDeletages = true;
		bool bShouldGatherInternalActors = true;

		/* Custom filter function used to filter actors descriptors. */
		TUniqueFunction<bool(const FWorldPartitionActorDesc*)> FilterActorDescFunc;

		/** Function called when container is initialized. */
		TUniqueFunction<void(UActorDescContainerInstance*)> OnInitializedFunc;
	};


	DECLARE_MULTICAST_DELEGATE_OneParam(FActorDescContainerInstanceInitializeDelegate, UActorDescContainerInstance*);
	static ENGINE_API FActorDescContainerInstanceInitializeDelegate OnActorDescContainerInstanceInitialized;

	DECLARE_EVENT_OneParam(UActorDescContainerInstance, FActorDescInstanceAddedEvent, FWorldPartitionActorDescInstance*);
	FActorDescInstanceAddedEvent OnActorDescInstanceAddedEvent;

	DECLARE_EVENT_OneParam(UActorDescContainerInstance, FActorDescInstanceRemovedEvent, FWorldPartitionActorDescInstance*);
	FActorDescInstanceRemovedEvent OnActorDescInstanceRemovedEvent;

	DECLARE_EVENT_OneParam(UActorDescContainerInstance, FActorDescInstanceUpdatingEvent, FWorldPartitionActorDescInstance*);
	FActorDescInstanceUpdatingEvent OnActorDescInstanceUpdatingEvent;

	DECLARE_EVENT_OneParam(UActorDescContainerInstance, FActorDescInstanceUpdatedEvent, FWorldPartitionActorDescInstance*);
	FActorDescInstanceUpdatedEvent OnActorDescInstanceUpdatedEvent;

	DECLARE_EVENT_OneParam(UActorDescContainerInstance, FActorReplacedEvent, FWorldPartitionActorDescInstance*);
	FActorReplacedEvent OnActorReplacedEvent;
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FActorDescInstanceAddedToContainerEvent, UActorDescContainerInstance*, FWorldPartitionActorDescInstance*);
	UE_EXPERIMENTAL(5.8, "This new static delegate is experimental and the API is subject to change.")
	static ENGINE_API FActorDescInstanceAddedToContainerEvent OnActorDescInstanceAddedToContainer;
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FActorDescInstanceRemovedFromContainerEvent, UActorDescContainerInstance*, FWorldPartitionActorDescInstance*);
	UE_EXPERIMENTAL(5.8, "This new static delegate is experimental and the API is subject to change.")
	static ENGINE_API FActorDescInstanceRemovedFromContainerEvent OnActorDescInstanceRemovedFromContainer;

	ENGINE_API virtual void Initialize(const FInitializeParams& InParams);
	bool IsInitialized() const { return bIsInitialized; }
	ENGINE_API virtual void Uninitialize();

	ENGINE_API UWorldPartition* GetTopWorldPartition() const;
	ENGINE_API UWorldPartition* GetOuterWorldPartition() const;
	ENGINE_API const FTransform& GetTransform() const;
	ENGINE_API const FLinkerInstancingContext* GetInstancingContext() const;
	const FActorContainerID& GetContainerID() const { return ContainerID; }
	FGuid GetContainerActorGuid() const { return ContainerActorGuid; }

	ENGINE_API static FName GetContainerPackageNameFromWorld(UWorld* InWorld);
	ENGINE_API FName GetContainerPackage() const;
	ENGINE_API FGuid GetContentBundleGuid() const;
	ENGINE_API const UExternalDataLayerAsset* GetExternalDataLayerAsset() const;
	ENGINE_API bool HasExternalContent() const;
	ENGINE_API FString GetExternalActorPath() const;
	ENGINE_API FString GetExternalObjectPath() const;
		
	virtual void GetPropertyOverridesForActor(const FActorContainerID& InContainerID, const FGuid& InActorGuid, TArray<FWorldPartitionRuntimeCellPropertyOverride>& OutPropertyOverrides) const {}

	ENGINE_API TUniquePtr<FWorldPartitionActorDescInstance>* GetActorDescInstancePtr(const FGuid& InActorGuid) const;
	ENGINE_API FWorldPartitionActorDescInstance* GetActorDescInstance(const FGuid& InActorGuid) const;
	ENGINE_API FWorldPartitionActorDescInstance& GetActorDescInstanceChecked(const FGuid& InActorGuid) const;

	ENGINE_API const FWorldPartitionActorDescInstance* GetActorDescInstanceByPath(const FString& ActorPath) const;
	ENGINE_API const FWorldPartitionActorDescInstance* GetActorDescInstanceByPath(const FSoftObjectPath& ActorPath) const;
	ENGINE_API const FWorldPartitionActorDescInstance* GetActorDescInstanceByName(FName ActorName) const;
		
	ENGINE_API bool IsActorDescHandled(const AActor* Actor) const;

	const UActorDescContainer* GetContainer() const { return Container; }
	UActorDescContainer* GetContainer() { return Container; }
	uint32 GetActorDescInstanceCount() const { return ActorDescList.Num(); }
	bool IsEmpty() const { return ActorDescList.IsEmpty(); }
		
	ENGINE_API void LoadAllActors(TArray<FWorldPartitionReference>& OutReferences);

	const TMap<FGuid, TObjectPtr<UActorDescContainerInstance>>& GetChildContainerInstances() const { return ChildContainerInstances; }
		
	const UActorDescContainerInstance* GetParentContainerInstance() const { return ParentContainerInstance.Get(); }
protected:
	virtual void RegisterContainer(const FInitializeParams& InParams);
	virtual void UnregisterContainer();
	void SetContainer(UActorDescContainer* InContainer) { Container = InContainer; }

	virtual FWorldPartitionActorDesc* GetActorDesc(const FGuid& InActorGuid) const;
	virtual FWorldPartitionActorDesc* GetActorDescChecked(const FGuid& InActorGuid) const;

	virtual FWorldPartitionActorDescInstance CreateActorDescInstance(FWorldPartitionActorDesc* InActorDesc) { return FWorldPartitionActorDescInstance(this, InActorDesc); }
private:
	void OnContainerReplaced(UActorDescContainer* InOldContainer, UActorDescContainer* InNewContainer);
	void SetContainerPackage(FName InContainerPackageName);
		
	void RegisterDelegates();
	void UnregisterDelegates();
	bool ShouldRegisterDelegates() const;
	
	FWorldPartitionActorDescInstance* AddActor(FWorldPartitionActorDesc* InActorDesc);
	ENGINE_API void RemoveActor(const FGuid& InActorGuid);

	FWorldPartitionActorDescInstance* AddActorDescInstance(FWorldPartitionActorDescInstance&& InActorDescInstance);
	void RemoveActorDescInstance(TUniquePtr<FWorldPartitionActorDescInstance>* InActorDescInstance);

	void OnActorDescAdded(FWorldPartitionActorDesc* InActorDesc);
	void OnActorDescRemoved(FWorldPartitionActorDesc* InActorDesc);
	void OnActorDescPreUpdate(FWorldPartitionActorDesc* InActorDesc);
	void OnActorDescUpdating(FWorldPartitionActorDesc* InActorDesc);
	void OnActorDescUpdated(FWorldPartitionActorDesc* InActorDesc);

	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewObjectMap);
	void OnEditorActorReplaced(AActor* InOldActor, AActor* InNewActor);

	void OnRegisterChildContainerInstance(const FGuid& InActorGuid, UActorDescContainerInstance* InChildContainerInstance);
	void OnUnregisterChildContainerInstance(const FGuid& InActorGuid);

	friend class UGameFeatureActionConvertContentBundleWorldPartitionBuilder;
#endif

protected:
#if WITH_EDITORONLY_DATA
	FSoftObjectPath												WorldContainerPath;
	FSoftObjectPath												SourceWorldContainerPath;
	TOptional<FLinkerInstancingContext>							InstancingContext;
			
	FActorContainerID											ContainerID;
	FGuid														ContainerActorGuid;
	TOptional<FTransform>										Transform;
	TWeakObjectPtr<const UActorDescContainerInstance>			ParentContainerInstance;

	UPROPERTY(Transient)
	TObjectPtr<UActorDescContainer>								Container;
		
	UPROPERTY(Transient)
	TMap<FGuid, TObjectPtr<UActorDescContainerInstance>>		ChildContainerInstances;

	bool														bIsInitialized;
	bool														bRegisteredDelegates;
	bool														bCreateChildContainerHierarchy;
#endif
};
