// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "GameFramework/Actor.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include <type_traits>

class UWorldPartition;

template<class ActorDescContPtrType>
class TActorDescContainerInstanceCollection
{
#if WITH_EDITOR
public:
	template<class> friend class TActorDescContainerInstanceCollection;

	TActorDescContainerInstanceCollection() = default;

	template<class U>
	TActorDescContainerInstanceCollection(std::initializer_list<U> ActorDescContainerInstanceArray);

	template<class U>
	TActorDescContainerInstanceCollection(const TArray<U>& ActorDescContainerInstances);

	virtual ~TActorDescContainerInstanceCollection();

	void AddContainer(ActorDescContPtrType Container);
	bool RemoveContainer(ActorDescContPtrType Container);
	bool Contains(const FName& ContainerPackageName) const;
	ActorDescContPtrType FindContainer(const FName& ContainerPackageName) const;

	template<class U>
	void Append(const TActorDescContainerInstanceCollection<U>& OtherCollection);

	void Empty();
	bool IsEmpty() const { return ActorDescContainerInstanceCollection.IsEmpty(); }
	uint32 GetActorDescContainerCount() const { return ActorDescContainerInstanceCollection.Num(); }

	FWorldPartitionActorDescInstance* GetActorDescInstance(const FGuid& Guid);
	const FWorldPartitionActorDescInstance* GetActorDescInstance(const FGuid& Guid) const;

	const FWorldPartitionActorDescInstance* GetActorDescInstanceByPath(const FString& ActorPath) const;
	const FWorldPartitionActorDescInstance* GetActorDescInstanceByPath(const FSoftObjectPath& ActorPath) const;
	const FWorldPartitionActorDescInstance* GetActorDescInstanceByName(FName ActorName) const;

	ActorDescContPtrType GetActorDescContainerInstance(const FGuid& ActorGuid) const;
	ActorDescContPtrType FindHandlingContainerInstance(const AActor* Actor) const;

	bool RemoveActor(const FGuid& ActorGuid)
		requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>);

	void OnPackageDeleted(UPackage* Package)
		requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>);

	void LoadAllActors(TArray<FWorldPartitionReference>& OutReferences)
		requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>);

	DECLARE_EVENT_OneParam(UWorldPartition, FActorDescInstanceAddedEvent, FWorldPartitionActorDescInstance*);
	FActorDescInstanceAddedEvent OnActorDescInstanceAddedEvent;

	DECLARE_EVENT_OneParam(UWorldPartition, FActorDescInstanceRemovedEvent, FWorldPartitionActorDescInstance*);
	FActorDescInstanceRemovedEvent OnActorDescInstanceRemovedEvent;

	DECLARE_EVENT_OneParam(UWorldPartition, FActorReplacedEvent, FWorldPartitionActorDescInstance*);
	FActorReplacedEvent OnActorReplacedEvent;

	void ForEachActorDescContainerInstanceBreakable(TFunctionRef<bool(ActorDescContPtrType)> Func, bool bRecursive = false);
	void ForEachActorDescContainerInstanceBreakable(TFunctionRef<bool(ActorDescContPtrType)> Func, bool bRecursive = false) const;

	void ForEachActorDescContainerInstance(TFunctionRef<void(ActorDescContPtrType)> Func, bool bRecursive = false);
	void ForEachActorDescContainerInstance(TFunctionRef<void(ActorDescContPtrType)> Func, bool bRecursive = false) const;

protected:
	virtual void OnCollectionChanged() {};
	virtual bool ShouldRegisterDelegates() const { return true; }

	TArray<ActorDescContPtrType> ActorDescContainerInstanceCollection;

private:
	void RegisterDelegates(ActorDescContPtrType Container);
	void UnregisterDelegates(ActorDescContPtrType Container);

	void OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* ActorDescInstance);
	void OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* ActorDescInstance);
	void OnActorReplaced(FWorldPartitionActorDescInstance* ActorDescInstance);

public:
	template<bool bConst, class ActorType>
	class TBaseIterator
	{
		static_assert(TIsDerivedFrom<ActorType, AActor>::IsDerived, "Type is not derived from AActor.");

	protected:
		typedef UActorDescContainerInstance ContainerType;
		typedef TArray<ActorDescContPtrType> ContainerCollectionType;
		typedef std::conditional_t<bConst, typename ContainerCollectionType::TConstIterator, typename ContainerCollectionType::TIterator> ContainerIteratorType;
		typedef std::conditional_t<bConst, typename ContainerType::TConstIterator<ActorType>, typename ContainerType::TIterator<ActorType>> ActDescIteratorType;

		typedef FWorldPartitionActorDescInstance ValueType;
		typedef std::conditional_t<bConst, const ValueType*, ValueType*> ReturnType;

	public:
		template<class T>
		TBaseIterator(T* Collection)
			: ContainerIterator(Collection->ActorDescContainerInstanceCollection)
		{
			if (ContainerIterator)
			{
				ActorsIterator = MakeUnique<ActDescIteratorType>(*ContainerIterator);

				if (!*ActorsIterator)
				{
					AdvanceToRelevantActorInNextContainer();
				}
			}
		}

		/**
		 * Iterates to next suitable actor desc.
		 */
		void operator++()
		{
			++(*ActorsIterator);

			if (!*ActorsIterator)
			{
				AdvanceToRelevantActorInNextContainer();
			}
		}

		/**
		 * Returns the current suitable actor desc pointed at by the Iterator
		 *
		 * @return	Current suitable actor desc
		 */
		inline ReturnType operator*() const
		{
			return StaticCast<ReturnType>(**ActorsIterator);
		}

		/**
		 * Returns the current suitable actor desc pointed at by the Iterator
		 *
		 * @return	Current suitable actor desc
		 */
		inline ReturnType operator->() const
		{
			return StaticCast<ReturnType>(**ActorsIterator);
		}
		/**
		 * Returns whether the iterator has reached the end and no longer points
		 * to a suitable actor desc.
		 *
		 * @return true if iterator points to a suitable actor desc, false if it has reached the end
		 */
		inline explicit operator bool() const
		{
			return ActorsIterator ? (bool)*ActorsIterator : false;
		}

	protected:
		inline void AdvanceToRelevantActorInNextContainer()
		{
			while (!(*ActorsIterator) && ContainerIterator)
			{
				++ContainerIterator;
				if (ContainerIterator)
				{
					ActorsIterator = MakeUnique<ActDescIteratorType>(*ContainerIterator);
				}
			}
		}

		ContainerIteratorType ContainerIterator;
		TUniquePtr<ActDescIteratorType> ActorsIterator;
	};

	template <class ActorType = AActor>
	class TIterator : public TBaseIterator<false, ActorType>
	{
		typedef TBaseIterator<false, ActorType> BaseType;

	public:
		template<class T>
		TIterator(T* Collection)
			: BaseType(Collection)
		{}
	};

	template <class ActorType = AActor>
	class TConstIterator : public TBaseIterator<true, ActorType>
	{
		typedef TBaseIterator<true, ActorType> BaseType;

	public:
		template<class T>
		TConstIterator(T* Collection)
			: BaseType(Collection)
		{}
	};
#endif
};

#if WITH_EDITOR

template<class ActorDescContPtrType>
template<class U>
TActorDescContainerInstanceCollection<ActorDescContPtrType>::TActorDescContainerInstanceCollection(std::initializer_list<U> ActorDescContainerInstanceArray)
	: ActorDescContainerInstanceCollection(ActorDescContainerInstanceArray)
{
	ForEachActorDescContainerInstance([this](ActorDescContPtrType ActorDescContainerInstance)
	{
		RegisterDelegates(ActorDescContainerInstance);
	});
}

template<class ActorDescContPtrType>
template<class U>
TActorDescContainerInstanceCollection<ActorDescContPtrType>::TActorDescContainerInstanceCollection(const TArray<U>& ActorDescContainerInstances)
	: ActorDescContainerInstanceCollection(ActorDescContainerInstances)
{
	ForEachActorDescContainerInstance([this](ActorDescContPtrType ActorDescContainerInstance)
	{
		RegisterDelegates(ActorDescContainerInstance);
	});
}

template<class ActorDescContPtrType>
TActorDescContainerInstanceCollection<ActorDescContPtrType>::~TActorDescContainerInstanceCollection()
{
	Empty();
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::AddContainer(ActorDescContPtrType Container)
{
	ActorDescContainerInstanceCollection.Add(Container);
	RegisterDelegates(Container);
	OnCollectionChanged();
}

template<class ActorDescContPtrType>
bool TActorDescContainerInstanceCollection<ActorDescContPtrType>::RemoveContainer(ActorDescContPtrType Container)
{
	if (ActorDescContainerInstanceCollection.RemoveSwap(Container) > 0)
	{
		UnregisterDelegates(Container);
		OnCollectionChanged();
		return true;
	}
	return false;
}

template<class ActorDescContPtrType>
template<class U>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::Append(const TActorDescContainerInstanceCollection<U>& OtherCollection)
{
	if (OtherCollection.IsEmpty())
	{
		return;
	}

	ActorDescContainerInstanceCollection.Append(OtherCollection.ActorDescContainerInstanceCollection);
	int32 AppendedContainerIndex = ActorDescContainerInstanceCollection.Num() - OtherCollection.ActorDescContainerInstanceCollection.Num();
	for (; AppendedContainerIndex < ActorDescContainerInstanceCollection.Num(); ++AppendedContainerIndex)
	{
		RegisterDelegates(ActorDescContainerInstanceCollection[AppendedContainerIndex]);
	}

	OnCollectionChanged();
}

template<class ActorDescContPtrType>
bool TActorDescContainerInstanceCollection<ActorDescContPtrType>::Contains(const FName& ContainerPackageName) const
{
	return FindContainer(ContainerPackageName) != nullptr;
}

template<class ActorDescContPtrType>
ActorDescContPtrType TActorDescContainerInstanceCollection<ActorDescContPtrType>::FindContainer(const FName& ContainerPackageName) const
{
	auto ContainerPtr = ActorDescContainerInstanceCollection.FindByPredicate([&ContainerPackageName](ActorDescContPtrType ActorDescContainerInstance) { return ActorDescContainerInstance->GetContainerPackage() == ContainerPackageName; });
	return ContainerPtr != nullptr ? *ContainerPtr : nullptr;
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::Empty()
{
	ForEachActorDescContainerInstance([this](ActorDescContPtrType ActorDescContainerInstance)
	{
		UnregisterDelegates(ActorDescContainerInstance);
	});

	ActorDescContainerInstanceCollection.Empty();
	OnCollectionChanged();
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::RegisterDelegates(ActorDescContPtrType Container)
{
	if (ShouldRegisterDelegates())
	{
		ConstCast(Container)->OnActorDescInstanceAddedEvent.AddRaw(this, &TActorDescContainerInstanceCollection<ActorDescContPtrType>::OnActorDescInstanceAdded);
		ConstCast(Container)->OnActorDescInstanceRemovedEvent.AddRaw(this, &TActorDescContainerInstanceCollection<ActorDescContPtrType>::OnActorDescInstanceRemoved);
		ConstCast(Container)->OnActorReplacedEvent.AddRaw(this, &TActorDescContainerInstanceCollection<ActorDescContPtrType>::OnActorReplaced);
	}
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::UnregisterDelegates(ActorDescContPtrType Container)
{
	if (ShouldRegisterDelegates())
	{
		ConstCast(Container)->OnActorDescInstanceAddedEvent.RemoveAll(this);
		ConstCast(Container)->OnActorDescInstanceRemovedEvent.RemoveAll(this);
		ConstCast(Container)->OnActorReplacedEvent.RemoveAll(this);
	}
}

template<class ActorDescContPtrType>
const FWorldPartitionActorDescInstance* TActorDescContainerInstanceCollection<ActorDescContPtrType>::GetActorDescInstance(const FGuid& Guid) const
{
	const FWorldPartitionActorDescInstance* ActorDescInstance = nullptr;
	ForEachActorDescContainerInstanceBreakable([&Guid, &ActorDescInstance](ActorDescContPtrType ActorDescContainerInstance)
	{
		ActorDescInstance = ActorDescContainerInstance->GetActorDescInstance(Guid);
		return ActorDescInstance == nullptr;
	});

	return ActorDescInstance;
}

template<class ActorDescContPtrType>
FWorldPartitionActorDescInstance* TActorDescContainerInstanceCollection<ActorDescContPtrType>::GetActorDescInstance(const FGuid& Guid)
{
	return const_cast<FWorldPartitionActorDescInstance*>(const_cast<const TActorDescContainerInstanceCollection*>(this)->GetActorDescInstance(Guid));
}

template<class ActorDescContPtrType>
const FWorldPartitionActorDescInstance* TActorDescContainerInstanceCollection<ActorDescContPtrType>::GetActorDescInstanceByPath(const FString& ActorPath) const
{
	const FWorldPartitionActorDescInstance* ActorDescInstance = nullptr;
	ForEachActorDescContainerInstanceBreakable([&ActorPath, &ActorDescInstance](ActorDescContPtrType ActorDescContainerInstance)
	{
		ActorDescInstance = ActorDescContainerInstance->GetActorDescInstanceByPath(ActorPath);
		return ActorDescInstance == nullptr;
	});

	return ActorDescInstance;
}

template<class ActorDescContPtrType>
const FWorldPartitionActorDescInstance* TActorDescContainerInstanceCollection<ActorDescContPtrType>::GetActorDescInstanceByPath(const FSoftObjectPath& ActorPath) const
{
	const FWorldPartitionActorDescInstance* ActorDescInstance = nullptr;
	ForEachActorDescContainerInstanceBreakable([&ActorPath, &ActorDescInstance](ActorDescContPtrType ActorDescContainerInstance)
	{
		ActorDescInstance = ActorDescContainerInstance->GetActorDescInstanceByPath(ActorPath);
		return ActorDescInstance == nullptr;
	});

	return ActorDescInstance;
}

template<class ActorDescContPtrType>
const FWorldPartitionActorDescInstance* TActorDescContainerInstanceCollection<ActorDescContPtrType>::GetActorDescInstanceByName(FName ActorName) const
{
	const FWorldPartitionActorDescInstance* ActorDescInstance = nullptr;
	ForEachActorDescContainerInstanceBreakable([&ActorName, &ActorDescInstance](ActorDescContPtrType ActorDescContainerInstance)
	{
		ActorDescInstance = ActorDescContainerInstance->GetActorDescInstanceByName(ActorName);
		return ActorDescInstance == nullptr;
	});

	return ActorDescInstance;
}

template<class ActorDescContPtrType>
ActorDescContPtrType TActorDescContainerInstanceCollection<ActorDescContPtrType>::GetActorDescContainerInstance(const FGuid& ActorGuid) const
{
	ActorDescContPtrType ActorDescContainerInstance = nullptr;
	ForEachActorDescContainerInstanceBreakable([&ActorGuid, &ActorDescContainerInstance](ActorDescContPtrType InActorDescContainerInstance)
	{
		if (InActorDescContainerInstance->GetActorDescInstance(ActorGuid) != nullptr)
		{
			ActorDescContainerInstance = InActorDescContainerInstance;
		}
		return ActorDescContainerInstance == nullptr;
	});

	return ActorDescContainerInstance;
}

template<class ActorDescContPtrType>
ActorDescContPtrType TActorDescContainerInstanceCollection<ActorDescContPtrType>::FindHandlingContainerInstance(const AActor* Actor) const
{
	// Actor is already in a container
	if (ActorDescContPtrType ActorDescContainerInstance = GetActorDescContainerInstance(Actor->GetActorGuid()))
	{
		return ActorDescContainerInstance;
	}

	// Actor is not yet stored in a container. Find which one should handle it.
	ActorDescContPtrType ActorDescContainerInstance = nullptr;
	ForEachActorDescContainerInstanceBreakable([&ActorDescContainerInstance, &Actor](ActorDescContPtrType InActorDescContainerInstance)
	{
		if (InActorDescContainerInstance->IsActorDescHandled(Actor))
		{
			ActorDescContainerInstance = InActorDescContainerInstance;
		}
		return ActorDescContainerInstance == nullptr;
	});

	return ActorDescContainerInstance;
}

template<class ActorDescContPtrType>
bool TActorDescContainerInstanceCollection<ActorDescContPtrType>::RemoveActor(const FGuid& ActorGuid)
	requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>)
{
	bool bRemoved = false;
	ForEachActorDescContainerInstanceBreakable([&ActorGuid, &bRemoved](ActorDescContPtrType ContainerInstance)
	{
		bRemoved = ContainerInstance->GetContainer()->RemoveActor(ActorGuid);
		return !bRemoved;
	});

	return bRemoved;
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::OnPackageDeleted(UPackage* Package)
	requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>)
{
	ForEachActorDescContainerInstance([Package](ActorDescContPtrType ContainerInstance)
	{
		ContainerInstance->GetContainer()->OnPackageDeleted(Package);
	});
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::LoadAllActors(TArray<FWorldPartitionReference>& OutReferences)
	requires (!std::is_const_v<std::remove_pointer_t<ActorDescContPtrType>>)
{
	ForEachActorDescContainerInstance([&OutReferences](ActorDescContPtrType ContainerInstance)
	{
		ContainerInstance->LoadAllActors(OutReferences);
	});
}


template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::ForEachActorDescContainerInstanceBreakable(TFunctionRef<bool(ActorDescContPtrType)> Func, bool bRecursive) const
{
	TFunction<bool(ActorDescContPtrType)> InvokeFunc = [&Func, &InvokeFunc, bRecursive](ActorDescContPtrType ActorDescContainerInstance)
	{
		if (!Func(ActorDescContainerInstance))
		{
			return false;
		}

		if (bRecursive)
		{
			for (auto [Guid, ChildActorDescContainerInstance] : ActorDescContainerInstance->GetChildContainerInstances())
			{
				if (!InvokeFunc(ChildActorDescContainerInstance))
				{
					return false;
				}
			}
		}

		return true;
	};

	for (ActorDescContPtrType ActorDescContainerInstance : ActorDescContainerInstanceCollection)
	{
		if (!InvokeFunc(ActorDescContainerInstance))
		{
			break;
		}
	}
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::ForEachActorDescContainerInstanceBreakable(TFunctionRef<bool(ActorDescContPtrType)> Func, bool bRecursive)
{
	const_cast<const TActorDescContainerInstanceCollection*>(this)->ForEachActorDescContainerInstanceBreakable(Func, bRecursive);
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::ForEachActorDescContainerInstance(TFunctionRef<void(ActorDescContPtrType)> Func, bool bRecursive) const
{
	ForEachActorDescContainerInstanceBreakable([&Func](ActorDescContPtrType ActorDescContainerInstance)
	{
		Func(ActorDescContainerInstance); return true;
	}, bRecursive);
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::ForEachActorDescContainerInstance(TFunctionRef<void(ActorDescContPtrType)> Func, bool bRecursive)
{
	const_cast<const TActorDescContainerInstanceCollection*>(this)->ForEachActorDescContainerInstance(Func, bRecursive);
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* ActorDescInstance)
{
	OnActorDescInstanceAddedEvent.Broadcast(ActorDescInstance);
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* ActorDescInstance)
{
	OnActorDescInstanceRemovedEvent.Broadcast(ActorDescInstance);
}

template<class ActorDescContPtrType>
void TActorDescContainerInstanceCollection<ActorDescContPtrType>::OnActorReplaced(FWorldPartitionActorDescInstance* ActorDescInstance)
{
	OnActorReplacedEvent.Broadcast(ActorDescInstance);
}

#endif 

using FActorDescContainerInstanceCollection = TActorDescContainerInstanceCollection<TObjectPtr<UActorDescContainerInstance>>;