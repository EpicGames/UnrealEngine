// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Subsystems/EngineSubsystem.h"
#include "WorldPartition/ActorDescContainer.h"
#include "ActorDescContainerSubsystem.generated.h"

#define UE_API ENGINE_API

UCLASS(MinimalAPI)
class UActorDescContainerSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

	friend class UActorDescContainer;

public:
	UActorDescContainerSubsystem()
#if WITH_EDITOR
		: ContainerManager(this)
#endif
	{}

	// Only create in editor
	bool ShouldCreateSubsystem(UObject* Outer) const override
	{ 
#if WITH_EDITOR
		return true;
#else
		return false; 
#endif
	}

#if WITH_EDITOR
	ENGINE_API static UActorDescContainerSubsystem* Get();
	ENGINE_API static UActorDescContainerSubsystem& GetChecked();

	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	void OnAssetCollision(FAssetData& A, FAssetData& B, FAssetData*& Keep);

	DECLARE_EVENT_OneParam(UActorDescContainerSubsystem, FContainerUpdatedEvent, FName);
	DECLARE_EVENT_TwoParams(UActorDescContainerSubsystem, FContainerReplacedEvent, UActorDescContainer*, UActorDescContainer*);

	FContainerUpdatedEvent& ContainerUpdated() { return OnContainerUpdated; }
	FContainerReplacedEvent& ContainerReplaced() { return OnContainerReplaced; }
	
	template <class ContainerType = UActorDescContainer>
	ContainerType* RegisterContainer(const typename UActorDescContainer::FInitializeParams& InitParams) { return ContainerManager.RegisterContainer<ContainerType>(InitParams); }
	UActorDescContainer* GetActorDescContainer(const FString& InContainerName) { return ContainerManager.GetActorDescContainer(InContainerName); }
	const UActorDescContainer* GetActorDescContainer(const FString& InContainerName) const { return ContainerManager.GetActorDescContainer(InContainerName); }
	void RegisterContainer(UActorDescContainer* Container) { ContainerManager.RegisterContainer(Container); }
	void UnregisterContainer(UActorDescContainer* Container) { ContainerManager.UnregisterContainer(Container); }
	FBox GetContainerBounds(const FString& ContainerName, bool bIsEditorBounds = false) const { return ContainerManager.GetContainerBounds(ContainerName, bIsEditorBounds); }
	void UpdateContainerBounds(const FString& ContainerName) { ContainerManager.UpdateContainerBounds(ContainerName); }
	void SetContainerPackage(UActorDescContainer* Container, FName ContainerPackageName) { ContainerManager.SetContainerPackage(Container, ContainerPackageName); }
	void NotifyContainerUpdated(FName ContainerPackage)
	{
		ContainerManager.UpdateContainerBoundsFromPackage(ContainerPackage);
		OnContainerUpdated.Broadcast(ContainerPackage);
	}
	void NotifyContainerReplaced(UActorDescContainer* OldContainer, UActorDescContainer* NewContainer)
	{
		OnContainerReplaced.Broadcast(OldContainer, NewContainer);
	}
#endif

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
#endif
	//~ End UObject Interface

private:
#if WITH_EDITOR
	FContainerUpdatedEvent OnContainerUpdated;
	FContainerReplacedEvent OnContainerReplaced;

	class FContainerManager
	{
		FContainerManager(UActorDescContainerSubsystem* InOwner)
			: Owner(InOwner)
		{}

		FContainerManager()
			: Owner(nullptr)
		{}

		friend class UActorDescContainerSubsystem;

		struct FRegisteredContainer
		{
			FRegisteredContainer()
				: Container(nullptr)
				, RefCount(0)
				, Bounds(ForceInit)
				, EditorBounds(ForceInit)
			{}

			UE_API void AddReferencedObjects(FReferenceCollector& Collector);
			UE_API void UpdateBounds();

			TObjectPtr<UActorDescContainer> Container;
			uint32 RefCount;
			FBox Bounds;
			FBox EditorBounds;
		};

		UE_API void AddReferencedObjects(FReferenceCollector& Collector);

	public:
		template <class ContainerType>
		ContainerType* RegisterContainer(const typename UActorDescContainer::FInitializeParams& InitParams)
		{
			check(Owner);
			FRegisteredContainer* RegisteredContainer = &RegisteredContainers.FindOrAdd(InitParams.ContainerName);
			UActorDescContainer* ActorDescContainer = RegisteredContainer->Container;
			check(RegisteredContainer->RefCount == 0 || ContainerType::StaticClass() == ActorDescContainer->GetClass());

			if (RegisteredContainer->RefCount++ == 0)
			{
				ActorDescContainer = NewObject<UActorDescContainer>(Owner, ContainerType::StaticClass(), NAME_None, RF_Transient);
				RegisteredContainer->Container = ActorDescContainer;
							
				// This will potentially invalidate RegisteredContainer due to RegisteredContainers reallocation
				ActorDescContainer->Initialize(InitParams);

				check(InitParams.ContainerName == ActorDescContainer->GetContainerName());
				RegisteredContainer = &RegisteredContainers.FindChecked(ActorDescContainer->GetContainerName());
				RegisteredContainer->UpdateBounds();
			}

			return Cast<ContainerType>(ActorDescContainer);
		}

		void RegisterContainer(UActorDescContainer* Container)
		{
			FRegisteredContainer& RegisteredContainer = RegisteredContainers.FindChecked(Container->GetContainerName());
			RegisteredContainer.RefCount++;
		}

		UActorDescContainer* GetActorDescContainer(const FString& InContainerName)
		{ 
			FRegisteredContainer* RegisteredContainer = RegisteredContainers.Find(InContainerName);
			return RegisteredContainer ? RegisteredContainer->Container : nullptr;
		}

		const UActorDescContainer* GetActorDescContainer(const FString& InContainerName) const 
		{
			return const_cast<FContainerManager*>(this)->GetActorDescContainer(InContainerName);
		}
				
		UE_API void UnregisterContainer(UActorDescContainer* Container);

		UE_API FBox GetContainerBounds(const FString& ContainerName, bool bIsEditorBounds = false) const;
		UE_API void UpdateContainerBounds(const FString& ContainerName);
		UE_API void UpdateContainerBoundsFromPackage(FName ContainerPackage);

		UE_API void SetContainerPackage(UActorDescContainer* Container, FName PackageName);

	private:
		TMap<FString, FRegisteredContainer> RegisteredContainers;
		UActorDescContainerSubsystem* Owner = nullptr;
	};

	FContainerManager ContainerManager;

	TMap<FName, TSet<FAssetData>> InvalidMapAssets;
#endif
};

#undef UE_API
