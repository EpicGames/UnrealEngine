// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescType.h"

#if WITH_EDITOR
// Default Iterator Value Type is the DescType itself
template<typename DescType, class ActorType>
struct TActorDescListIteratorValueType 
{ 
	using Type = DescType;
};
#endif

template<class DescType>
class TActorDescList
{
#if WITH_EDITOR
	friend struct FWorldPartitionImplBase;

	struct FActorGuidKeyFuncs : TDefaultMapKeyFuncs<FGuid, TUniquePtr<DescType>*, false>
	{
		static inline uint32 GetKeyHash(const FGuid& Key)
		{
			return HashCombineFast(HashCombineFast(Key.A, Key.B), HashCombineFast(Key.C, Key.D));
		}
	};

protected:
	using FActorDescArray = TChunkedArray<TUniquePtr<DescType>>;

public:
	using FGuidActorDescMap = TMap<FGuid, TUniquePtr<DescType>*, FDefaultSetAllocator, FActorGuidKeyFuncs>;

	TActorDescList() : bIsProxy(false) {}
	virtual ~TActorDescList() = default;

	// Non-copyable
	TActorDescList(const TActorDescList&) = delete;
	TActorDescList& operator=(const TActorDescList&) = delete;
		
	bool IsEmpty() const { return GetActorsByGuid().Num() == 0; }
	void Empty()
	{
		check(!bIsProxy || ActorsByGuid.IsEmpty());
		ActorsByGuid.Empty();
		ActorDescList.Empty();
	}

	template<bool bConst, class ActorType>
	class TBaseIterator
	{
		static_assert(TIsDerivedFrom<ActorType, AActor>::IsDerived, "Type is not derived from AActor.");

	protected:
		using MapType = FGuidActorDescMap;
		using ValueType = typename TActorDescListIteratorValueType<DescType, ActorType>::Type;
		using IteratorType = std::conditional_t<bConst, typename MapType::TConstIterator, typename MapType::TIterator>;
		using ListType = std::conditional_t<bConst, const TActorDescList<DescType>*, TActorDescList<DescType>*>;
		using ReturnType = std::conditional_t<bConst, const ValueType*, ValueType*>;

	public:
		TBaseIterator(ListType InActorDescList, UClass* InActorClass)
			: ActorsIterator(InActorDescList->GetActorsByGuid())
			, ActorClass(InActorClass)
			, bIsProxy(InActorDescList->bIsProxy)
		{
			check(ActorClass->IsNative());
			check(ActorClass->IsChildOf(ActorType::StaticClass()));

			if (ShouldSkip())
			{
				operator++();
			}
		}

		/**
		 * Iterates to next suitable actor desc
		 */
		inline void operator++()
		{
			do
			{
				++ActorsIterator;
			} while (ShouldSkip());
		}

		/**
		 * Removes the current iterator element
		 */
		inline void RemoveCurrent()
		{
			check(!bIsProxy);
			ActorsIterator.RemoveCurrent();
		}

		/**
		 * Returns the current suitable actor desc pointed at by the Iterator
		 *
		 * @return	Current suitable actor desc
		 */
		inline ReturnType operator*() const
		{
			return StaticCast<ReturnType>(ActorsIterator->Value->Get());
		}

		/**
		 * Returns the current suitable actor desc pointed at by the Iterator
		 *
		 * @return	Current suitable actor desc
		 */
		inline ReturnType operator->() const
		{
			return StaticCast<ReturnType>(ActorsIterator->Value->Get());
		}

		/**
		 * Returns whether the iterator has reached the end and no longer points
		 * to a suitable actor desc.
		 *
		 * @return true if iterator points to a suitable actor desc, false if it has reached the end
		 */
		inline explicit operator bool() const
		{
			return (bool)ActorsIterator;
		}

		/**
		 * Returns the actor class on which the iterator iterates on
		 *
		 * @return the actor class
		 */
		inline UClass* GetActorClass() const { return ActorClass; }

	protected:
		/**
		 * Determines whether the iterator currently points to a valid actor desc or not
		 * @return true if we should skip the actor desc
		 */
		inline bool ShouldSkip() const
		{
			if (!ActorsIterator)
			{
				return false;
			}

			return !ActorsIterator->Value->Get()->GetActorNativeClass()->IsChildOf(ActorClass);
		}

		IteratorType ActorsIterator;
		UClass* ActorClass;
		bool bIsProxy;
	};

	template <class ActorType = AActor>
	class TIterator : public TBaseIterator<false, ActorType>
	{
		using BaseType = TBaseIterator<false, ActorType>;

	public:
		TIterator(typename BaseType::ListType InActorDescList, UClass* InActorClass = nullptr)
			: BaseType(InActorDescList, InActorClass ? InActorClass : ActorType::StaticClass())
		{}
	};

	template <class ActorType = AActor>
	class TConstIterator : public TBaseIterator<true, ActorType>
	{
		using BaseType = TBaseIterator<true, ActorType>;

	public:
		TConstIterator(typename BaseType::ListType InActorDescList, UClass* InActorClass = nullptr)
			: BaseType(InActorDescList, InActorClass ? InActorClass : ActorType::StaticClass())
		{}
	};

	void AddActorDescriptor(DescType* ActorDesc);
	void RemoveActorDescriptor(DescType* ActorDesc);
		
	virtual FGuidActorDescMap& GetActorsByGuid() const { return GetProxyActorsByGuid(); }
protected:
	void SetIsProxy()
	{
		check(ActorsByGuid.IsEmpty());
		bIsProxy = true;
	}
	// Allow Child class to Proxy to another list
	virtual FGuidActorDescMap& GetProxyActorsByGuid() const { return const_cast<FGuidActorDescMap&>(ActorsByGuid); }

	TUniquePtr<DescType>* GetActorDescriptor(const FGuid& ActorGuid);
	const TUniquePtr<DescType>* GetActorDescriptor(const FGuid& ActorGuid) const;

	TUniquePtr<DescType>* GetActorDescriptorChecked(const FGuid& ActorGuid);
	const TUniquePtr<DescType>* GetActorDescriptorChecked(const FGuid& ActorGuid) const;

	FActorDescArray ActorDescList;
	FGuidActorDescMap ActorsByGuid;
	bool bIsProxy;
#endif
};

#if WITH_EDITOR
template<class DescType>
void TActorDescList<DescType>::AddActorDescriptor(DescType* ActorDesc)
{
	check(!bIsProxy);
	check(ActorDesc);

	if (TUniquePtr<DescType>* ExistingActorDesc = ActorsByGuid.FindRef(ActorDesc->GetGuid()))
	{
		UE_LOGF(LogWorldPartition, Error, "Duplicated actor descriptor detected:\n\tExisting: %ls\n\t     New: %ls", 
			*ExistingActorDesc->Get()->ToString(FWorldPartitionActorDesc::EToStringMode::Full), *ActorDesc->ToString(FWorldPartitionActorDesc::EToStringMode::Full));
	}
	else
	{
		TUniquePtr<DescType>* NewActorDesc = new(ActorDescList) TUniquePtr<DescType>(ActorDesc);
		ActorsByGuid.Add(ActorDesc->GetGuid(), NewActorDesc);
	}
}

template<class DescType>
void TActorDescList<DescType>::RemoveActorDescriptor(DescType* ActorDesc)
{
	check(!bIsProxy);
	check(ActorDesc);
	
	if (!ActorsByGuid.Remove(ActorDesc->GetGuid()))
	{
		UE_LOGF(LogWorldPartition, Error, "Removing unknown actor descriptor: %ls", *ActorDesc->ToString(FWorldPartitionActorDesc::EToStringMode::Full));
	}
}

template<class DescType>
TUniquePtr<DescType>* TActorDescList<DescType>::GetActorDescriptor(const FGuid& ActorGuid)
{
	return GetProxyActorsByGuid().FindRef(ActorGuid);
}

template<class DescType>
const TUniquePtr<DescType>* TActorDescList<DescType>::GetActorDescriptor(const FGuid& ActorGuid) const
{
	return GetProxyActorsByGuid().FindRef(ActorGuid);
}

template<class DescType>
TUniquePtr<DescType>* TActorDescList<DescType>::GetActorDescriptorChecked(const FGuid& ActorGuid)
{
	return GetProxyActorsByGuid().FindChecked(ActorGuid);
}

template<class DescType>
const TUniquePtr<DescType>* TActorDescList<DescType>::GetActorDescriptorChecked(const FGuid& ActorGuid) const
{
	return GetProxyActorsByGuid().FindChecked(ActorGuid);
}

// FActorDescList supports subclassing through ActorType's and FWorldPartitionActorDescType will return  the proper iterator value type
template<class ActorType>
struct TActorDescListIteratorValueType<FWorldPartitionActorDesc, ActorType>
{
	using Type = typename FWorldPartitionActorDescType<ActorType>::Type;
};
#endif

class FActorDescList : public TActorDescList<FWorldPartitionActorDesc>
{
#if WITH_EDITOR
public:
	ENGINE_API virtual FWorldPartitionActorDesc* GetActorDesc(const FGuid& Guid);
	ENGINE_API virtual const FWorldPartitionActorDesc* GetActorDesc(const FGuid& Guid) const;

	ENGINE_API virtual FWorldPartitionActorDesc& GetActorDescChecked(const FGuid& Guid);
	ENGINE_API virtual const FWorldPartitionActorDesc& GetActorDescChecked(const FGuid& Guid) const;

	int32 GetActorDescCount() const { return GetProxyActorsByGuid().Num(); }

	ENGINE_API FWorldPartitionActorDesc* AddActor(const AActor* InActor);
#endif
};
