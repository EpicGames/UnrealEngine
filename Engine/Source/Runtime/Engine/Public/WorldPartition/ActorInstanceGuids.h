// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

class AActor;
class FArchive;
class ULevel;

struct FActorInstanceGuid
{
	FGuid ActorGuid;
	FGuid ActorInstanceGuid;

	bool IsDefault();
#if WITH_EDITOR
	void InitializeFrom(const AActor& InActor);
#endif

	ENGINE_API static void ReleaseLevelInstanceGuid(ULevel* Level);
	ENGINE_API static void SetLevelInstanceGuid(ULevel* Level, ULevel* OwnerLevel, const FGuid& Guid, const FGuid& ResolvedGuid = FGuid());
	ENGINE_API static FGuid GetLevelInstanceGuid(ULevel* Level);

	ENGINE_API static FActorInstanceGuid GetActorGuids(const AActor& InActor);
	ENGINE_API static FGuid GetActorInstanceGuid(const AActor& InActor);
	ENGINE_API static void ReleaseActorInstanceGuid(const AActor& InActor);
	ENGINE_API static void Serialize(FArchive& Ar, AActor& InActor);
};

FArchive &operator <<(FArchive& Ar, FActorInstanceGuid& InActorInstanceGuids);
