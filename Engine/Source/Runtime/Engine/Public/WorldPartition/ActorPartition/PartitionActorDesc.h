// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Containers/Map.h"
#include "WorldPartition/WorldPartitionActorDesc.h"

class FPartitionActorDesc : public FWorldPartitionActorDesc
{
	friend class FPartitionActorDescFactory;

public:
	uint32 GridSize;
	int64 GridIndexX;
	int64 GridIndexY;
	int64 GridIndexZ;
	FGuid GridGuid;

	ENGINE_API FPartitionActorDesc();

protected:
	ENGINE_API virtual void Init(const AActor* InActor) override;
	ENGINE_API virtual bool Equals(const FWorldPartitionActorDesc* Other) const override;
	virtual uint32 GetSizeOf() const override { return sizeof(FPartitionActorDesc); }
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void TransferWorldData(const FWorldPartitionActorDesc* From) override;

	ENGINE_API void SetGridIndices(double LocationX, double LocationY, double LocationZ);
};
#endif
