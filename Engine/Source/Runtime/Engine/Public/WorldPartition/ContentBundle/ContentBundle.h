// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "WorldPartition/ContentBundle/ContentBundleStatus.h"
#include "WorldPartition/ContentBundle/ContentBundleBase.h"

class UContentBundleDescriptor;
class FContentBundleClient;
class IWorldPartitionCookPackageContext;
class URuntimeHashExternalStreamingObjectBase;

class FContentBundle : public FContentBundleBase
{
public:
	ENGINE_API FContentBundle(TSharedPtr<FContentBundleClient>& InClient, UWorld* InWorld);

	//~ Begin IContentBundle Interface
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	ENGINE_API virtual bool IsValid() const override;
	ENGINE_API virtual bool HasContent() const override;
	//~ End IContentBundle Interface

protected:
	//~ Begin IContentBundle Interface
	ENGINE_API virtual void DoInitialize() override;
	ENGINE_API virtual void DoUninitialize() override;
	ENGINE_API virtual void DoInjectContent() override;
	ENGINE_API virtual void DoRemoveContent() override;
	//~ End IContentBundle Interface
	
private:
#if WITH_EDITOR
	void InitializeForPIE();
#endif

	TObjectPtr<UPackage> ExternalStreamingObjectPackage;
	TObjectPtr<URuntimeHashExternalStreamingObjectBase> ExternalStreamingObject;
};
