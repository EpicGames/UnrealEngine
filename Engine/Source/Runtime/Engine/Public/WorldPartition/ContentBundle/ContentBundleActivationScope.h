// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/ContentBundle/ContentBundleEditorSubsystemInterface.h"

class FContentBundleEditor;

#if WITH_EDITOR
class FContentBundleActivationScope
{
public:
	FContentBundleActivationScope(FGuid InContentBundleGuid)
	{
		ContentBundleGuid = InContentBundleGuid;

		IContentBundleEditorSubsystemInterface* ContentBundleEditorSubsystem = IContentBundleEditorSubsystemInterface::Get();
		ContentBundleEditorSubsystem->PushContentBundleEditing();

		if (TSharedPtr<FContentBundleEditor> ContentBundleEditor = ContentBundleEditorSubsystem->GetEditorContentBundle(ContentBundleGuid))
		{
			ContentBundleEditorSubsystem->ActivateContentBundleEditing(ContentBundleEditor);
		}
	}
	~FContentBundleActivationScope()
	{
		IContentBundleEditorSubsystemInterface* ContentBundleEditorSubsystem = IContentBundleEditorSubsystemInterface::Get();
		if (TSharedPtr<FContentBundleEditor> ContentBundleEditor = ContentBundleEditorSubsystem->GetEditorContentBundle(ContentBundleGuid))
		{
			ContentBundleEditorSubsystem->DeactivateContentBundleEditing(ContentBundleEditor);
		}

		ContentBundleEditorSubsystem->PopContentBundleEditing();
	}

private:
	FGuid ContentBundleGuid;
};
#endif
