// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/ContentBundle/ContentBundleStatus.h"
#include "UObject/ObjectPtr.h"

class UContentBundleDescriptor;
class FContentBundleClient;
class IWorldPartitionCookPackageContext;

class FContentBundleBase
{
public:
	ENGINE_API FContentBundleBase(TSharedPtr<FContentBundleClient>& InClient, UWorld* InWorld);
	ENGINE_API virtual ~FContentBundleBase();

	ENGINE_API void Initialize();
	ENGINE_API void Uninitialize();

	ENGINE_API void InjectContent();
	ENGINE_API void RemoveContent();

	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector);

	virtual bool IsValid() const = 0;
	virtual bool HasContent() const = 0;

	ENGINE_API UWorld* GetInjectedWorld() const;
	ENGINE_API const FString& GetDisplayName() const;
	ENGINE_API const FColor& GetDebugColor() const;
	ENGINE_API const TWeakPtr<FContentBundleClient>& GetClient() const;
	ENGINE_API const UContentBundleDescriptor* GetDescriptor() const;
	EContentBundleStatus GetStatus() const { return Status; }
	ENGINE_API FString GetExternalStreamingObjectPackageName() const;
	ENGINE_API FString GetExternalStreamingObjectPackagePath() const;

protected:
	ENGINE_API void SetStatus(EContentBundleStatus NewStatus);

	virtual void DoInitialize() = 0;
	virtual void DoUninitialize() = 0;
	virtual void DoInjectContent() = 0;
	virtual void DoRemoveContent() = 0;

private:
	TWeakPtr<FContentBundleClient> Client;
	UWorld* InjectedWorld;
	TObjectPtr<const UContentBundleDescriptor> Descriptor;
	EContentBundleStatus Status;
};
