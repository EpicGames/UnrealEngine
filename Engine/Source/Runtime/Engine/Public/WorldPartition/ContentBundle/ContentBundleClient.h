// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"

#include "WorldPartition/ContentBundle/ContentBundleStatus.h"

#include "ContentBundleClient.generated.h"

class UContentBundleDescriptor;
class UWorld;

UENUM()
enum class EContentBundleClientState
{
	Unregistered,
	Registered,
	ContentInjectionRequested,
	ContentRemovalRequested,
	
	// Failed state
	RegistrationFailed,
};

UENUM()
enum class EWorldContentState
{
	NoContent,
	ContentBundleInjected
};

class FContentBundleClient
{
	friend class UContentBundleEngineSubsystem;
	friend class FContentBundleBase;

public:
	static ENGINE_API TSharedPtr<FContentBundleClient> CreateClient(const UContentBundleDescriptor* InContentBundleDescriptor, FString const& InDisplayName);

	ENGINE_API FContentBundleClient(const UContentBundleDescriptor* InContentBundleDescriptor, FString const& InDisplayName);
	virtual ~FContentBundleClient() = default;

	const UContentBundleDescriptor* GetDescriptor() const { return ContentBundleDescriptor.Get(); }

	ENGINE_API void RequestContentInjection();
	ENGINE_API void RequestRemoveContent();
	
	ENGINE_API void RequestUnregister();

	EContentBundleClientState GetState() const { return State; }

	FString const& GetDisplayName() const { return DisplayName; }

	ENGINE_API virtual bool ShouldInjectContent(UWorld* World) const;
	ENGINE_API virtual bool ShouldRemoveContent(UWorld* World) const;

protected:
	virtual void DoOnContentRegisteredInWorld(UWorld* InjectedWorld) {};
	virtual void DoOnContentInjectedInWorld(EContentBundleStatus InjectionStatus, UWorld* InjectedWorld) {};
	virtual void DoOnContentRemovedFromWorld(UWorld* InjectedWorld) {};

	virtual void DoOnClientToUnregister() {};

private:
	ENGINE_API bool HasContentToRemove() const;

	ENGINE_API void OnContentRegisteredInWorld(EContentBundleStatus ContentBundleStatus, UWorld* World);
	ENGINE_API void OnContentInjectedInWorld(EContentBundleStatus InjectionStatus, UWorld* InjectedWorld);
	ENGINE_API void OnContentRemovedFromWorld(EContentBundleStatus RemovalStatus, UWorld* InjectedWorld);

	ENGINE_API void SetState(EContentBundleClientState State);
	ENGINE_API void SetWorldContentState(UWorld* World, EWorldContentState State);
	ENGINE_API EWorldContentState GetWorldContentState(UWorld* World) const;

	TWeakObjectPtr<const UContentBundleDescriptor> ContentBundleDescriptor;
	
	TMap<TWeakObjectPtr<UWorld>, EWorldContentState> WorldContentStates;

	FString DisplayName;

	EContentBundleClientState State;
};
