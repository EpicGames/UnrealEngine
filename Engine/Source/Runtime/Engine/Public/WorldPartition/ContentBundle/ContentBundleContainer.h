// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_EDITOR
#include "Misc/TVariant.h"
#endif

class FContentBundleBase;
class FContentBundle;
class FContentBundleEditor;
class FContentBundleClient;
class IWorldPartitionCookPackageContext;
class UContentBundleDescriptor;
class FReferenceCollector;

/*
* Stores the content bundles for a given world.
* In Editor builds, the content bundles can be store as 2 different types:
*	1. Editor and Non-Game Worlds:
*		- The content bundles are stored as FContentBundleEditor, which support editing of content bundles.
*		- They are stored in TSharedPtr to allow the UI to keep TWeakPtr.
*	2. Editor and Game Worlds (PIE)
*		- The content bundles are stored as FContentBundles, which support injections of streaming objects and cells in a runtime hash.
*		- They are stored in TUniquePtr.
* 
* A TVariant is use to express that behavior in editor builds. In Editor and Non-Game Worlds the variant will contain FContentBundleEditors. In Game-Worlds it will contain FContentBundles.
* 
* In Non-Editor builds, the content bundles are always stored as FContentBundles.
*/
class FContentBundleContainer
{
public:
	FContentBundleContainer(UWorld* WorldToInjectIn);
	~FContentBundleContainer();

	UWorld* GetInjectedWorld() const;

	void Initialize();
	void Deinitialize();

	void AddReferencedObjects(FReferenceCollector& Collector);

	uint32 GetNumContentBundles() const;

	const TArray<TUniquePtr<FContentBundle>>& GetGameContentBundles() const;
	TArray<TUniquePtr<FContentBundle>>& GetGameContentBundles();

	bool InjectContentBundle(FContentBundleClient& ContentBundleClient);
	bool RemoveContentBundle(FContentBundleClient& ContentBundleClient);

#if WITH_EDITOR
	TSharedPtr<FContentBundleEditor> GetEditorContentBundle(const UContentBundleDescriptor* Descriptor) const;
	TSharedPtr<FContentBundleEditor> GetEditorContentBundle(const FGuid& ContentBundleGuid) const;

	const TArray<TSharedPtr<FContentBundleEditor>>& GetEditorContentBundles() const;
	TArray<TSharedPtr<FContentBundleEditor>>& GetEditorContentBundles();

	bool UseEditorContentBundle() const;
#endif

	void ForEachContentBundle(TFunctionRef<void(FContentBundleBase*)> Func) const;
	void ForEachContentBundleBreakable(TFunctionRef<bool(FContentBundleBase*)> Func) const;

private:
	FContentBundleBase* GetContentBundle(const FContentBundleClient& ContentBundleClient);

	void RegisterContentBundleClientEvents();
	void UnregisterContentBundleClientEvents();

	void OnContentBundleClientRegistered(TSharedPtr<FContentBundleClient>& ContentBundleClient);
	void OnContentBundleClientUnregistered(FContentBundleClient& ContentBundleClient);
	void OnContentBundleClientContentInjectionRequested(FContentBundleClient& ContentBundleClient);
	void OnContentBundleClientContentRemovalRequested(FContentBundleClient& ContentBundleClient);

	FContentBundleBase& InitializeContentBundle(TSharedPtr<FContentBundleClient>& ContentBundleClient);
	void DeinitializeContentBundle(FContentBundleBase& ContentBundle);

	void InitializeContentBundlesForegisteredClients();
	void DeinitializeContentBundles();

	bool InjectContentBundle(FContentBundleBase& ContentBundle);
	bool RemoveContentBundle(FContentBundleBase& ContentBundle);

#if WITH_EDITOR
	void OnPreGenerateStreaming(TArray<FString>* OutPackageToGenerate);
	void OnBeginCook(IWorldPartitionCookPackageContext& CookContext);
	void OnEndCook(IWorldPartitionCookPackageContext& CookContext);
#endif

	using ContentBundleGameArray = TArray<TUniquePtr<FContentBundle>>;
#if WITH_EDITOR
	using ContentBundleEditorArray = TArray<TSharedPtr<FContentBundleEditor>>;
	TVariant<ContentBundleEditorArray, ContentBundleGameArray> ContentBundlesVariant;
#else
	ContentBundleGameArray ContentBundles;
#endif

	UWorld* InjectedWorld;
};