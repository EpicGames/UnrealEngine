// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "ContentBundleDescriptor.generated.h"

class UActorDescContainer;
class AWorldDataLayers;
struct FColor;

UCLASS(MinimalAPI)
class UContentBundleDescriptor : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	const FString& GetDisplayName() const { return DisplayName; }
	const FColor& GetDebugColor() const { return DebugColor; }
	ENGINE_API FString GetPackageRoot() const;
	const FGuid& GetGuid() const { return Guid; }

#if WITH_EDITOR
	ENGINE_API void InitializeObject(const FString& InContentBundleName);

	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//~ End UObject Interface
#endif

	ENGINE_API bool IsValid() const;

	// Helper method that returns a compact string for a given content bundle ID
	static ENGINE_API FString GetContentBundleCompactString(const FGuid& InContentBundleID);

private:
	ENGINE_API void InitDebugColor();

	UPROPERTY(EditAnywhere, Category = BaseInformation)
	FString DisplayName;

	UPROPERTY(EditAnywhere, DuplicateTransient, Category = BaseInformation)
	FColor DebugColor;

	UPROPERTY(VisibleAnywhere, DuplicateTransient, Category = BaseInformation, AdvancedDisplay)
	FGuid Guid;
};
