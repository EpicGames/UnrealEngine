// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_EDITOR
#include "WorldPartition/ContentBundle/WorldDataLayerReference.h"
#include "WorldPartition/ContentBundle/ContentBundleBase.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageGenerator.h"
#endif

#include "ContentBundleEditor.generated.h"

class AActor;

#if WITH_EDITOR

class URuntimeHashExternalStreamingObjectBase;
class UWorldPartitionRuntimeCell;
class UActorDescContainerInstance;
class UWorldPartitionActorDescInstance;

class FContentBundleEditor : public FContentBundleBase, IWorldPartitionCookPackageGenerator
{
	friend class UContentBundleUnsavedActorMonitor;

public:
	ENGINE_API FContentBundleEditor(TSharedPtr<FContentBundleClient>& InClient, UWorld* InWorld);
	ENGINE_API ~FContentBundleEditor();

	//~ Begin IContentBundle Interface
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	ENGINE_API virtual bool IsValid() const override;
	ENGINE_API virtual bool HasContent() const override;
	//~ End IContentBundle Interface

	const FGuid& GetTreeItemID() const { return TreeItemID; }

	ENGINE_API void StartEditing();
	ENGINE_API void StopEditing();
	bool IsBeingEdited() const { return bIsBeingEdited; }

	ENGINE_API void InjectBaseContent();

	ENGINE_API bool AddActor(AActor* InActor);
	ENGINE_API bool ContainsActor(const AActor* InActor) const;
	ENGINE_API bool GetActors(TArray<AActor*>& Actors);
	ENGINE_API bool HasUserPlacedActors() const;
	ENGINE_API uint32 GetActorCount() const;
	ENGINE_API uint32 GetUnsavedActorAcount() const;

	ENGINE_API void ReferenceAllActors();
	ENGINE_API void UnreferenceAllActors();

	UE_DEPRECATED(5.4, "Use GetActorDescContainerInstance instead")
	TWeakObjectPtr<UActorDescContainer> GetActorDescContainer() const { return nullptr; };

	const TWeakObjectPtr<UActorDescContainerInstance> GetActorDescContainerInstance() const { return ActorDescContainerInstance; }


	ENGINE_API void GenerateStreaming(TArray<FString>* OutPackageToGenerate, bool bIsPIE);

	URuntimeHashExternalStreamingObjectBase* GetStreamingObject() const { return ExternalStreamingObject; }

	// Cooking
	ENGINE_API void OnBeginCook(IWorldPartitionCookPackageContext& CookContext);
	ENGINE_API void OnEndCook(IWorldPartitionCookPackageContext& CookContext);
	bool HasCookedContent() const { return ExternalStreamingObject != nullptr; }
	//~Begin IWorldPartitionCookPackageGenerator
	ENGINE_API virtual bool GatherPackagesToCook(class IWorldPartitionCookPackageContext& CookContext) override;
	ENGINE_API virtual bool PopulateGeneratorPackageForCook(class IWorldPartitionCookPackageContext& CookContext, const TArray<FWorldPartitionCookPackage*>& PackagesToCook, TArray<UPackage*>& OutModifiedPackages) override;
	ENGINE_API virtual bool PopulateGeneratedPackageForCook(class IWorldPartitionCookPackageContext& CookContext, const FWorldPartitionCookPackage& PackageToCook, TArray<UPackage*>& OutModifiedPackages) override;
	ENGINE_API virtual UWorldPartitionRuntimeCell* GetCellForPackage(const FWorldPartitionCookPackage& PackageToCook) const override;
	//~End IWorldPartitionCookPackageGenerator

protected:
	//~ Begin IContentBundle Interface
	ENGINE_API virtual void DoInitialize() override;
	ENGINE_API virtual void DoUninitialize() override;
	ENGINE_API virtual void DoInjectContent() override;
	ENGINE_API virtual void DoRemoveContent() override;
	//~ End IContentBundle Interface

private:
	ENGINE_API void OnUnsavedActorDeleted(AActor* Actor);

	ENGINE_API void BroadcastChanged();
	ENGINE_API UPackage* CreateActorPackage(const FName& ActorName) const;
	ENGINE_API FName BuildWorlDataLayersName() const;

	ENGINE_API void RegisterDelegates();
	ENGINE_API void UnregisterDelegates();
	
	ENGINE_API void OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* ActorDesc);
	ENGINE_API void OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* ActorDesc);

	TObjectPtr<class UContentBundleUnsavedActorMonitor> UnsavedActorMonitor;

	TWeakObjectPtr<UActorDescContainerInstance> ActorDescContainerInstance;
	FWorldDataLayersReference WorldDataLayersActorReference;

	TArray<FWorldPartitionReference> ForceLoadedActors;

	TMap<uint32, const UWorldPartitionRuntimeCell*> CookPackageIdsToCell;
	TObjectPtr<URuntimeHashExternalStreamingObjectBase> ExternalStreamingObject;

	// Used by Content Bundle Outliner to link tree item to FContentBundle instance.
	FGuid TreeItemID;

	bool bIsBeingEdited;
	bool bIsInCook;
};

#endif

UCLASS()
class UContentBundleUnsavedActorMonitor : public UObject
{
	GENERATED_BODY()

public:
	~UContentBundleUnsavedActorMonitor();

#if WITH_EDITOR
	void Initialize(FContentBundleEditor& InContentBundleEditor);
	void Uninitialize();

	void StartListenOnActorEvents();
	void StopListeningOnActorEvents();

	void MonitorActor(AActor* InActor);
	bool StopMonitoringActor(AActor* InActor);

	uint32 GetActorCount() const { return UnsavedActors.Num(); }

	bool IsMonitoringActors() const { return !UnsavedActors.IsEmpty(); }
	bool IsMonitoring(const AActor* Actor) const;

	const TArray<TWeakObjectPtr<AActor>>& GetUnsavedActors() const { return UnsavedActors; }
#endif

private:
#if WITH_EDITOR
	void OnActorDeleted(AActor* InActor);
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<TWeakObjectPtr<AActor>> UnsavedActors;
#endif

#if WITH_EDITOR
	FContentBundleEditor* ContentBundle;
#endif
};
