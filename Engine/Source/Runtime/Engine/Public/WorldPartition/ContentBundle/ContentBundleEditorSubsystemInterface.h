// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class FContentBundleEditor;

#if WITH_EDITOR
class IContentBundleEditorSubsystemInterface
{
public:
	static IContentBundleEditorSubsystemInterface* Get() { return Instance; }

	virtual ~IContentBundleEditorSubsystemInterface() = default;

	virtual void NotifyContentBundleAdded(const FContentBundleEditor* ContentBundle) = 0;
	virtual void NotifyContentBundleRemoved(const FContentBundleEditor* ContentBundle) = 0;
	virtual void NotifyContentBundleInjectedContent(const FContentBundleEditor* ContentBundle) = 0;
	virtual void NotifyContentBundleRemovedContent(const FContentBundleEditor* ContentBundle) = 0;
	virtual void NotifyContentBundleChanged(const FContentBundleEditor* ContentBundle) = 0;

	virtual TSharedPtr<FContentBundleEditor> GetEditorContentBundle(const FGuid& ContentBundleGuid) const = 0;
	
	virtual bool IsEditingContentBundle() const = 0;
	virtual bool ActivateContentBundleEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor) const = 0;
	virtual bool DeactivateContentBundleEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor) const = 0;
	virtual bool DeactivateCurrentContentBundleEditing() const = 0;
	virtual void PushContentBundleEditing() { PushContentBundleEditing(false); }
	virtual void PushContentBundleEditing(bool bDuplicateContext) = 0;
	virtual void PopContentBundleEditing() = 0;

	DECLARE_EVENT_OneParam(UContentBundleEditorSubsystem, FOnContentBundleAdded, const FContentBundleEditor*);
	FOnContentBundleAdded& OnContentBundleAdded() { return ContentBundleAdded; }

	DECLARE_EVENT_OneParam(UContentBundleEditorSubsystem, FOnContentBundleRemoved, const FContentBundleEditor*);
	FOnContentBundleRemoved& OnContentBundleRemoved() { return ContentBundleRemoved; }

	DECLARE_EVENT_OneParam(UContentBundleEditorSubsystem, FOnContentBundleRemovedContent, const FContentBundleEditor*);
	FOnContentBundleRemovedContent& OnContentBundleRemovedContent() { return ContentBundleRemovedContent; }

	DECLARE_EVENT_OneParam(UContentBundleEditorSubsystem, FOnContentBundleChanged, const FContentBundleEditor*);
	FOnContentBundleChanged& OnContentBundleChanged() { return ContentBundleChanged; }

protected:
	static ENGINE_API void SetInstance(IContentBundleEditorSubsystemInterface* InInstance);
	static ENGINE_API IContentBundleEditorSubsystemInterface* Instance;

private:
	FOnContentBundleAdded ContentBundleAdded;
	FOnContentBundleRemoved ContentBundleRemoved;
	FOnContentBundleRemovedContent ContentBundleRemovedContent;
	FOnContentBundleChanged ContentBundleChanged;
};
#endif
