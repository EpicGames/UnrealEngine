// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/World.h"
#include "Subsystems/EngineSubsystem.h"

#include "ContentBundleEngineSubsystem.generated.h"

class FContentBundleClient;
class UContentBundleDescriptor;
class UContentBundleTypeFactory;
class URuntimeHashExternalStreamingObjectBase;

UCLASS(Config = Engine, MinimalAPI)
class UContentBundleEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;

	ENGINE_API TSharedPtr<FContentBundleClient> RegisterContentBundle(const UContentBundleDescriptor* Descriptor, const FString& ClientDisplayName);
	ENGINE_API void UnregisterContentBundle(FContentBundleClient& Client);

	ENGINE_API void RequestContentInjection(FContentBundleClient& Client);
	ENGINE_API void RequestContentRemoval(FContentBundleClient& Client);

	ENGINE_API const UContentBundleDescriptor* GetContentBundleDescriptor(const FGuid& ContentBundleGuid) const;

	DECLARE_MULTICAST_DELEGATE_OneParam(FContentBundleRegisterDelegate, TSharedPtr<FContentBundleClient>&);
	FContentBundleRegisterDelegate OnContentBundleClientRegistered;

	DECLARE_MULTICAST_DELEGATE_OneParam(FContentBundleEventDelegate, FContentBundleClient&);
	FContentBundleEventDelegate OnContentBundleClientUnregistered;
	FContentBundleEventDelegate OnContentBundleClientRequestedContentInjection;
	FContentBundleEventDelegate OnContentBundleClientRequestedContentRemoval;

	TArrayView<TSharedPtr<FContentBundleClient>> GetContentBundleClients() { return MakeArrayView(ContentBundleClients); }

	static ENGINE_API UContentBundleEngineSubsystem* Get();

#if WITH_EDITOR
	void SetEditingContentBundleGuid(FGuid InGuid) { EditingContentBundleGuid = InGuid; }
	FGuid GetEditingContentBundleGuid() { return EditingContentBundleGuid; }
#endif
private:
	ENGINE_API void OnPreWorldInitialization(UWorld* World, const UWorld::InitializationValues IVS);
	ENGINE_API void OnWorldPostCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	ENGINE_API TSharedPtr<FContentBundleClient>* FindRegisteredClient(const UContentBundleDescriptor* Descriptor);
	ENGINE_API TSharedPtr<FContentBundleClient>* FindRegisteredClient(FContentBundleClient& InClient);

	TArray<TSharedPtr<FContentBundleClient>> ContentBundleClients;

	UPROPERTY(Config)
	TSoftClassPtr<UContentBundleTypeFactory> ContentBundleTypeFactoryClass;

	UPROPERTY(Transient)
	TObjectPtr<UContentBundleTypeFactory> ContentBundleTypeFactory;

#if WITH_EDITOR
	// @todo_ow: This will no longer be needed once we move ContentBundles to External Data Layers but currently the ActorPartitionSubsystem needs to get the current ContentBundleGuid to build a proper actor name
	FGuid EditingContentBundleGuid;
#endif
};
