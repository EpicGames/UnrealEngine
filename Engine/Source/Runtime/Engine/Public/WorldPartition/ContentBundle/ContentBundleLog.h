// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogContentBundle, Log, All);

class FContentBundleBase;	
class FContentBundleClient;
class FContentBundleContainer;
class UContentBundleDescriptor;
class UWorld;

namespace ContentBundle
{
	namespace Log
	{
		ENGINE_API FString MakeDebugInfoString(const FContentBundleBase& ContentBundle);
		ENGINE_API FString MakeDebugInfoString(const FContentBundleClient& ContentBundleClient);
		ENGINE_API FString MakeDebugInfoString(const FContentBundleContainer& ContentBundleContainer);
		ENGINE_API FString MakeDebugInfoString(const FContentBundleClient& ContentBundleClient, UWorld* World);
	}
}