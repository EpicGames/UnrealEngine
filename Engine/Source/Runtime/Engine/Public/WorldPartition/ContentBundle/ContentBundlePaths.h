// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/StringView.h"
#include "UObject/AssetRegistryTagsContext.h"
class FContentBundleBase;

namespace ContentBundlePaths
{
	inline constexpr FStringView ContentBundleFolder = TEXTVIEW("/ContentBundle/");
	inline constexpr FStringView GeneratedFolder = TEXTVIEW("/_Generated_/");
	inline constexpr FStringView ContentBundleAssetRegistryTagString = TEXTVIEW("ContentBundleGuids");	

	// Returns "/ContentBundle/"
	constexpr FStringView GetContentBundleFolder() { return ContentBundleFolder; }
	
	// Returns "ContentBundle"
	constexpr FStringView GetContentBundleFolderName()
	{
		// Start at index 1 to remove the first "/", Lenght - 2 to remove the trailing "/"
		return FStringView(&GetContentBundleFolder()[1], GetContentBundleFolder().Len() - 2);
	}

	// Returns "/ContentBundle/"
	constexpr FStringView GetGeneratedFolder() { return GeneratedFolder; }
	
	// Returns "_Generated_"
	constexpr FStringView GetGeneratedFolderName()
	{
		// Start at index 1 to remove the first "/", Lenght - 2 to remove the trailing "/"
		return FStringView(&GetGeneratedFolder()[1], GetGeneratedFolder().Len() - 2);
	}

	ENGINE_API FName GetContentBundleGuidsAssetRegistryTag();
	ENGINE_API void AddRegistryTags(FAssetRegistryTagsContext Context, TArray<FGuid>& ContentBundleGuids);
	ENGINE_API TArray<FGuid> ParseContentBundleGuids(const FString& GuidsStr);
	ENGINE_API TArray<FGuid> ParseContentBundleGuids(const FAssetData& AssetData);
	ENGINE_API void ForEachContentBundleLevelPackagePath(const FString& InLevelPackageName, TFunctionRef<void(const FString&)> Func);

	// Return value uses a reduced amount of character to avoid reaching character limit when cooking 
	// return format is /{MountPoint}/CB/{ContentBundle_ShortUID}/{LevelPath}/{GeneratedFolder}/
	FString GetCookedContentBundleLevelFolder(const FContentBundleBase& ContentBundle);
	
	// return format is {LevelPath}
	FString GetRelativeLevelFolder(const FContentBundleBase& ContentBundle);

#if WITH_EDITOR
	// return format is /{MountPoint}/{ContentBundleFolder}/{ContentBundleUID}/
	ENGINE_API bool BuildContentBundleExternalActorPath(const FString& InContenBundleMountPoint, const FGuid& InContentBundleGuid, FString& OutContentBundleRootPath);

	// return format is /{MountPoint}/{ExternalActorFolder}/{ContentBundleFolder}/{ContentBundleUID}
	ENGINE_API bool BuildContentBundleActorsRootPath(const FString& InContenBundleMountPoint, const FGuid& InContentBundleGuid,  FString& OutContainerPackagePath);

	// return format is /{MountPoint}/{ContentBundleFolder}/{ContentBundleUID}/{LevelPath}/
	ENGINE_API bool BuildActorDescContainerPackagePath(const FString& InContenBundleMountPoint, const FGuid& InContentBundleGuid, const FString& InLevelPackagePath, FString& OutContainerPackagePath);

	// InContentBundlePath format is : */{ContentBundleFolder}/{ContentBundleUID}/{RelativePath}
	// return format is {RelativePath}, empty otherwise
	ENGINE_API FStringView GetRelativePath(FStringView InContentBundlePath);

	// return true if InPackage follow format : */{ContentBundleFolder}/{ContentBundleUID}/*, false otherwise
	ENGINE_API bool IsAContentBundlePath(FStringView InContentBundlePath);

	// return an ExternalActor path following the format : /{MountPoint}/{ExternalActorFolder}/{ContentBundleFolder}/{ContentBundleUID}/{LevelPath}/{ExternalActorPackagePath}
	ENGINE_API FString MakeExternalActorPackagePath(const FString& ContentBundleExternalActorFolder, const FString& ActorName);

	// return true if InPackage follow format : /{MountPoint}/{ExternalActorFolder}/{ContentBundleFolder}/{ContentBundleUID}/*, false otherwise
	ENGINE_API bool IsAContentBundleExternalActorPackagePath(FStringView InPackagePath);

	// InContentBundleExternalActorPackagePath format is : /{MountPoint}/{ExternalActorFolder}/{ContentBundleFolder}/{ContentBundleUID}/{LevelPath}/{ExternalActorPackagePath}
	// return format is /{LevelPath}/{ExternalActorPackagePath}, empty otherwise
	ENGINE_API FStringView GetRelativeExternalActorPackagePath(FStringView InContentBundleExternalActorPackagePath);

	// InContentBundleExternalActorPackagePath format is : /{MountPoint}/{ExternalActorFolder}/{ContentBundleFolder}/{ContentBundleUID}/{LevelPath}/{ExternalActorPackagePath}
	// return format is {ContentBundleUID}, 0 otherwise
	ENGINE_API FGuid GetContentBundleGuidFromExternalActorPackagePath(FStringView InContentBundleExternalActorPackagePath);

	// InExternalActorPath format is : /{MountPoint}/{ExternalActorFolder}/{ContentBundleFolder}/{ContentBundleUID}/{LevelPath}/{ExternalActorPackagePath}
	// return format is /{ContentBundleFolder}/{ContentBundleUID}/{LevelPath}/{ExternalActorPackagePath}, empty otherwise
	ENGINE_API FStringView GetActorPathRelativeToExternalActors(FStringView InContentBundleExternalActorPackagePath);
#endif
}