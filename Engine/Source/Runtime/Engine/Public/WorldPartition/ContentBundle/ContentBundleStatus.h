// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "ContentBundleStatus.generated.h"

UENUM()
enum class EContentBundleStatus
{
	Registered,
	ReadyToInject,
	FailedToInject,
	ContentInjected,
	Unknown = -1
};