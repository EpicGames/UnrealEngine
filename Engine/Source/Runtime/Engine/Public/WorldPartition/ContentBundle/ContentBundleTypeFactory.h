// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "ContentBundleTypeFactory.generated.h"

class FContentBundleClient;
class UContentBundleDescriptor;

UCLASS(MinimalAPI)
class UContentBundleTypeFactory : public UObject
{
	GENERATED_BODY()

public:
	ENGINE_API virtual TSharedPtr<FContentBundleClient> CreateClient(const UContentBundleDescriptor* Descriptor, const FString& ClientDisplayName);
};
