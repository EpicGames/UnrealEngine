// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "WorldPartition/ContentBundle/ContentBundleContainer.h"

#include "ContentBundleWorldSubsystem.generated.h"

class FContentBundle;
class UContentBundleDescriptor;
class FContentBundleClient;
class FContentBundleEditor;
class UWorldPartition;
class UCanvas;

#if WITH_EDITOR
class URuntimeHashExternalStreamingObjectBase;
class UContentBundleDuplicateForPIEHelper;
#endif

UCLASS(MinimalAPI)
class UContentBundleManager : public UObject
{
	GENERATED_BODY()

public:
	ENGINE_API UContentBundleManager();

	ENGINE_API void Initialize();
	ENGINE_API void Deinitialize();

	ENGINE_API bool CanInject() const;

	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

#if WITH_EDITOR
	ENGINE_API bool GetEditorContentBundle(TArray<TSharedPtr<FContentBundleEditor>>& OutContentBundles);
	ENGINE_API TSharedPtr<FContentBundleEditor> GetEditorContentBundle(const UContentBundleDescriptor* Descriptor, const UWorld* ContentBundleWorld) const;
	ENGINE_API TSharedPtr<FContentBundleEditor> GetEditorContentBundle(const FGuid& ContentBundleGuid) const;

	UContentBundleDuplicateForPIEHelper* GetPIEDuplicateHelper() const { return PIEDuplicateHelper; }

	ENGINE_API void CheckForErrors() const;
#endif

	ENGINE_API const FContentBundleBase* GetContentBundle(const UWorld* InWorld, const FGuid& Guid) const;
	ENGINE_API void DrawContentBundlesStatus(const UWorld* InWorld, UCanvas* Canvas, FVector2D& Offset) const;

private:
	ENGINE_API uint32 GetContentBundleContainerIndex(const UWorld* InjectedWorld) const;
	ENGINE_API const TUniquePtr<FContentBundleContainer>* GetContentBundleContainer(const UWorld* InjectedWorld) const;
	ENGINE_API TUniquePtr<FContentBundleContainer>* GetContentBundleContainer(const UWorld* InjectedWorld);

	ENGINE_API void OnWorldPartitionInitialized(UWorldPartition* WorldPartition);
	ENGINE_API void OnWorldPartitionUninitialized(UWorldPartition* WorldPartition);

	TArray<TUniquePtr<FContentBundleContainer>> ContentBundleContainers;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UContentBundleDuplicateForPIEHelper> PIEDuplicateHelper;
#endif
};



UCLASS()
class UContentBundleDuplicateForPIEHelper : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	void Initialize();
	void Deinitialize();

	bool StoreContentBundleStreamingObect(const FContentBundleEditor& ContentBundleEditor, URuntimeHashExternalStreamingObjectBase* StreamingObject);
	URuntimeHashExternalStreamingObjectBase* RetrieveContentBundleStreamingObject(const FContentBundle& ContentBundle) const;

	uint32 GetStreamingObjectCount() { return StreamingObjects.Num(); }
	void Clear() { StreamingObjects.Empty(); }
#endif

private:
	void OnPIEEnded(const bool bIsSimulating);

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TMap<FGuid, TObjectPtr<URuntimeHashExternalStreamingObjectBase>> StreamingObjects;
#endif

};
