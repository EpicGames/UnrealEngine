// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionHandle.h"
#include "Misc/TVariant.h"

class UActorDescContainer;
class AWorldDataLayers;
struct FActorSpawnParameters;

class FWorldDataLayersReference
{
public:
	ENGINE_API FWorldDataLayersReference();

	ENGINE_API FWorldDataLayersReference(UActorDescContainerInstance* ContainerInstance, FName WorldDataLayerName); // Retrieve Actor
	ENGINE_API FWorldDataLayersReference(const FActorSpawnParameters& SpawnParameters); // Retrieve and Create Actor if Not Found, FActorSpawnParameters::Name must be set
	ENGINE_API FWorldDataLayersReference(FWorldDataLayersReference&& Other);

	ENGINE_API ~FWorldDataLayersReference();

	bool IsValid() const { return Get() != nullptr; }

	AWorldDataLayers* operator->() { return const_cast<AWorldDataLayers*>(const_cast<const FWorldDataLayersReference*>(this)->operator->()); }
	const AWorldDataLayers* operator->() const { return Get(); }

	ENGINE_API const AWorldDataLayers* Get() const;
	AWorldDataLayers* Get() { return const_cast<AWorldDataLayers*>(const_cast<const FWorldDataLayersReference*>(this)->Get()); }

	ENGINE_API void Reset();

	ENGINE_API FWorldDataLayersReference& operator=(FWorldDataLayersReference&& Other);

private:
	ENGINE_API bool TrySetReference(UActorDescContainerInstance* ContainerInstance, FName WorldDataLayerName);

	TVariant<AWorldDataLayers*, FWorldPartitionReference> WorldDataLayersVariant;
};

#endif // WITH_EDITOR
