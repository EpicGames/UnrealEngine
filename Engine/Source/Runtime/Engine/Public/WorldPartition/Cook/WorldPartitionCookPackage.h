// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "Hash/CityHash.h"
#include "Misc/Paths.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "UObject/Package.h"
#include "Hash/Blake3.h"

using FWorldPartitionPackageHash = FBlake3Hash;
using FWorldPartitionPackageHashBuilder = FBlake3;

#else
#include "Containers/StringView.h"
#endif

struct FWorldPartitionCookPackage
{
#if WITH_EDITOR

	enum class EType
	{
		Unknown,
		Level,
		Generic
	};

	using IDType = uint64;

	static IDType MakeCookPackageID(const FString& InRoot, const FString& InRelativeFilename)
	{
		check(!InRoot.IsEmpty() && !InRelativeFilename.IsEmpty());
		check(InRoot[0] == '/' && InRoot[InRoot.Len() - 1] != '/'); // Root is assumed to be in the format "/Root"
		check(InRelativeFilename[0] == '/' && InRelativeFilename[InRelativeFilename.Len() - 1] != '/'); // RelativeFileName is assumed to be in the format "/Relativefilename

		// Avoid doing string copies as this function is often called during cook when bridging between Cook code & WorldPartition code.
		// Compute a hash for both InRoot & InRelativeFilename. Then combine them instead of creating a new fullpath string and computing the hash on it.
		uint64 RootHash = CityHash64(reinterpret_cast<const char*>(*InRoot), InRoot.Len() * sizeof(TCHAR));
		uint64 RelativePathHash = CityHash64(reinterpret_cast<const char*>(*InRelativeFilename), InRelativeFilename.Len() * sizeof(TCHAR));
		return RootHash ^ RelativePathHash;
	}

	// PathComponents (Root & RelativePath members) need to follow the "/<PathComponent>" format for the PackageId computation to work.
	static FString SanitizePathComponent(const FString& Path)
	{
		FString SanitizedPath = TEXT("/") + Path;
		FPaths::RemoveDuplicateSlashes(SanitizedPath);

		if (SanitizedPath[SanitizedPath.Len() - 1] == '/')
		{
			SanitizedPath.RemoveAt(SanitizedPath.Len() - 1, 1);
		}

		return SanitizedPath;
	}

	static FString MakeGeneratedFullPath(const FString& InRoot, const FString& InRelativeFilename)
	{
		TStringBuilderWithBuffer<TCHAR, NAME_SIZE> FullPath;
		FullPath += TEXT("/");
		FullPath += InRoot;
		FullPath += GeneratedFolder;
		FullPath += InRelativeFilename;

		return FPaths::RemoveDuplicateSlashes(*FullPath);
	}

	FWorldPartitionCookPackage(const FString& InRoot, const FString& InRelativePath, EType InType, const FWorldPartitionPackageHash& InGenerationHash)
		: Root(SanitizePathComponent(InRoot)),
		RelativePath(SanitizePathComponent(InRelativePath)),
		PackageId(MakeCookPackageID(Root, RelativePath)),
		Type(InType),
		GenerationHash(InGenerationHash)
	{
	}

	FString GetFullGeneratedPath() const { return MakeGeneratedFullPath(Root, RelativePath); }

	UPackage* GetPackage() const { return FindObject<UPackage>(nullptr, *GetFullGeneratedPath()); }

	const FString Root;
	const FString RelativePath;
	const IDType PackageId;
	const EType Type;
	const FWorldPartitionPackageHash GenerationHash;
#endif

	/** Returns "_Generated_" */
	static FStringView GetGeneratedFolderName() { return FStringView(&GeneratedFolder[1], GeneratedFolder.Len() - 2); }

private:
	static constexpr FStringView GeneratedFolder = TEXTVIEW("/_Generated_/");
};