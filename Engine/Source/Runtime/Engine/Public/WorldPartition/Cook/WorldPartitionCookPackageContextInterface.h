// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "Cooker/CookDependency.h"

class IWorldPartitionCookPackageGenerator;
class IWorldPartitionCookPackageObject;
struct FWorldPartitionCookPackage;
class AActor;

struct FWorldPartitionCookPackageContextParams
{
	TArray<TSubclassOf<AActor>> FilteredClasses;
};

class IWorldPartitionCookPackageContext
{
public:
	virtual ~IWorldPartitionCookPackageContext() = default;

	virtual void RegisterPackageCookPackageGenerator(IWorldPartitionCookPackageGenerator* Generator) = 0;
	virtual void UnregisterPackageCookPackageGenerator(IWorldPartitionCookPackageGenerator* Generator) = 0;

	virtual const FWorldPartitionCookPackage* AddPackageToGenerate(IWorldPartitionCookPackageGenerator* Generator, IWorldPartitionCookPackageObject* InCookPackageObject, const FString& Root, const FString& RelativePath) = 0;
	virtual FString GetGeneratedPackagePath(IWorldPartitionCookPackageObject* InCookPackageObject) const = 0;

	virtual bool GatherPackagesToCook(const FWorldPartitionCookPackageContextParams& Params = FWorldPartitionCookPackageContextParams()) = 0;
	virtual const FWorldPartitionCookPackageContextParams& GetParams() const = 0;

	virtual void ReportSaveDependency(UE::Cook::FCookDependency CookDependency) const = 0;
};

#endif
