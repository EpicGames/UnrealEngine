// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR
struct FWorldPartitionCookPackage;
class IWorldPartitionCookPackageContext;
class UWorldPartitionRuntimeCell;
#endif

class IWorldPartitionCookPackageGenerator
{
#if WITH_EDITOR
public:
	virtual ~IWorldPartitionCookPackageGenerator() = default;

	virtual bool GatherPackagesToCook(IWorldPartitionCookPackageContext& CookContext) = 0;
	virtual bool PrepareGeneratorPackageForCook(IWorldPartitionCookPackageContext& CookContext, TArray<UPackage*>& OutModifiedPackages) { return true; }
	virtual bool PopulateGeneratorPackageForCook(IWorldPartitionCookPackageContext& CookContext, const TArray<FWorldPartitionCookPackage*>& InPackagesToCook, TArray<UPackage*>& OutModifiedPackages) = 0;
	virtual bool PopulateGeneratedPackageForCook(IWorldPartitionCookPackageContext& CookContext, const FWorldPartitionCookPackage& InPackageToCook, TArray<UPackage*>& OutModifiedPackages) = 0;
	virtual UWorldPartitionRuntimeCell* GetCellForPackage(const FWorldPartitionCookPackage& PackageToCook) const = 0;
#endif
};
