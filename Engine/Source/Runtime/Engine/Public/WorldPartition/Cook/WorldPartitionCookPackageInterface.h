// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreFwd.h"
#include "UObject/Interface.h"
#include "WorldPartitionCookPackage.h"
#include "WorldPartitionCookPackageInterface.generated.h"

class UExternalDataLayerAsset;
class IWorldPartitionCookPackageContext;

UINTERFACE(MinimalAPI)
class UWorldPartitionCookPackageObject : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IWorldPartitionCookPackageObject
{
	GENERATED_IINTERFACE_BODY()

public:
#if WITH_EDITOR
	virtual bool IsLevelPackage() const = 0;
	virtual const UExternalDataLayerAsset* GetExternalDataLayerAsset() const = 0;
	virtual FString GetPackageNameToCreate() const = 0;
	virtual FWorldPartitionPackageHash GetGenerationHash() const = 0;
	virtual bool OnPrepareGeneratorPackageForCook(TArray<UPackage*>& OutModifiedPackages) = 0;
	virtual bool OnPopulateGeneratorPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InPackage) = 0;
	virtual bool OnPopulateGeneratedPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InPackage, TArray<UPackage*>& OutModifiedPackages) = 0;
#endif
};