// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "ActorDataLayer.h"
#include "Math/Color.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.8, "The contents of this header have been fully deprecated. See DeprecatedDataLayer.h for details on particular cases.") 

#include "WorldPartition/DataLayer/Deprecated/DeprecatedDataLayer.h"
