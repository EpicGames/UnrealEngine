// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"

#include "WorldPartition/DataLayer/DataLayerType.h" 

#include "DataLayerAsset.generated.h"

class AActor;

UENUM(BlueprintType)
enum class EDataLayerLoadFilter : uint8
{
	/** Data Layer is considered by the client and the server. Client runtime state is replicated. */
	None,
	/** Data Layer is only considered by the client. */
	ClientOnly,
	/** Data layer is only considered by the server. */
	ServerOnly
};

UCLASS(BlueprintType, editinlinenew, MinimalAPI)
class UDataLayerAsset : public UDataAsset
{
	GENERATED_UCLASS_BODY()

	friend class UDataLayerConversionInfo;

#if WITH_EDITOR
	//~ Begin UObject interface
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	//~ End UObject interface

	//~ Begin UDataAsset interface
	ENGINE_API virtual FText GetDisplayNameText() const override;
	//~ End UDataAsset interface

public:
	virtual bool CanEditDataLayerType() const;
	ENGINE_API void SetType(EDataLayerType Type);
	ENGINE_API void SetLoadFilter(EDataLayerLoadFilter LoadFilter);
	void SetDebugColor(FColor InDebugColor) { DebugColor = InDebugColor; }
	ENGINE_API bool CanBeReferencedByActor(AActor* InActor) const;
	static ENGINE_API bool CanBeReferencedByActor(const TSoftObjectPtr<UDataLayerAsset>& InDataLayerAsset, AActor* InActor);

	virtual void OnCreated();
#endif

	ENGINE_API bool IsPrivate() const;

	UFUNCTION(Category = "Data Layer", BlueprintCallable)
	virtual EDataLayerType GetType() const { return DataLayerType; }

	UFUNCTION(Category = "Data Layer|Runtime", BlueprintCallable)
	bool IsRuntime() const { return !IsPrivate() && DataLayerType == EDataLayerType::Runtime; }

	UFUNCTION(Category = "Data Layer|Runtime", BlueprintCallable)
	FColor GetDebugColor() const { return DebugColor; }

	bool SupportsActorFilters() const { return bSupportsActorFilters; }

	UFUNCTION(Category = "Data Layer|Runtime", BlueprintCallable)
	bool IsClientOnly() const { return IsRuntime() && LoadFilter == EDataLayerLoadFilter::ClientOnly; }

	UFUNCTION(Category = "Data Layer|Runtime", BlueprintCallable)
	bool IsServerOnly() const { return IsRuntime() && LoadFilter == EDataLayerLoadFilter::ServerOnly; }

protected:
	/** Whether the Data Layer affects actor runtime loading */
	UPROPERTY(Category = "Data Layer", EditAnywhere)
	EDataLayerType DataLayerType;

	UPROPERTY(Category = "Actor Filter", EditAnywhere)
	bool bSupportsActorFilters;

private:
	UPROPERTY(Category = "Runtime", EditAnywhere)
	FColor DebugColor;

	UPROPERTY(Category = "Runtime", EditAnywhere)
	EDataLayerLoadFilter LoadFilter;
};
