// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"

#if WITH_EDITOR

class UDataLayerInstance;
class UWorld;

/**
 * FDataLayerEditorContext
 */

struct FDataLayerEditorContext
{
public:
	static const uint32 EmptyHash = 0;
	FDataLayerEditorContext() : Hash(FDataLayerEditorContext::EmptyHash) {}
	ENGINE_API FDataLayerEditorContext(UWorld* InWorld, const TArray<FName>& InDataLayerInstances);
	inline bool IsEmpty() const { return (Hash == FDataLayerEditorContext::EmptyHash) && DataLayerInstances.IsEmpty(); }
	inline uint32 GetHash() const { return Hash; }
	inline const TArray<FName>& GetDataLayerInstanceNames() const { return DataLayerInstances; }
private:
	uint32 Hash;
	TArray<FName> DataLayerInstances;
};
#endif
