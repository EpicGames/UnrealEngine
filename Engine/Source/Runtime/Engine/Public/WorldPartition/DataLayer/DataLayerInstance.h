// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "WorldPartition/DataLayer/DataLayerType.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationErrorHandler.h"

#include "DataLayerInstance.generated.h"

class AWorldDataLayers;
class FText;
class IStreamingGenerationErrorHandler;
class UWorld;
class UDataLayerAsset;
class UExternalDataLayerInstance;
class FDataLayerInstanceDesc;

UENUM(BlueprintType)
enum class EDataLayerRuntimeState : uint8
{
	// Unloaded
	Unloaded,
	
	// Loaded (meaning loaded but not visible)
	Loaded,

	// Activated (meaning loaded and visible)
	Activated
};

UENUM(BlueprintType)
enum class EOverrideBlockOnSlowStreaming : uint8
{
	// Uses default runtime partition 'Block on Slow Streaming' setting
	NoOverride,
	// Blocks on slow streaming (Overrides runtime partition 'Block on Slow Streaming' setting)
	Blocking,
	// Doesn't block on slow streaming (Overrides runtime partition 'Block on Slow Streaming' setting)
	NotBlocking
};

const inline TCHAR* GetDataLayerRuntimeStateName(EDataLayerRuntimeState State)
{
	switch (State)
	{
	case EDataLayerRuntimeState::Unloaded: return TEXT("Unloaded");
	case EDataLayerRuntimeState::Loaded: return TEXT("Loaded");
	case EDataLayerRuntimeState::Activated: return TEXT("Activated");
	default: check(0);
	}
	return TEXT("Invalid");
}

UCLASS(Config = Engine, PerObjectConfig, BlueprintType, AutoCollapseCategories = ("Data Layer|Advanced"), AutoExpandCategories = ("Data Layer|Editor", "Data Layer|Advanced|Runtime"), MinimalAPI)
class UDataLayerInstance : public UObject
{
	GENERATED_UCLASS_BODY()

	friend class UDataLayerConversionInfo;
	friend class FDataLayerInstanceDetails;

public:
	ENGINE_API virtual void PostLoad() override;
	
	template<class T>
	T* GetTypedOuter() const
	{
		static_assert(!std::is_same<T, UWorld>::value, "Use GetOuterWorld instead");
		static_assert(!std::is_same<T, ULevel>::value, "Use GetOuterWorld()->PersistentLevel instead");
		static_assert(!std::is_same<T, AWorldDataLayers>::value, "Use GetOuterWorldDataLayers or GetDirectOuterWorldDataLayers instead");
		return Super::GetTypedOuter<T>();
	}

	ENGINE_API virtual UWorld* GetOuterWorld() const;
	ENGINE_API AWorldDataLayers* GetOuterWorldDataLayers() const;
	ENGINE_API AWorldDataLayers* GetDirectOuterWorldDataLayers() const;

#if WITH_EDITOR
	//~ Begin UObject interface
	ENGINE_API virtual bool IsAsset() const override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	//~ End UObject interface

	// Getters/Setters
	ENGINE_API bool SetParent(UDataLayerInstance* InParent);
	ENGINE_API void SetVisible(bool bIsVisible);
	ENGINE_API void SetIsInitiallyVisible(bool bIsInitiallyVisible);
	ENGINE_API void SetIsLoadedInEditor(bool bIsLoadedInEditor, bool bFromUserChange);
	void SetIsLocked(bool bInIsLocked) { bIsLocked = bInIsLocked; }
	ENGINE_API void SetInitialRuntimeState(EDataLayerRuntimeState InInitialRuntimeState);

	ENGINE_API bool IsEffectiveLoadedInEditor() const;
	bool IsInitiallyLoadedInEditor() const { return bIsInitiallyLoadedInEditor; }
	bool IsLoadedInEditor() const { return bIsLoadedInEditor; }
	bool IsLoadedInEditorChangedByUserOperation() const { return bIsLoadedInEditorChangedByUserOperation; }
	ENGINE_API virtual bool IsReadOnly(FText* OutReason = nullptr) const;
	virtual bool IsIncludedInActorFilterDefault() const { return false; }
	EOverrideBlockOnSlowStreaming GetOverrideBlockOnSlowStreaming() const { return OverrideBlockOnSlowStreaming; }
	int32 GetStreamingPriority() const { return StreamingPriority; }

	// Data Layer Instance features support
	ENGINE_API bool CanBeChildOf(const UDataLayerInstance* InParent, FText* OutReason = nullptr) const;
	ENGINE_API virtual bool CanUserAddActors(FText* OutReason = nullptr) const;
	ENGINE_API virtual bool CanUserRemoveActors(FText* OutReason = nullptr) const;
	ENGINE_API virtual bool CanAddActor(AActor* Actor, FText* OutReason = nullptr) const;
	ENGINE_API virtual bool CanRemoveActor(AActor* Actor, FText* OutReason = nullptr) const;
	ENGINE_API virtual bool CanBeInActorEditorContext() const;
	virtual bool CanBeRemoved() const { return true; }
	virtual bool CanEditDataLayerShortName() const { return false; }
	virtual bool SupportsActorFilters() const { return false; }
	virtual const UExternalDataLayerInstance* GetRootExternalDataLayerInstance() const { return nullptr; }

	// Actor assignation and removal
	ENGINE_API virtual bool AddActor(AActor* Actor) const;
	ENGINE_API virtual bool RemoveActor(AActor* Actor) const;

	// Actor Editor Context
	ENGINE_API bool IsActorEditorContextCurrentColorized() const;
	ENGINE_API bool IsInActorEditorContext() const;
	ENGINE_API bool AddToActorEditorContext();
	ENGINE_API bool RemoveFromActorEditorContext();

	// Validation
	ENGINE_API virtual bool Validate(IStreamingGenerationErrorHandler* ErrorHandler) const;

	// Helpers
	ENGINE_API virtual const TCHAR* GetDataLayerIconName() const;
	static ENGINE_API FText GetDataLayerText(const UDataLayerInstance* InDataLayer);
	static bool GetAssetRegistryInfoFromPackage(FName InDataLayerInstancePackageName, FDataLayerInstanceDesc& OutDataLayerInstanceDesc);
	static bool GetAssetRegistryInfoFromPackage(const FAssetData& InAsset, FDataLayerInstanceDesc& OutDataLayerInstanceDesc);

#endif // WITH_EDITOR

	UFUNCTION(Category = "Data Layer", BlueprintCallable)
	virtual EDataLayerType GetType() const { return EDataLayerType::Unknown; }

	UFUNCTION(Category = "Data Layer|Editor", BlueprintCallable)
	ENGINE_API bool IsInitiallyVisible() const;

	UFUNCTION(Category = "Data Layer|Editor", BlueprintCallable)
	ENGINE_API bool IsVisible() const;

	UFUNCTION(Category = "Data Layer|Editor", BlueprintCallable)
	ENGINE_API bool IsEffectiveVisible() const;

	UFUNCTION(Category = "Data Layer|Runtime", BlueprintCallable)
	virtual bool IsRuntime() const { return false; }

	UFUNCTION(Category = "Data Layer|Runtime", BlueprintCallable)
	virtual bool IsClientOnly() const { return false; }

	UFUNCTION(Category = "Data Layer|Runtime", BlueprintCallable)
	virtual bool IsServerOnly() const { return false; }

	UFUNCTION(Category = "Data Layer|Runtime", BlueprintCallable)
	virtual FColor GetDebugColor() const { return FColor::Black; }

	UFUNCTION(Category = "Data Layer|Runtime", BlueprintCallable)
	EDataLayerRuntimeState GetInitialRuntimeState() const { return IsRuntime() ? InitialRuntimeState : EDataLayerRuntimeState::Unloaded; }

	UFUNCTION(Category = "Data Layer", BlueprintCallable)
	virtual FString GetDataLayerShortName() const { return TEXT("Invalid Data Layer"); }

	UFUNCTION(Category = "Data Layer", BlueprintCallable)
	virtual FString GetDataLayerFullName() const { return TEXT("Invalid Data Layer"); }

	virtual bool CanHaveChildDataLayerInstance(const UDataLayerInstance* InChildDataLayerInstance) const { return InChildDataLayerInstance != nullptr; }
	virtual bool CanHaveParentDataLayerInstance() const { return true; }

	const UDataLayerInstance* GetParent() const { return Parent; }
	UDataLayerInstance* GetParent() { return Parent; }

	virtual bool CanEditInitialRuntimeState() const { return IsRuntime(); }
	ENGINE_API EDataLayerRuntimeState GetRuntimeState() const;
	ENGINE_API EDataLayerRuntimeState GetEffectiveRuntimeState() const;

	UFUNCTION(Category = "Data Layer", BlueprintCallable)
	virtual const UDataLayerAsset* GetAsset() const { return nullptr; }

	const TArray<TObjectPtr<UDataLayerInstance>>& GetChildren() const { return Children; }
	ENGINE_API void ForEachChild(TFunctionRef<bool(const UDataLayerInstance*)> Operation) const;
	
	UE_DEPRECATED(5.8, "Use GetFName instead")
	virtual FName GetDataLayerFName() const { return GetFName(); }

protected:
#if WITH_EDITOR
	ENGINE_API bool IsLocked(FText* OutReason) const;
	ENGINE_API bool IsParentDataLayerTypeCompatible(const UDataLayerInstance* InParent, FText* OutReasonText = nullptr, IStreamingGenerationErrorHandler::EDataLayerHierarchyInvalidReason* OutReason = nullptr) const;

	virtual bool PerformAddActor(AActor* InActor) const { return false; }
	virtual bool PerformRemoveActor(AActor* InActor) const { return false;  }
	virtual void PerformSetDataLayerShortName(const FString& InNewShortName) {}
#endif

#if WITH_EDITORONLY_DATA
	/** Whether actors associated with the DataLayer are visible in the viewport */
	UPROPERTY(Transient)
	uint32 bIsVisible : 1;

	/** Whether actors associated with the Data Layer should be initially visible in the viewport when loading the map */
	UPROPERTY(Category = "Editor", EditAnywhere)
	uint32 bIsInitiallyVisible : 1;

	/** Determines the default value of the data layer's loaded state in editor if it hasn't been changed in data layer outliner by the user */
	UPROPERTY(Category = "Editor", EditAnywhere, meta = (DisplayName = "Is Initially Loaded"))
	uint32 bIsInitiallyLoadedInEditor : 1;

	/** Wheter the data layer is loaded in editor (user setting) */
	UPROPERTY(Transient)
	uint32 bIsLoadedInEditor : 1;
	uint32 bUndoIsLoadedInEditor : 1;

	/** Whether this data layer editor visibility was changed by a user operation */
	UPROPERTY(Transient)
	uint32 bIsLoadedInEditorChangedByUserOperation : 1;

	/** Whether this data layer is locked, which means the user can't change actors assignation, remove or rename it */
	UPROPERTY()
	uint32 bIsLocked : 1;

	UPROPERTY(Category = "Runtime|Advanced", EditAnywhere)
	EOverrideBlockOnSlowStreaming OverrideBlockOnSlowStreaming;

	/** Determines the streaming priority (lower value means higher priority), relative to target grid cells. Priority 0 means the same priority as target grid cells. */
	UPROPERTY(Category = "Runtime|Advanced", EditAnywhere)
	int32 StreamingPriority;
#endif

	/** Initial runtime state of this data layer instance. Only supported if it's runtime and not client/server only. */
	UPROPERTY(Category = "Runtime", EditAnywhere)
	EDataLayerRuntimeState InitialRuntimeState;

private:

#if WITH_EDITOR
	ENGINE_API void OnRemovedFromWorldDataLayers();
	void ClearLoadedInEditorChangedByUserOperation() { bIsLoadedInEditorChangedByUserOperation = false; }
	ENGINE_API void RemoveChild(UDataLayerInstance* DataLayer);
#endif
	ENGINE_API void AddChild(UDataLayerInstance* DataLayer);

	UPROPERTY()
	TObjectPtr<UDataLayerInstance> Parent;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UDataLayerInstance>> Children;

	friend class AWorldDataLayers;
	friend class UDataLayerManager;
	friend class FDataLayerUtils;
};
