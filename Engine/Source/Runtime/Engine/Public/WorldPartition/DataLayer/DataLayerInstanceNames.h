// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Algo/Unique.h"

#include "DataLayerInstanceNames.generated.h"

namespace WorldPartitionTests
{
	class FDataLayerInstanceNamesTest;
}

USTRUCT()
struct FDataLayerInstanceNames
{
	GENERATED_USTRUCT_BODY()

	FDataLayerInstanceNames()
	: bIsFirstDataLayerExternal(false)
	{
#if WITH_EDITOR
		bIsForcedEmptyNonExternalDataLayers = false;
#endif
	}

	FDataLayerInstanceNames(const TArray<FName>& InDataLayers, bool bInIsFirstDataLayerExternal)
	{
		const int32 NonExternalDataLayersOffset = bInIsFirstDataLayerExternal && InDataLayers.Num() ? 1 : 0;
		const int32 NumNonExternalDataLayers = InDataLayers.Num() - NonExternalDataLayersOffset;
		const FName ExternalDataLayer = (NonExternalDataLayersOffset > 0) ? InDataLayers[0] : NAME_None;
		Initialize(MakeArrayView(NumNonExternalDataLayers ? &InDataLayers[NonExternalDataLayersOffset] : nullptr, NumNonExternalDataLayers), ExternalDataLayer);
	}

	FDataLayerInstanceNames(const TArray<FName>& InNonExternalDataLayers, FName InExternalDataLayer)
	{
		Initialize(InNonExternalDataLayers, InExternalDataLayer);
	}

	const FName GetExternalDataLayer() const
	{
		check(!bIsFirstDataLayerExternal || !DataLayers.IsEmpty());
		return bIsFirstDataLayerExternal ? DataLayers[0] : NAME_None;
	}

#if WITH_EDITOR
	void Append(const FDataLayerInstanceNames& Other)
	{
		check(!HasExternalDataLayer() || !Other.HasExternalDataLayer() || (GetExternalDataLayer() == Other.GetExternalDataLayer()));

		if (!Other.HasExternalDataLayer())
		{
			DataLayers.Append(Other.GetNonExternalDataLayers());
		}
		else
		{
			TArrayView<const FName> ThisNonDataLayersView(GetNonExternalDataLayers());
			TArray<FName> ThisNonDataLayers(ThisNonDataLayersView.GetData(), ThisNonDataLayersView.Num());
			DataLayers = Other.ToArray();
			bIsFirstDataLayerExternal = Other.bIsFirstDataLayerExternal;
			DataLayers.Append(ThisNonDataLayers);
		}

		if (bIsForcedEmptyNonExternalDataLayers)
		{
			ForceEmptyNonExternalDataLayers();
		}

		// Remove potential duplicates from sorted non-external data layers
		GetNonExternalDataLayers().Sort(FNameFastLess());
		DataLayers.SetNum(Algo::Unique(DataLayers));
	}
#endif

	/**
	* DO NOT USE DIRECTLY
	* STL-like iterators to enable range-based for loop support.
	*/
	TArray<FName>::RangedForIteratorType     		 begin() { return DataLayers.begin(); }
	TArray<FName>::RangedForConstIteratorType		 begin() const { return DataLayers.begin(); }
	TArray<FName>::RangedForIteratorType      		 end() { return DataLayers.end(); }
	TArray<FName>::RangedForConstIteratorType		 end() const { return DataLayers.end(); }
	TArray<FName>::RangedForReverseIteratorType      rbegin() { return DataLayers.rbegin(); }
	TArray<FName>::RangedForConstReverseIteratorType rbegin() const { return DataLayers.rbegin(); }
	TArray<FName>::RangedForReverseIteratorType      rend() { return DataLayers.rend(); }
	TArray<FName>::RangedForConstReverseIteratorType rend() const { return DataLayers.rend(); }

	TArrayView<const FName> GetNonExternalDataLayers() const
	{
		static TArray<FName> EmptyArray;
		check(!bIsFirstDataLayerExternal || !DataLayers.IsEmpty());
		const int32 Offset = bIsFirstDataLayerExternal ? 1 : 0;
		const int32 NonExternalDataLayersCount = DataLayers.Num() - Offset;
		return NonExternalDataLayersCount > 0 ? MakeArrayView(&DataLayers[Offset], NonExternalDataLayersCount) : EmptyArray;
	}

	int32 Num() const { return DataLayers.Num(); }
	bool IsEmpty() const { return DataLayers.IsEmpty(); }
	bool Contains(FName InDataLayer) const { return DataLayers.Contains(InDataLayer); }
	bool HasExternalDataLayer() const { return bIsFirstDataLayerExternal; }
	const TArray<FName>& ToArray() const { return DataLayers; }

private:

	void Initialize(TArrayView<const FName> InNonExternalDataLayers, FName InExternalDataLayer)
	{
#if WITH_EDITOR
		bIsForcedEmptyNonExternalDataLayers = false;
#endif
		bIsFirstDataLayerExternal = !InExternalDataLayer.IsNone();

		DataLayers.Reserve(InNonExternalDataLayers.Num() + (bIsFirstDataLayerExternal ? 1 : 0));
		if (bIsFirstDataLayerExternal)
		{
			DataLayers.Add(InExternalDataLayer);
		}
		for (FName DataLayer : InNonExternalDataLayers)
		{
			if (DataLayer != InExternalDataLayer)
			{
				DataLayers.AddUnique(DataLayer);
			}
		}
	}

#if WITH_EDITOR
	bool IsForcedEmptyNonExternalDataLayers() const { return bIsForcedEmptyNonExternalDataLayers; }
	void ForceEmptyNonExternalDataLayers()
	{
		bIsForcedEmptyNonExternalDataLayers = true;
		if (bIsFirstDataLayerExternal)
		{
			DataLayers = { DataLayers[0] };
		}
		else
		{
			DataLayers.Empty();
		}
	}
	bool bIsForcedEmptyNonExternalDataLayers;
#endif

	UPROPERTY()
	bool bIsFirstDataLayerExternal;

	UPROPERTY()
	TArray<FName> DataLayers;

	friend class FStreamingGenerationActorDescView;
	friend class WorldPartitionTests::FDataLayerInstanceNamesTest;
};