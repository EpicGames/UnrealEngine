// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"

#include "DataLayerInstancePrivate.generated.h"

UCLASS(Config = Engine, PerObjectConfig, Within = WorldDataLayers, AutoExpandCategories = ("Data Layer|Editor"), MinimalAPI)
class UDataLayerInstancePrivate : public UDataLayerInstance
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	ENGINE_API static FName MakeName();
	ENGINE_API void OnCreated();

	//~ Begin UObject Interface
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	//~ End UObject Interface

	//~ Begin UDataLayerInstance Interface
	ENGINE_API virtual bool IsReadOnly(FText* OutReason = nullptr) const override;
	virtual bool SupportsActorFilters() const override { return GetAsset()->SupportsActorFilters(); }
	virtual bool IsIncludedInActorFilterDefault() const override { return bIsIncludedInActorFilterDefault; }
	virtual bool CanEditDataLayerShortName() const override { return true; }
	virtual bool CanAddActor(AActor* InActor, FText* OutReason = nullptr) const override;
	//~ End UDataLayerInstance Interface
#endif

	//~ Begin UDataLayerInstance Interface
	virtual const UDataLayerAsset* GetAsset() const override { return DataLayerAsset; }
	virtual EDataLayerType GetType() const override { return DataLayerAsset->GetType(); }
	virtual bool IsRuntime() const override { return DataLayerAsset->IsRuntime(); }
	virtual FColor GetDebugColor() const override { return DataLayerAsset->GetDebugColor(); }
	virtual FString GetDataLayerShortName() const override { return ShortName; }
	virtual FString GetDataLayerFullName() const override { return DataLayerAsset->GetPathName(); }
	//~ End UDataLayerInstance Interface

protected:
#if WITH_EDITOR
	//~ Begin UDataLayerInstance Interface
	ENGINE_API virtual bool PerformAddActor(AActor* InActor) const override;
	ENGINE_API virtual bool PerformRemoveActor(AActor* InActor) const override;
	virtual void PerformSetDataLayerShortName(const FString& InNewShortName) override { ShortName = InNewShortName; }
	//~ End UDataLayerInstance Interface
#endif

private:
	UPROPERTY()
	FString ShortName;

	UPROPERTY(Category = "Asset", VisibleAnywhere, Instanced)
	TObjectPtr<UDataLayerAsset> DataLayerAsset;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Category = "Actor Filter", EditAnywhere, meta = (DisplayName = "Is Included", ToolTip = "Whether actors assigned to this DataLayer are included by default when used in a filter"))
	bool bIsIncludedInActorFilterDefault;
#endif
};
