// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreFwd.h"
#include "UObject/Interface.h"
#include "DataLayerInstanceProviderInterface.generated.h"

class UDataLayerInstance;
class UExternalDataLayerAsset;
class UExternalDataLayerInstance;

UINTERFACE(MinimalAPI)
class UDataLayerInstanceProvider : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IDataLayerInstanceProvider
{
	GENERATED_IINTERFACE_BODY()

public:
	virtual TSet<TObjectPtr<UDataLayerInstance>>& GetDataLayerInstances() = 0;
	virtual const TSet<TObjectPtr<UDataLayerInstance>>& GetDataLayerInstances() const = 0;
	virtual const UExternalDataLayerInstance* GetRootExternalDataLayerInstance() const = 0;
	virtual const UExternalDataLayerAsset* GetRootExternalDataLayerAsset() const;
};

