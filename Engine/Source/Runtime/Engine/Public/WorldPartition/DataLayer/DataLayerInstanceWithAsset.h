// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"

#include "DataLayerInstanceWithAsset.generated.h"

UCLASS(Config = Engine, PerObjectConfig, AutoCollapseCategories = ("Data Layer|Advanced"), AutoExpandCategories = ("Data Layer|Editor", "Data Layer|Advanced|Runtime"), MinimalAPI)
class UDataLayerInstanceWithAsset : public UDataLayerInstance
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	//~ Begin UObject interface
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const;
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface

	//~ Begin UDataLayerInstance interface
	ENGINE_API virtual bool IsReadOnly(FText* OutReason = nullptr) const override;
	ENGINE_API virtual bool CanAddActor(AActor* InActor, FText* OutReason = nullptr) const override;
	ENGINE_API virtual bool CanRemoveActor(AActor* InActor, FText* OutReason = nullptr) const override;
	ENGINE_API virtual bool CanBeInActorEditorContext() const override;
	ENGINE_API virtual bool SupportsActorFilters() const override;
	ENGINE_API virtual bool IsIncludedInActorFilterDefault() const override;
	ENGINE_API virtual bool Validate(IStreamingGenerationErrorHandler* ErrorHandler) const override;
	ENGINE_API virtual const UExternalDataLayerInstance* GetRootExternalDataLayerInstance() const override;
	//~ End UDataLayerInstance interface

	static const UDataLayerAsset* GetDataLayerAsset(const UDataLayerAsset* Asset) { return Asset; } // Used when creating a DataLayerInstance
	ENGINE_API static FName MakeName(const UDataLayerAsset* InDataLayerAsset);
	ENGINE_API static TSubclassOf<UDataLayerInstanceWithAsset> GetDataLayerInstanceClass();
	ENGINE_API virtual void OnCreated(const UDataLayerAsset* Asset);
	virtual bool CanEditDataLayerAsset() const { return true; }
#endif

	//~ Begin UDataLayerInstance interface
	ENGINE_API virtual UWorld* GetOuterWorld() const override;
	virtual const UDataLayerAsset* GetAsset() const override { return DataLayerAsset; }
	virtual EDataLayerType GetType() const override { return DataLayerAsset ? DataLayerAsset->GetType() : EDataLayerType::Unknown; }
	virtual bool IsRuntime() const override { return DataLayerAsset && DataLayerAsset->IsRuntime(); }
	virtual bool IsClientOnly() const override { return DataLayerAsset && DataLayerAsset->IsClientOnly(); }
	virtual bool IsServerOnly() const override { return DataLayerAsset && DataLayerAsset->IsServerOnly(); }
	virtual FColor GetDebugColor() const override { return DataLayerAsset ? DataLayerAsset->GetDebugColor() : FColor::Black; }
	virtual FString GetDataLayerShortName() const override { return DataLayerAsset ? DataLayerAsset->GetName() : GetFName().ToString(); }
	virtual FString GetDataLayerFullName() const override { return DataLayerAsset ? DataLayerAsset->GetPathName() : GetFName().ToString(); }
	//~ End UDataLayerInstance interface

protected:
#if WITH_EDITOR
	ENGINE_API virtual bool PerformAddActor(AActor* InActor) const;
	ENGINE_API virtual bool PerformRemoveActor(AActor* InActor) const;

	bool bSkipCheckReadOnlyForSubLevels;
#endif

private:
	UPROPERTY(Category = "Data Layer", EditAnywhere, meta = (DisallowedClasses = "/Script/Engine.ExternalDataLayerAsset"))
	TObjectPtr<const UDataLayerAsset> DataLayerAsset;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Category = "Data Layer|Actor Filter", EditAnywhere, meta = (DisplayName = "Is Included", ToolTip = "Whether actors assigned to this DataLayer are included by default when used in a filter"))
	bool bIsIncludedInActorFilterDefault;
#endif

#if WITH_EDITOR
	// Used to compare state pre/post undo
	TObjectPtr<const UDataLayerAsset> CachedDataLayerAsset;
#endif

	friend class UDataLayerConversionInfo;
	friend class ULevelInstanceSubsystem;
};
