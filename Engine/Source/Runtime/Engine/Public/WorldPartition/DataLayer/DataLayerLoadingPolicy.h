// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "WorldPartition/WorldPartition.h"
#include "DataLayerLoadingPolicy.generated.h"

#define UE_API ENGINE_API

class UDataLayerInstance;

UCLASS(MinimalAPI, Within = DataLayerManager)
class UDataLayerLoadingPolicy : public UObject
{
	GENERATED_BODY()
#if WITH_EDITOR
public:
	UE_API virtual bool ResolveIsLoadedInEditor(TArray<const UDataLayerInstance*>& InDataLayers) const;

protected:
	UE_API EWorldPartitionDataLayersLogicOperator GetDataLayersLogicOperator() const;
#endif
};

#undef UE_API
