// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerEditorContext.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "Templates/SubclassOf.h"
#include "DataLayerManager.generated.h"

class UDataLayerAsset;
class UDataLayerInstanceWithAsset;
class UDataLayerInstance;
class UDataLayerLoadingPolicy;
class UExternalDataLayerAsset;
class ExternalDataLayerInstance;
class UActorDescContainer;
class UActorDescContainerInstance;
class UCanvas;
class UWorld;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnDataLayerInstanceRuntimeStateChanged, const UDataLayerInstance*, DataLayer, EDataLayerRuntimeState, State);

#if WITH_EDITOR
class FDataLayersEditorBroadcast
{
public:
	static ENGINE_API FDataLayersEditorBroadcast& Get();
	static ENGINE_API void StaticOnActorDataLayersEditorLoadingStateChanged(bool bIsFromUserChange);
	/** Broadcasts whenever one or more DataLayers editor loading state changed */
	DECLARE_EVENT_OneParam(FDataLayersEditorBroadcast, FOnActorDataLayersEditorLoadingStateChanged, bool /*bIsFromUserChange*/);
	FOnActorDataLayersEditorLoadingStateChanged& OnActorDataLayersEditorLoadingStateChanged() { return DataLayerEditorLoadingStateChanged; }

private:
	FOnActorDataLayersEditorLoadingStateChanged DataLayerEditorLoadingStateChanged;

	friend class UDataLayerManager;
};
#endif

UCLASS(Config = Engine, Within = WorldPartition, MinimalAPI)
class UDataLayerManager : public UObject
{
	GENERATED_BODY()

public:
	template <class T>
	static UDataLayerManager* GetDataLayerManager(const T* InObject)
	{
		UWorldPartition* WorldPartition = IsValid(InObject) ? FWorldPartitionHelpers::GetWorldPartition(InObject) : nullptr;
		return WorldPartition ? WorldPartition->GetDataLayerManager() : nullptr;
	}

	//~ Begin Blueprint interface

	/** Returns all Data Layer instances. */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	ENGINE_API TArray<UDataLayerInstance*> GetDataLayerInstances() const;

	/** Returns the Data Layer instance referencing the provided Data Layer asset (if any). */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	ENGINE_API const UDataLayerInstance* GetDataLayerInstanceFromAsset(const UDataLayerAsset* InDataLayerAsset) const;

	/** Returns the Data Layer instance matching the provided Data Layer instance name (if any). */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	ENGINE_API const UDataLayerInstance* GetDataLayerInstanceFromName(const FName& InDataLayerInstanceName) const;

	/** Changes the Data Layer instance runtime state.
	  * If recursive is set to true, the runtime state will also be applied to all child Data Layer instances.
	  * Note:
	  *  - Changing the runtime state of a Client-Only Data Layer instance must be done on the client side or else it will have no effect.
	  *  - Changing the runtime state of a Server-Only Data Layer instance can only be done on the server side or else it will have no effect.
	  *  - Changing the runtime state of a runtime Data Layer instance (with no Load Filter set on the asset) must be done on the server side 
	  *    or else it will have no effect. The runtime state will then be replicated on the client.
	  * (see Data Layer asset Load Filter for more details)
	  */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	ENGINE_API bool SetDataLayerInstanceRuntimeState(const UDataLayerInstance* InDataLayerInstance, EDataLayerRuntimeState InState, bool bInIsRecursive = false);

	/** Finds a matching Data Layer instance referencing the provided Data Layer asset and changes its runtime state (if any).
	  * If recursive is set to true, the runtime state will also be applied to all child Data Layer instances.
	  * Note:
	  *  - Changing the runtime state of a Client-Only Data Layer instance must be done on the client side or else it will have no effect.
	  *  - Changing the runtime state of a Server-Only Data Layer instance can only be done on the server side or else it will have no effect.
	  *  - Changing the runtime state of a runtime Data Layer instance (with no Load Filter set on the asset) must be done on the server side
	  *    or else it will have no effect. The runtime state will then be replicated on the client.
	  * (see Data Layer asset Load Filter for more details)
	  */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	ENGINE_API bool SetDataLayerRuntimeState(const UDataLayerAsset* InDataLayerAsset, EDataLayerRuntimeState InState, bool bInIsRecursive = false);

	/** Returns the Data Layer instance runtime state. */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	ENGINE_API EDataLayerRuntimeState GetDataLayerInstanceRuntimeState(const UDataLayerInstance* InDataLayerInstance) const;

	/** Finds a matching Data Layer instance referencing the provided Data Layer asset and returns the Data Layer Instance runtime state. */
	UFUNCTION(BlueprintCallable, Category = DataLayers)
	ENGINE_API EDataLayerRuntimeState GetDataLayerInstanceEffectiveRuntimeState(const UDataLayerInstance* InDataLayerInstance) const;

	/** Called when a Data Layer instance runtime state has changed. */
	UPROPERTY(BlueprintAssignable)
	FOnDataLayerInstanceRuntimeStateChanged OnDataLayerInstanceRuntimeStateChanged;
	
	//~ End Blueprint interface

	template<class T>
	const UDataLayerInstance* GetDataLayerInstance(const T& InDataLayerIdentifier) const;

	template<class T>
	TArray<const UDataLayerInstance*> GetDataLayerInstances(const TArray<T>& InDataLayerIdentifiers) const;

#if WITH_EDITOR
	template<class T>
	TArray<FName> GetDataLayerInstanceNames(const TArray<T>& InDataLayerIdentifiers) const;
#else
	template<class T>
	UE_DEPRECATED(5.8, "This method is Tool-only since the removal of FActorDataLayer.")
	TArray<FName> GetDataLayerInstanceNames(const TArray<T>& InDataLayerIdentifiers) const { return {}; }
#endif // WITH_EDITOR

	ENGINE_API void ForEachDataLayerInstance(TFunctionRef<bool(UDataLayerInstance*)> Func);
	ENGINE_API void ForEachDataLayerInstance(TFunctionRef<bool(UDataLayerInstance*)> Func) const;

	//~ Begin Runtime State
	ENGINE_API const TSet<FName>& GetEffectiveActiveDataLayerNames() const;
	ENGINE_API const TSet<FName>& GetEffectiveLoadedDataLayerNames() const;
	ENGINE_API bool IsAnyDataLayerInEffectiveRuntimeState(TArrayView<const FName> InDataLayerNames, EDataLayerRuntimeState InState) const;
	ENGINE_API bool IsAllDataLayerInEffectiveRuntimeState(TArrayView<const FName> InDataLayerNames, EDataLayerRuntimeState InState) const;
	//~ End Runtime State

private:
	//~ Begin Initialization/Deinitialization
	ENGINE_API UDataLayerManager();
	ENGINE_API void Initialize();
	ENGINE_API void DeInitialize();
	//~ End Initialization/Deinitialization

	//~ Begin Debugging
	ENGINE_API void DrawDataLayersStatus(UCanvas* Canvas, FVector2D& Offset) const;
	ENGINE_API void DumpDataLayers(FOutputDevice& OutputDevice) const;
	ENGINE_API TArray<UDataLayerInstance*> ConvertArgsToDataLayers(const TArray<FString>& InArgs);
	//~ End Debugging

	ENGINE_API AWorldDataLayers* GetWorldDataLayers() const;
	ENGINE_API const UDataLayerInstance* GetDataLayerInstanceFromAssetName(const FName& InDataLayerAssetPathName) const;
	ENGINE_API void BroadcastOnDataLayerInstanceRuntimeStateChanged(const UDataLayerInstance* InDataLayer, EDataLayerRuntimeState InState);
	ENGINE_API static bool IsAnyDataLayerInEffectiveRuntimeState(TArrayView<const FName> InDataLayerNames, EDataLayerRuntimeState InState, const FWorldDataLayersEffectiveStates& InEffectiveStates);
	ENGINE_API static bool IsAllDataLayerInEffectiveRuntimeState(TArrayView<const FName> InDataLayerNames, EDataLayerRuntimeState InState, const FWorldDataLayersEffectiveStates& InEffectiveStates);

	/** Console command used to toggle activation of a Data Layer */
	static ENGINE_API class FAutoConsoleCommand ToggleDataLayerActivation;

	/** Console command used to set Runtime Data Layer state */
	static ENGINE_API class FAutoConsoleCommand SetDataLayerRuntimeStateCommand;

	/** Console command used to list Data Layers */
	static ENGINE_API class FAutoConsoleCommandWithOutputDevice DumpDataLayersCommand;

	/** Data layers load time */
	mutable TMap<const UDataLayerInstance*, double> ActiveDataLayersLoadTime;

	friend class UWorldPartition;
	friend class AWorldDataLayers;
	friend class UDataLayerSubsystem;
	friend class UDataLayerInstanceWithAsset;
	friend class UExternalDataLayerManager;
	friend class UExternalDataLayerInstance;
	friend class UWorldPartitionStreamingPolicy;
	friend class UWorldPartitionRuntimeCell;
	friend class UWorldPartitionSubsystem;
	friend class FWorldPartitionDebugHelper;
	friend struct FWorldPartitionStreamingContext;

#if WITH_EDITOR
private:
	//~ Begin Editor Context
	ENGINE_API void PushActorEditorContext(bool bDuplicateContext) const;
	ENGINE_API void PopActorEditorContext() const;
	ENGINE_API TArray<UDataLayerInstance*> GetActorEditorContextDataLayers() const;
	ENGINE_API TArray<AWorldDataLayers*> GetActorEditorContextWorldDataLayers() const;
	ENGINE_API uint32 GetDataLayerEditorContextHash() const;
	//~ End Editor Context

	//~ Begin WorldPartitionActorDescInstance
	ENGINE_API bool CanResolveDataLayers() const;
	ENGINE_API void ResolveActorDescContainersDataLayers() const;
	
	void OnActorDescContainerInstanceInitialized(UActorDescContainerInstance* InActorDescContainerInstance);
	void ResolveActorDescContainerInstanceDataLayers(UActorDescContainerInstance* InActorDescContainerInstance) const;
	void ResolveActorDescInstanceDataLayers(FWorldPartitionActorDescInstance* InActorDescInstance) const;
	void ResolveActorDescContainerInstanceDataLayersInternal(UActorDescContainerInstance* InActorDescContainerInstance, FWorldPartitionActorDescInstance* InActorDescInstance) const;
	ENGINE_API static FWorldPartitionReference LoadWorldDataLayersActor(UActorDescContainerInstance* InActorDescContainerInstance);

	//~ End

	//~ Begin Editor loading
	ENGINE_API TSubclassOf<UDataLayerLoadingPolicy> GetDataLayerLoadingPolicyClass() const;
	ENGINE_API bool ResolveIsLoadedInEditor(const TArray<FName>& InDataLayerInstanceNames) const;
	//~ End Editor loading

	ENGINE_API static TSubclassOf<UDataLayerInstanceWithAsset> GetDataLayerInstanceWithAssetClass();

	// Helper
	ENGINE_API TArray<const UDataLayerInstance*> GetRuntimeDataLayerInstances(const TArray<FName>& InDataLayerInstanceNames) const;

	//~ Begin User settings
	ENGINE_API void UpdateDataLayerEditorPerProjectUserSettings() const;
	ENGINE_API void GetUserLoadedInEditorStates(TArray<const UDataLayerInstance*>& OutDataLayersLoadedInEditor, TArray<const UDataLayerInstance*>& OutDataLayersNotLoadedInEditor) const;
	//~ End User settings

	/** Used by Editor Context */
	mutable int32 DataLayerActorEditorContextID;

	/** Whether manager can resolve data layers */
	bool bCanResolveDataLayers;

	friend struct SDataLayerTreeLabel;
	friend class IWorldPartitionActorLoaderInterface;
	friend class FWorldPartitionStreamingGenerator;
	friend class FWorldPartitionActorDesc;
	friend class UWorldPartitionRuntimeSpatialHash;
	friend class UWorldPartitionRuntimeHashSet;
	friend class UWorldPartitionConvertCommandlet;
	friend class FDataLayersBroadcast;
	friend class FDataLayerUtils;
	friend class FDataLayerMode;
	friend class FDataLayerHierarchy;
	friend class FActorBrowsingMode;
	friend class UDataLayerEditorSubsystem;
	friend class UActorPartitionSubsystem;
	friend class UActorDescContainerInstance;
	friend class AWorldSettings;
	friend class AActor;
	friend class FDataLayerEditorModule;
#endif

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY(Config)
	TSoftClassPtr<UDataLayerLoadingPolicy> DataLayerLoadingPolicyClass;

	UPROPERTY(Config)
	TSoftClassPtr<UDataLayerInstanceWithAsset> DataLayerInstanceWithAssetClass;

	UPROPERTY(Transient)
	TObjectPtr<UDataLayerLoadingPolicy> DataLayerLoadingPolicy;

	FWorldPartitionReference WorldDataLayersActor;
#endif
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template<class T>
const UDataLayerInstance* UDataLayerManager::GetDataLayerInstance(const T& InDataLayerIdentifier) const
{
	AWorldDataLayers* WorldDataLayers = GetWorldDataLayers();
	return WorldDataLayers ? WorldDataLayers->GetDataLayerInstance(InDataLayerIdentifier) : nullptr;
}

template<class T>
TArray<const UDataLayerInstance*> UDataLayerManager::GetDataLayerInstances(const TArray<T>& InDataLayerIdentifiers) const
{
	static TArray<const UDataLayerInstance*> EmptyArray;
	AWorldDataLayers* WorldDataLayers = GetWorldDataLayers();
	return WorldDataLayers ? WorldDataLayers->GetDataLayerInstances(InDataLayerIdentifiers) : EmptyArray;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
template<class T>
TArray<FName> UDataLayerManager::GetDataLayerInstanceNames(const TArray<T>& InDataLayerIdentifiers) const
{
	static TArray<FName> EmptyArray;
	AWorldDataLayers* WorldDataLayers = GetWorldDataLayers();
	return WorldDataLayers ? WorldDataLayers->GetDataLayerInstanceNames(InDataLayerIdentifiers) : EmptyArray;
}
#endif // WITH_EDITOR
