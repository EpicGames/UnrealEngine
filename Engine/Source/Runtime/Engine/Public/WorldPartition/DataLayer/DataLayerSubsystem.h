// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "DataLayerSubsystem.generated.h"

/**
 * UDataLayerSubsystem
 */

class UDataLayerLoadingPolicy;
class UCanvas;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnDataLayerRuntimeStateChanged, const UDataLayerInstance*, DataLayer, EDataLayerRuntimeState, State);

#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorldDataLayersPostRegister, AWorldDataLayers*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorldDataLayersPreUnregister, AWorldDataLayers*);
#endif

/** This class is deprecated, it has been replaced by DataLayerManager. */
UCLASS(Config = Engine, MinimalAPI)
class UDataLayerSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

protected:
	ENGINE_API virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override;

public:
	//~ Begin Blueprint callable functions

	/** Find a Data Layer by its asset. */
	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetDataLayerInstanceFromAsset instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use the equivalent function in DataLayerManager"))
	ENGINE_API UDataLayerInstance* GetDataLayerInstanceFromAsset(const UDataLayerAsset* InDataLayerAsset) const;

	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetDataLayerInstanceRuntimeState instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use the equivalent function in DataLayerManager"))
	ENGINE_API EDataLayerRuntimeState GetDataLayerInstanceRuntimeState(const UDataLayerAsset* InDataLayerAsset) const;

	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetDataLayerInstanceEffectiveRuntimeState instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use the equivalent function in DataLayerManager"))
	ENGINE_API EDataLayerRuntimeState GetDataLayerInstanceEffectiveRuntimeState(const UDataLayerAsset* InDataLayerAsset) const;

	/** Set the Data Layer state using its name. */
	UE_DEPRECATED(5.3, "Use UDataLayerManager::SetDataLayerInstanceRuntimeState instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, BlueprintAuthorityOnly, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use the equivalent function in DataLayerManager"))
	ENGINE_API void SetDataLayerInstanceRuntimeState(const UDataLayerAsset* InDataLayerAsset, EDataLayerRuntimeState InState, bool bInIsRecursive = false);

	/** Called when a Data Layer changes state. */
	UE_DEPRECATED(5.3, "Use UDataLayerManager::OnDataLayerInstanceRuntimeStateChanged instead.")
	UPROPERTY(BlueprintAssignable)
	FOnDataLayerRuntimeStateChanged OnDataLayerRuntimeStateChanged;

	//~ End Blueprint callable functions

	template<class T>
	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetDataLayerInstance instead.")
	UDataLayerInstance* GetDataLayerInstance(const T& InDataLayerIdentifier, const ULevel* InLevelContext = nullptr) const;

	template<class T>
	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetDataLayerInstances instead.")
	TArray<const UDataLayerInstance*> GetDataLayerInstances(const TArray<T>& InDataLayerIdentifiers, const ULevel* InLevelContext = nullptr) const;

	template<class T>
	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetDataLayerInstanceNames instead.")
	TArray<FName> GetDataLayerInstanceNames(const TArray<T>& InDataLayerIdentifiers, const ULevel* InLevelContext = nullptr) const;

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	ENGINE_API const UDataLayerInstance* GetDataLayerInstanceFromAssetName(const FName& InDataLayerAssetPathName) const;

	UE_DEPRECATED(5.3, "Use UDataLayerManager::SetDataLayerRuntimeState instead.")
	ENGINE_API void SetDataLayerRuntimeState(const UDataLayerInstance* InDataLayerInstance, EDataLayerRuntimeState InState, bool bInIsRecursive = false);

	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetDataLayerRuntimeState instead.")
	ENGINE_API EDataLayerRuntimeState GetDataLayerRuntimeState(const UDataLayerInstance* InDataLayer) const;

	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetDataLayerRuntimeStateByName instead.")
	ENGINE_API EDataLayerRuntimeState GetDataLayerRuntimeStateByName(const FName& InDataLayerName) const;

	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetDataLayerEffectiveRuntimeState instead.")
	ENGINE_API EDataLayerRuntimeState GetDataLayerEffectiveRuntimeState(const UDataLayerInstance* InDataLayer) const;

	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetDataLayerEffectiveRuntimeStateByName instead.")
	ENGINE_API EDataLayerRuntimeState GetDataLayerEffectiveRuntimeStateByName(const FName& InDataLayerName) const;

	UE_DEPRECATED(5.3, "Use UDataLayerManager::IsAnyDataLayerInEffectiveRuntimeState instead.")
	ENGINE_API bool IsAnyDataLayerInEffectiveRuntimeState(const TArray<FName>& InDataLayerNames, EDataLayerRuntimeState InState) const;

	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetEffectiveActiveDataLayerNames instead.")
	ENGINE_API const TSet<FName>& GetEffectiveActiveDataLayerNames() const;

	UE_DEPRECATED(5.3, "Use UDataLayerManager::GetEffectiveLoadedDataLayerNames instead.")
	ENGINE_API const TSet<FName>& GetEffectiveLoadedDataLayerNames() const;

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void GetDataLayerDebugColors(TMap<FName, FColor>& OutMapping) const {}

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	ENGINE_API void DrawDataLayersStatus(UCanvas* Canvas, FVector2D& Offset) const;

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	ENGINE_API void DumpDataLayers(FOutputDevice& OutputDevice) const;

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void RegisterWorldDataLayer(AWorldDataLayers* WorldDataLayers) {}

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void UnregisterWorldDataLayer(AWorldDataLayers* WorldDataLayers) {}

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void ForEachWorldDataLayer(TFunctionRef<bool(AWorldDataLayers*)> Func) {}

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void ForEachWorldDataLayer(TFunctionRef<bool(AWorldDataLayers*)> Func) const {}

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void ForEachDataLayer(TFunctionRef<bool(UDataLayerInstance*)> Func, const ULevel* InLevelContext = nullptr) {}

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void ForEachDataLayer(TFunctionRef<bool(UDataLayerInstance*)> Func, const ULevel* InLevelContext = nullptr) const {}

#if WITH_EDITOR
	UE_DEPRECATED(5.3, "This function has been deprecated.")
	static ENGINE_API TArray<const UDataLayerInstance*> GetRuntimeDataLayerInstances(UWorld* InWorld, const TArray<FName>& InDataLayerInstanceNames);

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	bool ResolveIsLoadedInEditor(const TArray<FName>& InDataLayerInstanceNames) const { return false; }

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	bool CanResolveDataLayers() const { return false; }

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	bool RemoveDataLayer(const UDataLayerInstance* InDataLayer) { return false; }

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	int32 RemoveDataLayers(const TArray<UDataLayerInstance*>& InDataLayerInstances) { return 0; }

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void UpdateDataLayerEditorPerProjectUserSettings() const {}

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void GetUserLoadedInEditorStates(TArray<FName>& OutDataLayersLoadedInEditor, TArray<FName>& OutDataLayersNotLoadedInEditor) const {}

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void PushActorEditorContext() const {}

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	void PopActorEditorContext() const {}

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	TArray<UDataLayerInstance*> GetActorEditorContextDataLayers() const { return TArray<UDataLayerInstance*>(); }

	UE_DEPRECATED(5.3, "This function has been deprecated.")
	uint32 GetDataLayerEditorContextHash() const { return 0; }
#endif

	PRAGMA_DISABLE_DEPRECATION_WARNINGS // Suppress compiler warning on override of deprecated function

	UE_DEPRECATED(5.0, "Use SetDataLayerRuntimeState instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, BlueprintAuthorityOnly, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use the equivalent function in DataLayerManager"))
	void SetDataLayerState(const FActorDataLayer& InDataLayer, EDataLayerState InState) { SetDataLayerRuntimeState(InDataLayer, (EDataLayerRuntimeState)InState); }

	UE_DEPRECATED(5.0, "Use SetDataLayerRuntimeStateByLabel instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, BlueprintAuthorityOnly, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use the equivalent function in DataLayerManager"))
	void SetDataLayerStateByLabel(const FName& InDataLayerLabel, EDataLayerState InState) { SetDataLayerRuntimeStateByLabel(InDataLayerLabel, (EDataLayerRuntimeState)InState); }

	UE_DEPRECATED(5.0, "Use GetDataLayerRuntimeState instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use the equivalent function in DataLayerManager"))
	EDataLayerState GetDataLayerState(const FActorDataLayer& InDataLayer) const { return (EDataLayerState)GetDataLayerRuntimeState(InDataLayer); }

	UE_DEPRECATED(5.0, "Use GetDataLayerRuntimeStateByLabel instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use the equivalent function in DataLayerManager"))
	EDataLayerState GetDataLayerStateByLabel(const FName& InDataLayerLabel) const { return (EDataLayerState)GetDataLayerRuntimeStateByLabel(InDataLayerLabel); }

	UE_DEPRECATED(5.0, "Use GetDataLayerRuntimeState instead.")
	EDataLayerState GetDataLayerState(const UDataLayerInstance* InDataLayer) const { return (EDataLayerState)GetDataLayerRuntimeState(InDataLayer); }

	UE_DEPRECATED(5.0, "Use UDataLayerManager::GetDataLayerInstanceRuntimeState instead.")
	EDataLayerState GetDataLayerStateByName(const FName& InDataLayerName) const { return (EDataLayerState)GetDataLayerRuntimeStateByName(InDataLayerName); }

	UE_DEPRECATED(5.0, "Use UDataLayerManager::IsAnyDataLayerInEffectiveRuntimeState instead.")
	bool IsAnyDataLayerInState(const TArray<FName>& InDataLayerNames, EDataLayerState InState) const { return IsAnyDataLayerInEffectiveRuntimeState(InDataLayerNames, (EDataLayerRuntimeState)InState); }

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.0, "GetActiveDataLayerNames will be removed.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "GetActiveDataLayerNames will be removed."))
	const TSet<FName>& GetActiveDataLayerNames() const { static TSet<FName> EmptySet; return EmptySet; }

	UE_DEPRECATED(5.0, "GetLoadedDataLayerNames will be removed.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "GetLoadedDataLayerNames will be removed."))
	const TSet<FName>& GetLoadedDataLayerNames() const { static TSet<FName> EmptySet; return EmptySet; }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "Use GetDataLayerInstanceFromAsset instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use GetDataLayerInstanceFromAsset in DataLayerManager"))
	ENGINE_API UDataLayerInstance* GetDataLayer(const FActorDataLayer& InDataLayer) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.1, "Use GetDataLayerInstanceFromAsset instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use GetDataLayerInstanceFromAsset in DataLayerManager"))
	ENGINE_API UDataLayerInstance* GetDataLayerFromName(FName InDataLayerName) const;

	UE_DEPRECATED(all, "Use GetDataLayerInstanceFromAsset instead.")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use GetDataLayerInstanceFromAsset in DataLayerManager"))
	UDataLayerInstance* GetDataLayerFromLabel(FName InDataLayerLabel) const { return nullptr; }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "Use GetDataLayerInstanceRuntimeState instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use GetDataLayerInstanceRuntimeState in DataLayerManager"))
	ENGINE_API EDataLayerRuntimeState GetDataLayerRuntimeState(const FActorDataLayer& InDataLayer) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.1, "Use GetDataLayerInstanceRuntimeState instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use GetDataLayerInstanceRuntimeState in DataLayerManager"))
	ENGINE_API EDataLayerRuntimeState GetDataLayerRuntimeStateByLabel(const FName& InDataLayerLabel) const;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "Use GetDataLayerInstanceEffectiveRuntimeState instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use GetDataLayerInstanceEffectiveRuntimeState in DataLayerManager"))
	ENGINE_API EDataLayerRuntimeState GetDataLayerEffectiveRuntimeState(const FActorDataLayer& InDataLayer) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.1, "Use GetDataLayerInstanceEffectiveRuntimeState instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, meta = (DeprecatedFunction, DeprecationMessage = "DataLayerSubsystem is deprecated, use GetDataLayerInstanceEffectiveRuntimeState in DataLayerManager"))
	ENGINE_API EDataLayerRuntimeState GetDataLayerEffectiveRuntimeStateByLabel(const FName& InDataLayerLabel) const;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "Use SetDataLayerRuntimeState() with UDataLayerAsset* overload instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, BlueprintAuthorityOnly)
	ENGINE_API void SetDataLayerRuntimeState(const FActorDataLayer& InDataLayer, EDataLayerRuntimeState InState, bool bInIsRecursive = false);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.1, "Use SetDataLayerInstanceRuntimeState instead.")
	UFUNCTION(BlueprintCallable, Category = DataLayers, BlueprintAuthorityOnly)
	ENGINE_API void SetDataLayerRuntimeStateByLabel(const FName& InDataLayerLabel, EDataLayerRuntimeState InState, bool bInIsRecursive = false);

private:
	UE_DEPRECATED(5.3, "Use UDataLayerManager::DataLayerLoadingPolicyClass instead.")
	UPROPERTY(Config)
	TSoftClassPtr<UDataLayerLoadingPolicy> DataLayerLoadingPolicyClass;

	friend class UDataLayerManager;
};

template<class T>
UDataLayerInstance* UDataLayerSubsystem::GetDataLayerInstance(const T& InDataLayerIdentifier, const ULevel* InLevelContext /* = nullptr */) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? const_cast<UDataLayerInstance*>(DataLayerManager->GetDataLayerInstance(InDataLayerIdentifier)) : nullptr;
}

template<class T>
TArray<FName> UDataLayerSubsystem::GetDataLayerInstanceNames(const TArray<T>& InDataLayerIdentifiers, const ULevel* InLevelContext /* = nullptr */) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetDataLayerInstanceNames(InDataLayerIdentifiers) : TArray<FName>();
}

template<class T>
TArray<const UDataLayerInstance*> UDataLayerSubsystem::GetDataLayerInstances(const TArray<T>& InDataLayerIdentifiers, const ULevel* InLevelContext /* = nullptr */) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetDataLayerInstances(InDataLayerIdentifiers) : TArray<const UDataLayerInstance*>();
}
