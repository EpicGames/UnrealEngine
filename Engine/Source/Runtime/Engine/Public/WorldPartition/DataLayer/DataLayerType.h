// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLayerType.generated.h"

UENUM(BlueprintType)
enum class EDataLayerType : uint8
{
	Runtime,
	Editor,
	Unknown UMETA(Hidden),

	Size UMETA(Hidden)
};