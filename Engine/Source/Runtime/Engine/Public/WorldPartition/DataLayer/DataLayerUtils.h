// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "WorldPartition/DataLayer/DataLayerType.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerInstanceNames.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"

class UDataLayerManager;
class FDataLayerInstanceDesc;
class FWorldDataLayersActorDesc;
class FWorldPartitionActorDesc;
class UActorDescContainer;
class AWorldDataLayers;

class FDataLayerUtils
{
public:
#if WITH_EDITOR
	static const TCHAR* GetDataLayerIconName(EDataLayerType DataLayerType)
	{
		static constexpr const TCHAR* IconNameByType[static_cast<int>(EDataLayerType::Size)] = { TEXT("DataLayer.Runtime") , TEXT("DataLayer.Editor"), TEXT("") };
		return IconNameByType[static_cast<uint32>(DataLayerType)];
	}

	UE_DEPRECATED(5.4, "Use ResolveDataLayerInstanceNames instead")
	static TArray<FName> ResolvedDataLayerInstanceNames(const UDataLayerManager* InDataLayerManager, const FWorldPartitionActorDesc* InActorDesc, const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs = TArray<const FWorldDataLayersActorDesc*>())
	{
		return ResolveDataLayerInstanceNames(InDataLayerManager, InActorDesc, InWorldDataLayersActorDescs).ToArray();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.4, "Use IWorldPartitionActorDescInstanceView version instead")
	static bool ResolveRuntimeDataLayerInstanceNames(const UDataLayerManager* InDataLayerManager, const class FWorldPartitionActorDescView& InActorDescView, const FStreamingGenerationActorDescViewMap& ActorDescViewMap, TArray<FName>& OutRuntimeDataLayerInstanceNames) { return false; }

	UE_DEPRECATED(5.4, "This function is no longer used")
	static TArray<const FWorldDataLayersActorDesc*> FindWorldDataLayerActorDescs(const FStreamingGenerationActorDescViewMap& ActorDescViewMap) { return TArray<const FWorldDataLayersActorDesc*>(); }

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Calls Callback for each data layer instance that an actor belongs to.
	 *
	 * @param InDataLayerManager If provided, we'll try to fetch the data layer instances for the current world
	 * @param InActorDesc The actor for which we check the assigned instances
	 * @param InWorldDataLayersActorDescs List of Data Layers to consider when Data Layer Manager is not available
	 * @param Callback The function to call for each instance; only one of the two pointers will be valid
	 * @return false if we failed to resolve data layer instances
	 */
	static ENGINE_API bool ForEachDataLayerInstance(const UDataLayerManager* InDataLayerManager, const FWorldPartitionActorDesc* InActorDesc, const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs, TFunctionRef<void(const UDataLayerInstance*, const FDataLayerInstanceDesc*, bool /** bExternal */)> Callback);

	static ENGINE_API FDataLayerInstanceNames ResolveDataLayerInstanceNames(const UDataLayerManager* InDataLayerManager, const FWorldPartitionActorDesc* InActorDesc, const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs = TArray<const FWorldDataLayersActorDesc*>());

	static ENGINE_API bool ResolveRuntimeDataLayerInstanceNames(const UDataLayerManager* InDataLayerManager, const IWorldPartitionActorDescInstanceView& InActorDescView, const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs, FDataLayerInstanceNames& OutRuntimeDataLayerInstanceNames);

	static ENGINE_API const FDataLayerInstanceDesc* GetDataLayerInstanceDescFromInstanceName(const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs, const FName& DataLayerInstanceName);

	static ENGINE_API const FDataLayerInstanceDesc* GetDataLayerInstanceDescFromAssetPath(const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs, const FName& DataLayerAssetPath);

	static ENGINE_API bool AreWorldDataLayersActorDescsSane(const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs);

	static ENGINE_API FString GenerateUniqueDataLayerShortName(const UDataLayerManager* InDataLayerManager, const FString& InNewShortName);

	static ENGINE_API bool SetDataLayerShortName(UDataLayerInstance* InDataLayerInstance, const FString& InNewShortName);

	static ENGINE_API bool FindDataLayerByShortName(const UDataLayerManager* InDataLayerManager, const FString& InShortName, TSet<UDataLayerInstance*>& OutDataLayerInstances);

	static ENGINE_API bool AreDataLayerTypesCompatible(EDataLayerType ParentDataLayerType, EDataLayerType ChildDataLayerType, bool bIsParentExternalDataLayer, FText* OutReason = nullptr);

	/**
	 * Returns true if a cell should be considered client-only based on its data layer instances.
	 * When the list contains a mix of External Data Layer (EDL) and regular data layers,
	 * the EDL is skipped as long as it is not server-only (i.e. its load filter is None or ClientOnly).
	 * Only the regular data layers are tested via IsClientOnly().
	 * When the list contains a single entry (EDL or regular), its IsClientOnly() value is used directly.
	 * At most one EDL is expected in the list.
	 */
	static ENGINE_API bool IsClientOnlyFromDataLayers(const TArray<const UDataLayerInstance*>& InCellDataLayerInstances);
#endif

	static FString GetSanitizedDataLayerShortName(FString InShortName)
	{
		return InShortName.TrimStartAndEnd().Replace(TEXT("\""), TEXT(""));
	}
};
