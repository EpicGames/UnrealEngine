// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UDataLayerInstance;

#if WITH_EDITOR
class FDataLayersID
{
public:
	ENGINE_API FDataLayersID();
	ENGINE_API FDataLayersID(const TArray<const UDataLayerInstance*>& DataLayerInstance);

	bool operator==(const FDataLayersID& Other) const
	{
		return Hash == Other.Hash;
	}

	bool operator!=(const FDataLayersID& Other) const
	{
		return !(*this == Other);
	}

	uint32 GetHash() const
	{
		return Hash;
	}

	friend FArchive& operator<<(FArchive& InAr, FDataLayersID& InDataLayersID)
	{
		return InAr << InDataLayersID.Hash;
	}

	friend uint32 GetTypeHash(const FDataLayersID& InDataLayersID)
	{
		return InDataLayersID.Hash;
	}

private:
	uint32 Hash;
};
#endif
