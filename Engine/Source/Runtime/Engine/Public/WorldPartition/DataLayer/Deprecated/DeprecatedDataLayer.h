// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Math/Color.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"

#include "DeprecatedDataLayer.generated.h"

// These deprecated reflected types and members must not be removed for the duration of UE5, even after 2 versions post-deprecation, to prevent data losses and allow loading and upgrading old assets.

UENUM(BlueprintType)
enum class UE_DEPRECATED(all, "Use EDataLayerRuntimeState instead.") EDataLayerState : uint8
{
	Unloaded,
	Loaded,
	Activated
};

static_assert(EDataLayerRuntimeState::Unloaded < EDataLayerRuntimeState::Loaded && EDataLayerRuntimeState::Loaded < EDataLayerRuntimeState::Activated, "Streaming Query code is dependent on this being true");

struct UE_DEPRECATED(5.8, "This class is deprecated and only present for backward compatibility purposes") FActorDataLayer;

USTRUCT(BlueprintType)
struct FActorDataLayer
{
	GENERATED_USTRUCT_BODY()

	FActorDataLayer()
	: Name(NAME_None)
	{}

	FActorDataLayer(const FName& InName)
	: Name(InName)
	{}

	inline bool operator==(const FActorDataLayer& Other) const { return Name == Other.Name; }
	inline bool operator<(const FActorDataLayer& Other) const { return Name.FastLess(Other.Name); }

	inline operator FName() const { return Name; }

	/** The name of this layer */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "FActorDataLayer is deprecated."))
	FName Name;

	friend uint32 GetTypeHash(const FActorDataLayer& Value)
	{
		return GetTypeHash(Value.Name);
	}
};

class UE_DEPRECATED(all, "Use UDataLayerInstance & UDataLayerAsset to create DataLayers") UDEPRECATED_DataLayer;

UCLASS(Config = Engine, PerObjectConfig, Within = WorldDataLayers, BlueprintType, Deprecated, MinimalAPI)
class UDEPRECATED_DataLayer : public UObject
{
	GENERATED_BODY()

	friend class UDeprecatedDataLayerInstance;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	bool Equals(const FActorDataLayer& ActorDataLayer) const { return false; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	FName GetDataLayerLabel() const { return FName{}; }

	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	bool IsInitiallyVisible() const { return false; }

	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	bool IsVisible() const { return false; }

	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	bool IsEffectiveVisible() const { return false; }

	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	FColor GetDebugColor() const { return FColor{}; }

	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	bool IsRuntime() const { return false; }
	
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	EDataLayerRuntimeState GetInitialRuntimeState() const { return EDataLayerRuntimeState::Unloaded; }

	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	bool IsDynamicallyLoaded() const { return false; }

	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	bool IsInitiallyActive() const { return IsRuntime() && GetInitialRuntimeState() == EDataLayerRuntimeState::Activated; }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	EDataLayerState GetInitialState() const { return EDataLayerState::Unloaded; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	uint32 bIsInitiallyActive_DEPRECATED : 1;

	UPROPERTY(Transient, meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	uint32 bIsVisible : 1;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	uint32 bIsInitiallyVisible : 1;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	uint32 bIsInitiallyLoadedInEditor : 1;

	UPROPERTY(Transient, meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	uint32 bIsLoadedInEditor : 1;

	UPROPERTY(Transient, meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	uint32 bIsLoadedInEditorChangedByUserOperation : 1;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	uint32 bIsLocked : 1;
#endif

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	FName DataLayerLabel;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	uint32 bIsRuntime : 1;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	EDataLayerRuntimeState InitialRuntimeState;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	FColor DebugColor;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	TObjectPtr<UDEPRECATED_DataLayer> Parent_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use UDataLayerInstance & UDataLayerAsset to create DataLayers"))
	TArray<TObjectPtr<UDEPRECATED_DataLayer>> Children_DEPRECATED;
};

class UE_DEPRECATED(all, "Only for use by the conversion commandlet") UDeprecatedDataLayerInstance;

UCLASS(Config = Engine, PerObjectConfig, Within = WorldDataLayers, AutoCollapseCategories = ("Data Layer|Advanced"), AutoExpandCategories = ("Data Layer|Editor", "Data Layer|Advanced|Runtime"), MinimalAPI)
class UDeprecatedDataLayerInstance final : public UDataLayerInstance
{
	GENERATED_BODY()


#if WITH_EDITOR
	//~ Begin UObject Interface
	virtual void PostLoad() override
	{
		if (DebugColor == FColor::Black)
		{
			DebugColor = FColor::MakeRandomSeededColor(GetTypeHash(GetFName().ToString()));
		}

		Super::PostLoad();
	}
	//~ End UObject Interface
#endif

public:
	UDeprecatedDataLayerInstance()
	: DataLayerType(EDataLayerType::Editor)
	{}

#if WITH_EDITOR
	static FName MakeName()
	{
		return FName(FString::Format(TEXT("DataLayer_{0}"), { FGuid::NewGuid().ToString() }));
	}
	void OnCreated()
	{
		Modify(/*bAlwaysMarkDirty*/false);

		FDataLayerUtils::SetDataLayerShortName(this, TEXT("DataLayer"));

		DeprecatedDataLayerFName = TEXT("");

		DebugColor = FColor::MakeRandomSeededColor(GetTypeHash(GetFName().ToString()));
	}
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FActorDataLayer GetActorDataLayer() const
	{
		return FActorDataLayer(GetFName());
	}

	static FName MakeName(const class UDEPRECATED_DataLayer* DeprecatedDataLayer) { return FName{}; }

	void OnCreated(const class UDEPRECATED_DataLayer* DeprecatedDataLayer) {}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	
	FName GetDataLayerLabel() const { return Label; }

private:
#if WITH_EDITOR
	//~ Begin UDataLayerInstance Interface
	virtual bool PerformAddActor(AActor* InActor) const override { return false; }
	virtual bool PerformRemoveActor(AActor* InActor) const override { return false; }
	virtual bool CanEditDataLayerShortName() const override { return true; }
	virtual void PerformSetDataLayerShortName(const FString& InNewShortName) { Label = *InNewShortName; }
	//~ Endif UDataLayerInstance Interface
#endif

	//~ Begin UDataLayerInstance Interface
	FName GetDataLayerFName() const { return !DeprecatedDataLayerFName.IsNone() ? DeprecatedDataLayerFName : GetFName(); }
	virtual EDataLayerType GetType() const override { return DataLayerType; }
	virtual bool IsRuntime() const override { return DataLayerType == EDataLayerType::Runtime; }
	virtual FColor GetDebugColor() const override { return DebugColor; }
	virtual FString GetDataLayerShortName() const override { return Label.ToString(); }
	virtual FString GetDataLayerFullName() const override { return TEXT("Deprecated_") + Label.ToString(); }
	//~ End UDataLayerInstance Interface

private:
	UPROPERTY()
	FName Label;

	UPROPERTY()
	FName DeprecatedDataLayerFName;

	UPROPERTY(Category = "Data Layer", EditAnywhere)
	EDataLayerType DataLayerType;

	UPROPERTY(Category = "Data Layer|Runtime", EditAnywhere, meta = (EditConditionHides, EditCondition = "DataLayerType == EDataLayerType::Runtime"))
	FColor DebugColor;

	friend class AWorldDataLayers;
	friend class UDataLayerConversionInfo;
	friend class UDataLayerToAssetCommandletContext;
	friend class UDataLayerToAssetCommandlet;
};