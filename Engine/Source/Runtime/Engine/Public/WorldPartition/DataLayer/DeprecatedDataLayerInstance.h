// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/Deprecated/DeprecatedDataLayer.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

UE_DEPRECATED_HEADER(5.8, "The contents of this header have been fully deprecated. See DeprecatedDataLayer.h for details on particular cases.") 

#include "WorldPartition/DataLayer/Deprecated/DeprecatedDataLayer.h"