// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerUID.h"

#include "ExternalDataLayerAsset.generated.h"

#define UE_API ENGINE_API

struct FAssetData;

UCLASS(MinimalAPI, BlueprintType, editinlinenew)
class UExternalDataLayerAsset : public UDataLayerAsset
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	static constexpr FColor EditorUXColor = FColor(255, 167, 26);
#endif

#if WITH_EDITOR
	//~ Begin UObject Interface
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	//~ End UObject Interface

	static UE_API bool GetAssetRegistryInfoFromPackage(const FAssetData& InAsset, FExternalDataLayerUID& OutExternalDataLayerUID);
#endif

	//~ Begin UDataLayerAsset Interface
#if WITH_EDITOR
	UE_API virtual void OnCreated() override;
	virtual bool CanEditDataLayerType() const override { return false; }
#endif
	virtual EDataLayerType GetType() const override { return EDataLayerType::Runtime; }
	//~ End UDataLayerAsset Interface

	const FExternalDataLayerUID& GetUID() const { return UID; }

private:
	UPROPERTY(VisibleAnywhere, Category = "Data Layer", AdvancedDisplay, DuplicateTransient, meta = (DisplayName = "External Data Layer UID"))
	FExternalDataLayerUID UID;
};

#undef UE_API
