// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectKey.h"
#include "Templates/SubclassOf.h"
#include "Subsystems/EngineSubsystem.h"
#include "Tickable.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInjectionPolicy.h"

#include "ExternalDataLayerEngineSubsystem.generated.h"

struct FAssetData;
class UExternalDataLayerAsset;

enum class EExternalDataLayerRegistrationState : uint8
{
	Unregistered,
	Registered,
	Active
};

UCLASS(Config = Engine, MinimalAPI)
class UExternalDataLayerEngineSubsystem : public UEngineSubsystem, public FTickableGameObject
{
	GENERATED_BODY()

public:
	ENGINE_API static UExternalDataLayerEngineSubsystem& Get();

#if WITH_EDITOR
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;

	virtual bool IsTickableInEditor() const override { return true; }
	virtual UWorld* GetTickableGameObjectWorld() const override;
#endif
	virtual bool IsTickable() const override;
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return (IsTemplate() ? ETickableTickType::Never : ETickableTickType::Conditional); }
	virtual TStatId GetStatId() const override;

	ENGINE_API void RegisterExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient);
	ENGINE_API void UnregisterExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient);
	ENGINE_API void ActivateExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient);
	ENGINE_API void DeactivateExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient);
	ENGINE_API bool IsExternalDataLayerAssetRegistered(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient = nullptr) const;
	ENGINE_API bool IsExternalDataLayerAssetActive(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient = nullptr) const;
	ENGINE_API EExternalDataLayerRegistrationState GetExternalDataLayerAssetRegistrationState(const UExternalDataLayerAsset* InExternalDataLayerAsset) const;
	ENGINE_API TArray<UObject*> GetClientsForExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset) const;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FExternalDataLayerAssetRegistrationStateChangedEventDelegate, const UExternalDataLayerAsset* ExternalDataLayerAsset, EExternalDataLayerRegistrationState OldState, EExternalDataLayerRegistrationState NewState);
	FExternalDataLayerAssetRegistrationStateChangedEventDelegate OnExternalDataLayerAssetRegistrationStateChanged;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FExternalDataLayerOverrideInjectionDelegate, const UWorld* InWorld, const UExternalDataLayerAsset* InExternalDataLayerAsset, bool& bOutOverrideValue);
	FExternalDataLayerOverrideInjectionDelegate OnExternalDataLayerOverrideInjection;
	ENGINE_API bool CanInjectOverride(const UWorld* InWorld, const UExternalDataLayerAsset* InExternalDataLayerAsset, bool& bOutOverrideValue) const;

private:
#if WITH_EDITOR
	void OnAssetsPreDelete(const TArray<UObject*>& Objects);
	void OnGetLevelExternalActorsPaths(const FString& InLevelPackageName, const FString& InPackageShortName, TArray<FString>& OutExternalActorsPaths);
	bool OnResolveLevelMountPoint(const FString& InLevelPackageName, const UObject* InLevelMountPointContext, FString& OutResolvedLevelMountPoint);
#endif
	ENGINE_API bool CanWorldInjectExternalDataLayerAsset(const UWorld* InWorld, const UExternalDataLayerAsset* InExternalDataLayerAsset, FText* OutFailureReason = nullptr) const;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Config)
	TSubclassOf<UExternalDataLayerInjectionPolicy> InjectionPolicyClass;

	UPROPERTY(Transient)
	TObjectPtr<UExternalDataLayerInjectionPolicy> InjectionPolicy;
#endif

	struct FRegisteredExternalDataLayers
	{
		TSet<FObjectKey> RegisteredClients;
		TSet<FObjectKey> ActiveClients;
		EExternalDataLayerRegistrationState State = EExternalDataLayerRegistrationState::Registered;
	};

	using FRegisteredExternalDataLayerAssetMap = TMap<TObjectPtr<const UExternalDataLayerAsset>, FRegisteredExternalDataLayers>;
	const FRegisteredExternalDataLayerAssetMap& GetRegisteredExternalDataLayerAssets() const { return ExternalDataLayerAssets; }

	FRegisteredExternalDataLayerAssetMap ExternalDataLayerAssets;

#if WITH_EDITOR
	TMap<TWeakObjectPtr<const UExternalDataLayerAsset>, FRegisteredExternalDataLayers> PreDeletedExternalDataLayerAssets;

	using FForcedExternalDataLayerInjectionKey = TPair<const UWorld*, const UExternalDataLayerAsset*>;
	TSet<FForcedExternalDataLayerInjectionKey> ForcedAllowInjection;

	FDelegateHandle LevelExternalActorsPathsProviderDelegateHandle;
	FDelegateHandle LevelMountPointResolverDelegateHandle;
#endif

	friend class UExternalDataLayerManager;
	friend class UGameFeatureActionConvertContentBundleWorldPartitionBuilder;
};
