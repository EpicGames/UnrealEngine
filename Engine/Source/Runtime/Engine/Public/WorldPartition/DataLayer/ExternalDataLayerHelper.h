// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreFwd.h"
#include "Containers/StringView.h"
#include "Templates/Function.h"

class AActor;
class FAssetRegistryTagsContext;
class UExternalDataLayerAsset;
class UExternalDataLayerInstance;
struct FExternalDataLayerUID;
struct FAssetData;

namespace UE::MeshPartition
{
	struct FWPActorPropertiesTransformer;
}

struct FMoveToExternalDataLayerParams
{
	FMoveToExternalDataLayerParams()
		: ExternalDataLayerInstance (nullptr)
		, bAllowNonUserManaged(false)
	{}

	FMoveToExternalDataLayerParams(const UExternalDataLayerInstance* InExternalDataLayerInstance, bool bInAllowNonUserManaged = false)
		: ExternalDataLayerInstance(InExternalDataLayerInstance)
		, bAllowNonUserManaged(bInAllowNonUserManaged)
	{}

	const UExternalDataLayerInstance* ExternalDataLayerInstance;
	bool bAllowNonUserManaged;
};

class FExternalDataLayerHelper
{
public:
	/** Returns the external streaming object package name */
	static FString GetExternalStreamingObjectPackageName(const UExternalDataLayerAsset* InExternalDataLayerAsset);

	/** Return true if succeeds building the external data layer root path (OutExternalDataLayerRootPath) using the provided mount point and EDL UID.
	 * Format is /{MountPoint}/{ExternalDataLayerFolder}/{EDL_UID}
	 */
	ENGINE_API static bool BuildExternalDataLayerRootPath(const FString& InEDLMountPoint, const FExternalDataLayerUID& InExternalDataLayerUID, FString& OutExternalDataLayerRootPath);

	/** Return true if succeeds building the external data layer actors root path (OutExternalDataLayerRootPath) using the provided mount point and EDL UID.
	 * Format is /{MountPoint}/{ExternalActorFolder}/{ExternalDataLayerFolder}/{EDL_UID}
	 */
	ENGINE_API static bool BuildExternalDataLayerActorsRootPath(const FString& InEDLMountPoint, const FExternalDataLayerUID& InExternalDataLayerUID, FString& OutExternalDataLayerRootPath);

	/** Return the external data layer level root path (OutExternalDataLayerLevelRootPath) using the ExternalDataLayerAsset and the level package path.
	  * Format is /{MountPoint}/{ExternalDataLayerFolder}/{EDL_UID}/{LevelPath}
	  */
	ENGINE_API static FString GetExternalDataLayerLevelRootPath(const UExternalDataLayerAsset* InExternalDataLayerAsset, const FString& InLevelPackagePath);

	/** Return the external data layer level root path (OutExternalDataLayerLevelRootPath) using the provided mount point, EDL UID and the level package path.
	  * Format is /{MountPoint}/{ExternalDataLayerFolder}/{EDL_UID}/{LevelPath}
	  */
	ENGINE_API static FString GetExternalDataLayerLevelRootPath(const FString& InEDLMountPoint, const FExternalDataLayerUID& InExternalDataLayerUID, const FString& InLevelPackagePath);

#if WITH_EDITOR
	/** Returns whether the provided path respects the format <start_path>/{ExternalDataLayerFolder}/{EDL_UID}/<end_path>. 
	 * If true, fills OutExternalDataLayerUID when provided.
	 */
	ENGINE_API static bool IsExternalDataLayerPath(FStringView InExternalDataLayerPath, FExternalDataLayerUID* OutExternalDataLayerUID = nullptr);

	/** Fills Asset Registry Tags Context with provided External Data Layer UIDs. */
	ENGINE_API static void AddAssetRegistryTags(FAssetRegistryTagsContext OutContext, const TArray<FExternalDataLayerUID>& InExternalDataLayerUIDs);

	/** Retrieves External Data Layers UIDs from provided Asset. */
	ENGINE_API static void GetExternalDataLayerUIDs(const FAssetData& Asset, TArray<FExternalDataLayerUID>& OutExternalDataLayerUIDs);

	/** Iterates through all possible External Data Layer Level Package Paths using Asset Registry. */
	ENGINE_API static void ForEachExternalDataLayerLevelPackagePath(const FString& InLevelPackageName, TFunctionRef<void(const FString&)> Func);

	/* Returns the external actor package relative path for an actor package of an actor using External Data Layers.
	 * InExternalDataLayerExternalActorPackagePath format is : /{MountPoint}/{ExternalActorFolder}/{ExternalDataLayerFolder}/{EDL_UID}/{ExternalActorPackagePath}
	 * return format is : /{ExternalActorPackagePath}, empty otherwise
	 */
	ENGINE_API static FStringView GetRelativeExternalActorPackagePath(FStringView InExternalDataLayerExternalActorPackagePath);
#endif

private:

#if WITH_EDITOR
	ENGINE_API static const UExternalDataLayerAsset* GetExternalDataLayerAssetFromObject(const UObject* InContextObject);

	/** 
	 * Validates that all actors can change their External Data Layer to the new provided value (supports passing null) 
	 * Returns false if any actor fails and fills OutFailureReason with the reason (if non-null). 
	 */
	ENGINE_API static bool CanMoveActorsToExternalDataLayer(const TArray<AActor*>& InActors, const FMoveToExternalDataLayerParams& InParams, FText* OutFailureReason = nullptr);

	/** 
	 * Changes all actors External Data Layer to the new provided value (supports passing null)
	 * Returns false if any actor fails and fills OutFailureReason with the reason (if non-null). 
	 */
	ENGINE_API static bool MoveActorsToExternalDataLayer(const TArray<AActor*>& InActors, const UExternalDataLayerInstance* InExternalDataLayerInstance, FText* OutFailureReason = nullptr);

	/**
	 * Changes all actors External Data Layer to the new provided value (supports passing null)
	 * Returns false if any actor fails and fills OutFailureReason with the reason (if non-null).
	 */
	ENGINE_API static bool MoveActorsToExternalDataLayer(const TArray<AActor*>& InActors, const FMoveToExternalDataLayerParams& InParams, FText* OutFailureReason = nullptr);

	friend class FDataLayerEditorModule;
	friend class UActorPartitionSubsystem;
	friend class UDataLayerEditorSubsystem;
	friend class UExternalDataLayerEngineSubsystem;
	friend class UContentBundleEditingSubmodule;
	friend class UGameFeatureActionConvertContentBundleWorldPartitionBuilder;
	friend struct UE::MeshPartition::FWPActorPropertiesTransformer;
#endif
	static constexpr FStringView GetExternalDataLayerFolder() { return ExternalDataLayerFolder; }
	static constexpr FStringView ExternalDataLayerFolder = TEXTVIEW("/EDL/");
};