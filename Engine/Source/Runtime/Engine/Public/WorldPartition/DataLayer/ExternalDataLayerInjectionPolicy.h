// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"

#include "ExternalDataLayerInjectionPolicy.generated.h"

#define UE_API ENGINE_API

class UWorld;
class UExternalDataLayerAsset;

UCLASS(MinimalAPI)
class UExternalDataLayerInjectionPolicy : public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	UE_API virtual bool CanInject(const UWorld* InWorld, const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient, FText* OutFailureReason = nullptr) const;
#endif
};

#undef UE_API
