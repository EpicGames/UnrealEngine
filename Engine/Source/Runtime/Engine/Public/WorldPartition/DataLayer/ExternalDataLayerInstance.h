// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "ExternalDataLayerInstance.generated.h"

class UDataLayerAsset;
class UExternalDataLayerAsset;

UCLASS(Config = Engine, PerObjectConfig, BlueprintType, AutoCollapseCategories = ("Data Layer|Advanced"), AutoExpandCategories = ("Data Layer|Editor", "Data Layer|Advanced|Runtime"), MinimalAPI)
class UExternalDataLayerInstance : public UDataLayerInstanceWithAsset
{
	GENERATED_BODY()

public:

	//~ Begin UDataLayerInstance Interface
	virtual UWorld* GetOuterWorld() const override;
	virtual bool CanHaveParentDataLayerInstance() const override { return false; }
	//~ End UDataLayerInstance Interface

	ENGINE_API const UExternalDataLayerAsset* GetExternalDataLayerAsset() const;

#if WITH_EDITOR
	//~ Begin UDataLayerInstanceWithAsset Interface
	ENGINE_API static FName MakeName(const UDataLayerAsset* InDataLayerAsset);
	ENGINE_API static TSubclassOf<UDataLayerInstanceWithAsset> GetDataLayerInstanceClass();
	virtual bool CanEditDataLayerAsset() const override { return false; }
	//~ End UDataLayerInstanceWithAsset Interface

	//~ Begin UDataLayerInstance Interface
	ENGINE_API virtual const TCHAR* GetDataLayerIconName() const override;
	ENGINE_API virtual void OnCreated(const UDataLayerAsset* Asset) override;
	ENGINE_API virtual bool IsReadOnly(FText* OutReason = nullptr) const override;
	ENGINE_API virtual bool CanUserAddActors(FText* OutReason = nullptr) const override;
	ENGINE_API virtual bool CanUserRemoveActors(FText* OutReason = nullptr) const override;
	ENGINE_API virtual bool CanAddActor(AActor* Actor, FText* OutReason = nullptr) const override;
	ENGINE_API virtual bool CanRemoveActor(AActor* Actor, FText* OutReason = nullptr) const override;
	ENGINE_API virtual bool CanHaveChildDataLayerInstance(const UDataLayerInstance* DataLayerInstance) const override;
	virtual bool CanBeRemoved() const override { return false; }
	//~ End UDataLayerInstance Interface
#endif

protected:
#if WITH_EDITOR
	//~ Begin UDataLayerInstanceWithAsset Interface
	ENGINE_API virtual bool PerformAddActor(AActor* Actor) const override;
	ENGINE_API virtual bool PerformRemoveActor(AActor* Actor) const override;
	//~ End UDataLayerInstanceWithAsset Interface
#endif

	friend class FExternalDataLayerHelper;
};