// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/DataLayer/WorldDataLayersActorDesc.h"
#include "WorldPartition/DataLayer/ExternalDataLayerEngineSubsystem.h"

#include "ExternalDataLayerManager.generated.h"

#define UE_API ENGINE_API

class UActorDescContainerInstance;
class UDataLayerManager;
class UExternalDataLayerAsset;
class UExternalDataLayerInstance;
class UWorldPartitionRuntimeCell;
class URuntimeHashExternalStreamingObjectBase;
class AWorldDataLayers;

UCLASS(MinimalAPI, Within = WorldPartition)
class UExternalDataLayerManager : public UObject
{
	GENERATED_BODY()

public:
	template <class T>
	static UExternalDataLayerManager* GetExternalDataLayerManager(const T* InObject)
	{
		UWorldPartition* WorldPartition = IsValid(InObject) ? FWorldPartitionHelpers::GetWorldPartition(InObject) : nullptr;
		return WorldPartition ? WorldPartition->GetExternalDataLayerManager() : nullptr;
	}

private:
	//~ Begin Initialization/Deinitialization
	UE_API UExternalDataLayerManager();
	UE_API void Initialize();
	bool IsInitialized() const { return bIsInitialized; }
	UE_API void DeInitialize();
	//~ End Initialization/Deinitialization

	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin Injection/Removal
	UE_API void UpdateExternalDataLayerInjectionState(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	UE_API bool CanInjectExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset, FText* OutReason = nullptr) const;
	UE_API bool IsExternalDataLayerInjected(const UExternalDataLayerAsset* InExternalDataLayerAsset) const;
	UE_API bool InjectExternalDataLayer(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	UE_API bool RemoveExternalDataLayer(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	UE_API bool InjectIntoGameWorld(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	UE_API bool RemoveFromGameWorld(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	UE_API bool RegisterExternalStreamingObjectForGameWorld(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	UE_API bool UnregisterExternalStreamingObjectForGameWorld(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	//~ End Injection/Removal

	// Monitor when EDL state changes
	UE_API void OnExternalDataLayerAssetRegistrationStateChanged(const UExternalDataLayerAsset* InExternalDataLayerAsset, EExternalDataLayerRegistrationState InOldState, EExternalDataLayerRegistrationState InNewState);

	UE_API FString GetExternalDataLayerLevelRootPath(const UExternalDataLayerAsset* InExternalDataLayerAsset) const;
	UE_API FString GetExternalStreamingObjectPackagePath(const UExternalDataLayerAsset* InExternalDataLayerAsset) const;
	UE_API const UExternalDataLayerInstance* GetExternalDataLayerInstance(const UExternalDataLayerAsset* InExternalDataLayerAsset) const;
	UE_API UExternalDataLayerInstance* GetExternalDataLayerInstance(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	UE_API UDataLayerManager& GetDataLayerManager() const;

#if WITH_EDITOR
	//~ Begin UObject Interface
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	//~ End UObject Interface

	// Used in editor
	UE_API UActorDescContainerInstance* RegisterExternalDataLayerActorDescContainer(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	UE_API bool UnregisterExternalDataLayerActorDescContainer(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	UE_API bool ValidateOnActorExternalDataLayerAssetChanged(AActor* InActor);
	UE_API bool RegisterExternalDataLayerInstance(UExternalDataLayerInstance* InExternalDataLayerInstance);
	UE_API bool UnregisterExternalDataLayerInstance(UExternalDataLayerInstance* InExternalDataLayerInstance);
	UE_API const UExternalDataLayerAsset* GetMatchingExternalDataLayerAssetForObjectPath(const FSoftObjectPath& InObjectPath) const;
	UE_API const UExternalDataLayerAsset* GetActorEditorContextCurrentExternalDataLayer() const;
	UE_API AWorldDataLayers* GetWorldDataLayers(const UExternalDataLayerAsset* InExternalDataLayerAsset, bool bInAllowCreate = false) const;
	UE_API FString GetActorPackageName(const UExternalDataLayerAsset* InExternalDataLayerAsset, const ULevel* InDestinationLevel, const FString& InActorPath) const;
	UE_API URuntimeHashExternalStreamingObjectBase* CreateExternalStreamingObjectUsingStreamingGeneration(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	bool HasInjectedExternalDataLayerAssets() const { return InjectedExternalDataLayerAssets.Num() > 0; }
	UE_API URuntimeHashExternalStreamingObjectBase* GetExternalStreamingObjectForDataLayerAsset(const UExternalDataLayerAsset* InAsset) const;

	// Used for PIE/-game
	UE_API void PrepareEditorGameWorld();
	UE_API void ShutdownEditorGameWorld();

	//~ Begin Cooking
	UE_API UWorldPartitionRuntimeCell* GetCellForCookPackage(const FString& InCookPackageName) const;
	UE_API URuntimeHashExternalStreamingObjectBase* GetExternalStreamingObjectForCookPackage(const FString& InCookPackageName) const;
	UE_API void ForEachExternalStreamingObjects(TFunctionRef<bool(URuntimeHashExternalStreamingObjectBase*)> Func) const;
	//~ End Cooking
#endif

private:
	bool IsRunningGameOrInstancedWorldPartition() const { return bIsRunningGameOrInstancedWorldPartition; }

	bool bIsInitialized;
	bool bIsRunningGameOrInstancedWorldPartition;

	UPROPERTY(Transient)
	TMap<TObjectPtr<const UExternalDataLayerAsset>, TObjectPtr<URuntimeHashExternalStreamingObjectBase>> ExternalStreamingObjects;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TSet<TObjectPtr<const UExternalDataLayerAsset>> InjectedExternalDataLayerAssets;
#endif

#if WITH_EDITOR
	using FExternalDataLayerContainerMap = TMap<TObjectPtr<const UExternalDataLayerAsset>, TObjectPtr<UActorDescContainerInstance>>;
	FExternalDataLayerContainerMap EDLContainerMap;
	TMap<TObjectPtr<const UExternalDataLayerAsset>, FWorldPartitionReference> EDLWorldDataLayersMap;
	TSet<TObjectPtr<const UExternalDataLayerAsset>> PreEditUndoExternalDataLayerAssets;
#endif

	friend class FDataLayerMode;
	friend class FDataLayerEditorModule;
	friend class FExternalDataLayerHelper;
	friend class AWorldDataLayers;
	friend class UWorldPartition;
	friend class UExternalDataLayerInstance;
	friend class UDataLayerEditorSubsystem;
	friend class ULevelInstanceSubsystem;
	friend class UContentBundleEditingSubmodule;
	friend class UWorldPartitionRuntimeLevelStreamingCell;
	friend class UGameFeatureActionConvertContentBundleWorldPartitionBuilder;
	friend struct FWorldPartitionUtils;
	friend class URuntimeHashExternalStreamingObjectBase;
};

#undef UE_API
