// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExternalDataLayerUID.generated.h"

#define UE_API ENGINE_API

struct FAssetData;

USTRUCT()
struct FExternalDataLayerUID
{
	GENERATED_BODY()

public:
	UE_API bool IsValid() const;
	operator uint32() const { return Value; }

	UE_API FString ToString() const;
#if WITH_EDITOR
	FExternalDataLayerUID(uint32 InValue = 0) { Value = InValue; }
	static UE_API bool Parse(const FString& InUIDString, FExternalDataLayerUID& OutUID);
#endif

private:
#if WITH_EDITOR
	static UE_API FExternalDataLayerUID NewUID();
#endif

	UPROPERTY(VisibleAnywhere, Category = "External Data Layer", AdvancedDisplay)
	uint32 Value = 0;

	friend uint32 GetTypeHash(const FExternalDataLayerUID& InUID) { return InUID.Value; }
	friend class UExternalDataLayerAsset;
};

#undef UE_API
