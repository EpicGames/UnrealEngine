// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class AActor;
class UDataLayerInstance;
class UExternalDataLayerAsset;

/**
 * The module holding all of the UI related pieces for DataLayer management
 */
class IDataLayerEditorModule : public IModuleInterface
{
public:
	virtual ~IDataLayerEditorModule() = default;
	virtual bool AddActorToDataLayers(AActor* Actor, const TArray<UDataLayerInstance*>& DataLayers) = 0;
	virtual void SetActorEditorContextCurrentExternalDataLayer(const UExternalDataLayerAsset* InExternalDataLayerAsset) = 0;
};
