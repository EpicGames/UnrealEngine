// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Info.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerInstanceProviderInterface.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "ExternalPackageHelper.h"
#include "WorldDataLayers.generated.h"

class UDataLayerInstance;
class UDataLayerInstanceWithAsset;
class UDataLayerAsset;
class UExternalDataLayerInstance;

USTRUCT()
struct FActorPlacementDataLayers
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FName> DataLayerInstanceNames;

	UPROPERTY()
	FName ExternalDataLayerName;

	UPROPERTY()
	FName CurrentColorizedDataLayerInstanceName;

	UPROPERTY()
	int32 ContextID = INT32_MAX;

	void Reset()
	{
		DataLayerInstanceNames.Reset();
		ExternalDataLayerName = NAME_None;
		CurrentColorizedDataLayerInstanceName = NAME_None;
	}
};

struct FWorldDataLayersEffectiveStates
{
	FWorldDataLayersEffectiveStates();
	const TSet<FName>& GetAllEffectiveActiveDataLayerNames() const;
	const TSet<FName>& GetAllEffectiveLoadedDataLayerNames() const;
	EDataLayerRuntimeState GetDataLayerEffectiveRuntimeStateByName(FName InDataLayerName) const;
	int32 GetUpdateEpoch() const { return UpdateEpoch; }

protected:
	void Reset();
	void SetReplicatedEffectiveActiveDataLayerNames(const TArray<FName>& InRepEffectiveActiveDataLayerNames);
	void SetReplicatedEffectiveLoadedDataLayerNames(const TArray<FName>& InRepEffectiveLoadedDataLayerNames);
	bool SetDataLayerEffectiveRuntimeState(FName InDataLayerName, bool bIsLocalDataLayer, EDataLayerRuntimeState NewEffectiveRuntimeState, EDataLayerRuntimeState& OutOldEffectiveRuntimeState);
	const TSet<FName>& GetReplicatedEffectiveActiveDataLayerNames() const { return ReplicatedEffectiveActiveDataLayerNames; }
	const TSet<FName>& GetReplicatedEffectiveLoadedDataLayerNames() const { return ReplicatedEffectiveLoadedDataLayerNames; }
	friend class AWorldDataLayers;

private:
	TSet<FName> ReplicatedEffectiveActiveDataLayerNames;
	TSet<FName> ReplicatedEffectiveLoadedDataLayerNames;
	TSet<FName> LocalEffectiveActiveDataLayerNames;
	TSet<FName> LocalEffectiveLoadedDataLayerNames;

	int32 UpdateEpoch;
	mutable int32 AllEffectiveActiveDataLayerNamesEpoch;
	mutable int32 AllEffectiveLoadedDataLayerNamesEpoch;
	mutable TSet<FName> AllEffectiveActiveDataLayerNames;
	mutable TSet<FName> AllEffectiveLoadedDataLayerNames;
};

/**
 * Actor containing data layers instances within a world.
 */
UCLASS(hidecategories = (Actor, HLOD, Cooking, Transform, Advanced, Display, Events, Object, Physics, Attachment, Info, Input, Blueprint, Layers, Tags, Replication), notplaceable, MinimalAPI)
class AWorldDataLayers : public AInfo, public IDataLayerInstanceProvider
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin AActor Interface
#if WITH_EDITOR
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	virtual bool ShouldLevelKeepRefIfExternal() const override;
	virtual bool IsEditorOnly() const override;
	virtual bool ShouldImport(FStringView ActorPropString, bool IsMovingLevel) override { return false; }
	virtual bool IsLockLocation() const { return true; }
	virtual bool IsUserManaged() const override { return false; }
	virtual bool ActorTypeSupportsDataLayer() const override { return false; }
	virtual bool ActorTypeSupportsExternalDataLayer() const override { return false; }
	ENGINE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	virtual void OnLoadedActorRemovedFromLevel() override;
	//~ End AActor Interface

	static ENGINE_API AWorldDataLayers* Create(UWorld* World, FName InWorldDataLayerName = NAME_None);
	static ENGINE_API AWorldDataLayers* Create(const FActorSpawnParameters& SpawnParameters);

	ENGINE_API bool IsEmpty() const;

	template<class DataLayerInstanceType, typename ...CreationsArgs>
	DataLayerInstanceType* CreateDataLayer(CreationsArgs... InCreationArgs);

	ENGINE_API bool RemoveDataLayer(const UDataLayerInstance* InDataLayer, bool bInResolveActorDescContainers = true);
	ENGINE_API int32 RemoveDataLayers(const TArray<UDataLayerInstance*>& InDataLayerInstances, bool bInResolveActorDescContainers = true);
	ENGINE_API void SetAllowRuntimeDataLayerEditing(bool bInAllowRuntimeDataLayerEditing);
	bool GetAllowRuntimeDataLayerEditing() const { return bAllowRuntimeDataLayerEditing; }

	ENGINE_API bool IsActorEditorContextCurrentColorized(const UDataLayerInstance* InDataLayerInstance) const;
	ENGINE_API bool IsInActorEditorContext(const UDataLayerInstance* InDataLayerInstance) const;
	ENGINE_API bool AddToActorEditorContext(UDataLayerInstance* InDataLayerInstance);
	ENGINE_API bool RemoveFromActorEditorContext(UDataLayerInstance* InDataLayerInstance);
	ENGINE_API void PushActorEditorContext(int32 InContextID, bool bDuplicateContext);
	ENGINE_API void PopActorEditorContext(int32 InContextID);
	ENGINE_API TArray<UDataLayerInstance*> GetActorEditorContextDataLayers() const;

	//~ Begin Helper Functions
	ENGINE_API TArray<const UDataLayerInstance*> GetDataLayerInstances(const TArray<UDataLayerAsset*>& InDataLayersAssets) const;
	ENGINE_API TArray<const UDataLayerInstance*> GetDataLayerInstances(const TArray<const UDataLayerAsset*>& InDataLayersAssets) const;
	ENGINE_API TArray<FName> GetDataLayerInstanceNames(const TArray<const UDataLayerAsset*>& InDataLayersAssets) const;
	//~ End Helper Functions

	// Allows overriding of DataLayers with PlayFromHere
	template<class T>
	UE_DEPRECATED(5.8, "Use SetDataLayerRuntimeState instead.")
	void OverwriteDataLayerRuntimeStates(const TArray<T>* InActiveDataLayers, const TArray<T>* InLoadedDataLayers);

	ENGINE_API bool IsReadOnly(FText* OutReason = nullptr) const;
	ENGINE_API bool IsSubWorldDataLayers() const;

	ENGINE_API bool SupportsExternalPackageDataLayerInstances() const;
	bool IsUsingExternalPackageDataLayerInstances() const { return bUseExternalPackageDataLayerInstances; }
	ENGINE_API bool SetUseExternalPackageDataLayerInstances(bool bInNewValue, bool bInInteractiveMode = false);


	ENGINE_API bool CanReferenceDataLayerAsset(const UDataLayerAsset* InDataLayerAsset, FText* OutFailureReason) const;
#endif

	static FName GetWorldPartitionWorldDataLayersName() { return FName("WorldDataLayers"); } // reserved for ULevel::WorldDataLayers
	
	ENGINE_API void DumpDataLayers(FOutputDevice& OutputDevice) const;
	ENGINE_API bool ContainsDataLayer(const UDataLayerInstance* InDataLayer) const;
	ENGINE_API const UDataLayerInstance* GetDataLayerInstance(const FName& InDataLayerInstanceName) const;
	ENGINE_API const UDataLayerInstance* GetDataLayerInstance(const UDataLayerAsset* InDataLayerAsset) const;
	ENGINE_API const UDataLayerInstance* GetDataLayerInstanceFromAssetName(const FName& InDataLayerAssetPathName) const;

	ENGINE_API void ForEachDataLayerInstance(TFunctionRef<bool(UDataLayerInstance*)> Func);
	ENGINE_API void ForEachDataLayerInstance(TFunctionRef<bool(UDataLayerInstance*)> Func) const;

	ENGINE_API TArray<const UDataLayerInstance*> GetDataLayerInstances(const TArray<FName>& InDataLayerInstanceNames) const;
	ENGINE_API const UExternalDataLayerInstance* GetExternalDataLayerInstance(const UExternalDataLayerAsset* InExternalDataLayerAsset) const;
	ENGINE_API bool IsExternalDataLayerWorldDataLayers() const;

	// DataLayer Runtime State
	ENGINE_API bool SetDataLayerRuntimeState(const UDataLayerInstance* InDataLayerInstance, EDataLayerRuntimeState InState, bool bIsRecursive = false);
	ENGINE_API EDataLayerRuntimeState GetDataLayerRuntimeStateByName(FName InDataLayerName) const;
	ENGINE_API EDataLayerRuntimeState GetDataLayerEffectiveRuntimeStateByName(FName InDataLAyerName) const;
	const TSet<FName>& GetEffectiveActiveDataLayerNames() const;
	const TSet<FName>& GetEffectiveLoadedDataLayerNames() const;
	UFUNCTION(NetMulticast, Reliable)
	ENGINE_API void OnDataLayerRuntimeStateChanged(const UDataLayerInstance* InDataLayer, EDataLayerRuntimeState InState);
	int32 GetDataLayersStateEpoch() const { return DataLayersStateEpoch; }

protected:
	ENGINE_API void InitializeDataLayerRuntimeStates();
	ENGINE_API void ResetDataLayerRuntimeStates(bool bFlushNet = true);

	UFUNCTION()
	ENGINE_API void OnRep_ActiveDataLayerNames();

	UFUNCTION()
	ENGINE_API void OnRep_LoadedDataLayerNames();

	UFUNCTION()
	ENGINE_API void OnRep_EffectiveActiveDataLayerNames();

	UFUNCTION()
	ENGINE_API void OnRep_EffectiveLoadedDataLayerNames();

private:
	enum class ESetDataLayerRuntimeStateError
	{
		NotRuntime,
		ClientOnlyFromServer,
		ServerOnlyFromClient,
		AuthoritativeFromClient
	};

	ENGINE_API const FWorldDataLayersEffectiveStates& GetEffectiveStates() const;

	// External Data Layers
	bool AddExternalDataLayerInstance(UExternalDataLayerInstance* ExternalDataLayerInstance);
	bool RemoveExternalDataLayerInstance(UExternalDataLayerInstance* ExternalDataLayerInstance);
	ENGINE_API UExternalDataLayerInstance* GetExternalDataLayerInstance(const UExternalDataLayerAsset* InExternalDataLayerAsset);

#if WITH_EDITOR
	// ExternalPackage Data Layer Instances
	ENGINE_API void InitializeExternalPackageDataLayerInstances();
#endif

	ENGINE_API void OnDataLayerManagerInitialized();
	ENGINE_API void OnDataLayerManagerDeinitialized();
	ENGINE_API void ResolveEffectiveRuntimeState(const UDataLayerInstance* InDataLayer, bool bInNotifyChange = true);
	ENGINE_API void DumpDataLayerRecursively(const UDataLayerInstance* DataLayer, FString Prefix, FOutputDevice& OutputDevice) const;
	bool CanChangeDataLayerRuntimeState(const UDataLayerInstance* InDataLayerInstance, ESetDataLayerRuntimeStateError* OutReason = nullptr) const;

	//~ Begin IDataLayerInstanceProvider interface
	ENGINE_API virtual TSet<TObjectPtr<UDataLayerInstance>>& GetDataLayerInstances() override;
	virtual const TSet<TObjectPtr<UDataLayerInstance>>& GetDataLayerInstances() const override { return const_cast<AWorldDataLayers*>(this)->GetDataLayerInstances(); }
	virtual const UExternalDataLayerInstance* GetRootExternalDataLayerInstance() const override { return RootExternalDataLayerInstance; }
	//~ End IDataLayerInstanceProvider interface

#if !WITH_EDITOR
	void UpdateAccelerationTable(const UDataLayerInstance* DataLayerInstance, bool bIsAdding);
#endif

#if WITH_EDITOR
	virtual bool ActorTypeIsMainWorldOnly() const override { return true; }

	ENGINE_API void AddDataLayerInstance(UDataLayerInstance* InDataLayerInstance);
	ENGINE_API void ResolveActorDescContainers();
	ENGINE_API void RemoveEditorDataLayers();
	void UpdateCurrentColorizedDataLayerInstance();

	friend class UDataLayerInstanceWithAsset;
	friend class UDataLayerToAssetCommandlet;

	// Used to compare state pre/post undo
	TSet<TObjectPtr<UDataLayerInstance>> CachedDataLayerInstances;
#endif

#if !WITH_EDITOR
	TMap<FName, const UDataLayerInstance*> InstanceNameToInstance;
	TMap<FString, const UDataLayerInstance*> AssetNameToInstance;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bUseExternalPackageDataLayerInstances;

	// True when Runtime Data Layer editing is allowed.
	UPROPERTY()
	bool bAllowRuntimeDataLayerEditing;

	UPROPERTY(Transient)
	FActorPlacementDataLayers CurrentDataLayers;

	TArray<FActorPlacementDataLayers> CurrentDataLayersStack;
#endif

	UPROPERTY()
	TObjectPtr<UExternalDataLayerInstance> RootExternalDataLayerInstance;

	UPROPERTY()
	TSet<TObjectPtr<UDataLayerInstance>> DataLayerInstances;

	/** Data layer instances stored in their external package (only used when UseExternalPackageDataLayerInstances is True) */
	UPROPERTY(Transient)
	TSet<TObjectPtr<UDataLayerInstance>> ExternalPackageDataLayerInstances;

	/** Temporary array containing data layer instances manually loaded from their external packages */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UDataLayerInstance>> LoadedExternalPackageDataLayerInstances;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UDataLayerInstance>> TransientDataLayerInstances;

	UE_DEPRECATED(5.8, "DeprecatedDataLayerNameToDataLayerInstance Property is no longer used.")
	UPROPERTY()
	TMap<FName, TWeakObjectPtr<UDataLayerInstance>> DeprecatedDataLayerNameToDataLayerInstance;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(all, "Convert Data layers to DataLayerInstances and DataLayerAssets using DataLayerToAsset CommandLet and use DataLayerInstances instead.")
	UPROPERTY()
	TSet<TObjectPtr<class UDEPRECATED_DataLayer>> WorldDataLayers_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY(Transient, Replicated, ReplicatedUsing=OnRep_ActiveDataLayerNames)
	TArray<FName> RepActiveDataLayerNames;
		
	UPROPERTY(Transient, Replicated, ReplicatedUsing=OnRep_LoadedDataLayerNames)
	TArray<FName> RepLoadedDataLayerNames;

	// TSet do not support replication so we replicate an array and update the set in the OnRep_ActiveDataLayerNames/OnRep_LoadedDataLayerNames
	TSet<FName> ActiveDataLayerNames;
	TSet<FName> LoadedDataLayerNames;

	TSet<FName> LocalActiveDataLayerNames;
	TSet<FName> LocalLoadedDataLayerNames;

	UPROPERTY(Transient, Replicated, ReplicatedUsing=OnRep_EffectiveActiveDataLayerNames)
	TArray<FName> RepEffectiveActiveDataLayerNames;
		
	UPROPERTY(Transient, Replicated, ReplicatedUsing=OnRep_EffectiveLoadedDataLayerNames)
	TArray<FName> RepEffectiveLoadedDataLayerNames;

	FWorldDataLayersEffectiveStates EffectiveStates;

	int32 DataLayersStateEpoch;

	friend class UWorldPartition;
	friend class UDataLayerManager;
	friend class UExternalDataLayerManager;
	friend class UDataLayerEditorSubsystem;
	friend struct FWorldDataLayersEffectiveStatesAccessor;
};

struct FWorldDataLayersEffectiveStatesAccessor
{
private:
	static const FWorldDataLayersEffectiveStates& Get(const AWorldDataLayers* InWorldDataLayers)
	{
		check(InWorldDataLayers);
		return InWorldDataLayers->GetEffectiveStates();
	}
	friend struct FWorldPartitionStreamingContext;
	friend class UWorldPartitionStreamingPolicy;
	friend class UDataLayerManager;
};


DEFINE_ACTORDESC_TYPE(AWorldDataLayers, FWorldDataLayersActorDesc);

#if WITH_EDITOR

template <typename T, typename = int>
struct HasDataLayerInstanceClass : std::false_type {};

template <typename T>
struct HasDataLayerInstanceClass<T, decltype(&T::GetDataLayerInstanceClass, 0)> : std::true_type {};

template<class DataLayerInstanceType, typename ...CreationsArgs>
DataLayerInstanceType* AWorldDataLayers::CreateDataLayer(CreationsArgs... InCreationArgs)
{
	UClass* ClassToUse = DataLayerInstanceType::StaticClass();
	if constexpr (HasDataLayerInstanceClass<DataLayerInstanceType>::value)
		ClassToUse = DataLayerInstanceType::GetDataLayerInstanceClass();
	UPackage* ExternalPackage = nullptr;
	UObject* OuterObject = this;
	FName NewObjectName;

	if (IsUsingExternalPackageDataLayerInstances())
	{
		check(SupportsExternalPackageDataLayerInstances());
		const ULevel* Level = GetLevel();
		const bool bIsTransient = (Level->IsInstancedLevel() && !Level->IsPersistentLevel()) || Level->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor);
		check(!bIsTransient);

		// We generate a globally unique name to avoid any potential clash of 2 users creating the same object
		const FGuid NewObjectGuid = FGuid::NewGuid();
		FString ShortName = DataLayerInstanceType::StaticClass()->GetName() + TEXT("_UID_") + NewObjectGuid.ToString(EGuidFormats::UniqueObjectGuid);
		TStringBuilderWithBuffer<TCHAR, NAME_SIZE> GloballyUniqueObjectPath;
		GloballyUniqueObjectPath += OuterObject->GetPathName();
		GloballyUniqueObjectPath += TEXT(".");
		GloballyUniqueObjectPath += ShortName;

		ExternalPackage = FExternalPackageHelper::CreateExternalPackage(OuterObject, *GloballyUniqueObjectPath, FExternalPackageHelper::GetDefaultExternalPackageFlags(), GetRootExternalDataLayerAsset());
		check(ExternalPackage);
		NewObjectName = *ShortName;
	}
	else
	{
		NewObjectName = DataLayerInstanceType::MakeName(Forward<CreationsArgs>(InCreationArgs)...);
	}

	DataLayerInstanceType* NewDataLayer = NewObject<DataLayerInstanceType>(OuterObject, ClassToUse, NewObjectName, RF_Transactional, nullptr, /*bCopyTransientsFromClassDefaults*/false, /*InstanceGraph*/nullptr, ExternalPackage);
	check(NewDataLayer != nullptr);
	NewDataLayer->OnCreated(Forward<CreationsArgs>(InCreationArgs)...);
	AddDataLayerInstance(NewDataLayer);

	ResolveActorDescContainers();
	return NewDataLayer;
}

template<class IdentifierType>
void AWorldDataLayers::OverwriteDataLayerRuntimeStates(const TArray<IdentifierType>* InActiveDataLayers, const TArray<IdentifierType>* InLoadedDataLayers)
{
	if (GetLocalRole() == ROLE_Authority)
	{
		// Handle Wrapper types
		auto IdentifierToInstance = [this](const IdentifierType& Identifier) -> const UDataLayerInstance*
		{
			if constexpr (std::is_same_v<IdentifierType, TSoftObjectPtr<UDataLayerAsset>>)
			{
				return GetDataLayerInstance(Identifier.Get());
			}
			else
			{
				return GetDataLayerInstance(Identifier);
			}
		};

		FlushNetDormancy();

		// This should get called before game starts. It doesn't send out events
		check(!GetWorld()->bMatchStarted);

		if (InActiveDataLayers)
		{
			ActiveDataLayerNames.Empty(InActiveDataLayers->Num());
			for (const IdentifierType& DataLayerIdentitier : *InActiveDataLayers)
			{
				const UDataLayerInstance* DataLayerInstance = IdentifierToInstance(DataLayerIdentitier);
				if (DataLayerInstance && DataLayerInstance->IsRuntime())
				{
					ActiveDataLayerNames.Add(DataLayerInstance->GetDataLayerFName());
				}
			}
			MARK_PROPERTY_DIRTY_FROM_NAME(AWorldDataLayers, RepActiveDataLayerNames, this);
			RepActiveDataLayerNames = ActiveDataLayerNames.Array();
		}
		

		if(InLoadedDataLayers)
		{
			LoadedDataLayerNames.Empty(InLoadedDataLayers->Num());
			for (const IdentifierType& DataLayerIdentitier : *InLoadedDataLayers)
			{
				const UDataLayerInstance* DataLayerInstance = IdentifierToInstance(DataLayerIdentitier);
				if (DataLayerInstance && DataLayerInstance->IsRuntime())
				{
					LoadedDataLayerNames.Add(DataLayerInstance->GetDataLayerFName());
				}
			}
			MARK_PROPERTY_DIRTY_FROM_NAME(AWorldDataLayers, RepLoadedDataLayerNames, this);
			RepLoadedDataLayerNames = LoadedDataLayerNames.Array();
		}

		ForEachDataLayerInstance([this](class UDataLayerInstance* DataLayerInstance)
		{
			if (DataLayerInstance->IsRuntime())
			{
				ResolveEffectiveRuntimeState(DataLayerInstance, /*bNotifyChange*/false);
			}
			return true;
		});

		MARK_PROPERTY_DIRTY_FROM_NAME(AWorldDataLayers, RepEffectiveActiveDataLayerNames, this);
		RepEffectiveActiveDataLayerNames = EffectiveStates.GetReplicatedEffectiveActiveDataLayerNames().Array();
		MARK_PROPERTY_DIRTY_FROM_NAME(AWorldDataLayers, RepEffectiveLoadedDataLayerNames, this);
		RepEffectiveLoadedDataLayerNames = EffectiveStates.GetReplicatedEffectiveLoadedDataLayerNames().Array();
	}
}

#endif // WITH_EDITOR
