// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/DataLayer/DataLayerType.h"
#include "Misc/Optional.h"

enum class EDataLayerRuntimeState : uint8;

class FDataLayerInstanceDesc
{
public:
	ENGINE_API FDataLayerInstanceDesc();
	ENGINE_API void Init(class UDataLayerInstance* DataLayerInstance);
	friend FArchive& operator<<(FArchive& Ar, FDataLayerInstanceDesc& DataLayerInstanceDesc);
	friend bool operator == (const FDataLayerInstanceDesc& Lhs, const FDataLayerInstanceDesc& Rhs);
	friend bool operator < (const FDataLayerInstanceDesc& Lhs, const FDataLayerInstanceDesc& Rhs);

	FName GetName() const { return Name; }
	FName GetParentName() const { return ParentName; }
	bool IsUsingAsset() const { return bIsUsingAsset; }
	FName GetAssetPath() const { return AssetPath; }
	ENGINE_API EDataLayerType GetDataLayerType() const;
	ENGINE_API FString GetShortName() const;
	bool IsIncludedInActorFilterDefault() const { return bIsIncludedInActorFilterDefault; }
	ENGINE_API EDataLayerRuntimeState GetInitialRuntimeState() const;
	ENGINE_API bool SupportsActorFilters() const;

private:
	ENGINE_API class UDataLayerAsset* GetAsset() const;

	// DataLayerInstance Name
	FName Name;
	// Parent DataLayerInstance Name
	FName ParentName;
	// We can't rely on a valid AssetPath to determine if DataLayerInstance is UDataLayerInstanceWithAsset/UDataLayerInstancePrivate
	bool bIsUsingAsset;

	//~ Begin UDataLayerInstance
	// DataLayer Asset Path 
	FName AssetPath;
	// Returns if data layer should be included by default in FWorldPartitionActorFilter.
	bool bIsIncludedInActorFilterDefault;
	// DataLayer Asset is Private
	bool bIsPrivate;
	// Initial runtime state of this data layer instance.
	EDataLayerRuntimeState InitialRuntimeState;
	//~ End UDataLayerInstance

	// If DataLayer Asset is Private store the SupportsActorFilter() flag
	bool bPrivateDataLayerSupportsActorFilter;
	// If DataLayer Asset is Private store the ShortName (also used for UDeprecatedDataLayerInstance)
	FString PrivateShortName;

	//~ Begin UDeprecatedDataLayerInstance
	bool bDeprecatedIsRuntime;
	//~ End UDeprecatedDataLayerInstance

	friend class UDataLayerInstance;
};

/**
 * ActorDesc for AWorldDataLayers actors.
 */
class FWorldDataLayersActorDesc : public FWorldPartitionActorDesc
{
public:
	ENGINE_API FWorldDataLayersActorDesc();
	bool IsValid() const { return bIsValid; }
	ENGINE_API const TArray<FDataLayerInstanceDesc>& GetDataLayerInstances() const;
	ENGINE_API const FDataLayerInstanceDesc* GetDataLayerInstanceFromInstanceName(FName InDataLayerInstanceName) const;
	ENGINE_API const FDataLayerInstanceDesc* GetDataLayerInstanceFromAssetPath(FName InDataLayerAssetPath) const;

protected:
	//~ Begin FWorldPartitionActorDesc Interface.
	ENGINE_API virtual void Init(const AActor* InActor) override;
	ENGINE_API virtual bool Equals(const FWorldPartitionActorDesc* Other) const override;
	virtual uint32 GetSizeOf() const override { return sizeof(FWorldDataLayersActorDesc); }
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	virtual bool IsResaveNeeded() const override { return !IsValid(); }
	virtual bool IsRuntimeRelevant(const FWorldPartitionActorDescInstance* InActorDescInstance) const override;
	//~ End FWorldPartitionActorDesc Interface.

private:

	void ForEachDataLayerInstanceDesc(TFunctionRef<bool(const FDataLayerInstanceDesc&)> Func) const;
	const TArray<FDataLayerInstanceDesc>& GetExternalPackageDataLayerInstances() const;

	TArray<FDataLayerInstanceDesc> DataLayerInstances;
	mutable TOptional<TArray<FDataLayerInstanceDesc>> ExternalPackageDataLayerInstances;
	bool bIsValid;
	bool bIsExternalDataLayerWorldDataLayers;
	bool bUseExternalPackageDataLayerInstances;
};
#endif
