// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionActorDescInstanceViewInterface.h"

class AWorldDataLayers;
class UDataLayerAsset;
class UDataLayerInstance;
class UDataLayerInstanceWithAsset;
class UHLODLayer;
struct FActorDescViewMutator;

// deprecated
class FWorldPartitionActorDescView;

class IStreamingGenerationErrorHandler
{
public:
	virtual ~IStreamingGenerationErrorHandler() = default;

	/**
	 * Called when an actor has an invalid runtime grid.
	 */
	virtual void OnInvalidRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, FName GridName) = 0;

	/**
	 * Called when an actor references an invalid actor.
	 */
	virtual void OnInvalidReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const FGuid& ReferenceGuid, IWorldPartitionActorDescInstanceView* ReferenceActorDescView) = 0;

	/**
	 * Called when an actor references an actor using a different grid placement.
	 */
	virtual void OnInvalidReferenceGridPlacement(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView) = 0;

	/**
	 * Called when there's an error with the data layers used by an actor and one or more referenced actor.
	 */
	enum class EDataLayerInvalidReason
	{
		ReferencedActorDifferentRuntimeDataLayers,
		ReferencedActorDifferentExternalDataLayer
	};

	virtual void OnInvalidReferenceDataLayers(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView, EDataLayerInvalidReason Reason) = 0;

	/**
	 * Called when an actor references an actor using a different RuntimeGrid.
	 */
	virtual void OnInvalidReferenceRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView) = 0;

	/**
	 * Called when an actor contains runtime data layers with different types of LoadFilter.
	 */
	virtual void OnDataLayersLoadFilterMismatch(const IWorldPartitionActorDescInstanceView& ActorDescView) = 0;

	/**
	 * Called when the world references a streamed actor.
	 */
	enum class EWorldReferenceInvalidReason
	{
		ReferencedActorIsSpatiallyLoaded,
		ReferencedActorHasDataLayers
	};

	virtual void OnInvalidWorldReference(const IWorldPartitionActorDescInstanceView& ActorDescView, EWorldReferenceInvalidReason Reason) = 0;

	/**
	 * Called when a data layer instance does not have a data layer asset
	 */
	virtual void OnInvalidReferenceDataLayerAsset(const UDataLayerInstanceWithAsset* DataLayerInstance) = 0;

	/**
	 * Used to identify a data layer hierarchy type mismatch error
	 */
	enum class EDataLayerHierarchyInvalidReason
	{
		ClientOnlyDataLayerCantBeChild,
		ServerOnlyDataLayerCantBeChild,
		IncompatibleDataLayerType
	};

	/**
	 * Called when a data layer is not of the same type as its parent
	 */
	virtual void OnDataLayerHierarchyTypeMismatch(const UDataLayerInstance* DataLayerInstance, const UDataLayerInstance* Parent, EDataLayerHierarchyInvalidReason Reason) = 0;

	/**
 	 * Called when there's an error with a data layer used by a WorldDataLayers actor
 	 */
	virtual void OnInvalidWorldDataLayersReference(const AWorldDataLayers* WorldDataLayers, const UDataLayerInstance* DataLayerInstance, const FText& Reason) = 0;

	/**
	 * Called when two data layer instances share the same asset
	 */
	virtual void OnDataLayerAssetConflict(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerInstanceWithAsset* ConflictingDataLayerInstance) = 0;

	/**
	 * Called when the data layer asset is not compatible with its data layer asset.
	 */
	virtual void OnInvalidDataLayerAssetType(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerAsset* DataLayerAsset) = 0;

	/**
	 * Called when an actor needs to be resaved.
	 */
	virtual void OnActorNeedsResave(const IWorldPartitionActorDescInstanceView& ActorDescView) = 0;

	/**
	 * Used to identify a level instance actor error
	 */
	enum class ELevelInstanceInvalidReason
	{
		WorldAssetNotFound,
		WorldAssetDontContainActorsMetadata,
		WorldAssetIncompatiblePartitioned,
		WorldAssetHasInvalidContainer,
		CirculalReference
	};

	virtual void OnLevelInstanceInvalidWorldAsset(const IWorldPartitionActorDescInstanceView& ActorDescView, FName WorldAsset, ELevelInstanceInvalidReason Reason) = 0;

	/**
	 * Called when an actor references another actor with a different set of actor filter.
	 */
	virtual void OnInvalidActorFilterReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView) = 0;

	/**
	 * Called when an actor has an invalid HLOD layer.
	 */
	virtual void OnInvalidHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView) = 0;

	enum class EHLODLayerUnsupportedReason
	{
		ActorClassUnsupportedByLayerType
	};

	/**
	 * Called when an actor has a HLOD layer that doesn't support that actor.
	 */
	virtual void OnUnsupportedHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView, const UHLODLayer* HLODLayer, EHLODLayerUnsupportedReason Reason) = 0;

	/**
	 * Reason an actor descriptor mutator override was rejected.
	 */
	enum class EActorDescViewMutatorRejectedReason
	{
		DivergesClusterRuntimeGrid,
		DivergesClusterIsSpatiallyLoaded
	};

	/**
	 * Called when a mutator override was rejected because applying it would have split reference-connected actors.
	 */
	virtual void OnRejectedActorDescViewMutator(const IWorldPartitionActorDescInstanceView& ActorDescView, const FActorDescViewMutator& RejectedOverrides, EActorDescViewMutatorRejectedReason Reason) = 0;

	// Helpers
	static ENGINE_API FString GetActorName(const IWorldPartitionActorDescInstanceView& ActorDescView);
	static ENGINE_API FString GetFullActorName(const IWorldPartitionActorDescInstanceView& ActorDescView);
};
#endif
