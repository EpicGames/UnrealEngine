// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationErrorHandler.h"

class FStreamingGenerationLogErrorHandler : public IStreamingGenerationErrorHandler
{
public:
	ENGINE_API virtual void OnInvalidRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, FName GridName) override;
	ENGINE_API virtual void OnInvalidReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const FGuid& ReferenceGuid, IWorldPartitionActorDescInstanceView* ReferenceActorDescView) override;
	ENGINE_API virtual void OnInvalidReferenceGridPlacement(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView) override;
	ENGINE_API virtual void OnInvalidReferenceDataLayers(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView, EDataLayerInvalidReason Reason) override;
	ENGINE_API virtual void OnInvalidWorldReference(const IWorldPartitionActorDescInstanceView& ActorDescView, EWorldReferenceInvalidReason Reason) override;
	ENGINE_API virtual void OnInvalidReferenceRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView) override;
	ENGINE_API virtual void OnDataLayersLoadFilterMismatch(const IWorldPartitionActorDescInstanceView& ActorDescView) override;
	ENGINE_API virtual void OnInvalidReferenceDataLayerAsset(const UDataLayerInstanceWithAsset* DataLayerInstance) override;
	ENGINE_API virtual void OnInvalidDataLayerAssetType(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerAsset* DataLayerAsset) override;
	ENGINE_API virtual void OnDataLayerHierarchyTypeMismatch(const UDataLayerInstance* DataLayerInstance, const UDataLayerInstance* Parent, EDataLayerHierarchyInvalidReason Reason) override;
	ENGINE_API virtual void OnInvalidWorldDataLayersReference(const AWorldDataLayers* WorldDataLayers, const UDataLayerInstance* DataLayerInstance, const FText& Reason) override;
	ENGINE_API virtual void OnDataLayerAssetConflict(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerInstanceWithAsset* ConflictingDataLayerInstance) override;
	ENGINE_API virtual void OnActorNeedsResave(const IWorldPartitionActorDescInstanceView& ActorDescView) override;
	ENGINE_API virtual void OnLevelInstanceInvalidWorldAsset(const IWorldPartitionActorDescInstanceView& ActorDescView, FName WorldAsset, ELevelInstanceInvalidReason Reason) override;
	ENGINE_API virtual void OnInvalidActorFilterReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView) override;
	ENGINE_API virtual void OnInvalidHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView) override;
	ENGINE_API virtual void OnUnsupportedHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView, const UHLODLayer* HLODLayer, EHLODLayerUnsupportedReason Reason) override;
	ENGINE_API virtual void OnRejectedActorDescViewMutator(const IWorldPartitionActorDescInstanceView& ActorDescView, const FActorDescViewMutator& RejectedOverrides, EActorDescViewMutatorRejectedReason Reason) override;
};
#endif
