// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "UObject/SoftObjectPath.h"
#include "WorldPartitionActorFilter.generated.h"

enum class EWorldPartitionActorFilterType : uint8
{
	None = 0,
	Loading = 1 << 1,
	Packing = 1 << 0,
	All = Packing | Loading,
};
ENUM_CLASS_FLAGS(EWorldPartitionActorFilterType);

USTRUCT()
struct FWorldPartitionActorFilter
{
	GENERATED_USTRUCT_BODY()

	FWorldPartitionActorFilter() {}
	ENGINE_API ~FWorldPartitionActorFilter();

	ENGINE_API FWorldPartitionActorFilter(const FWorldPartitionActorFilter& Other);
	ENGINE_API FWorldPartitionActorFilter(FWorldPartitionActorFilter&& Other);

	ENGINE_API FWorldPartitionActorFilter& operator=(const FWorldPartitionActorFilter& Other);
	ENGINE_API FWorldPartitionActorFilter& operator=(FWorldPartitionActorFilter&& Other);

#if WITH_EDITORONLY_DATA
	FWorldPartitionActorFilter(const FString& InDisplayName) : DisplayName(InDisplayName) {}
	
	ENGINE_API void AddChildFilter(const FGuid& InGuid, FWorldPartitionActorFilter* InChildFilter);
	ENGINE_API void RemoveChildFilter(const FGuid& InGuid);
	ENGINE_API void ClearChildFilters();
	
	ENGINE_API void Override(const FWorldPartitionActorFilter& Other);

	const TMap<FGuid, FWorldPartitionActorFilter*>& GetChildFilters() const { return ChildFilters; }
	FWorldPartitionActorFilter* GetParentFilter() const { return Parent; }

	struct FDataLayerFilter
	{
		FDataLayerFilter() {}

		FDataLayerFilter(const FString& InDisplayName, bool bInIncluded)
			: bIncluded(bInIncluded)
			, DisplayName(InDisplayName) {}

		// True if DataLayer actors should be included
		bool bIncluded;

		// Transient
		FString DisplayName;

	};

	// Transient
	FString DisplayName;
	// List of DataLayer Assets to Include or Exclude, missing DataLayer Assets will use default behavior
	TMap<FSoftObjectPath, FDataLayerFilter> DataLayerFilters;

	friend FArchive& operator<<(FArchive& Ar, FWorldPartitionActorFilter& Filter);
	ENGINE_API bool Serialize(FArchive& Ar);

	// Operators.
	ENGINE_API bool operator==(const FWorldPartitionActorFilter& Other) const;

	bool operator!=(const FWorldPartitionActorFilter& Other) const
	{
		return !(*this == Other);
	}

	// Needed for Copy/Paste/ResetToDefault
	ENGINE_API bool ExportTextItem(FString& ValueStr, FWorldPartitionActorFilter const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	ENGINE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);
	ENGINE_API FString ToString() const;

	static ENGINE_API void RequestFilterRefresh(bool bIsFromUserChange);

	DECLARE_MULTICAST_DELEGATE(FOnWorldPartitionActorFilterChanged);
	static FOnWorldPartitionActorFilterChanged& GetOnWorldPartitionActorFilterChanged() { return OnWorldPartitionActorFilterChanged; }
private:
	// Transient
	FWorldPartitionActorFilter* Parent = nullptr;
	// Map of FWorldPartitionActorFilters per Child Level Instance, recursive
	TMap<FGuid, FWorldPartitionActorFilter*> ChildFilters;
	// Static Event for when some filter changes
	static ENGINE_API FOnWorldPartitionActorFilterChanged OnWorldPartitionActorFilterChanged;
#endif
};

#if WITH_EDITORONLY_DATA
template<> struct TStructOpsTypeTraits<FWorldPartitionActorFilter> : public TStructOpsTypeTraitsBase2<FWorldPartitionActorFilter>
{
	enum
	{
		WithSerializer = true,
		WithIdenticalViaEquality = true,
		WithImportTextItem = true,
		WithExportTextItem = true
	};
};
#endif

