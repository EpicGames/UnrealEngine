// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODObject.h"

#include "CustomHLODActor.generated.h"

UCLASS(MinimalAPI)
class AWorldPartitionCustomHLOD : public AActor, public IWorldPartitionHLODObject
{
	GENERATED_BODY()

public:
	ENGINE_API AWorldPartitionCustomHLOD(const FObjectInitializer& ObjectInitializer);

	// ~Begin IWorldPartitionHLODObject interface
	virtual UObject* GetUObject() const override;
	ENGINE_API virtual ULevel* GetHLODLevel() const override;
	ENGINE_API virtual FString GetHLODNameOrLabel() const override;
	virtual bool DoesRequireWarmup() const override;
	ENGINE_API virtual TSet<UObject*> GetAssetsToWarmup() const override;
	ENGINE_API virtual void SetVisibility(bool bIsVisible) override;
	ENGINE_API virtual const FGuid& GetSourceCellGuid() const override;
	ENGINE_API virtual bool IsStandalone() const override;
	ENGINE_API virtual const FGuid& GetStandaloneHLODGuid() const override;
	ENGINE_API virtual bool IsCustomHLOD() const override;
	ENGINE_API virtual const FGuid& GetCustomHLODGuid() const override;
	// ~End IWorldPartitionHLODObject interface

	// ~Begin AActor Interface
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	// ~End AActor Interface

protected:
#if WITH_EDITOR
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif
private:
	UPROPERTY(Category=CustomHLOD, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Mesh,Rendering,Physics,Components|StaticMesh", AllowPrivateAccess = "true"))
	TObjectPtr<class UStaticMeshComponent> StaticMeshComponent;

	UPROPERTY()
	FGuid HLODInstanceGuid;
};