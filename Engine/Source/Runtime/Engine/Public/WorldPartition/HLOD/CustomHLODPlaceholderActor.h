// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

UE_DEPRECATED_HEADER(5.8, "Use WorldPartition/HLOD/HLODSourceCellPlaceholderActor.h instead of WorldPartition/HLOD/CustomHLODPlaceholderActor.h")
#include "WorldPartition/HLOD/HLODSourceCellPlaceholderActor.h"

#if WITH_EDITOR
UE_DEPRECATED(5.8, "AWorldPartitionCustomHLODPlaceholder has been renamed to AWorldPartitionHLODSourceCellPlaceholder")
typedef AWorldPartitionHLODSourceCellPlaceholder AWorldPartitionCustomHLODPlaceholder;
#endif
