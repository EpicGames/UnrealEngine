// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Components/SceneComponent.h"
#include "Net/Serialization/FastArraySerializer.h"
#include "DestructibleHLODComponent.generated.h"


class UMaterialInterface;
class UMaterialInstanceDynamic;
class UTexture2DDynamic;
class FTexture2DDynamicResource;
class UHLODInstancedStaticMeshComponent;

// Entry for a damaged actor
USTRUCT()
struct FWorldPartitionDestructibleHLODDamagedActorState : public FFastArraySerializerItem
{
	GENERATED_USTRUCT_BODY()

public:
	static const uint8 MAX_HEALTH = 0xFF;

	FWorldPartitionDestructibleHLODDamagedActorState()
		: ActorIndex(INDEX_NONE)
		, ActorHealth(MAX_HEALTH)
	{
	}

	FWorldPartitionDestructibleHLODDamagedActorState(int32 InActorIndex)
		: ActorIndex(InActorIndex)
		, ActorHealth(MAX_HEALTH)
	{
	}

	bool operator == (const FWorldPartitionDestructibleHLODDamagedActorState& Other) const
	{
		return ActorIndex == Other.ActorIndex && ActorHealth == Other.ActorHealth;
	}

	UPROPERTY()
	int32 ActorIndex;

	UPROPERTY()
	uint8 ActorHealth;
};


// Replicated state of the destructible HLOD
USTRUCT()
struct FWorldPartitionDestructibleHLODState : public FFastArraySerializer
{
	GENERATED_BODY()

public:
	bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms)
	{
		return FFastArraySerializer::FastArrayDeltaSerialize<FWorldPartitionDestructibleHLODDamagedActorState, FWorldPartitionDestructibleHLODState>(DamagedActors, DeltaParms, *this);
	}

	void Initialize(UWorldPartitionDestructibleHLODComponent* InDestructibleHLODComponent);
	void SetActorHealth(int32 InActorIndex, uint8 InActorHealth);

	const bool IsClient() const { return bIsClient; }
	const bool IsServer() const { return bIsServer; }

	// ~ FFastArraySerializer Contract Begin
	void PostReplicatedAdd(const TArrayView<int32>& AddedIndices, int32 FinalSize);
	void PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize);
	// ~ FFastArraySerializer Contract End

private:
	void ApplyDamagedActorState(int32 DamagedActorIndex);

private:
	UPROPERTY()
	TArray<FWorldPartitionDestructibleHLODDamagedActorState> DamagedActors;

	UPROPERTY(NotReplicated)
	TObjectPtr<UWorldPartitionDestructibleHLODComponent> OwnerComponent;

	// Server only, map of actors indices to their damage info in the DamagedActors array
	TArray<int32> ActorsToDamagedActorsMapping;	

	bool bIsServer = false;
	bool bIsClient = false;
	int32 NumDestructibleActors = 0;
};


template<> struct TStructOpsTypeTraits<FWorldPartitionDestructibleHLODState> : public TStructOpsTypeTraitsBase2<FWorldPartitionDestructibleHLODState>
{
	enum { WithNetDeltaSerializer = true };
};


/**
 * For a given source actor, this represent the range of instances that represents it once batched in an "Instancing" HLOD component.
 * For a static mesh component, expect InstanceCount to be 1, whereas an instanced static mesh component will have multiple instances.
 */
USTRUCT()
struct FComponentInstanceMapping
{
	GENERATED_BODY()

	static FComponentInstanceMapping Make(uint32 ComponentIndex, uint32 InstanceStart, uint32 InstanceCount)
	{
		checkf((ComponentIndex & ~kComponentIndexMask) == 0, TEXT("ComponentIndex exceeds %u bits"), kComponentIndexBits);
		checkf((InstanceCount & ~kItemCountMask) == 0, TEXT("InstanceCount exceeds %u bits"), kItemCountBits);
		FComponentInstanceMapping Mapping;
		Mapping.A = ((ComponentIndex & kComponentIndexMask) << kComponentIndexShift) | (InstanceCount & kItemCountMask);
		Mapping.B = InstanceStart; // full 32-bit Start
		return Mapping;
	}

	void Decode(uint32& ComponentIndex, uint32& OutInstanceStart, uint32& OutInstanceCount) const
	{
		ComponentIndex = (A >> kComponentIndexShift) & kComponentIndexMask;
		OutInstanceCount = A & kItemCountMask;
		OutInstanceStart = B;
	}

private:
	UPROPERTY()
	uint32 A = 0; // [ComponentIndex bits | InstanceCount bits]
	
	UPROPERTY()
	uint32 B = 0; // InstanceStart (32-bit)

	static constexpr uint32 kComponentIndexBits = 11;
	static constexpr uint32 kItemCountBits		= 21;
	static_assert(kComponentIndexBits + kItemCountBits == 32, "Bit count must total 32");

	// --- Derived masks/shifts ---
	static constexpr uint32 kComponentIndexShift = kItemCountBits;
	static constexpr uint32 kComponentIndexMask	 = (1u << kComponentIndexBits) - 1u;
	static constexpr uint32 kItemCountMask = (1u << kItemCountBits) - 1u;
};

/**
 * For a given source actor which was potentially comprised of multiple components, this serves as storage to retrieve the associated FComponentInstanceMappings
 * In order to be efficient in the most common scenario where an actor only has one component stored in ISM, this struct is flexible and can fulfill two purpose
 * 1) Standard storage
 *		This struct serves as a mean to retrieve the range of FComponentInstanceMappings associated with this actor from the
 *		FHLODInstancingPackedMappingData::ComponentsMapping array. In this case, IsInline() return false, and GetComponentsMappingRange() should be used
 *		to retreive the range.
 * 2) Inline storage
 *		In the common case where only a single component (either SM or ISM) end up in HLODs, we don't need to store multiple FComponentInstanceMapping values for it.
 *		The added indirection and storage in FHLODInstancingPackedMappingData::ComponentsMapping is inefficient. To avoid this, we use this struct to store the
 *		component mapping directly (inline). In this case, IsInline() return true, and GetInline() should be used to retrieve the (component index + instance start + instance count).
 * You can use FHLODInstancingPackedMappingData::ForEachActorInstancingMapping() that will abstract all of that and will give you all the mappings directly.
 */
USTRUCT()
struct FActorInstanceMappingsRef
{
	GENERATED_BODY()

	static FActorInstanceMappingsRef MakeMappingRange(uint32 RangeOffset, uint32 RangeCount)
	{
		checkf((RangeOffset & ~kRangeOffsetMask) == 0, TEXT("RangeOffset exceeds available storage"));
		FActorInstanceMappingsRef Mapping;
		Mapping.A = (RangeOffset & kRangeOffsetMask);
		Mapping.B = RangeCount;
		return Mapping;
	}

	static FActorInstanceMappingsRef MakeMappingInline(uint32 ComponentIndex, uint32 InstanceStart, uint32 InstanceCount)
	{
		checkf((ComponentIndex & ~kComponentIndexMask) == 0, TEXT("ComponentIndex exceeds %u bits"), kComponentIndexBits);
		checkf((InstanceCount & ~kItemCountMask) == 0, TEXT("InstanceCount exceeds %u bits"), kItemCountBits);
		FActorInstanceMappingsRef Mapping;
		Mapping.A = kInlineTagMask
			| ((ComponentIndex & kComponentIndexMask) << kComponentIndexShift)
			| (InstanceCount & kItemCountMask);
		Mapping.B = InstanceStart;
		return Mapping;
	}

	bool IsInline() const
	{
		return (A & kInlineTagMask) != 0;
	}

	void GetComponentsMappingRange(uint32& OutRangeOffset, uint32& OutRangeCount) const
	{
		check(!IsInline());
		OutRangeOffset = (A & kRangeOffsetMask);
		OutRangeCount = B;
	}

	void GetInline(uint32& OutComponentIndex, uint32& OutInstanceStart, uint32& OutInstanceCount) const
	{
		check(IsInline());
		OutComponentIndex = (A >> kComponentIndexShift) & kComponentIndexMask;
		OutInstanceCount = A & kItemCountMask;
		OutInstanceStart = B;
	}

private:
	UPROPERTY()
	uint32 A = 0; // [inline bit] + (inline ? [kComponentIndexBits + kItemCountBits] : [kOffsetBits]
	
	UPROPERTY()
	uint32 B = 0; // inline ? InstanceStart

	// Storage bits
	static constexpr uint32 kInlineFlagBits		= 1;
	static constexpr uint32 kRangeOffsetBits	= 31;
	static constexpr uint32 kComponentIndexBits = 10;
	static constexpr uint32 kItemCountBits		= 21;
	static_assert(kInlineFlagBits + kComponentIndexBits + kItemCountBits == 32, "Bit count must total 32");
	static_assert(kInlineFlagBits + kRangeOffsetBits == 32, "Bit count must total 32");

	// Masks & shifts
	static constexpr uint32 kInlineTagMask		 = (1u << kRangeOffsetBits);
	static constexpr uint32 kRangeOffsetMask	 = (1u << kRangeOffsetBits) - 1u;
	static constexpr uint32 kComponentIndexShift = kItemCountBits;
	static constexpr uint32 kComponentIndexMask	 = (1u << kComponentIndexBits) - 1u;
	static constexpr uint32 kItemCountMask		 = (1u << kItemCountBits) - 1u;
};


USTRUCT()
struct FHLODInstancingPackedMappingData
{
	GENERATED_BODY()

	// Array of HLOD ISMC. ComponentsMapping entries are indexing into it.
	UPROPERTY()
	TArray<TObjectPtr<UHLODInstancedStaticMeshComponent>> ISMCs;

	// Compacted components mappings for each actors.
	// Entries for a given actor are consecutive. Use PerActorMappingData to index into it.
	UPROPERTY()
	TArray<FComponentInstanceMapping> ComponentsMapping;

	// For a given actor, either provides the range of entries for it in the ComponentsMapping array.
	// OR
	// If there's a single entry, it is found inline in the FActorInstanceMappingsRef struct.
	UPROPERTY()
	TMap<uint32, FActorInstanceMappingsRef> PerActorMappingData;

	// Utility to iterate over all mapping entries for a given actor.
	void ForEachActorInstancingMapping(uint32 InActorIndex, TFunctionRef<void(UHLODInstancedStaticMeshComponent*, uint32, uint32)> InFunc) const
	{
		if (const FActorInstanceMappingsRef* ActorMappingData = PerActorMappingData.Find(InActorIndex))
		{
			if (ActorMappingData->IsInline())
			{
				uint32 ComponentIndex, InstanceStart, InstanceCount;
				ActorMappingData->GetInline(ComponentIndex, InstanceStart, InstanceCount);

				InFunc(ISMCs[ComponentIndex], InstanceStart, InstanceCount);
			}
			else
			{
				uint32 RangeOffset, RangeCount;
				ActorMappingData->GetComponentsMappingRange(RangeOffset, RangeCount);

				for (uint32 i = 0; i < RangeCount; ++i)
				{
					const FComponentInstanceMapping& It = ComponentsMapping[RangeOffset + i];
					uint32 ComponentIndex, InstanceStart, InstanceCount;
					It.Decode(ComponentIndex, InstanceStart, InstanceCount);

					InFunc(ISMCs[ComponentIndex], InstanceStart, InstanceCount);
				}
			}
		}
	}
};


UCLASS(MinimalAPI, HideDropDown, NotPlaceable, HideCategories = (Tags, Sockets, ComponentTick, ComponentReplication, Activation, Cooking, Events, AssetUserData, Collision))
class UWorldPartitionDestructibleHLODComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API const TArray<FName>& GetDestructibleActors() const;
	ENGINE_API void DestroyActor(int32 ActorIndex);
	ENGINE_API void DamageActor(int32 ActorIndex, float DamagePercent);
	ENGINE_API void ApplyDamagedActorState(int32 ActorIndex, const uint8 ActorHealth);
	ENGINE_API void OnDestructionStateUpdated();

#if WITH_EDITOR
	ENGINE_API void SetDestructibleActors(const TArray<FName>& InDestructibleActors);
	ENGINE_API void SetDestructibleHLODMaterial(UMaterialInterface* InDestructibleMaterial);
	ENGINE_API void SetHLODInstancingPackedMappingData(FHLODInstancingPackedMappingData&& InHLODInstancingPackedMappingData);	
#endif
	
private:
	ENGINE_API virtual void BeginPlay() override;

	void SetupVisibilityTexture();
	void UpdateVisibilityTexture();

private:
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DestructibleHLODMaterial;

	UPROPERTY()
	FHLODInstancingPackedMappingData DestructibleHLODInstancesMappingData;

	UPROPERTY(Transient, Replicated)
	FWorldPartitionDestructibleHLODState DestructibleHLODState;

	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> VisibilityMaterial;

	UPROPERTY(Transient)
	TObjectPtr<UTexture2DDynamic> VisibilityTexture;

	uint32 VisibilityTextureSize;

	// Client only, visibility buffer that is meant to be sent to the GPU
	TArray<uint8> VisibilityBuffer;

	// Name of the destructible actors from the source cell.
	UPROPERTY()
	TArray<FName> DestructibleActors;
};

// Deprecated: Subclass
UCLASS(Deprecated, MinimalAPI)
class UDEPRECATED_UWorldPartitionDestructibleHLODMeshComponent : public UWorldPartitionDestructibleHLODComponent
{
	GENERATED_UCLASS_BODY()
};