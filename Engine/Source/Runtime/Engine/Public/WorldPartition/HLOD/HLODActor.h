// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/HLOD/HLODStats.h"
#include "WorldPartition/HLOD/HLODBuilder.h"
#include "WorldPartition/HLOD/HLODRebuildPolicy.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODObject.h"

#if WITH_EDITOR
#include "UObject/ObjectSaveContext.h"
#endif // WITH_EDITOR

#include "HLODActor.generated.h"

class UHLODLayer;
class UWorldPartitionHLODSourceActors;
class UHLODRebuildPolicyData;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogHLODHash, Log, All);

#if WITH_EDITOR
struct FWorldPartitionHLODBuildEvent
{
	enum class EEventType
	{
		BeginBuild,
		EndBuild
	};

	EEventType EventType;
	TObjectPtr<AWorldPartitionHLOD> TargetHLODActor;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionHLODBuildEventDelegate, const FWorldPartitionHLODBuildEvent&)
DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionHLODActorEditorRegisteredDelegate, AWorldPartitionHLOD*)
DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionHLODActorEditorUnregisteredDelegate, AWorldPartitionHLOD*)
#endif

UENUM(BlueprintType)
enum class EExportHLODMeshOrigin : uint8
{
	Actor   UMETA(DisplayName = "Actor Origin"),		// Origin is at actor location
	World   UMETA(DisplayName = "World Origin"),        // Origin is at world <0,0,0>
};

USTRUCT(BlueprintType)
struct FExportHLODAssetsParams
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Export Options")
	EExportHLODMeshOrigin MeshOrigin = EExportHLODMeshOrigin::World;

	UPROPERTY(EditAnywhere, Category = "Export Options", meta=(ContentDir))
	FDirectoryPath ExportRootPath;

	bool bTestExportOnly = false;
#endif
};

UCLASS(NotPlaceable, MinimalAPI, HideCategories=(Rendering, Replication, Collision, Physics, Navigation, Networking, Input, Actor, LevelInstance, Cooking))
class AWorldPartitionHLOD : public AActor, public IWorldPartitionHLODObject
{
	GENERATED_UCLASS_BODY()

	friend class FHLODActorDesc;
	typedef TMap<FName, int64> FStats;

public:
	inline uint32 GetLODLevel() const { return LODLevel; }

	// ~Begin AActor interface
	virtual bool IsHLODRelevant() const override { return true; }
	// ~End AActor interface

	// ~Begin IWorldPartitionHLODObject interface
	virtual UObject* GetUObject() const override { return const_cast<AWorldPartitionHLOD*>(this); }
	ENGINE_API virtual ULevel* GetHLODLevel() const override;
	ENGINE_API virtual FString GetHLODNameOrLabel() const override;
	virtual bool DoesRequireWarmup() const override { return bRequireWarmup; }
	ENGINE_API virtual TSet<UObject*> GetAssetsToWarmup() const override;
	ENGINE_API virtual void SetVisibility(bool bIsVisible) override;
	ENGINE_API virtual const FGuid& GetSourceCellGuid() const override;
	ENGINE_API virtual bool IsStandalone() const override;
	ENGINE_API virtual const FGuid& GetStandaloneHLODGuid() const override;
	ENGINE_API virtual bool IsCustomHLOD() const override;
	ENGINE_API virtual const FGuid& GetCustomHLODGuid() const override;
	// ~End IWorldPartitionHLODObject interface

#if WITH_EDITOR
	ENGINE_API static const FName NAME_HLODHash_AssetTag;

	ENGINE_API void SetHLODComponents(const TArray<UActorComponent*>& InHLODComponents);

	ENGINE_API void SetSourceActors(UWorldPartitionHLODSourceActors* InSourceActors);
	ENGINE_API UWorldPartitionHLODSourceActors* GetSourceActors();
	ENGINE_API const UWorldPartitionHLODSourceActors* GetSourceActors() const;

	ENGINE_API void SetInputStats(const FHLODBuildInputStats& InInputStats);
	ENGINE_API const FHLODBuildInputStats& GetInputStats() const;

	void SetRequireWarmup(bool InRequireWarmup) { bRequireWarmup = InRequireWarmup; }
	ENGINE_API void SetIsStandalone(bool bInIsStandalone);

	ENGINE_API void SetSourceCellGuid(const FGuid& InSourceCellGuid);
	inline void SetLODLevel(uint32 InLODLevel) { LODLevel = InLODLevel; }

	ENGINE_API const FBox& GetHLODBounds() const;
	ENGINE_API void SetHLODBounds(const FBox& InBounds);

	double GetMinVisibleDistance() const { return MinVisibleDistance; }
	void SetMinVisibleDistance(double InMinVisibleDistance) { MinVisibleDistance = InMinVisibleDistance; }

	FName GetHLODResourcesPackagePath() const { return HLODResourcesPackagePath; }
	void SetHLODResourcesPackagePath(FName InHLODResourcesPackagePath) { HLODResourcesPackagePath = InHLODResourcesPackagePath; }

	ENGINE_API UPackage* GetHLODResourcesPackage() const;

	ENGINE_API bool BuildHLOD(bool bForceBuild = false);
	ENGINE_API bool ShouldBuildHLOD() const;

	ENGINE_API int64 GetStat(FName InStatName) const;
	void SetStat(FName InStatName, int64 InStatValue) { HLODStats.Add(InStatName, InStatValue); }
	void ResetStats() { HLODStats.Reset(); }

	ENGINE_API static FWorldPartitionHLODBuildEventDelegate& GetHLODBuildEventDelegate();
	ENGINE_API static FWorldPartitionHLODActorEditorRegisteredDelegate& GetHLODActorEditorRegisteredDelegate();
	ENGINE_API static FWorldPartitionHLODActorEditorUnregisteredDelegate& GetHLODActorEditorUnregisteredDelegate();

	UFUNCTION(BlueprintCallable, Category = "World Partition", meta = (DevelopmentOnly))
	ENGINE_API TArray<UObject*> ExportHLODAssets(const struct FExportHLODAssetsParams& ExportHLODAssetsParams, FString& OutErrorMessage) const;
	
	ENGINE_API void UpdateHLODBuildReportHeader();
	ENGINE_API void UpdateHLODBuildReportContent();

	ENGINE_API const FHLODRebuildPolicyDataSet& GetHLODRebuildPolicyDataSet() const;
	ENGINE_API void SetHLODRebuildPolicyDataSet(const FHLODRebuildPolicyDataSet& InHLODRebuildPolicyDataSet);

	UE_DEPRECATED(5.8, "Hash data is now stored in a UHLODRebuildPolicyData object.")
	uint32 GetHLODHash() const { return HLODHash_DEPRECATED; }
	UE_DEPRECATED(5.8, "Hash data is now stored in a UHLODRebuildPolicyData object.")
	uint32 ComputeHLODHash() const { return 0; }
	UE_DEPRECATED(5.8, "Hash data is now stored in a UHLODRebuildPolicyData object.")
	void SetHLODHash(uint32 InHLODHash, const FString& InHLODBuildReportContent) {}
	UE_DEPRECATED(5.8, "Use override with no parameter.")
	ENGINE_API void UpdateHLODBuildReportContent(const FString& InHLODBuildReportContent);

private:
	const FStats& GetStats() const { return HLODStats; }

#endif // WITH_EDITOR

protected:
	//~ Begin UObject Interface.
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual bool IsEditorOnly() const override;
	ENGINE_API virtual bool NeedsLoadForServer() const override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	ENGINE_API virtual void RerunConstructionScripts() override;
	virtual bool CanEditChange(const FProperty* InProperty) const override { return false; }
	virtual bool CanEditChangeComponent(const UActorComponent* Component, const FProperty* InProperty) const override { return false; }
#endif
	//~ End UObject Interface.

	//~ Begin AActor Interface.
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PreRegisterAllComponents() override;
	ENGINE_API virtual void PostRegisterAllComponents() override;
	ENGINE_API virtual void PostUnregisterAllComponents() override;
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	virtual bool SupportsIncrementalPreRegisterComponents() const override
	{
		return false;
	}
	virtual bool SupportsIncrementalPreUnregisterComponents() const override
	{
		return false;
	}
	virtual bool IsComponentRelevantForNavigation(UActorComponent* Component) const override
	{
		return false;
	}
	virtual bool IsRuntimeOnly() const override { return true; }
#if WITH_EDITOR
	ENGINE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	ENGINE_API virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const override;

	virtual bool ShouldImport(FStringView ActorPropString, bool IsMovingLevel) override { return false; }
	virtual bool IsLockLocation() const override { return true; }
	virtual bool IsUserManaged() const override { return false; }
	virtual bool IsActorLabelEditable() const override { return false; }
#endif
	//~ End AActor Interface.

private:
#if WITH_EDITOR
	ENGINE_API void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources);
	FString GenerateHLODBuildReportHeaderString() const;
#endif

	void OnForceDisableShadowsCVarChanged(IConsoleVariable* CVar);
	void ApplyCastShadowToComponents(bool bInShouldCastShadows);

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UWorldPartitionHLODSourceActors> SourceActors;

	UPROPERTY()
	FHLODBuildInputStats InputStats;

	UPROPERTY()
	FBox HLODBounds;

	UPROPERTY()
	double MinVisibleDistance;

	UPROPERTY()
	TMap<FName, int64> HLODStats;

	UPROPERTY()
	FString HLODBuildReport;

	UPROPERTY(DuplicateTransient)
	TArray<TObjectPtr<UHLODRebuildPolicyData>> HLODRebuildPolicyDataSet;

	UPROPERTY()
	FName HLODResourcesPackagePath;
#endif

	UPROPERTY()
	uint32 LODLevel;

	UPROPERTY()
	bool bRequireWarmup;

	UPROPERTY()
	FGuid SourceCellGuid;

	UPROPERTY()
	FGuid StandaloneHLODGuid;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TSoftObjectPtr<UWorldPartitionRuntimeCell> SourceCell_DEPRECATED;

	UPROPERTY()
	FName SourceCellName_DEPRECATED;

	UPROPERTY()
	TArray<FWorldPartitionRuntimeCellObjectMapping> HLODSubActors_DEPRECATED;

	UPROPERTY()
	TObjectPtr<const UHLODLayer> SubActorsHLODLayer_DEPRECATED;

	UPROPERTY()
	uint32 HLODHash_DEPRECATED;
#endif

#if WITH_EDITOR
	static FWorldPartitionHLODBuildEventDelegate HLODBuildEventDelegate;
	static FWorldPartitionHLODActorEditorRegisteredDelegate HLODActorEditorRegisteredDelegate;
	static FWorldPartitionHLODActorEditorUnregisteredDelegate HLODActorEditorUnregisteredDelegate;
#endif
};

DEFINE_ACTORDESC_TYPE(AWorldPartitionHLOD, FHLODActorDesc);
