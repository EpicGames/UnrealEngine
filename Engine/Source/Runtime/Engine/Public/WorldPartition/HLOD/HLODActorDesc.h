// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Containers/Map.h"

#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/DataLayer/DataLayersID.h"
#include "WorldPartition/HLOD/HLODStats.h"

class UHLODLayer;
class AWorldPartitionHLOD;

/**
 * ActorDesc for AWorldPartitionHLOD.
 */
class FHLODActorDesc : public FWorldPartitionActorDesc
{
	friend class AWorldPartitionHLOD;
	friend class FHLODActorDescFactory;

public:
	typedef TMap<FName, int64>	FStats;

	inline const TArray<FGuid>& GetChildHLODActors() const { return ChildHLODActors; }
	inline const TArray<FGuid>& GetExternalChildHLODActors() const { return ExternalChildHLODActors; }
	inline const FTopLevelAssetPath& GetSourceHLODLayer() const { return SourceHLODLayer; }
	inline const FStats& GetStats() const { return HLODStats; }
	ENGINE_API int64 GetStat(FName InStatName) const;

	// Test if HLOD actor descriptors are equals, but without considering properties that may change after the HLOD actor is built.
	ENGINE_API bool EqualsUnbuilt(const FHLODActorDesc& Other) const;

protected:
	ENGINE_API FHLODActorDesc();

	//~ Begin FWorldPartitionActorDesc Interface.
	ENGINE_API virtual void Init(const AActor* InActor) override;
	ENGINE_API virtual bool Equals(const FWorldPartitionActorDesc* Other) const override;
	virtual uint32 GetSizeOf() const override { return sizeof(FHLODActorDesc); }
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	virtual bool IsRuntimeRelevant(const FWorldPartitionActorDescInstance* InActorDescInstance) const override;
	//~ End FWorldPartitionActorDesc Interface.

	TArray<FGuid> ChildHLODActors;

	TArray<FGuid> ExternalChildHLODActors;

	FTopLevelAssetPath SourceHLODLayer;

	FStats HLODStats;
};
#endif
