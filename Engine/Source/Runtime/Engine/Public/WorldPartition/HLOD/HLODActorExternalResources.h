// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"

#if WITH_EDITOR
#include "UObject/ObjectSaveContext.h"
#endif

#include "HLODActorExternalResources.generated.h"

UCLASS()
class UHLODActorExternalResources : public UDataAsset
{
	GENERATED_BODY()

protected:
#if WITH_EDITOR
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif // WITH_EDITOR

public:
	UPROPERTY(VisibleAnywhere, Category=Resources)
	TArray<TObjectPtr<UObject> > Resources;
};