// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "HLODBlueprintLibrary.generated.h"

class AActor;

UCLASS(MinimalAPI)
class UWorldPartitionHLODBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category="HLOD")
	static bool BuildHLODForActors(const TArray<AActor*>& InActors);

	UFUNCTION(BlueprintCallable, Category="HLOD")
	static bool BuildHLODForVolume(const FBox& InBox);
};