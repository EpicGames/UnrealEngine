// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/Transform.h"
#include "Templates/SubclassOf.h"

#include "HLODBuilder.generated.h"

class AActor;
class FHLODHashBuilder;
class UActorComponent;
class UHLODInstancedStaticMeshComponent;
class UHLODInstancedSkinnedMeshComponent;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogHLODBuilder, Log, All);

/**
 * Base class for all HLOD Builder settings
 */
UCLASS(MinimalAPI, DefaultToInstanced)
class UHLODBuilderSettings : public UObject
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
public:
	/**
	 * Hash the class settings.
	 * This is used to detect settings changes and trigger an HLOD rebuild if necessary.
	 * The provided hash builder state will be updated to reflect the hashed content.
	 */
	virtual void ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const {}

	/**
	 * Whether these HLOD settings are going to result in the created mesh referencing external materials.
	 * This is used to prevent HLOD[N] from referencing private materials from an HLOD[N-1]
	 * By default most HLOD methods are baking down their textures and uses their own unique material instance.
	 */
	virtual bool IsReusingSourceMaterials() const { return false; }

	UE_DEPRECATED(5.7, "GetCRC() has been replaced by ComputeHLODHash()")
	virtual uint32 GetCRC() const final { return 0; }
#endif // WITH_EDITOR
};


/**
 * Provide context for the HLOD generation
 */
struct FHLODBuildContext
{
	UE_DEPRECATED(5.8, "Replaced by TargetWorld to disambiguate vs BuildWorld")
	UWorld*	World;

	/** World for which HLODs are being built */
	UWorld* TargetWorld;

	/** Temporary world used to perform the HLOD build */
	UWorld* BuildWorld;

	/** Components that will be represented by this HLOD */
	TArray<UActorComponent*> SourceComponents;

	/** Outer to use for generated assets */
	UObject* AssetsOuter;

	/** Base name to use for generated assets */
	FString	AssetsBaseName;

	// Location of this HLOD actor in the world
	FVector WorldPosition;

	/** Minimum distance at which the HLOD is expected to be displayed */
	double MinVisibleDistance;

	/** Flags to set for generated assets */
	EObjectFlags AssetsObjectFlags;
};

/**
 * Keep track of assets used as input to the HLOD generation of a given HLOD actor, along with the number of occurences.
 */
USTRUCT()
struct FHLODBuildInputReferencedAssets
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FTopLevelAssetPath, uint32> StaticMeshes;
};

/**
 * Referenced assets per HLOD builder.
 */
USTRUCT()
struct FHLODBuildInputStats
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FName, FHLODBuildInputReferencedAssets> BuildersReferencedAssets;
};

/**
 * Result of the HLOD build of a single actor.
 */
struct FHLODBuildResult
{
	FHLODBuildInputStats		InputStats;
	TArray<UActorComponent*>	HLODComponents;
};

/**
 * Base class for all HLOD builders
 * This class takes as input a group of components, and should return component(s) that will be included in the HLOD actor.
 */
UCLASS(Abstract, Config = Editor, MinimalAPI)
class UHLODBuilder : public UObject
{
	 GENERATED_UCLASS_BODY()

#if WITH_EDITOR
public:
	/**
	 * Provide builder settings before a Build.
	 */
	ENGINE_API void SetHLODBuilderSettings(const UHLODBuilderSettings* InHLODBuilderSettings);

	/**
	 * Build an HLOD representation of the input actors.
	 * Components returned by this method needs to be properly outered & assigned to your target (HLOD) actor.
	 */
	ENGINE_API FHLODBuildResult Build(const FHLODBuildContext& InHLODBuildContext) const;

	UE_DEPRECATED(5.2, "Use Build() method that takes a single FHLODBuildContext parameter.")
	TArray<UActorComponent*> Build(const FHLODBuildContext& InHLODBuildContext, const TArray<AActor*>& InSourceActors) const { return Build(InHLODBuildContext).HLODComponents; }

	/**
	 * Return the setting subclass associated with this builder.
	 */
	ENGINE_API virtual TSubclassOf<UHLODBuilderSettings> GetSettingsClass() const;

	/**
	 * Should return true if components generated from this builder need a warmup phase before being made visible.
	 * If your components are using virtual textures or Nanite meshes, this should return true, as it will be necessary
	 * to warmup the VT & Nanite caches before transitionning to HLOD. Otherwise, it's likely the initial first frames
	 * could show a low resolution texture or mesh.
	 */
	ENGINE_API virtual bool RequiresWarmup() const;

	/**
	 * For a given component, compute a unique hash from the properties that are relevant to HLOD generation.
	 * Used to detect changes to the source components of an HLOD.
	 * The base version can only support hashing of static mesh components. HLOD builder subclasses
	 * should override this method and compute the hash of component types they support as input.
	 * The provided hash builder state will be updated to reflect the hashed content.
	 * Will return true if an hash could be computed, false otherwise.
	 */
	ENGINE_API virtual bool ComputeHLODHash(class FHLODHashBuilder& InHashBuilder, const UActorComponent* InSourceComponent) const;

	/**
	 * Components created with this method need to be properly outered & assigned to your target actor.
	 */
	virtual TArray<UActorComponent*> Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const PURE_VIRTUAL(UHLODBuilder::Build, return {};);

	/**
	 * From a set of components, compute a unique hash from their properties that are relevant to HLOD generation.
	 * Used to detect changes to the source actors and trigger an HLOD rebuild if necessary.
	 * The provided hash builder state will be updated to reflect the hashed content.
	 */
	static ENGINE_API void ComputeHLODHash(class FHLODHashBuilder& HashBuilder, const TArray<UActorComponent*>& InSourceComponents);

	/** 
	 * Get the InstancedStaticMeshComponent subclass that should be used when creating instanced static mesh HLODs.
	 */
	static ENGINE_API TSubclassOf<UHLODInstancedStaticMeshComponent> GetInstancedStaticMeshComponentClass();

	/**
	 * Get the InstancedStaticMeshComponent subclass that should be used when creating instanced skinned mesh HLODs.
	 */
	static ENGINE_API TSubclassOf<UHLODInstancedSkinnedMeshComponent> GetInstancedSkinnedMeshComponentClass();

protected:
	virtual bool ShouldIgnoreBatchingPolicy() const { return false; }

	static ENGINE_API TArray<UActorComponent*> BatchInstances(const TArray<UActorComponent*>& InSourceComponents);
	static ENGINE_API TArray<UActorComponent*> BatchInstances(const TArray<UActorComponent*>& InSourceComponents, TFunctionRef<bool(const FBox&)> InFilterFunc);

	template <typename TComponentClass>
	static inline TArray<TComponentClass*> FilterComponents(const TArray<UActorComponent*>& InSourceComponents)
	{
		TArray<TComponentClass*> FilteredComponents;
		FilteredComponents.Reserve(InSourceComponents.Num());
		Algo::TransformIf(InSourceComponents, FilteredComponents, [](UActorComponent* SourceComponent) { return Cast<TComponentClass>(SourceComponent); }, [](UActorComponent* SourceComponent) { return Cast<TComponentClass>(SourceComponent); });
		if (InSourceComponents.Num() != FilteredComponents.Num())
		{
			UE_LOGF(LogHLODBuilder, Warning, "Excluding %d components from the HLOD build.", InSourceComponents.Num() - FilteredComponents.Num());
		}
		return FilteredComponents;
	}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
protected:
	UPROPERTY()
	TObjectPtr<const UHLODBuilderSettings> HLODBuilderSettings;

private:
	UPROPERTY(Config)
	TSubclassOf<UHLODInstancedStaticMeshComponent> HLODInstancedStaticMeshComponentClass;

	UPROPERTY(Config)
	TSubclassOf<UHLODInstancedSkinnedMeshComponent> HLODInstancedSkinnedMeshComponentClass;
#endif
};


/**
 * Null HLOD builder that ignores it's input and generate no component.
 */
UCLASS(HideDropdown, MinimalAPI)
class UNullHLODBuilder : public UHLODBuilder
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	virtual bool RequiresWarmup() const { return false; }
	virtual bool ComputeHLODHash(FHLODHashBuilder& HashBuilder, const UActorComponent* InSourceComponent) const { return false; }
	virtual TArray<UActorComponent*> Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const { return {}; }
#endif
};
