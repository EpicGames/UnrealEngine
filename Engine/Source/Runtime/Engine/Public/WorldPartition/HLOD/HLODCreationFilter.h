// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box.h"
#include "Templates/SharedPointer.h"

#define UE_API ENGINE_API

struct FHLODCreationFilterContext
{
	TOptional<FBox> Bounds;
};

struct IHLODCreationFilter
{
	virtual ~IHLODCreationFilter() = default;
	virtual bool IsActive() const = 0;
	virtual bool PassesFilter(const FHLODCreationFilterContext& InFilterContext) const = 0;
};

namespace UE::HLOD::CreationFilter
{
	UE_API bool HasAnyActiveFilters(const TArray<TSharedPtr<IHLODCreationFilter>>& InFilters);
	UE_API bool PassesFilters(const TArray<TSharedPtr<IHLODCreationFilter>>& InFilters, const FHLODCreationFilterContext& InFilterContext);
}

struct FHLODCreationVolumeFilter : IHLODCreationFilter
{
	FHLODCreationVolumeFilter() = default;
	UE_API FHLODCreationVolumeFilter(TArray<FBox>&& InFilterVolumes);

	UE_API virtual bool IsActive() const override;
	UE_API virtual bool PassesFilter(const FHLODCreationFilterContext& InFilterContext) const override;

	TArray<FBox> FilterVolumes;
};

#undef UE_API