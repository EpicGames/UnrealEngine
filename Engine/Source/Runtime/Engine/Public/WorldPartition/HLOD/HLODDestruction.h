// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/Interface.h"
#include "HLODDestruction.generated.h"


class UWorldPartitionDestructibleHLODComponent;


USTRUCT(BlueprintType)
struct FWorldPartitionHLODDestructionTag
{
	GENERATED_BODY()

	FWorldPartitionHLODDestructionTag()
		: HLODDestructionComponent()
		, ActorIndex(INDEX_NONE)
	{
	}

	bool IsValid() const
	{
		return HLODDestructionComponent != nullptr && ActorIndex != INDEX_NONE;
	}

	UPROPERTY()
	TObjectPtr<UWorldPartitionDestructibleHLODComponent> HLODDestructionComponent;

	UPROPERTY()
	int32 ActorIndex;
};


UINTERFACE(NotBlueprintable, meta=(DisplayName="Destructible in HLOD Interface"), MinimalAPI)
class UWorldPartitionDestructibleInHLODInterface : public UInterface
{
	GENERATED_BODY()
};


class IWorldPartitionDestructibleInHLODInterface
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, meta=(BlueprintInternalUseOnly = "true"))
	ENGINE_API void SetHLODDestructionTag(const FWorldPartitionHLODDestructionTag& InDestructionTag);

	UFUNCTION(BlueprintNativeEvent, meta=(BlueprintInternalUseOnly = "true"))
	ENGINE_API FWorldPartitionHLODDestructionTag GetHLODDestructionTag() const;
};


UCLASS(MinimalAPI)
class UWorldPartitionDestructibleInHLODSupportLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "HLOD Destruction")
	static ENGINE_API void DestroyInHLOD(const TScriptInterface<IWorldPartitionDestructibleInHLODInterface>& DestructibleInHLOD);

	UFUNCTION(BlueprintCallable, Category="HLOD Destruction")
	static ENGINE_API void DamageInHLOD(const TScriptInterface<IWorldPartitionDestructibleInHLODInterface>& DestructibleInHLOD, float DamagePercent);
};
