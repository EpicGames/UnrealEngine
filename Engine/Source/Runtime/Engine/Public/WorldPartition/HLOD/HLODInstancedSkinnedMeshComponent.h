// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Animation/AnimBank.h"
#include "HLODInstancedSkinnedMeshComponent.generated.h"


UCLASS(HideDropDown, NotPlaceable, MinimalAPI)
class UHLODInstancedSkinnedMeshComponent : public UInstancedSkinnedMeshComponent
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	ENGINE_API virtual TUniquePtr<FSkinnedMeshComponentDescriptor> AllocateISMComponentDescriptor() const;
#endif
};


// ISM descriptor class based on FSkinnedMeshComponentDescriptor
USTRUCT()
struct FHLODSkinnedMeshComponentDescriptor : public FSkinnedMeshComponentDescriptor
{
	GENERATED_BODY()

#if WITH_EDITOR
	ENGINE_API FHLODSkinnedMeshComponentDescriptor();

	ENGINE_API virtual void InitFrom(const UInstancedSkinnedMeshComponent* Component, bool bInitBodyInstance = true) override;
#endif
};
