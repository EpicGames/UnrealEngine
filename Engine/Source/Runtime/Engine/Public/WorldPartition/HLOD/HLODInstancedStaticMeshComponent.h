// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "ISMPartition/ISMComponentBatcher.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "HLODInstancedStaticMeshComponent.generated.h"


UCLASS(HideDropDown, NotPlaceable, MinimalAPI)
class UHLODInstancedStaticMeshComponent : public UInstancedStaticMeshComponent
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	ENGINE_API virtual TUniquePtr<FISMComponentDescriptor> AllocateISMComponentDescriptor() const;

	typedef TArray<FISMComponentBatcher::FComponentToInstancesMapping> FSourceComponentsToInstancesMap;

	ENGINE_API void SetSourceComponentsToInstancesMap(FSourceComponentsToInstancesMap&& InSourceComponentsToInstances);
	ENGINE_API const FSourceComponentsToInstancesMap& GetSourceComponentsToInstancesMap() const;

private:
	// Transient data, only available during HLOD builds
	FSourceComponentsToInstancesMap SourceComponentsToInstances;
#endif
};


// ISM descriptor class based on FISMComponentDescriptor
USTRUCT()
struct FHLODISMComponentDescriptor : public FISMComponentDescriptor
{
	GENERATED_BODY()

#if WITH_EDITOR
	ENGINE_API FHLODISMComponentDescriptor();

	ENGINE_API virtual void InitFrom(const UStaticMeshComponent* Component, bool bInitBodyInstance = true) override;
	ENGINE_API virtual void InitComponent(UInstancedStaticMeshComponent* ISMComponent) const override;
#endif
};
