// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"

#include "EngineDefines.h"
#include "MeshMerge/MeshMergingSettings.h"
#include "MeshMerge/MeshProxySettings.h"
#include "MeshMerge/MeshApproximationSettings.h"

#include "WorldPartition/HLOD/HLODBuilder.h"

#include "HLODLayer.generated.h"

class AActor;
class AWorldPartitionHLOD;
class UHLODBuilder;
class UHLODBuilderSettings;
class FHLODHashBuilder;
class UMaterial;
class UWorldPartition;
class UWorldPartitionHLODModifier;
class FWorldPartitionActorDesc;

UENUM()
enum class EHLODLayerType : uint8
{
	Instancing				UMETA(DisplayName = "Instancing"),
	MeshMerge				UMETA(DisplayName = "Merged Mesh"),
	MeshSimplify			UMETA(DisplayName = "Simplified Mesh"),
	MeshApproximate			UMETA(DisplayName = "Approximated Mesh"),
	Custom					UMETA(DisplayName = "Custom Builder"),
	CustomHLODActor			UMETA(DisplayName = "Custom HLOD Actor"),
};

UENUM()
enum class EHLODLayerEditorLoadingBehavior : uint8
{
	/** Use default behavior: Instancing layers are not loaded (except if non-spatial), other layer types are loaded */
	Default					UMETA(DisplayName = "Default"),
	/** Always load HLODs from this layer in the editor */
	ForceLoaded				UMETA(DisplayName = "Always Loaded"),
	/** Never load HLODs from this layer in the editor */
	ForceNotLoaded			UMETA(DisplayName = "Never Loaded"),
};

UCLASS(Blueprintable, MinimalAPI)
class UHLODLayer : public UObject
{
	GENERATED_UCLASS_BODY()
	
#if WITH_EDITOR
public:
	/** Get the default engine HLOD layers setup */
	static ENGINE_API UHLODLayer* GetEngineDefaultHLODLayersSetup();

	/** Duplicate the provided HLOD layers setup */
	static ENGINE_API UHLODLayer* DuplicateHLODLayersSetup(UHLODLayer* HLODLayer, const FString& DestinationPath, const FString& Prefix);

	EHLODLayerType GetLayerType() const { return LayerType; }
	void SetLayerType(EHLODLayerType InLayerType) { LayerType = InLayerType; }
	const TSubclassOf<UHLODBuilder> GetHLODBuilderClass() const { return HLODBuilderClass; }
	const UHLODBuilderSettings* GetHLODBuilderSettings() const { return HLODBuilderSettings; }
	const TSubclassOf<AWorldPartitionHLOD> GetHLODActorClass() const { return HLODActorClass; }
	const TSubclassOf<UWorldPartitionHLODModifier> GetHLODModifierClass() const { return HLODModifierClass; }
	bool GetForceRayTracingFarField() const { return bForceRayTracingFarField; }
	UHLODLayer* GetParentLayer() const { return ParentLayer; }
	void SetParentLayer(UHLODLayer* InParentLayer) { ParentLayer = InParentLayer; }
	UHLODLayer* GetLinkedLayer() const { return LinkedLayer; }
	EHLODLayerEditorLoadingBehavior GetEditorLoadingBehavior() const { return EditorLoadingBehavior; }
	ENGINE_API bool DoesRequireWarmup() const;
	ENGINE_API void ComputeHLODHash(FHLODHashBuilder& InHLODHashBuilder) const;
	ENGINE_API bool MustBeLoadedOnServer() const;

	UE_DEPRECATED(5.7, "Deprecated. These streaming grid properties are now specified in the partition's settings.")
	ENGINE_API FName GetRuntimeGrid(uint32 InHLODLevel) const;
	UE_DEPRECATED(5.7, "Deprecated. These streaming grid properties are now specified in the partition's settings.")
	bool IsSpatiallyLoaded() const { return bIsSpatiallyLoaded; }
	UE_DEPRECATED(5.7, "Deprecated. These streaming grid properties are now specified in the partition's settings.")
	void SetIsSpatiallyLoaded(bool bInIsSpatiallyLoaded) { bIsSpatiallyLoaded = bInIsSpatiallyLoaded; }
	UE_DEPRECATED(5.7, "Deprecated. These streaming grid properties are now specified in the partition's settings.")
	int32 GetCellSize() const { return !bIsSpatiallyLoaded ? 0 : CellSize; }
	UE_DEPRECATED(5.7, "Deprecated. These streaming grid properties are now specified in the partition's settings.")
	double GetLoadingRange() const { return !bIsSpatiallyLoaded ? WORLD_MAX : LoadingRange; }
	UE_DEPRECATED(5.7, "Deprecated. These streaming grid properties are now specified in the partition's settings.")
	static ENGINE_API FName GetRuntimeGridName(uint32 InLODLevel, int32 InCellSize, double InLoadingRange);

	// Get name of properties
	static const FName GetLayerTypePropertyName() { return GET_MEMBER_NAME_CHECKED(UHLODLayer, LayerType); };
	static const FName GetHLODBuilderSettingsPropertyName() { return GET_MEMBER_NAME_CHECKED(UHLODLayer, HLODBuilderSettings); }

private:
	//~ Begin UObject Interface.
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static ENGINE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
	//~ End UObject Interface.
#endif

#if WITH_EDITORONLY_DATA
private:
	/** Type of HLOD generation to use */
	UPROPERTY(EditAnywhere, Category=General)
	EHLODLayerType LayerType;

	/** HLOD Builder class */
	UPROPERTY(EditAnywhere, Category=General, meta = (DisplayName = "HLOD Builder Class", EditConditionHides, EditCondition = "LayerType == EHLODLayerType::Custom"))
	TSubclassOf<UHLODBuilder> HLODBuilderClass;

	UPROPERTY(VisibleAnywhere, Instanced, NoClear, Category=General, meta = (NoResetToDefault, EditInline, ShowInnerProperties))
	TObjectPtr<UHLODBuilderSettings> HLODBuilderSettings;

	/** Whether HLOD actors generated for this layer will be spatially loaded */
	UPROPERTY(EditAnywhere, Category=General)
	uint32 bIsSpatiallyLoaded : 1;

	/** Cell size of the runtime grid created to encompass HLOD actors generated for this HLOD Layer */
	UPROPERTY(EditAnywhere, Category=General)
	int32 CellSize;

	/** Loading range of the runtime grid created to encompass HLOD actors generated for this HLOD Layer */
	UPROPERTY(EditAnywhere, Category=General)
	double LoadingRange;

	/** HLOD Layer to assign to the generated HLOD actors */
	UPROPERTY(EditAnywhere, Category=General)
	TObjectPtr<UHLODLayer> ParentLayer;
	
	/** HLOD Layer used to control visiblity of Custom HLOD Actors. Custom HLOD Actors become visible when actors from the linked HLOD Layer are unloaded. */
	UPROPERTY(EditAnywhere, Category=General, AdvancedDisplay, meta = (EditConditionHides, EditCondition = "LayerType == EHLODLayerType::CustomHLODActor"))
	TObjectPtr<UHLODLayer> LinkedLayer;

	/** Specify a custom HLOD Actor class, the default is AWorldPartitionHLOD */
	UPROPERTY(EditAnywhere, Category = General, AdvancedDisplay, meta = (DisplayName = "HLOD Actor Class"))
	TSubclassOf<AWorldPartitionHLOD> HLODActorClass;

	/** HLOD Modifier class, to allow changes to the HLOD at runtime */
	UPROPERTY(EditAnywhere, Category = General, AdvancedDisplay, meta = (DisplayName = "HLOD Modifier Class"))
	TSubclassOf<UWorldPartitionHLODModifier> HLODModifierClass;

	/**
	 * When enabled, static mesh components generated for this HLOD layer will be included in the ray tracing far field,
	 * even if the HLOD is spatially loaded. This is useful for cases where the last HLOD level is spatially
	 * loaded but should still contribute to ray traced reflections and global illumination.
	 */
	UPROPERTY(EditAnywhere, Category = General, AdvancedDisplay)
	uint32 bForceRayTracingFarField : 1;

	/**
	 * Controls whether HLODs from this layer are loaded in the editor.
	 * - Default: Instancing layers are not loaded (except if non-spatial), other layer types are loaded
	 * - Always Loaded: Force load HLODs from this layer in editor
	 * - Never Loaded: Prevent loading HLODs from this layer in editor
	 */
	UPROPERTY(EditAnywhere, Category = General, AdvancedDisplay)
	EHLODLayerEditorLoadingBehavior EditorLoadingBehavior;

	friend class FWorldPartitionHLODLayerDetailsCustomization;

private:
	friend class FWorldPartitionHLODUtilities;

	UPROPERTY()
	FMeshMergingSettings MeshMergeSettings_DEPRECATED;
	UPROPERTY()
	FMeshProxySettings MeshSimplifySettings_DEPRECATED;
	UPROPERTY()
	FMeshApproximationSettings MeshApproximationSettings_DEPRECATED;
	UPROPERTY()
	TSoftObjectPtr<UMaterialInterface> HLODMaterial_DEPRECATED;
	UPROPERTY()
	uint32 bAlwaysLoaded_DEPRECATED : 1;
#endif
};
