// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"

#include "HLODModifier.generated.h"


class UActorComponent;
class UHLODBuilder;
struct FHLODBuildContext;

/**
 * Base class for all runtime HLOD modifiers
 */
UCLASS(Abstract, MinimalAPI)
class UWorldPartitionHLODModifier : public UObject
{
	 GENERATED_UCLASS_BODY()

public:
	virtual bool CanModifyHLOD(TSubclassOf<UHLODBuilder> InHLODBuilderClass) const PURE_VIRTUAL(UWorldPartitionHLODModifier::CanModifyHLOD, return false;);
	
	virtual void BeginHLODBuild(const FHLODBuildContext& InHLODBuildContext) PURE_VIRTUAL(UWorldPartitionHLODModifier::BeginHLODBuild, );
	virtual void EndHLODBuild(TArray<UActorComponent*>& InOutComponents) PURE_VIRTUAL(UWorldPartitionHLODModifier::EndHLODBuild, );

	virtual bool MustBeLoadedOnServer() const { return false; }
};
