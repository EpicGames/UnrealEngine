// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "UObject/Interface.h"
#include "HLODProviderInterface.generated.h"


class AWorldPartitionHLOD;


UINTERFACE(MinimalAPI)
class UWorldPartitionHLODProvider : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IWorldPartitionHLODProvider
{
	GENERATED_IINTERFACE_BODY()

public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FPackageProcessedDelegate, UPackage*)

	UE_DEPRECATED(5.7, "Use FPackageProcessedDelegate along with FBuildHLODActorParams::OnPackageProcessed")
	DECLARE_DELEGATE_RetVal_OneParam(bool, FPackageModifiedDelegate, UPackage*)

	struct FBuildHLODActorParams
	{
		bool bForceRebuild = false;

		FPackageProcessedDelegate OnPackageProcessed;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.7, "Use OnPackageProcessed. Test if packaged is modified with Package->IsDirty()")
		FPackageModifiedDelegate OnPackageModified;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	virtual bool BuildHLODActor(const FBuildHLODActorParams& BuildHLODActorParams) = 0;
};
