// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "HLODRebuildPolicy.generated.h"

class AWorldPartitionHLOD;
class UActorComponent;

/**
 * Decision type returned when a rebuild policy is evaluated.
 */
UENUM()
enum class EHLODRebuildPolicyDecision : uint8
{
	None,			//!< No decision.
	ApproveRebuild,	//!< At least one HLOD rebuild policy must approve the build for it to be performed.
	RejectRebuild	//!< If a single rebuild policy rejects the build, it is aborted.
};

/**
 * Data associated with the policy evaluation.
 * Stored in the HLOD actor so that it can be used as the baseline when performing the next evaluation.
 */
UCLASS(Abstract, MinimalAPI)
class UHLODRebuildPolicyData : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	virtual FString ToString() const { return FString(); };
};

typedef TArray<TObjectPtr<UHLODRebuildPolicyData>> FHLODRebuildPolicyDataSet;

/**
 * Logic and settings associated with a rebuild policy.
  */
UCLASS(Abstract, MinimalAPI, Config = Editor)
class UHLODRebuildPolicy : public UObject
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	static ENGINE_API FHLODRebuildPolicyDataSet ComputeDataForRebuildPolicies(const AWorldPartitionHLOD* InHLODActor, const TArray<UActorComponent*>& InSourceComponents, const bool bInForComparison);
	static ENGINE_API EHLODRebuildPolicyDecision Evaluate(const AWorldPartitionHLOD* InHLODActor, const FHLODRebuildPolicyDataSet& InOldData, const FHLODRebuildPolicyDataSet& InNewData);

protected:
	virtual TSubclassOf<UHLODRebuildPolicyData> GetRebuildPolicyDataType() const PURE_VIRTUAL(UHLODRebuildPolicy::GetRebuildPolicyDataType, return UHLODRebuildPolicyData::StaticClass(); );
	virtual EHLODRebuildPolicyDecision Evaluate(const AWorldPartitionHLOD* InHLODActor, const UHLODRebuildPolicyData* InOldData, const UHLODRebuildPolicyData* InNewData, FString& OutReason) const PURE_VIRTUAL(UHLODRebuildPolicy::Evaluate, return EHLODRebuildPolicyDecision::None; );
	virtual UHLODRebuildPolicyData* ComputeDataForRebuildPolicy(const AWorldPartitionHLOD* InHLODActor, const TArray<UActorComponent*>& InSourceComponents, const UHLODRebuildPolicyData* InExistingPolicyData) const PURE_VIRTUAL(UHLODRebuildPolicy::ComputeDataForRebuildPolicy, return nullptr; );
#endif

	static TArray<TSubclassOf<UHLODRebuildPolicy>> GetRebuildPoliciesClasses();

	// HLOD Rebuild Policies to evaluate. These are evaluated one after the other, so the order is important.
	UPROPERTY(Config)
	TArray<FSoftObjectPath> HLODRebuildPolicies;
};
