// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "UObject/ObjectKey.h"
#include "Containers/Ticker.h"
#include "Templates/Function.h"
#include "HLODRuntimeSubsystem.generated.h"

class IWorldPartitionHLODObject;
class AWorldPartitionHLOD;
class FSceneViewFamily;
class FHLODResourcesResidencySceneViewExtension;
class UWorldPartition;
class UWorldPartitionRuntimeCell;
class URuntimeHashExternalStreamingObjectBase;
class UWorld;

DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionHLODObjectRegisteredEvent, IWorldPartitionHLODObject* /* InHLODObject */);
DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionHLODObjectUnregisteredEvent, IWorldPartitionHLODObject* /* InHLODObject */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FWorldPartitionHLODForEachHLODObjectInCellEvent, const UWorldPartitionRuntimeCell* /* InCell */, TFunction<void(IWorldPartitionHLODObject*)> /* InFunc */);

UE_DEPRECATED(5.6, "Use FWorldPartitionHLODObjectRegisteredEvent instead")
DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionHLODActorRegisteredEvent, AWorldPartitionHLOD* /* InHLODActor */);
UE_DEPRECATED(5.6, "Use FWorldPartitionHLODObjectUnregisteredEvent instead")
DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionHLODActorUnregisteredEvent, AWorldPartitionHLOD* /* InHLODActor */);

/**
 * UWorldPartitionHLODRuntimeSubsystem
 */
UCLASS(MinimalAPI)
class UWorldPartitionHLODRuntimeSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	ENGINE_API UWorldPartitionHLODRuntimeSubsystem();

	//~ Begin USubsystem Interface.
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;
	//~ End USubsystem Interface.

	//~ Begin UWorldSubsystem Interface.
	ENGINE_API virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override;
	//~ End UWorldSubsystem Interface.

	ENGINE_API void RegisterHLODObject(IWorldPartitionHLODObject* InWorldPartitionHLOD);
	ENGINE_API void UnregisterHLODObject(IWorldPartitionHLODObject* InWorldPartitionHLOD);

	ENGINE_API void OnCellShown(const UWorldPartitionRuntimeCell* InCell);
	ENGINE_API void OnCellHidden(const UWorldPartitionRuntimeCell* InCell);

	ENGINE_API bool CanMakeVisible(const UWorldPartitionRuntimeCell* InCell);
	ENGINE_API bool CanMakeInvisible(const UWorldPartitionRuntimeCell* InCell);

	ENGINE_API FWorldPartitionHLODForEachHLODObjectInCellEvent& GetForEachHLODObjectInCellEvent();

	static ENGINE_API bool IsHLODEnabled();
	static ENGINE_API bool IsWarmupEnabled();

	ENGINE_API const TArray<IWorldPartitionHLODObject*>& GetHLODObjectsForCell(const UWorldPartitionRuntimeCell* InCell) const;
	FWorldPartitionHLODObjectRegisteredEvent& OnHLODObjectRegisteredEvent() { return HLODObjectRegisteredEvent; }
	FWorldPartitionHLODObjectUnregisteredEvent& OnHLODObjectUnregisteredEvent() { return HLODObjectUnregisteredEvent; }

	ENGINE_API void OnExternalStreamingObjectInjected(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject);
	ENGINE_API void OnExternalStreamingObjectRemoved(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject);

	ENGINE_API void OnCVarsChanged();

#if !UE_BUILD_SHIPPING
	uint32 GetNumOutdatedHLODObjects() const { return OutdatedHLODObjects.Num(); }
#endif

	UE_DEPRECATED(5.6, "Use RegisterHLODObject instead")
	void RegisterHLODActor(AWorldPartitionHLOD* InWorldPartitionHLOD) {}
	UE_DEPRECATED(5.6, "Use UnregisterHLODObject instead")
	void UnregisterHLODActor(AWorldPartitionHLOD* InWorldPartitionHLOD) {}

	UE_DEPRECATED(5.6, "Use GetHLODObjectsForCell instead")
	const TArray<AWorldPartitionHLOD*>& GetHLODActorsForCell(const UWorldPartitionRuntimeCell* InCell) const
	{
		static TArray<AWorldPartitionHLOD*> Empty;
		return Empty;
	}

#if !UE_BUILD_SHIPPING
	UE_DEPRECATED(5.6, "Use GetNumOutdatedHLODObjects instead")
	uint32 GetNumOutdatedHLODActors() const { return 0; }
#endif

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Use OnHLODObjectRegisteredEvent instead")
	FWorldPartitionHLODActorRegisteredEvent& OnHLODActorRegisteredEvent() { return HLODActorRegisteredEvent; }
	UE_DEPRECATED(5.6, "Use OnHLODObjectUnregisteredEvent instead")
	FWorldPartitionHLODActorUnregisteredEvent& OnHLODActorUnregisteredEvent() {	return HLODActorUnregisteredEvent; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
	UE_DEPRECATED(5.5, "Use UWorldPartitionHLODEditorSubsystem::WriteHLODStatsCSV()")
	static bool WriteHLODStatsCSV(UWorld* InWorld, const FString& InFilename) { return false; }
#endif

	UE_DEPRECATED(5.4, "You should perform this logic on the game side.")
	void SetHLODAlwaysLoadedCullDistance(int32 InCullDistance) {}
	
private:
	void ForEachHLODObjectInCell(const UWorldPartitionRuntimeCell* InCell, TFunction<void(IWorldPartitionHLODObject*)> InFunc);
	FWorldPartitionHLODForEachHLODObjectInCellEvent ForEachHLODObjectInCellEvent;

	struct FCellData
	{
		bool bIsCellVisible = false;
		TArray<IWorldPartitionHLODObject*> LoadedHLODs;	// HLOD representation of the cell itself
	};

	struct FWorldPartitionHLODRuntimeData
	{
		TMap<FGuid, FCellData> CellsData;
	};

	TMap<TObjectPtr<UWorldPartition>, FWorldPartitionHLODRuntimeData> WorldPartitionsHLODRuntimeData;
	ENGINE_API const FCellData* GetCellData(const UWorldPartitionRuntimeCell* InCell) const;
	ENGINE_API FCellData* GetCellData(const UWorldPartitionRuntimeCell* InCell);
	ENGINE_API FCellData* GetCellData(IWorldPartitionHLODObject* InWorldPartitionHLOD);
	ENGINE_API FCellData* GetCellData(const UWorldPartition* InWorldPartition, const FGuid& InCellGuid);

	// Keep track of the state of warmup for an HLOD object
	struct FWorldPartitionHLODWarmupState
	{		
		uint32 WarmupLastRequestedFrame = INDEX_NONE;
		uint32 WarmupCallsUntilReady = INDEX_NONE;
		FBox WarmupBounds;
	};
	typedef TMap<IWorldPartitionHLODObject*, FWorldPartitionHLODWarmupState> FHLODWarmupStateMap;
	FHLODWarmupStateMap HLODObjectsToWarmup;
	
	// Keep track of all HLOD objects currently warming up for a given level
	// If there are any, the OnCleanupLevelDelegateHandle member should be bound
	struct FHLODLevelState
	{
		TSet<IWorldPartitionHLODObject*> HLODObjectsWarmingUp;
		FDelegateHandle OnCleanupLevelDelegateHandle;
	};
	typedef TMap<ULevel*, FHLODLevelState> FHLODLevelStateMap;
	FHLODLevelStateMap HLODLevelWarmupStates;

	FWorldPartitionHLODWarmupState& AddHLODObjectToWarmup(IWorldPartitionHLODObject* InHLODObject);
	void RemoveHLODObjectFromWarmup(IWorldPartitionHLODObject* InHLODObject);
	void OnCleanupLevel(ULevel* InCell);

	ENGINE_API void OnBeginRenderViews(const FSceneViewFamily& InViewFamily);
	ENGINE_API bool PrepareToWarmup(const UWorldPartitionRuntimeCell* InCell, IWorldPartitionHLODObject* InHLODObject);
	ENGINE_API bool ShouldPerformWarmup() const;
	ENGINE_API bool ShouldPerformWarmupForCell(const UWorldPartitionRuntimeCell* InCell) const;
	bool bCachedShouldPerformWarmup;

	/** Console command used to turn on/off loading & rendering of world partition HLODs */
	static ENGINE_API class FAutoConsoleCommand EnableHLODCommand;

	static ENGINE_API bool WorldPartitionHLODEnabled;

	friend class FHLODResourcesResidencySceneViewExtension;
	TSharedPtr<FHLODResourcesResidencySceneViewExtension, ESPMode::ThreadSafe> SceneViewExtension;

	ENGINE_API void OnWorldPartitionInitialized(UWorldPartition* InWorldPartition);
	ENGINE_API void OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition);

	FWorldPartitionHLODObjectRegisteredEvent	HLODObjectRegisteredEvent;
	FWorldPartitionHLODObjectUnregisteredEvent	HLODObjectUnregisteredEvent;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FWorldPartitionHLODActorRegisteredEvent		HLODActorRegisteredEvent;
	FWorldPartitionHLODActorUnregisteredEvent	HLODActorUnregisteredEvent;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Referenced Cell GUID -> HLOD Actor
	TMap<FGuid, TSet<IWorldPartitionHLODObject*>> StandaloneHLODObjectsReferencingUnloadedCells;
	
	// Cell GUID -> WorldPartition
	TMap<FGuid, TObjectPtr<UWorldPartition>> StandaloneHLODCellToWorldPartitionMap;

#if !UE_BUILD_SHIPPING
	TSet<IWorldPartitionHLODObject*> OutdatedHLODObjects;
#endif
};
