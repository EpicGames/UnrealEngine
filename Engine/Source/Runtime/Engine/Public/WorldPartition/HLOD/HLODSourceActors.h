// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HLODSourceActors.generated.h"


class FHLODHashBuilder;
class ULevelStreaming;
class UHLODLayer;


UCLASS(Abstract, MinimalAPI)
class UWorldPartitionHLODSourceActors : public UObject
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	virtual bool LoadSourceActors(bool& bOutDirty, UWorld* TargetWorld) const PURE_VIRTUAL(UWorldPartitionHLODSourceActors::LoadSourceActors, return false; );
	ENGINE_API virtual void ComputeHLODHash(FHLODHashBuilder& HashBuilder) const;
	virtual bool IsHLODRelevant(AActor* InActor) const PURE_VIRTUAL(UWorldPartitionHLODSourceActors::IsHLODRelevant, return false; );

	ENGINE_API void SetHLODLayer(const UHLODLayer* HLODLayer);
	ENGINE_API const UHLODLayer* GetHLODLayer() const;

	UE_DEPRECATED(5.7, "GetHLODHash() has been replaced by ComputeHLODHash()")
	virtual uint32 GetHLODHash() const final { return 0; }

	UE_DEPRECATED(5.7, "Use LoadSourceActors() override with a TargetWorld instead")
	virtual ULevelStreaming* LoadSourceActors(bool& bOutDirty) const final { return nullptr; }
#endif

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<const UHLODLayer> HLODLayer;
#endif
};
