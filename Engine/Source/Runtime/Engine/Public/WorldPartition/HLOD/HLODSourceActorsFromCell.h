// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/HLOD/HLODSourceActors.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "HLODSourceActorsFromCell.generated.h"


UCLASS(MinimalAPI)
class UWorldPartitionHLODSourceActorsFromCell : public UWorldPartitionHLODSourceActors
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	ENGINE_API virtual bool LoadSourceActors(bool& bOutDirty, UWorld* TargetWorld) const override;
	ENGINE_API virtual void ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const override;
	ENGINE_API virtual bool IsHLODRelevant(AActor* InActor) const override;

	ENGINE_API static uint32 GetSourceActorsHash(const TArray<FWorldPartitionRuntimeCellObjectMapping>& InSourceActors);

	ENGINE_API void SetActors(const TArray<FWorldPartitionRuntimeCellObjectMapping>&& InSourceActors);
	ENGINE_API const TArray<FWorldPartitionRuntimeCellObjectMapping>& GetActors() const;
#endif

private:
#if WITH_EDITOR
	// Dirty source actors must be duplicated (no on-disk version to load), and duplication clears the duplicate's guids.
	// Map each duplicate to its original's instance guid so IsHLODRelevant can still match.
	mutable TMap<TWeakObjectPtr<AActor>, FGuid> DirtyDuplicateActorsToOriginalGuids;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FWorldPartitionRuntimeCellObjectMapping> Actors;
#endif
};
