// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/HLOD/HLODSourceActors.h"
#include "HLODSourceActorsFromLevel.generated.h"


UCLASS(MinimalAPI)
class UWorldPartitionHLODSourceActorsFromLevel : public UWorldPartitionHLODSourceActors
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	ENGINE_API virtual bool LoadSourceActors(bool& bOutDirty, UWorld* TargetWorld) const override;
	ENGINE_API virtual void ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const override;
	ENGINE_API virtual bool IsHLODRelevant(AActor* InActor) const override;

	ENGINE_API void SetSourceLevel(const UWorld* InSourceLevel);
	ENGINE_API const TSoftObjectPtr<UWorld>& GetSourceLevel() const;
#endif

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TSoftObjectPtr<UWorld> SourceLevel;
#endif
};
