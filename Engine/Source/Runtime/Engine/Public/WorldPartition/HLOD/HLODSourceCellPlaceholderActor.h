// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Misc/Guid.h"

#include "HLODSourceCellPlaceholderActor.generated.h"

class FWorldPartitionActorDescInstance;

#if WITH_EDITOR
enum class EHLODSourceCellPlaceholderType : uint8
{
	None,
	CustomHLOD,
	StandaloneHLOD
};
#endif

UCLASS(MinimalAPI, NotPlaceable, Transient)
class AWorldPartitionHLODSourceCellPlaceholder : public AActor
{
	GENERATED_BODY()

public:
	ENGINE_API AWorldPartitionHLODSourceCellPlaceholder(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	void InitFrom(const FWorldPartitionActorDescInstance* InHLODActorDescInstance);
	virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const override;
	ENGINE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	ENGINE_API const FGuid& GetHLODActorGuid() const;
	ENGINE_API EHLODSourceCellPlaceholderType GetPlaceholderType() const;
	void SetPlaceholderType(EHLODSourceCellPlaceholderType InPlaceholderType);

	UE_DEPRECATED(5.8, "GetCustomHLODActorGuid has been renamed to GetHLODActorGuid")
	const FGuid& GetCustomHLODActorGuid() const { return GetHLODActorGuid(); }

private:
	FGuid HLODActorGuid;
	EHLODSourceCellPlaceholderType PlaceholderType = EHLODSourceCellPlaceholderType::None;
	const FWorldPartitionActorDescInstance* HLODActorDescInstance;
#endif
};
