// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/HLOD/HLODSourceCellPlaceholderActor.h"

#if WITH_EDITOR
class FHLODSourceCellPlaceholderActorDesc : public FWorldPartitionActorDesc
{
public:
	virtual void Init(const AActor* InActor) override;
	virtual bool Equals(const FWorldPartitionActorDesc* Other) const override;
	const FGuid& GetHLODActorGuid() const;
	EHLODSourceCellPlaceholderType GetPlaceholderType() const;

protected:
	virtual uint32 GetSizeOf() const override;
	virtual void Serialize(FArchive& Ar) override;

	FGuid HLODActorGuid;
	EHLODSourceCellPlaceholderType PlaceholderType = EHLODSourceCellPlaceholderType::None;
};
#endif
