// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


struct FWorldPartitionHLODStats
{
	// Input
	static ENGINE_API const FName InputActorCount;
	static ENGINE_API const FName InputTriangleCount;
	static ENGINE_API const FName InputVertexCount;

	// Mesh
	static ENGINE_API const FName MeshInstanceCount;
	static ENGINE_API const FName MeshNaniteTriangleCount;
	static ENGINE_API const FName MeshNaniteVertexCount;
	static ENGINE_API const FName MeshTriangleCount;
	static ENGINE_API const FName MeshVertexCount;
	static ENGINE_API const FName MeshUVChannelCount;

	// Material
	static ENGINE_API const FName MaterialBaseColorTextureSize;
	static ENGINE_API const FName MaterialNormalTextureSize;
	static ENGINE_API const FName MaterialEmissiveTextureSize;
	static ENGINE_API const FName MaterialMetallicTextureSize;
	static ENGINE_API const FName MaterialRoughnessTextureSize;
	static ENGINE_API const FName MaterialSpecularTextureSize;
	
	// Memory
	static ENGINE_API const FName MemoryMeshResourceSizeBytes;
	static ENGINE_API const FName MemoryTexturesResourceSizeBytes;
	static ENGINE_API const FName MemoryDiskSizeBytes;

	// Build
	static ENGINE_API const FName BuildTimeLoadMilliseconds;
	static ENGINE_API const FName BuildTimeBuildMilliseconds;
	static ENGINE_API const FName BuildTimeTotalMilliseconds;
};
