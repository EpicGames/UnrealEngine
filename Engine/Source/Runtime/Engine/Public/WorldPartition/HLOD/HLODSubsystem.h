// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WorldPartition/HLOD/HLODRuntimeSubsystem.h"


UE_DEPRECATED(5.4, "Use UWorldPartitionHLODRuntimeSubsystem")
typedef UWorldPartitionHLODRuntimeSubsystem UHLODSubsystem;
