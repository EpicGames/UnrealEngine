// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Containers/Set.h"
#include "Misc/Guid.h"

class ULevel;
class UObject;

class IWorldPartitionHLODObject
{
public:
	/** Returns the associated UObject if any (nullptr if none) */
	virtual UObject* GetUObject() const = 0;

	/** Returns the HLOD object associated level */
	virtual ULevel* GetHLODLevel() const = 0;

	/** Return the name or the label of the HLOD object */
	virtual FString GetHLODNameOrLabel() const = 0;

	/** Returns whether the HLOD object requires warmup */
	virtual bool DoesRequireWarmup() const = 0;

	/** Gather the list of assets that should be warmed up before this HLOD object is made visible */
	virtual TSet<UObject*> GetAssetsToWarmup() const = 0;

	/** Changes the visibility of the HLOD object */
	virtual void SetVisibility(bool bIsVisible) = 0;

	/** Returns the associated source cell guid of this HLOD object. */
	virtual const FGuid& GetSourceCellGuid() const = 0;

	/** Returns whether the HLOD object is part of Standalone HLOD. */
	virtual bool IsStandalone() const = 0;

	/** Returns Standalone HLOD Guid */
	virtual const FGuid& GetStandaloneHLODGuid() const = 0;

	/** Returns whether the HLOD objects represents custom HLOD */
	virtual bool IsCustomHLOD() const = 0;

	/** Returns Custom HLOD Guid */
	virtual const FGuid& GetCustomHLODGuid() const = 0;
};