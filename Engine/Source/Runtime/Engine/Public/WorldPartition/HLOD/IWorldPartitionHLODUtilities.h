// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionStreamingGenerationContext.h"
#include "WorldPartition/DataLayer/DataLayersID.h"

#if WITH_EDITOR

#include "WorldPartition/HLOD/HLODRebuildPolicy.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"

class AActor;
class UWorldPartition;
class UHLODLayer;
class UHLODBuilder;
class UHLODBuilderSettings;
class AWorldPartitionHLOD;

struct IHLODCreationFilter;

struct FHLODCreationContext
{
	TMap<FName, FWorldPartitionHandle> HLODActorDescs;
	TArray<FWorldPartitionReference> ActorReferences;
	TMap<FName, TWeakObjectPtr<AWorldPartitionHLOD>> UnsavedHLODActors;
};

struct FHLODCreationParams
{
	UWorldPartition* WorldPartition;
	UWorld* TargetWorld;

	FGuid CellGuid;
	FString CellName;
	TUniqueFunction<FName(const UHLODLayer*)> GetRuntimeGrid;
	uint32 HLODLevel;
	FGuid ContentBundleGuid;
	TArray<const UDataLayerInstance*> DataLayerInstances;
	bool bIsStandalone;
	TArray<TSharedPtr<IHLODCreationFilter>> Filters;
	FName HLODResourcesFolder;

	const UExternalDataLayerAsset* GetExternalDataLayerAsset() const
	{
		auto IsAnExternalDataLayerPred = [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsA<UExternalDataLayerInstance>(); };
		if (const UDataLayerInstance* const* ExternalDataLayerInstance = DataLayerInstances.FindByPredicate(IsAnExternalDataLayerPred))
		{
			return CastChecked<UExternalDataLayerInstance>(*ExternalDataLayerInstance)->GetExternalDataLayerAsset();
		}
		return nullptr;
	}

	double MinVisibleDistance;
	
	UE_DEPRECATED(5.7, "CellBounds member is not used anymore.")
	FBox CellBounds;
};

struct FHLODBuildParams
{
	AWorldPartitionHLOD* HLODActor = nullptr;
	bool bForceBuild = false;
	bool bTestOnly = false;
};

/**
 * Tools for building HLODs in WorldPartition
 */
class IWorldPartitionHLODUtilities
{
public:
	virtual ~IWorldPartitionHLODUtilities() = default;

	/**
	 * Create HLOD actors for a given cell
	 *
	 * @param	InCreationContext	HLOD creation context object
	 * @param	InCreationParams	HLOD creation parameters object
	 * @param	InActors			The actors for which we'll build an HLOD representation
	 */
	virtual TArray<AWorldPartitionHLOD*> CreateHLODActors(FHLODCreationContext& InCreationContext, const FHLODCreationParams& InCreationParams, const TArray<IStreamingGenerationContext::FActorInstance>& InActors) = 0;

	/**
	 * Build HLOD for the specified AWorldPartitionHLOD actor.
	 *
	 * @param 	InBuildParams		Parameters to use for the HLOD build
	 * @return True if the HLOD actor was updated, false otherwise.
	 */
	virtual bool BuildHLOD(const FHLODBuildParams& InBuildParams) = 0;

	/**
	 * Retrieve the HLOD Builder class to use for the given HLODLayer.
	 * 
	 * @param	InHLODLayer		HLODLayer
	 * @return The HLOD builder subclass to use for building HLODs for the provided HLOD layer.
	 */
	virtual TSubclassOf<UHLODBuilder> GetHLODBuilderClass(const UHLODLayer* InHLODLayer) = 0;

	/**
	 * Create the HLOD builder settings for the provided HLOD layer object. The type of settings created will depend on the HLOD layer type.
	 *
	 * @param 	InHLODLayer		The HLOD layer for which we'll create a setting object
	 * @return A newly created UHLODBuilderSettings object, outered to the provided HLOD layer.
	 */
	virtual UHLODBuilderSettings* CreateHLODBuilderSettings(UHLODLayer* InHLODLayer) = 0;

	/**
	 * HLOD build evaluator delegate
	 * @param HLODActor			The HLOD actor to be rebuilt.
	 * @param OldPolicyDataSet	The previously stored policy data set of the inputs to the HLOD build.
	 * @param NewPolicyDataSet	The newly computed policy data set of the inputs to the HLOD build.
	 * @return The HLOD rebuild decision
	 */
	DECLARE_DELEGATE_RetVal_ThreeParams(EHLODRebuildPolicyDecision, FHLODBuildEvaluator, AWorldPartitionHLOD*, const FHLODRebuildPolicyDataSet&, const FHLODRebuildPolicyDataSet&);

	/**
	 * Provide a delegate that will be used to evaluate whether an HLOD build should be performed.
	 * @param The evaluator delegate.
	 */
	virtual void SetHLODBuildEvaluator(FHLODBuildEvaluator BuildEvaluatorDelegate) = 0;

	UE_DEPRECATED(5.2, "Use the overload that passes the DataLayersInstances via InCreationParams")
	virtual TArray<AWorldPartitionHLOD*> CreateHLODActors(FHLODCreationContext& InCreationContext, const FHLODCreationParams& InCreationParams, const TArray<IStreamingGenerationContext::FActorInstance>& InActors, const TArray<const UDataLayerInstance*>& InDataLayerInstances)
	{
		const_cast<FHLODCreationParams&>(InCreationParams).DataLayerInstances = InDataLayerInstances;
		return CreateHLODActors(InCreationContext, InCreationParams, InActors);
	}

	UE_DEPRECATED(5.8, "Use the overload that passes a FHLODBuildParams")
	virtual bool BuildHLOD(AWorldPartitionHLOD* InHLODActor) final
	{
		FHLODBuildParams Params;
		Params.HLODActor = InHLODActor;
		Params.bForceBuild = false;
		return BuildHLOD(Params);
	}
};
#endif
