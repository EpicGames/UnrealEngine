// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilities.h"

#if WITH_EDITOR

class IWorldPartitionHLODUtilitiesModule : public IModuleInterface
{
public:
	virtual IWorldPartitionHLODUtilities* GetUtilities() = 0;
};

#endif
