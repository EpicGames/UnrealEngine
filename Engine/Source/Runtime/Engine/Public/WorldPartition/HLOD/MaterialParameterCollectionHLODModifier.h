// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/HLOD/HLODModifier.h"
#include "WorldPartition/HLOD/HLODBuilder.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "MaterialParameterCollectionHLODModifier.generated.h"

class UMaterialParameterCollection;

/** A scalar parameter */
USTRUCT()
struct FHLODModifierScalarParameter
{
	GENERATED_USTRUCT_BODY()

	FHLODModifierScalarParameter()
	: OverrideValue(0.0f)
	{
		ParameterName = FName("Scalar");
	}

	UPROPERTY(EditAnywhere, Category=Parameter)
	FName ParameterName;

	UPROPERTY(EditAnywhere, Category=Parameter)
	float OverrideValue;
};


UCLASS(BlueprintType, Blueprintable)
class UMaterialParameterCollectionHLODModifier : public UWorldPartitionHLODModifier
{
	GENERATED_UCLASS_BODY()

	virtual bool CanModifyHLOD(TSubclassOf<UHLODBuilder> InHLODBuilderClass) const;
	virtual void BeginHLODBuild(const FHLODBuildContext& InHLODBuildContext);
	virtual void EndHLODBuild(TArray<UActorComponent*>& InOutComponents);


	UPROPERTY(EditAnywhere, Category=Material, DisplayName="Material Parameter Collection")
	TObjectPtr<UMaterialParameterCollection> MPC;

	UPROPERTY(EditAnywhere, Category=Material, Meta = (TitleProperty = "ParameterName"))
	TArray<FHLODModifierScalarParameter> ScalarParameters;

private:
	void ApplyScalarParameterOverrides(UMaterialParameterCollectionInstance* Instance);
	void RestoreScalarParameterValues(UMaterialParameterCollectionInstance* Instance);

	const FHLODBuildContext* HLODBuildContext;
	TArray<float> CachedScalarParameterValues;
};
