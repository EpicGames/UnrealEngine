// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelInstance/LevelInstanceActorGuid.h"
#include "LevelInstance/LevelInstanceActorImpl.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "StandaloneHLODActor.generated.h"

UCLASS(NotPlaceable, MinimalAPI, HideCategories=(Rendering, Replication, Collision, Physics, Navigation, Networking, Input, Actor, LevelInstance, Cooking))
class AWorldPartitionStandaloneHLOD : public AActor, public ILevelInstanceInterface
{
	GENERATED_BODY()

public:
	ENGINE_API AWorldPartitionStandaloneHLOD(const FObjectInitializer& ObjectInitializer);

	//~ Begin ILevelInstanceInterface
	ENGINE_API virtual const FLevelInstanceID& GetLevelInstanceID() const override;
	ENGINE_API virtual bool HasValidLevelInstanceID() const override;
	ENGINE_API virtual const FGuid& GetLevelInstanceGuid() const override;
	ENGINE_API virtual const TSoftObjectPtr<UWorld>& GetWorldAsset() const override;
	ENGINE_API virtual bool IsLoadingEnabled() const override;
	ENGINE_API virtual bool SetWorldAsset(TSoftObjectPtr<UWorld> InWorldAsset) override;

#if WITH_EDITOR
	ENGINE_API virtual ULevelInstanceComponent* GetLevelInstanceComponent() const override;
	ENGINE_API virtual ELevelInstanceRuntimeBehavior GetDesiredRuntimeBehavior() const override;
	ENGINE_API virtual ELevelInstanceRuntimeBehavior GetDefaultRuntimeBehavior() const override;
#endif // WITH_EDITOR
	//~ End ILevelInstanceInterface

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual bool CanEditChangeComponent(const UActorComponent* Component, const FProperty* InProperty) const override;
#endif
	//~ End UObject Interface.

	//~ Begin AActor Interface.
	ENGINE_API void PostRegisterAllComponents() override;
	ENGINE_API void PostUnregisterAllComponents() override;
	
#if WITH_EDITOR
	ENGINE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
#endif
	//~ End AActor Interface.

protected:
	UPROPERTY()
	TSoftObjectPtr<UWorld> WorldAsset;
	
	UPROPERTY(Transient)
	FGuid LevelInstanceSpawnGuid;

private:
	FLevelInstanceActorGuid LevelInstanceActorGuid;
	FLevelInstanceActorImpl LevelInstanceActorImpl;
};