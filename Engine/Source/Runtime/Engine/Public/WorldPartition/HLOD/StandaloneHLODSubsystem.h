// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "WorldPartition/WorldPartitionActorDesc.h"

#include "StandaloneHLODSubsystem.generated.h"

class AWorldPartitionStandaloneHLOD;
class FLevelInstanceActorDesc;
class UActorDescContainerInstance;

UCLASS(MinimalAPI)
class UWorldPartitionStandaloneHLODSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

#if WITH_EDITOR
private:
	struct FStandaloneHLODActorParams
	{
		FGuid Guid;
		FTransform Transform;
		FString WorldPackageName;
		FString ActorLabel;
	};

public:

	//~ Begin USubsystem Interface.
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;
	//~ End USubsystem Interface.

	//~ Begin UWorldSubsystem Interface.
	ENGINE_API virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override;
	ENGINE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	//~ End UWorldSubsystem Interface.

	ENGINE_API void UpdateStandaloneHLODActors(FStandaloneHLODActorParams InStandaloneHLODActorParams);
	ENGINE_API void DeleteStandaloneHLODActors(FGuid InGuid);

	ENGINE_API void UpdateStandaloneHLODActorsRecursive(const FLevelInstanceActorDesc& InLevelInstanceActorDesc, const FTransform InActorTransform, bool bChildrenOnly);
	ENGINE_API void DeleteStandaloneHLODActorsRecursive(const FLevelInstanceActorDesc& InLevelInstanceActorDesc);

	ENGINE_API void ForEachStandaloneHLODActor(TFunctionRef<void(AWorldPartitionStandaloneHLOD*)> Func) const;
	ENGINE_API void ForEachStandaloneHLODActorWithLevelInstanceGuid(TFunctionRef<void(AWorldPartitionStandaloneHLOD*, const FGuid& LevelInstanceGuid)> Func) const;
	ENGINE_API void ForEachStandaloneHLODActorFiltered(FGuid InGuid, TFunctionRef<void(AWorldPartitionStandaloneHLOD*)> Func) const;

	static ENGINE_API bool GetStandaloneHLODFolderPathAndPackagePrefix(const FString& InWorldPackageName, FString& OutFolderPath, FString& OutPackagePrefix);

	ENGINE_API bool GetChildHLODActorsContainer(const FActorContainerID& InContainerID, FActorContainerID& OutChildContainerID) const;

private:
	void OnWorldPartitionInitialized(UWorldPartition* InWorldPartition);
	void OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition);

	void OnActorChanged(AActor* InActor);
	void OnActorDeleted(AActor* InActor);
	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

	void OnActorDescContainerInstanceRegistered(UActorDescContainerInstance* InContainerInstance);
	void OnActorDescContainerInstanceUnregistered(UActorDescContainerInstance* InContainerInstance);

	void OnLevelAddedToWorld(ULevel* Level, UWorld* World);

	TMap<FGuid, TArray<AWorldPartitionStandaloneHLOD*>> StandaloneHLODActors;
	TMap<FActorContainerID, FActorContainerID> HLODContainerToChildHLODActorsContainer;

	TMap<FName, TMap<int32, FName>> CachedHLODSetups;
	bool bRefreshCachedHLODSetups;
#endif
};