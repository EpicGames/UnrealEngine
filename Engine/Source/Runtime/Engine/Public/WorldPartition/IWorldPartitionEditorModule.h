// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SubclassOf.h"

class UWorldPartitionBuilder;

/**
 * The module holding all of the UI related pieces for WorldPartition
 */
class IWorldPartitionEditorModule : public IModuleInterface
{
public:
	virtual ~IWorldPartitionEditorModule() = default;

	static ENGINE_API IWorldPartitionEditorModule& Get();

	virtual bool ConvertMap(const FString& InLongPackageName) = 0;

	UE_DEPRECATED(5.1, "Use RunBuilder with UWorld* instead.")
	virtual bool RunBuilder(TSubclassOf<UWorldPartitionBuilder> BuilderClass, const FString& InLongPackageName) { return false; }
		
	struct FRunBuilderParams
	{
		TSubclassOf<UWorldPartitionBuilder> BuilderClass;
		UWorld* World = nullptr;
		FString ExtraArgs;
		FText OperationDescription;
	};

	virtual bool RunBuilder(TSubclassOf<UWorldPartitionBuilder> BuilderClass, UWorld* InWorld);
	virtual bool RunBuilder(const FRunBuilderParams& Params) = 0;

	virtual int32 GetPlacementGridSize() const = 0;
	virtual int32 GetInstancedFoliageGridSize() const = 0;
	virtual int32 GetMinimapLowQualityWorldUnitsPerPixelThreshold() const = 0;
	
	virtual bool GetEnableLoadingInEditor() const = 0;
	virtual void SetEnableLoadingInEditor(bool bInEnableLoadingInEditor) = 0;

	virtual bool GetEnableStreamingGenerationLogOnPIE() const = 0;
	virtual void SetEnableStreamingGenerationLogOnPIE(bool bInEnableLoadingInEditor) = 0;

	virtual bool GetDisablePIE() const = 0;
	virtual void SetDisablePIE(bool bInDisablePIE) = 0;

	virtual bool GetDisableBugIt() const = 0;
	virtual void SetDisableBugIt(bool bInDisableBugIt) = 0;

	virtual bool GetAdvancedMode() const = 0;
	virtual void SetAdvancedMode(bool bInAdvancedMode) = 0;

	virtual bool GetRequireExplicitHLODLayerPartitionAssignation() const = 0;
	virtual void SetRequireExplicitHLODLayerPartitionAssignation(bool bInRequireExplicitHLODLayerPartitionAssignation) = 0;

	virtual bool IsEditingContentBundle() const = 0;
	virtual bool IsEditingContentBundle(const FGuid& ContentBundleGuid) const = 0;

	virtual bool GetShowHLODsInEditor() const = 0;
	virtual void SetShowHLODsInEditor(bool bInShowHLODsInEditor) = 0;

	virtual bool GetShowHLODsOverLoadedRegions() const = 0;
	virtual void SetShowHLODsOverLoadedRegions(bool bInShowHLODsOverLoadedRegions) = 0;

	virtual double GetHLODInEditorMinDrawDistance() const = 0;
	virtual void SetHLODInEditorMinDrawDistance(double InMinDrawDistance) = 0;

	virtual double GetHLODInEditorMaxDrawDistance() const = 0;
	virtual void SetHLODInEditorMaxDrawDistance(double InMaxDrawDistance) = 0;

	virtual bool IsHLODInEditorAllowed(UWorld* InWorld, FText* OutDisallowedReason = nullptr) const = 0;

	/** Returns if the partitioned world contains errors. */
	virtual bool HasErrors(UWorld* World) const = 0;

	/** Repairs detected errors in the partitioned world. */
	virtual void RepairErrors(UWorld* World) = 0;

	/**
     * Parameters for the WriteHLODStats() method.
     */	
	struct FWriteHLODStatsParams
	{
		enum class EStatsType : uint8
		{
			Default,
			InputDetails
		};

		UWorld* World = nullptr;
		FString Filename;
		EStatsType StatsType = EStatsType::Default;
	};

	/** Writes various HLOD stats to a file. */
	virtual bool WriteHLODStats(const FWriteHLODStatsParams& Params) const = 0;

	/** Triggered when a world is added. */
	DECLARE_EVENT_OneParam(IWorldPartitionEditorModule, FWorldPartitionCreated, UWorld*);

	/** Triggered when the editor is about to launch a commandlet. Can be used to modify the builder params. */
	DECLARE_EVENT_OneParam(IWorldPartitionEditorModule, FOnPreExecuteCommandlet, FRunBuilderParams&);

	/** Triggered when the editor launches a commandlet. Can be used to provide project specific arguments. */
	DECLARE_EVENT_OneParam(IWorldPartitionEditorModule, FOnExecuteCommandlet, TArray<FString>&);

	/** Triggered when the editor has launched a commandlet. */
	DECLARE_EVENT(IWorldPartitionEditorModule, FOnPostExecuteCommandlet);

	/** Return the world added event. */
	virtual FWorldPartitionCreated& OnWorldPartitionCreated() = 0;

	/** Return the commandlet pre-execution event */
	virtual FOnPreExecuteCommandlet& OnPreExecuteCommandlet() = 0;

	/** Return the commandlet execution event */
	virtual FOnExecuteCommandlet& OnExecuteCommandlet() = 0;

	/** Return the commandlet post-execution event */
	virtual FOnPostExecuteCommandlet& OnPostExecuteCommandlet() = 0;
};
