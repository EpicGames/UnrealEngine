// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "IWorldPartitionObjectResolver.generated.h"

USTRUCT()
struct FWorldPartitionResolveData
{
	GENERATED_USTRUCT_BODY()

	FWorldPartitionResolveData() {}

	UE_DEPRECATED(5.4, "Please use FTopLevelAssetPath for the asset path directly")
	FWorldPartitionResolveData(const FActorContainerID& InContainerID, const FString& InSourceWorldAssetPath)
		: ContainerID(InContainerID), SourceWorldAssetPath(InSourceWorldAssetPath) {}

	FWorldPartitionResolveData(const FActorContainerID& InContainerID, const FTopLevelAssetPath& InSourceWorldAssetPath)
		: ContainerID(InContainerID), SourceWorldAssetPath(InSourceWorldAssetPath) {}

	bool IsValid() const { return !ContainerID.IsMainContainer(); }
		
	ENGINE_API bool ResolveObject(UWorld* InWorld, const FSoftObjectPath& InObjectPath, UObject*& OutObject) const;

	UPROPERTY()
	FActorContainerID ContainerID;

	UPROPERTY()
	FTopLevelAssetPath SourceWorldAssetPath;
};

UINTERFACE(MinimalAPI)
class UWorldPartitionObjectResolver
	: public UInterface
{
public:
	GENERATED_BODY()
};

class IWorldPartitionObjectResolver
{
public:
	GENERATED_BODY()

#if WITH_EDITOR
	virtual void SetWorldPartitionResolveData(const FWorldPartitionResolveData& InEmbeddingData) = 0;
#endif
	virtual const FWorldPartitionResolveData& GetWorldPartitionResolveData() const = 0;
};
