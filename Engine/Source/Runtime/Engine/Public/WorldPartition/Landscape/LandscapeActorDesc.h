// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "WorldPartition/ActorPartition/PartitionActorDesc.h"

/**
 * ActorDesc for LandscapeActors.
 */
class FLandscapeActorDesc : public FPartitionActorDesc
{
public:
	ENGINE_API virtual void Init(const AActor* InActor) override;
	ENGINE_API virtual bool Equals(const FWorldPartitionActorDesc* Other) const override;
	ENGINE_API virtual const FGuid& GetSceneOutlinerParent() const override;

protected:
	virtual uint32 GetSizeOf() const override { return sizeof(FLandscapeActorDesc); }
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

private:
	FGuid LandscapeActorGuid;
};
#endif
