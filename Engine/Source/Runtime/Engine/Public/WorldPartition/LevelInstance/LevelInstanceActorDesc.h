// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectPtr.h"
#include "Containers/Map.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/HLOD/StandaloneHLODActor.h"
#include "WorldPartition/LevelInstance/LevelInstancePropertyOverrideDesc.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"

class ULevelInstanceSubsystem;
class FWorldPartitionActorDescInstance;
class IStreamingGenerationErrorHandler;
class IWorldPartitionActorDescInstanceView;
class UActorDescContainerInstance;
enum class ELevelInstanceRuntimeBehavior : uint8;

/**
 * ActorDesc for Actors that are part of a LevelInstanceActor Level.
 */
class FLevelInstanceActorDesc : public FWorldPartitionActorDesc
{
public:
	ENGINE_API FLevelInstanceActorDesc();
	ENGINE_API virtual ~FLevelInstanceActorDesc() override;
		
	ENGINE_API virtual bool IsChildContainerInstance() const override;

	virtual EWorldPartitionActorFilterType GetChildContainerFilterType() const { return IsChildContainerInstance() ? EWorldPartitionActorFilterType::Loading : EWorldPartitionActorFilterType::None; }
	virtual FName GetChildContainerPackage() const override { return WorldAsset.GetLongPackageFName(); }
	virtual FString GetChildContainerName() const override;
	virtual const FWorldPartitionActorFilter* GetChildContainerFilter() const override { return &Filter; }
		
	ENGINE_API virtual void CheckForErrors(const IWorldPartitionActorDescInstanceView* InActorDescView, IStreamingGenerationErrorHandler* ErrorHandler) const override;

	ENGINE_API FTransform GetChildContainerTransform() const;
	virtual UActorDescContainer* GetChildContainer() const override { check(IsChildContainerInstance()); return ChildContainer.Get(); }

	virtual bool HasStandaloneHLOD() const override { return bHasStandaloneHLOD; }
	virtual ELevelInstanceRuntimeBehavior GetDesiredRuntimeBehavior() const { return DesiredRuntimeBehavior; }

protected:
	ENGINE_API virtual void Init(const AActor* InActor) override;
	ENGINE_API virtual void Init(const FWorldPartitionActorDescInitData& DescData) override;
	ENGINE_API virtual bool Equals(const FWorldPartitionActorDesc* Other) const override;
	ENGINE_API virtual void TransferFrom(const FWorldPartitionActorDesc* From) override;
	virtual uint32 GetSizeOf() const override { return sizeof(FLevelInstanceActorDesc); }
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void SetContainer(UActorDescContainer* InContainer) override;
	ENGINE_API virtual UActorDescContainerInstance* CreateChildContainerInstance(const FWorldPartitionActorDescInstance* InActorDescInstance) const override;
	ENGINE_API virtual bool GetChildContainerInstance(const FWorldPartitionActorDescInstance* InActorDescInstance, FContainerInstance& OutContainerInstance) const override;
	ENGINE_API virtual UWorldPartition* GetLoadedChildWorldPartition(const FWorldPartitionActorDescInstance* InActorDescInstance) const override;
	FLevelInstancePropertyOverrideDesc* GetOverrideDesc() const { return PropertyOverrideDesc.Get(); }
	ENGINE_API static bool ValidateCircularReference(const UActorDescContainerInstance* InParentContainer, FName InChildContainerPackage);

	FSoftObjectPath WorldAsset;
	ELevelInstanceRuntimeBehavior DesiredRuntimeBehavior;

	TWeakObjectPtr<UActorDescContainer> ChildContainer;

	FWorldPartitionActorFilter Filter;
	bool bIsChildContainerInstance;
	bool bHasStandaloneHLOD;

	FSoftObjectPath PropertyOverrideAsset;
	TSharedPtr<FLevelInstancePropertyOverrideDesc> PropertyOverrideDesc;
private:
	ENGINE_API bool IsChildContainerInstanceInternal() const;
	ENGINE_API void RegisterChildContainer();
	ENGINE_API void UnregisterChildContainer();
	ENGINE_API void UpdateBounds();
};
#endif
