// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "Misc/Optional.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "LevelInstance/LevelInstancePropertyOverrideAsset.h"
#include "LevelInstanceContainerInstance.generated.h"

class ULevelInstancePropertyOverrideAsset;
struct FWorldPartitionRuntimeCellPropertyOverride;

UCLASS(MinimalAPI)
class ULevelInstanceContainerInstance : public UActorDescContainerInstance
{
	GENERATED_BODY()

	ULevelInstanceContainerInstance() {}
	virtual ~ULevelInstanceContainerInstance() = default;
	
#if WITH_EDITOR
protected:
	friend class FLevelInstanceActorDesc;
	friend class ULevelStreamingLevelInstance;
	friend class ULevelStreamingLevelInstanceEditorPropertyOverride;
	friend class ULevelInstanceSubsystem;

	ENGINE_API void SetOverrideContainerAndAsset(UActorDescContainer* InOverrideContainer, ULevelInstancePropertyOverrideAsset* InAsset);
	
	virtual void Initialize(const FInitializeParams& InParams) override;
	virtual void Uninitialize();

	virtual void GetPropertyOverridesForActor(const FActorContainerID& InContainerID, const FGuid& InActorGuid, TArray<FWorldPartitionRuntimeCellPropertyOverride>& OutPropertyOverrides) const override;
	ENGINE_API void GetPropertyOverridesForActor(const FActorContainerID& InContainerID, const FActorContainerID& InContextContainerID, const FGuid& InActorGuid, TArray<FLevelInstanceActorPropertyOverride>& OutPropertyOverrides) const;

	virtual void RegisterContainer(const FInitializeParams& InParams) override;
	virtual void UnregisterContainer() override;
	
	virtual FWorldPartitionActorDesc* GetActorDesc(const FGuid& InActorGuid) const override;
	virtual FWorldPartitionActorDesc* GetActorDescChecked(const FGuid& InActorGuid) const override;

	FWorldPartitionActorDesc* GetOverrideActorDesc(const FGuid& InActorGuid, const FActorContainerPath& Path = FActorContainerPath()) const;

	FWorldPartitionActorDescInstance CreateActorDescInstance(FWorldPartitionActorDesc* InActorDesc) override { return FWorldPartitionActorDescInstance(this, GetActorDescChecked(InActorDesc->GetGuid())); }
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UActorDescContainer> OverrideContainer;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UActorDescContainer>> ParentContainerReferences;

	// This is a Weak Ptr because parent level instance can get unloaded first and we don't want to cause a leak as we are going to get unloaded afterwards anyways
	UPROPERTY(Transient)
	TWeakObjectPtr<ULevelInstancePropertyOverrideAsset> PropertyOverrideAsset;
		
	UPROPERTY(Transient)
	TMap<FActorContainerID, FActorContainerPath> ContainerIDToContainerPath;
#endif
};
