// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "WorldPartition/WorldPartitionActorDesc.h"
#include "LevelInstance/LevelInstancePropertyOverrideAsset.h"

struct FAssetData;

// Shared so that existing containers can keep it alive even if owning LevelInstance might have released it
class FLevelInstancePropertyOverrideDesc : public TSharedFromThis<FLevelInstancePropertyOverrideDesc>
{
	friend class FLevelInstanceActorDesc;
	friend class ULevelInstancePropertyOverrideAsset;
	friend class ULevelInstancePropertyOverrideContainer;
	friend class ULevelInstanceContainerInstance;
public:
	virtual ~FLevelInstancePropertyOverrideDesc();
		
protected:
	FName GetWorldPackage() const { return WorldAsset.GetLongPackageFName(); }
	FSoftObjectPath GetAssetPath() const { return AssetPath; }
	FName GetAssetPackage() const { return PackageName; }

	void Init(const ULevelInstancePropertyOverrideAsset* InPropertyOverride);
	friend FArchive& operator<<(FArchive& Ar, FLevelInstancePropertyOverrideDesc& InPropertyOverrideDesc);
	void TransferNonEditedContainers(const FLevelInstancePropertyOverrideDesc* InExistingOverrideDesc);

	// Returns the overriden ActorDesc for an Actor part of BaseContainer or any of its Child containers
	FWorldPartitionActorDesc* GetOverrideActorDesc(const FGuid& InActorGuid, const FActorContainerPath& InContainerPath);
	const FWorldPartitionActorDesc* GetOverrideActorDesc(const FGuid& InActorGuid, const FActorContainerPath& InContainerPath) const;

	void SetContainerForActorDescs(UActorDescContainer* InContainer);

	const TMap<FActorContainerPath, TMap<FGuid, TSharedPtr<FWorldPartitionActorDesc>>>& GetActorDescsPerContainer() const { return ActorDescsPerContainer; }
	
	void SerializeTo(TArray<uint8>& OutPayload);
	void SerializeFrom(const TArray<uint8>& InPayload);

	// Utility methods to find Base ActorDescs
	const FWorldPartitionActorDesc* GetBaseDescByGuid(const FActorContainerPath& InContainerPath, const FGuid& InActorGuid) const;
	const UActorDescContainer* GetBaseContainer(const UActorDescContainer* InContainer, const FActorContainerPath& InContainerPath) const;

	FString GetContainerName() const;
	static FString GetContainerNameFromAssetPath(const FSoftObjectPath& InAssetPath);
	static FString GetContainerNameFromAsset(ULevelInstancePropertyOverrideAsset* InAsset);

	UActorDescContainer* GetBaseContainer() const { return BaseContainer; }
private:
	FSoftObjectPath AssetPath;
	FSoftObjectPath WorldAsset;
	FName PackageName;
		
	UActorDescContainer* BaseContainer = nullptr;

	// Override ActorDescs are stored as shared pointers because we have FLevelInstanceActorDesc::Init which can be called multiple times for an actor (Every time CreateActorDesc is called)
	// This will result in the new FLevelInstanceActorDesc copying some ActorDescs from the previous FLevelInstanceActorDesc (the one still in the parent Container)
	// For this operation to not trash the previous FLevelInstanceActorDesc we need to share those ptrs
	TMap<FActorContainerPath, TMap<FGuid, TSharedPtr<FWorldPartitionActorDesc>>> ActorDescsPerContainer;
};

#endif