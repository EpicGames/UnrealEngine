// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterSpatial.h"

#if WITH_EDITOR
/** Actor loader adapter that will do a spatial query based on the actor's brush  */
class FLoaderAdapterActor : public ILoaderAdapterSpatial
{
public:
	ENGINE_API FLoaderAdapterActor(AActor* InActor);

	//~ Begin IWorldPartitionActorLoaderInterface::ILoader interface
	ENGINE_API virtual TOptional<FBox> GetBoundingBox() const override;
	ENGINE_API virtual TOptional<FString> GetLabel() const override;
	//~ End IWorldPartitionActorLoaderInterface::ILoader interface

protected:
	//~ Begin IWorldPartitionActorLoaderInterface::ILoaderAdapterSpatial interface
	ENGINE_API virtual bool Intersect(const FBox& Box) const override;
	//~ End IWorldPartitionActorLoaderInterface::ILoaderAdapterSpatial interface

	AActor* Actor;
};
#endif
