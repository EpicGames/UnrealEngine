// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterList.h"

/** Loader adapter that contains an actor list. */
class FLoaderAdapterActorList: public FLoaderAdapterList
{
public:
	ENGINE_API FLoaderAdapterActorList(UWorld* InWorld);

	ENGINE_API void AddActors(const TArray<FGuid>& ActorGuids);
	ENGINE_API void AddActors(const TArray<FWorldPartitionHandle>& ActorHandles);

	ENGINE_API void RemoveActors(const TArray<FGuid>& ActorGuids);
	ENGINE_API void RemoveActors(const TArray<FWorldPartitionHandle>& ActorHandles);

	ENGINE_API bool ContainsActor(const FGuid& ActorGuid) const;
	ENGINE_API bool ContainsActor(const FWorldPartitionHandle& ActorHandle) const;

protected:
	//~ Begin IWorldPartitionActorLoaderInterface::ILoaderAdapterList interface
	ENGINE_API virtual bool PassActorDescFilter(const FWorldPartitionHandle& ActorHandle) const override;
	//~ End IWorldPartitionActorLoaderInterface::ILoaderAdapterList interface

	TSet<FWorldPartitionHandle> ActorsToRemove;
};
#endif
