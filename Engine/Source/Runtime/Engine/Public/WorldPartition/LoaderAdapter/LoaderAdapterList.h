// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"

#if WITH_EDITOR
/** Base class for actor loaders that contains a specific list of actors */
class FLoaderAdapterList : public IWorldPartitionActorLoaderInterface::ILoaderAdapter
{
public:
	ENGINE_API FLoaderAdapterList(UWorld* InWorld);
	virtual ~FLoaderAdapterList() = default;

	const TSet<FWorldPartitionHandle>& GetActors() const { return Actors; }

protected:
	//~ Begin ILoaderAdapter interface
	ENGINE_API virtual void ForEachActor(TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const override;
	//~ End ILoaderAdapter interface

private:
	ENGINE_API void HandleActorContainer(const FWorldPartitionHandle& InActor, TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const;

protected:

	TSet<FWorldPartitionHandle> Actors;
};
#endif
