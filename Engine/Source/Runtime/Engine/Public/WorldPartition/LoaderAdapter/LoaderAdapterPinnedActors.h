// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "WorldPartition/LoaderAdapter/LoaderAdapterActorList.h"

class AActor;
class FWorldPartitionActorDesc;
class FWorldPartitionActorDescInstance;

class FLoaderAdapterPinnedActors : public FLoaderAdapterActorList
{
public:
	FLoaderAdapterPinnedActors(UWorld* InWorld)
		: FLoaderAdapterActorList(InWorld)
	{}

	UE_DEPRECATED(5.4, "Use FWorldPartitionActorDescInstance version instead")
	static bool SupportsPinning(FWorldPartitionActorDesc* InActorDesc) { return false;}

	static ENGINE_API bool SupportsPinning(FWorldPartitionActorDescInstance* InActorDescInstance);
	static ENGINE_API bool SupportsPinning(AActor* InActor);

protected:
	ENGINE_API virtual bool PassActorDescFilter(const FWorldPartitionHandle& ActorHandle) const override;
};

#endif
