// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"

#if WITH_EDITOR
class UWorldPartition;

/** Base class for actor loaders that requires spatial queries */
class ILoaderAdapterSpatial : public IWorldPartitionActorLoaderInterface::ILoaderAdapter
{
public:
	ENGINE_API ILoaderAdapterSpatial(UWorld* InWorld);
	virtual ~ILoaderAdapterSpatial() = default;

protected:
	//~ Begin ILoaderAdapter interface
	ENGINE_API virtual void ForEachActor(TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const override;
	//~ End ILoaderAdapter interface

	// Private interface
	virtual bool Intersect(const FBox& Box) const =0;

private:
	ENGINE_API void HandleIntersectingContainer(UWorldPartition* InWorldPartition, const FBox& InBoundingBox, TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const;

protected:
	bool bIncludeSpatiallyLoadedActors;
	bool bIncludeNonSpatiallyLoadedActors;
};
#endif
