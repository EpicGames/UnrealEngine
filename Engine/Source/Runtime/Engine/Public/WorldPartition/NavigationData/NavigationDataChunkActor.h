// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorPartition/PartitionActor.h"
#include "Containers/Array.h"
#include "NavigationDataChunkActor.generated.h"

class UNavigationDataChunk;
class UNavigationSystemBase;

UCLASS(NotPlaceable, MinimalAPI)
class ANavigationDataChunkActor : public APartitionActor
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	//~ Begin UObject Interface
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

	//~ Begin APartitionActor Interface
	ENGINE_API virtual uint32 GetDefaultGridSize(UWorld* InWorld) const override;
	//~ End APartitionActor Interface
	
	ENGINE_API void AddNavigationDataChunkInEditor(const UNavigationSystemBase& NavSys);
#endif // WITH_EDITOR

	const TArray<UNavigationDataChunk*>& GetNavDataChunk() const { return NavDataChunks; }
	TArray<TObjectPtr<UNavigationDataChunk>>& GetMutableNavDataChunk() { return NavDataChunks; }

	ENGINE_API void CollectNavData(const FBox& QueryBounds, FBox& OutTilesBounds);

#if WITH_EDITOR
	ENGINE_API void SetDataChunkActorBounds(const FBox& InBounds);

	//~ Begin AActor Interface.
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
	ENGINE_API virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const override;
	//~ End AActor Interface.
#endif // WITH_EDITOR

	virtual FBox GetBounds() const { return DataChunkActorBounds; }
	
protected:
	//~ Begin AActor Interface.
	ENGINE_API virtual void BeginPlay() override;
	ENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	ENGINE_API virtual void GetActorBounds(bool bOnlyCollidingComponents, FVector& OutOrigin, FVector& OutBoxExtent, bool bIncludeFromChildActors) const override;
	//~ End AActor Interface.

	ENGINE_API void AddNavigationDataChunkToWorld();
	ENGINE_API void RemoveNavigationDataChunkFromWorld();
	ENGINE_API void Log(const TCHAR* FunctionName) const;

	UPROPERTY()
	TArray<TObjectPtr<UNavigationDataChunk>> NavDataChunks;

	UPROPERTY()
	FBox DataChunkActorBounds;
};
