// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"

class FPackedLevelActorDesc : public FWorldPartitionActorDesc
{
public:
	FPackedLevelActorDesc() {}
	virtual ~FPackedLevelActorDesc() override = default;

protected:
	ENGINE_API virtual void Init(const AActor* InActor) override;
	ENGINE_API virtual void Init(const FWorldPartitionActorDescInitData& DescData) override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual bool Equals(const FWorldPartitionActorDesc* Other) const override;
	virtual uint32 GetSizeOf() const override { return sizeof(FPackedLevelActorDesc); }

	FSoftObjectPath WorldAsset;
	FWorldPartitionActorFilter ContainerFilter;
};
#endif
