// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionStreamingGenerationContext.h"
#include "RuntimePartition.generated.h"

#define UE_API ENGINE_API

/** Chooses a method for how to compute streaming cells bounds */
UENUM()
enum class ERuntimePartitionCellBoundsMethod : uint8
{
	UseContent,
	UseCellBounds,
	UseMinContentCellBounds
};

UCLASS(MinimalAPI, Abstract, EditInlineNew, CollapseCategories)
class URuntimePartition : public UObject
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	//~ Begin UObject Interface.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject Interface.

	UE_API URuntimePartition* CreateHLODRuntimePartition(int32 InHLODIndex) const;

	/*
	 * Represents a cell descriptor, generated by runtime partitions. This is a streaming cell containing actors, without taking into account data layers, content bundles, etc.
	 */
	struct FCellDesc
	{
		FName Name;
		bool bIsSpatiallyLoaded = false;
		bool bBlockOnSlowStreaming = false;
		bool bClientOnlyVisible = false;
		bool bIs2D = false;
		int32 Priority = 0;

		/** Optional level value that can be used to filter debug display */
		int32 Level = 0;

		/** Optional cell bounds for partitions that work on uniform grids */
		TOptional<FBox> CellBounds;

		TArray<const IStreamingGenerationContext::FActorSetInstance*> ActorSetInstances;
	};

	/*
	 * Represents a cell descriptor instance, which is an instance of a cell after being split into data layers, etc. and ready to be converted into a streaming level.
	 */
	struct FCellDescInstance : public FCellDesc
	{
		FCellDescInstance(const FCellDesc& InCellDesc, URuntimePartition* InSourcePartition, const TArray<const UDataLayerInstance*>& InDataLayerInstances, const FGuid& InContentBundleID);

		URuntimePartition* SourcePartition = nullptr;
		TArray<const UDataLayerInstance*> DataLayerInstances;
		FGuid ContentBundleID;
	};

	struct FGenerateStreamingParams
	{
		const TArray<const IStreamingGenerationContext::FActorSetInstance*>* ActorSetInstances;
	};

	struct FGenerateStreamingResult
	{
		TArray<FCellDesc> RuntimeCellDescs;
	};

	UE_API virtual void SetDefaultValues();
	virtual bool SupportsHLODs() const PURE_VIRTUAL(URuntimePartition::SupportsHLODs, return true;);
	virtual TOptional<FFixedGridInfo> GetFixedGridInfo() const { return {}; }
	UE_API virtual void InitHLODRuntimePartitionFrom(const URuntimePartition* InRuntimePartition, int32 InHLODIndex);
	virtual void UpdateHLODRuntimePartitionFrom(const URuntimePartition* InRuntimePartition) {}
#endif
	virtual bool IsValidPartitionTokens(const TArray<FName>& InPartitionTokens) const PURE_VIRTUAL(URuntimePartition::IsValidPartitionTokens, return false;);
#if WITH_EDITOR
	virtual bool GenerateStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingResult& OutResult) PURE_VIRTUAL(URuntimePartition::GenerateStreaming, return false;);
	virtual FArchive& AppendCellGuid(FArchive& InAr) { return InAr << Name << HLODIndex; }
#endif

	UPROPERTY()
	FName Name;

	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Meta = (DisplayPriority = 3, EditCondition = "HLODIndex == INDEX_NONE", EditConditionHides, HideEditConditionToggle))
	bool bBlockOnSlowStreaming = false;

	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Meta = (DisplayPriority = 3, EditCondition = "HLODIndex == INDEX_NONE", EditConditionHides, HideEditConditionToggle))
	bool bClientOnlyVisible;

	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Meta = (DisplayPriority = 3, EditCondition = "HLODIndex == INDEX_NONE", EditConditionHides, HideEditConditionToggle))
	int32 Priority;

	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Meta = (DisplayPriority = 3, EditCondition = "HLODIndex == INDEX_NONE", EditConditionHides, HideEditConditionToggle))
	ERuntimePartitionCellBoundsMethod BoundsMethod;

	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Meta = (DisplayPriority = 0))
	int32 LoadingRange;

	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Meta = (DisplayPriority = 10, DisplayName="Debug - Grid Color"))
	FLinearColor DebugColor;

	UPROPERTY()
	int32 HLODIndex;

protected:
#if WITH_EDITOR
	UE_API FCellDesc CreateCellDesc(const FString& InName, bool bInIsSpatiallyLoaded, int32 InLevel, const TArray<const IStreamingGenerationContext::FActorSetInstance*>& InActorSetInstances);
#endif
};

#undef UE_API
