// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldGridPreviewer.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartition.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "RuntimePartitionLHGrid.generated.h"

#define UE_API ENGINE_API

namespace UE::Private::WorldPartition
{
	struct FStreamingDescriptor;
};

UCLASS(MinimalAPI)
class URuntimePartitionLHGrid : public URuntimePartition
{
	GENERATED_BODY()

	friend class UWorldPartitionRuntimeHashSet;
	friend class UWorldPartitionRuntimeSpatialHash;
	friend struct UE::Private::WorldPartition::FStreamingDescriptor;

public:
#if WITH_EDITOR
	//~ Begin UObject Interface.
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PreEditChange(FProperty* InPropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject Interface.

	//~ Begin URuntimePartition interface
	virtual bool SupportsHLODs() const override { return true; }
	UE_API virtual void InitHLODRuntimePartitionFrom(const URuntimePartition* InRuntimePartition, int32 InHLODIndex) override;
	UE_API virtual void UpdateHLODRuntimePartitionFrom(const URuntimePartition* InRuntimePartition) override;
	UE_API virtual void SetDefaultValues() override;
#endif
	UE_API virtual bool IsValidPartitionTokens(const TArray<FName>& InPartitionTokens) const override;
#if WITH_EDITOR
	UE_API virtual bool GenerateStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingResult& OutResult) override;
	UE_API virtual FArchive& AppendCellGuid(FArchive& InAr) override;
	//~ End URuntimePartition interface

	uint32 GetCellSize() const
	{
		return CellSize;
	}

	UE_API virtual TOptional<FFixedGridInfo> GetFixedGridInfo() const override;
#endif

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = RuntimeSettings, meta=(DisplayPriority = 0))
	uint32 CellSize = 25600;

	UPROPERTY(EditAnywhere, Category = RuntimeSettings, meta=(DisplayPriority = 1, EditCondition="HLODIndex == INDEX_NONE", EditConditionHides))
	FVector Origin = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = RuntimeSettings, meta=(DisplayPriority = 1, EditCondition="HLODIndex == INDEX_NONE", EditConditionHides))
	bool bIs2D = false;

	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Transient, SkipSerialization, meta=(DisplayPriority=10, DisplayName = "Debug - Show Grid Preview"))
	bool bShowGridPreview = false;
#endif

#if WITH_EDITOR
	TUniquePtr<FWorldGridPreviewer> WorldGridPreviewer;
#endif
};

#undef UE_API
