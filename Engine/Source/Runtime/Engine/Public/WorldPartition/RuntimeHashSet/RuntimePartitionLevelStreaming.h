// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartition.h"
#include "RuntimePartitionLevelStreaming.generated.h"

UCLASS()
class URuntimePartitionLevelStreaming : public URuntimePartition
{
	GENERATED_BODY()

public:
	//~ Begin URuntimePartition interface
#if WITH_EDITOR
	virtual bool SupportsHLODs() const override { return true; }
#endif
	virtual bool IsValidPartitionTokens(const TArray<FName>& InPartitionTokens) const override;
#if WITH_EDITOR
	virtual bool GenerateStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingResult& OutResult) override;
#endif
	//~ End URuntimePartition interface
};