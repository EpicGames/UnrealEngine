// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartition.h"
#include "RuntimePartitionPersistent.generated.h"

UCLASS(HideDropdown)
class URuntimePartitionPersistent : public URuntimePartition
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	//~ Begin URuntimePartition interface
	virtual bool SupportsHLODs() const override { return false; }
#endif
	virtual bool IsValidPartitionTokens(const TArray<FName>& InPartitionTokens) const override { return true; }
#if WITH_EDITOR
	virtual bool GenerateStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingResult& OutResult) override;
	//~ End URuntimePartition interface
#endif
};
