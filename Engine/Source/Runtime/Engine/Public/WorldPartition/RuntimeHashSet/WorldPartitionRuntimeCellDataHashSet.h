// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionRuntimeCellData.h"
#include "WorldPartitionRuntimeCellDataHashSet.generated.h"

UCLASS(Within = WorldPartitionRuntimeCell, MinimalAPI)
class UWorldPartitionRuntimeCellDataHashSet : public UWorldPartitionRuntimeCellData
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	ENGINE_API virtual void DumpStateLog(FHierarchicalLogArchive& Ar) const;
#endif

	UPROPERTY()
	bool bIs2D;
};
