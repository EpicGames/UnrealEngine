// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Math/StaticSpatialIndex.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartition.h"
#include "WorldPartitionRuntimeHashSet.generated.h"

#define UE_API ENGINE_API

class UHLODLayer;
class URuntimePartitionPersistent;
struct FPropertyChangedChainEvent;

using FStaticSpatialIndexSorter = FStaticSpatialIndex::TNodeSorterHilbert<FStaticSpatialIndex::FSpatialIndexProfile3D, 65536>;
using FStaticSpatialIndexType = TStaticSpatialIndexRTree<TObjectPtr<UWorldPartitionRuntimeCell>, FStaticSpatialIndexSorter, FStaticSpatialIndex::FSpatialIndexProfile3D>;

using FStaticSpatialIndexSorter2D = FStaticSpatialIndex::TNodeSorterHilbert<FStaticSpatialIndex::FSpatialIndexProfile2D, 65536>;
using FStaticSpatialIndexType2D = TStaticSpatialIndexRTree<TObjectPtr<UWorldPartitionRuntimeCell>, FStaticSpatialIndexSorter2D, FStaticSpatialIndex::FSpatialIndexProfile2D>;

namespace UE::Private::WorldPartition
{
	struct FStreamingDescriptor;
};

/** Holds an HLOD setup for a particular partition class. */
USTRUCT()
struct FRuntimePartitionHLODSetup
{
	GENERATED_USTRUCT_BODY()

	/** Name for this HLOD layer setup */
	UPROPERTY(EditAnywhere, Category = RuntimeSettings)
	FName Name;

#if WITH_EDITORONLY_DATA
	/** Associated HLOD Layer objects */
	UPROPERTY(EditAnywhere, Category = RuntimeSettings, DisplayName = "HLOD Layers To Include")
	TArray<TObjectPtr<const UHLODLayer>> HLODLayers;

	/** Used as the "TitlePropery" when showing as an array item */
	UPROPERTY(Transient, VisibleAnywhere, Category = RuntimeSettings)
	FName RowDisplayName;
#endif

	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Instanced, Meta = (NoResetToDefault))
	TObjectPtr<URuntimePartition> PartitionLayer;

	/** Whether this HLOD setup is spatially loaded or not */
	UPROPERTY()
	bool bIsSpatiallyLoaded = true;
};

/** Holds settings for a runtime partition instance. */
USTRUCT()
struct FRuntimePartitionDesc
{
	GENERATED_USTRUCT_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    FRuntimePartitionDesc() = default;
    ~FRuntimePartitionDesc() = default;
    FRuntimePartitionDesc(const FRuntimePartitionDesc&) = default;
    FRuntimePartitionDesc(FRuntimePartitionDesc&&) = default;
    FRuntimePartitionDesc& operator=(const FRuntimePartitionDesc&) = default;
    FRuntimePartitionDesc& operator=(FRuntimePartitionDesc&&) = default;
    PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Name for this partition, used to map actors to it through the Actor.RuntimeGrid property  */
	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Meta = (EditCondition = "MainLayer != nullptr", HideEditConditionToggle))
	FName Name;

	/** Partition class */
	UE_DEPRECATED(5.8, "This property is replaced by MainLayer")
	UPROPERTY()
	TSubclassOf<URuntimePartition> Class;

	/** Main partition object */
	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Instanced, Meta = (DisplayName = "Main Partition", NoResetToDefault))
	TObjectPtr<URuntimePartition> MainLayer;

	/** HLOD setups used by this partition, one for each layers in the hierarchy */
	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Meta = (DisplayName = "HLOD Partitions", EditCondition = "MainLayer != nullptr", HideEditConditionToggle, TitleProperty="RowDisplayName"))
	TArray<FRuntimePartitionHLODSetup> HLODSetups;

#if WITH_EDITOR
	TObjectPtr<URuntimePartition> GetFirstSpatiallyLoadedHLODPartitionAncestor(int32 HLODSetupsIndex);
#endif
};

USTRUCT()
struct FRuntimePartitionStreamingData
{
	GENERATED_USTRUCT_BODY()

	friend class UWorldPartitionRuntimeHashSet;
	friend class URuntimeHashSetExternalStreamingObject;
	friend UE::Private::WorldPartition::FStreamingDescriptor;

	void CreatePartitionsSpatialIndex() const;
	void DestroyPartitionsSpatialIndex() const;
	bool HasSpatialIndex() const { return SpatialIndex.IsValid() || SpatialIndexForce2D.IsValid() || SpatialIndex2D.IsValid(); }

	ENGINE_API int32 GetLoadingRange() const;

protected:
	/** Name of the runtime partition, currently maps to target grids. */
	UPROPERTY()
	FName Name;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString DebugName;
#endif

	UPROPERTY()
	int32 LoadingRange = 0;

	UPROPERTY()
	TArray<TObjectPtr<UWorldPartitionRuntimeCell>> SpatiallyLoadedCells;

	UPROPERTY()
	TArray<TObjectPtr<UWorldPartitionRuntimeCell>> NonSpatiallyLoadedCells;

	// Transient
	mutable TUniquePtr<FStaticSpatialIndexType> SpatialIndex;
	mutable TUniquePtr<FStaticSpatialIndexType2D> SpatialIndexForce2D;
	mutable TUniquePtr<FStaticSpatialIndexType2D> SpatialIndex2D;
};

template<>
struct TStructOpsTypeTraits<FRuntimePartitionStreamingData> : public TStructOpsTypeTraitsBase2<FRuntimePartitionStreamingData>
{
	enum { WithCopy = false };
};

UCLASS(MinimalAPI)
class URuntimeHashSetExternalStreamingObject : public URuntimeHashExternalStreamingObjectBase
{
	GENERATED_BODY()

#if WITH_EDITOR
	UE_API virtual void DumpStateLog(FHierarchicalLogArchive& Ar) override;
#endif

public:
	//~ Begin UObject Interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface

	UE_API void CreatePartitionsSpatialIndex() const;
	UE_API void DestroyPartitionsSpatialIndex() const;

	UPROPERTY()
	TArray<FRuntimePartitionStreamingData> RuntimeStreamingData;
};

UCLASS(MinimalAPI)
class UWorldPartitionRuntimeHashSet final : public UWorldPartitionRuntimeHash
{
	GENERATED_UCLASS_BODY()

	friend UE::Private::WorldPartition::FStreamingDescriptor;
	friend class ULevelPackageDiskSizeMetric;
	friend class UWorldPartitionRuntimeSpatialHash;

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	ENGINE_API virtual void PostLoad() override;
#endif
	//~ End UObject Interface

public:
	ENGINE_API virtual bool Draw2D(FWorldPartitionDraw2DContext& DrawContext) const override;
	ENGINE_API virtual void Draw3D(const TArray<FWorldPartitionStreamingSource>& Sources) const override;
	ENGINE_API virtual bool ContainsRuntimeHash(const FString& Name) const override;
	ENGINE_API virtual const FGuid* GetStandaloneHLODActorSourceCellOverride(const FGuid& InActorGuid) const override;
	ENGINE_API virtual const FGuid* GetCustomHLODActorSourceCellOverride(const FGuid& InActorGuid) const override;

#if WITH_EDITOR
	// Streaming generation interface
	ENGINE_API virtual void SetDefaultValues() override;
	ENGINE_API virtual bool SupportsHLODs() const override;
	ENGINE_API virtual bool SetupHLODActors(const IStreamingGenerationContext* StreamingGenerationContext, const UWorldPartition::FSetupHLODActorsParams& Params) const override;
	ENGINE_API virtual bool GenerateStreaming(class UWorldPartitionStreamingPolicy* StreamingPolicy, const IStreamingGenerationContext* StreamingGenerationContext, TArray<FString>* OutPackagesToGenerate) override;
	ENGINE_API virtual FName GetDefaultGrid() const override;
	ENGINE_API virtual bool IsValidGrid(FName GridName, const UClass* ActorClass) const override;
	ENGINE_API virtual bool IsValidHLODLayer(FName GridName, const FSoftObjectPath& HLODLayerPath) const override;
	ENGINE_API virtual TOptional<FFixedGridInfo> GetFixedGridInfo(FName InGridName) const override;
	ENGINE_API virtual void DumpStateLog(FHierarchicalLogArchive& Ar) const override;
	ENGINE_API virtual void ForEachHLODLayer(TFunctionRef<bool(FName, FName, int32)> Func) const override; // RuntimePartitionName, HLODSetupName, HLODSetupIndex
	ENGINE_API virtual int32 ComputeHLODHierarchyDepth() const override;
	ENGINE_API virtual FName ResolveRuntimeGridNameForHLODLayer(FName RuntimeGrid, const FSoftObjectPath& HLODLayerPath) const override;
	ENGINE_API virtual bool CopyGridSetupFrom(const UWorldPartitionRuntimeHash* Other) override;
#endif

	// Helpers
	static ENGINE_API bool ParseGridName(FName GridName, TArray<FName>& MainPartitionTokens, TArray<FName>& HLODPartitionTokens);

#if WITH_EDITOR
	// Conversions
	static ENGINE_API UWorldPartitionRuntimeHashSet* CreateFrom(const UWorldPartitionRuntimeHash* SrcHash);
#endif

	// External streaming object interface
#if WITH_EDITOR
	virtual TSubclassOf<URuntimeHashExternalStreamingObjectBase> GetExternalStreamingObjectClass() const override { return URuntimeHashSetExternalStreamingObject::StaticClass(); }
#endif
	ENGINE_API virtual bool InjectExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject) override;
	ENGINE_API virtual bool RemoveExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject) override;

	// Streaming interface
	ENGINE_API virtual void ForEachStreamingCells(TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func) const override;
	ENGINE_API virtual void ForEachStreamingCellsQuery(const FWorldPartitionStreamingQuerySource& QuerySource, TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func, FWorldPartitionQueryCache* QueryCache) const override;
	ENGINE_API virtual void ForEachStreamingCellsSources(const TArray<FWorldPartitionStreamingSource>& Sources, TFunctionRef<bool(const UWorldPartitionRuntimeCell*, EStreamingSourceTargetState)> Func, const FWorldPartitionStreamingContext& Context = FWorldPartitionStreamingContext()) const override;

protected:
	ENGINE_API virtual bool SupportsWorldAssetStreaming(const FName& InTargetGrid) override;
	ENGINE_API virtual FGuid RegisterWorldAssetStreaming(const UWorldPartition::FRegisterWorldAssetStreamingParams& InParams) override;
	ENGINE_API virtual bool UnregisterWorldAssetStreaming(const FGuid& InWorldAssetStreamingGuid) override;
	ENGINE_API virtual TArray<UWorldPartitionRuntimeCell*> GetWorldAssetStreamingCells(const FGuid& InWorldAssetStreamingGuid) override;

private:
	ENGINE_API virtual void OnBeginPlay() override;

#if WITH_EDITOR
	ENGINE_API virtual bool HasStreamingContent() const override;
	ENGINE_API virtual void StoreStreamingContentToExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* OutExternalStreamingObject) override;
	ENGINE_API virtual void FlushStreamingContent() override;

	/** Generate the runtime partitions streaming descs. */
	bool GenerateRuntimePartitionsStreamingDescs(const IStreamingGenerationContext* StreamingGenerationContext, TMap<URuntimePartition*, TArray<URuntimePartition::FCellDescInstance>>& OutRuntimeCellDescs) const;

	struct FCellUniqueId
	{
		FString Name;
		FString InstanceSuffix;
		FGuid Guid;
	};

	FCellUniqueId GetCellUniqueId(const URuntimePartition::FCellDescInstance& InCellDescInstance) const;
#endif

	ENGINE_API void ForEachStreamingData(TFunctionRef<bool(const FRuntimePartitionStreamingData&)> Func) const;

	ENGINE_API void UpdateRuntimeDataGridMap();

public:
	ENGINE_API const URuntimePartition* ResolveRuntimePartition(FName GridName, bool bMainPartitionLayer = false) const;

	const URuntimePartition* ResolveRuntimePartitionForHLODLayer(FName GridName, const FSoftObjectPath& HLODLayerPath) const;

private:
	void FixupHLODSetup(FRuntimePartitionDesc& RuntimePartition);

	void RemoveIrrelevantCells(FRuntimePartitionStreamingData& StreamingData);

	/** Array of runtime partition descriptors */
	UPROPERTY(EditAnywhere, Category = RuntimeSettings, Meta = (TitleProperty = "Name"))
	TArray<FRuntimePartitionDesc> RuntimePartitions;

	UPROPERTY()
	TArray<FRuntimePartitionStreamingData> RuntimeStreamingData;

	UPROPERTY(Transient)
	TMap<FGuid, TObjectPtr<URuntimeHashSetExternalStreamingObject>> WorldAssetStreamingObjects;

	UPROPERTY()
	TMap<FGuid, FGuid> StandaloneHLODActorToSourceCellsMap;

	UPROPERTY()
	TMap<FGuid, FGuid> CustomHLODActorToSourceCellsMap;

	// Optimized data
	TMap<FName, TArray<const FRuntimePartitionStreamingData*>> RuntimeSpatiallyLoadedDataGridMap;
	TArray<const FRuntimePartitionStreamingData*> RuntimeNonSpatiallyLoadedDataGridList;

	friend class FWorldPartitionRuntimeHashSetDetails;
};

#undef UE_API
