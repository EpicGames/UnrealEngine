// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/Find.h"
#include "Algo/Transform.h"

#include "ProfilingDebugging/ScopedTimers.h"

#include "WorldPartition/WorldPartitionRuntimeSpatialHash.h"
#include "WorldPartition/WorldPartitionStreamingGenerationContext.h"
#include "WorldPartition/DataLayer/DataLayersID.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

#if WITH_EDITOR

extern ENGINE_API bool GRuntimeSpatialHashUseAlignedGridLevels;
extern ENGINE_API bool GRuntimeSpatialHashSnapNonAlignedGridLevelsToLowerLevels;
extern ENGINE_API bool GRuntimeSpatialHashPlaceSmallActorsUsingLocation;
extern ENGINE_API bool GRuntimeSpatialHashPlacePartitionActorsUsingLocation;

#endif

/**
  * Square 2D grid helper
  */
struct FSquare2DGridHelper
{
	struct FGrid2D
	{
		FVector2D Origin;
		int64 CellSize;
		int64 GridSize;

		inline FGrid2D(const FVector2D& InOrigin, int64 InCellSize, int64 InGridSize)
			: Origin(InOrigin)
			, CellSize(InCellSize)
			, GridSize(InGridSize)
		{}

		/**
		 * Validate that the coordinates fit the grid size
		 *
		 * @return true if the specified coordinates are valid
		 */
		inline bool IsValidCoords(const FGridCellCoord2& InCoords) const
		{
			if ((InCoords.X >= 0) && (InCoords.X < GridSize) && (InCoords.Y >= 0) && (InCoords.Y < GridSize))
			{
				int64 Index = (InCoords.Y * GridSize) + InCoords.X;
				if (ensureMsgf(Index >= 0, TEXT("World Partition reached the current limit of large world coordinates.")))
				{
					return true;
				}
			}
			return false;
		}

		/**
		 * Returns the cell bounds
		 *
		 * @return true if the specified index was valid
		 */
		inline bool GetCellBounds(int64 InIndex, FBox2D& OutBounds) const
		{
			if (InIndex >= 0 && InIndex <= (GridSize * GridSize))
			{
				const FGridCellCoord2 Coords(InIndex % GridSize, InIndex / GridSize);
				return GetCellBounds(Coords, OutBounds);
			}

			return false;
		}

		/**
		 * Returns the cell bounds
		 *
		 * @return true if the specified coord was valid
		 */
		inline bool GetCellBounds(const FGridCellCoord2& InCoords, FBox2D& OutBounds, bool bCheckIsValidCoord = true) const
		{
			if (!bCheckIsValidCoord || IsValidCoords(InCoords))
			{
				const FVector2D Min = (FVector2D(Origin) - FVector2D(GridSize * CellSize * 0.5)) + FVector2D(InCoords.X * CellSize, InCoords.Y * CellSize);
				const FVector2D Max = Min + FVector2D(CellSize, CellSize);
				OutBounds = FBox2D(Min, Max);
				return true;
			}

			return false;
		}

		/**
		 * Returns the cell coordinates of the provided position
		 *
		 * @return true if the position was inside the grid
		 */
		inline bool GetCellCoords(const FVector2D& InPos, FGridCellCoord2& OutCoords) const
		{
			OutCoords = FGridCellCoord2(
				FMath::FloorToInt(((InPos.X - Origin.X) / CellSize) + GridSize * 0.5),
				FMath::FloorToInt(((InPos.Y - Origin.Y) / CellSize) + GridSize * 0.5)
			);

			return IsValidCoords(OutCoords);
		}

		/**
		 * Returns the cells coordinates of the provided box
		 *
		 * @return true if the bounds was intersecting with the grid
		 */
		inline bool GetCellCoords(const FBox2D& InBounds2D, FGridCellCoord2& OutMinCellCoords, FGridCellCoord2& OutMaxCellCoords) const
		{
			GetCellCoords(InBounds2D.Min, OutMinCellCoords);
			if ((OutMinCellCoords.X >= GridSize) || (OutMinCellCoords.Y >= GridSize))
			{
				return false;
			}

			GetCellCoords(InBounds2D.Max, OutMaxCellCoords);
			if ((OutMaxCellCoords.X < 0) || (OutMaxCellCoords.Y < 0))
			{
				return false;
			}

			OutMinCellCoords.X = FMath::Clamp(OutMinCellCoords.X, 0LL, GridSize - 1);
			OutMinCellCoords.Y = FMath::Clamp(OutMinCellCoords.Y, 0LL, GridSize - 1);
			OutMaxCellCoords.X = FMath::Clamp(OutMaxCellCoords.X, 0LL, GridSize - 1);
			OutMaxCellCoords.Y = FMath::Clamp(OutMaxCellCoords.Y, 0LL, GridSize - 1);
			return true;
		}

		/**
		 * Returns the cell index of the provided coords
		 *
		 * @return true if the coords was inside the grid
		 */
		inline bool GetCellIndex(const FGridCellCoord2& InCoords, uint64& OutIndex) const
		{
			if (IsValidCoords(InCoords))
			{
				OutIndex = (InCoords.Y * GridSize) + InCoords.X;
				return true;
			}
			return false;
		}

		/**
		 * Returns the cell index of the provided position
		 *
		 * @return true if the position was inside the grid
		 */
		inline bool GetCellIndex(const FVector& InPos, uint64& OutIndex) const
		{
			FGridCellCoord2 Coords = FGridCellCoord2(
				FMath::FloorToInt(((InPos.X - Origin.X) / CellSize) + GridSize * 0.5),
				FMath::FloorToInt(((InPos.Y - Origin.Y) / CellSize) + GridSize * 0.5)
			);

			return GetCellIndex(Coords, OutIndex);
		}

		/**
		 * Get the number of intersecting cells of the provided box
		 *
		 * @return the number of intersecting cells
		 */
		int32 GetNumIntersectingCells(const FBox& InBox) const
		{
			FGridCellCoord2 MinCellCoords;
			FGridCellCoord2 MaxCellCoords;
			const FBox2D Bounds2D(FVector2D(InBox.Min), FVector2D(InBox.Max));

			if (GetCellCoords(Bounds2D, MinCellCoords, MaxCellCoords))
			{
				return (MaxCellCoords.X - MinCellCoords.X + 1) * (MaxCellCoords.Y - MinCellCoords.Y + 1);
			}

			return 0;
		}

		/**
		 * Runs a function on all intersecting cells for the provided box
		 *
		 * @return the number of intersecting cells
		 */
		int32 ForEachIntersectingCellsBreakable(const FBox& InBox, TFunctionRef<bool(const FGridCellCoord2&)> InOperation) const
		{
			int32 NumCells = 0;

			FGridCellCoord2 MinCellCoords;
			FGridCellCoord2 MaxCellCoords;
			const FBox2D Bounds2D(FVector2D(InBox.Min), FVector2D(InBox.Max));

			if (GetCellCoords(Bounds2D, MinCellCoords, MaxCellCoords))
			{
				for (int64 y = MinCellCoords.Y; y <= MaxCellCoords.Y; y++)
				{
					for (int64 x = MinCellCoords.X; x <= MaxCellCoords.X; x++)
					{
						const FGridCellCoord2 Coord(x, y);
						// Validate that generated coordinate is valid (in case we reached the 64-bit limit of cell index)
						if (IsValidCoords(Coord))
						{
							if (!InOperation(Coord))
							{
								return NumCells;
							}
							++NumCells;
						}
					}
				}
			}

			return NumCells;
		}

		int32 ForEachIntersectingCells(const FBox& InBox, TFunctionRef<void(const FGridCellCoord2&)> InOperation) const
		{
			return ForEachIntersectingCellsBreakable(InBox, [InOperation](const FGridCellCoord2& Vector) { InOperation(Vector); return true; });
		}

		/**
		 * Runs a function on all intersecting cells for the provided sphere
		 *
		 * @return the number of intersecting cells
		 */
		int32 ForEachIntersectingCells(const FSphere& InSphere, TFunctionRef<void(const FGridCellCoord2&)> InOperation) const
		{
			int32 NumCells = 0;

			// @todo_ow: rasterize circle instead?
			const FBox Box(InSphere.Center - FVector(InSphere.W), InSphere.Center + FVector(InSphere.W));
			const double SquareDistance = InSphere.W * InSphere.W;
			const FVector2D SphereCenter(InSphere.Center);

			ForEachIntersectingCells(Box, [this, SquareDistance, &SphereCenter, &InOperation, &NumCells](const FGridCellCoord2& Coords)
			{
				// No need to check validity of coords as it's already done
				const bool bCheckIsValidCoords = false;
				FBox2D CellBounds;
				GetCellBounds(Coords, CellBounds, bCheckIsValidCoords);

				FVector2D Delta = SphereCenter - FVector2D::Max(CellBounds.Min, FVector2D::Min(SphereCenter, CellBounds.Max));
				if ((Delta.X * Delta.X + Delta.Y * Delta.Y) < SquareDistance)
				{
					InOperation(Coords);
					NumCells++;
				}
			});

			return NumCells;
		}
		
		/**
		 * Runs a function on all intersecting cells for the provided spherical sector
		 *
		 * @return the number of intersecting cells
		 */
		int32 ForEachIntersectingCells(const FSphericalSector& InShape, TFunctionRef<void(const FGridCellCoord2&)> InOperation) const;

	private:

		bool DoesCircleSectorIntersectsCell(const FGridCellCoord2& Coords, const FSphericalSector& InShape) const;
	};

	struct FGridLevel : public FGrid2D
	{
#if WITH_EDITOR
		struct FGridCellDataChunk
		{
			FGridCellDataChunk(const TArray<const UDataLayerInstance*>& InDataLayers, const FGuid& InContentBundleID)
			{
				Algo::TransformIf(InDataLayers, DataLayers, [](const UDataLayerInstance* DataLayer) { return DataLayer->IsRuntime(); }, [](const UDataLayerInstance* DataLayer) { return DataLayer; });
				DataLayersID = FDataLayersID(DataLayers);
				ContentBundleID = InContentBundleID;
			}

			void AddActorSetInstance(const IStreamingGenerationContext::FActorSetInstance* ActorSetInstance) { ActorSetInstances.Add(ActorSetInstance); }
			const TArray<const IStreamingGenerationContext::FActorSetInstance*>& GetActorSetInstances() const { return ActorSetInstances; }
			bool HasDataLayers() const { return !DataLayers.IsEmpty(); }
			const TArray<const UDataLayerInstance*>& GetDataLayers() const { return DataLayers; }
			const FDataLayersID& GetDataLayersID() const { return DataLayersID; }
			FGuid GetContentBundleID() const { return ContentBundleID; }
			bool operator==(const FGridCellDataChunk& InGridCellDataChunk) const { return DataLayersID == InGridCellDataChunk.DataLayersID;}
			friend uint32 GetTypeHash(const FGridCellDataChunk& InGridCellDataChunk) { return GetTypeHash(InGridCellDataChunk.DataLayersID);}

		private:
			TArray<const IStreamingGenerationContext::FActorSetInstance*> ActorSetInstances;
			TArray<const UDataLayerInstance*> DataLayers;
			FDataLayersID DataLayersID;
			FGuid ContentBundleID;
		};
#endif

		struct FGridCell
		{
			FGridCell(const FGridCellCoord& InCoords)
				: Coords(InCoords)
			{}

#if WITH_EDITOR
			void AddActorSetInstance(const IStreamingGenerationContext::FActorSetInstance* ActorSetInstance)
			{
				const FDataLayersID DataLayersID = FDataLayersID(ActorSetInstance->DataLayers);
				FGridCellDataChunk& ActorDataChunk = DataChunks.FindOrAddByHash(DataLayersID.GetHash(), FGridCellDataChunk(ActorSetInstance->DataLayers, ActorSetInstance->ContentBundleID));
				ActorDataChunk.AddActorSetInstance(ActorSetInstance);
			}

			const TSet<FGridCellDataChunk>& GetDataChunks() const
			{
				return DataChunks;
			}

			const FGridCellDataChunk* GetNoDataLayersDataChunk() const
			{
				for (const FGridCellDataChunk& DataChunk : DataChunks)
				{
					if (!DataChunk.HasDataLayers())
					{
						return &DataChunk;
					}
				}
				return nullptr;
			}
#endif

			FGridCellCoord GetCoords() const
			{
				return Coords;
			}

		private:
			FGridCellCoord Coords;
#if WITH_EDITOR
			TSet<FGridCellDataChunk> DataChunks;
#endif
		};

		inline FGridLevel(const FVector2D& InOrigin, int64 InCellSize, int64 InGridSize, int32 InLevel)
			: FGrid2D(InOrigin, InCellSize, InGridSize)
			, Level(InLevel)
		{}

		/**
		 * Returns the cell at the specified grid coordinate
		 *
		 * @return the cell at the specified grid coordinate
		 */
		inline FGridCell& GetCell(const FGridCellCoord2& InCoords)
		{
			check(IsValidCoords(InCoords));

			uint64 CellIndex;
			verify(GetCellIndex(InCoords, CellIndex));

			int64 CellIndexMapping;
			int64* CellIndexMappingPtr = CellsMapping.Find(CellIndex);
			if (CellIndexMappingPtr)
			{
				CellIndexMapping = *CellIndexMappingPtr;
			}
			else
			{
				CellIndexMapping = Cells.Emplace(FGridCellCoord(InCoords.X, InCoords.Y, Level));
				CellsMapping.Add(CellIndex, CellIndexMapping);
			}

			return Cells[CellIndexMapping];
		}

		/**
		 * Returns the cell at the specified grid coordinate
		 *
		 * @return the cell at the specified grid coordinate
		 */
		inline const FGridCell& GetCell(const FGridCellCoord2& InCoords) const
		{
			check(IsValidCoords(InCoords));

			uint64 CellIndex;
			verify(GetCellIndex(InCoords, CellIndex));

			int64 CellIndexMapping = CellsMapping.FindChecked(CellIndex);

			const FGridCell& Cell = Cells[CellIndexMapping];
			check(Cell.GetCoords() == FGridCellCoord(InCoords.X, InCoords.Y, Level));
			return Cell;
		}

		int32 Level;
		TArray<FGridCell> Cells;
		TMap<int64, int64> CellsMapping;
	};

	ENGINE_API FSquare2DGridHelper(const FBox& InWorldBounds, const FVector& InOrigin, int64 InCellSize, bool bUseAlignedGridLevels);

	// Returns the lowest grid level
	inline FGridLevel& GetLowestLevel() { return Levels[0]; }

	// Returns the always loaded (top level) cell
	inline FGridLevel::FGridCell& GetAlwaysLoadedCell() { return Levels.Last().GetCell(FGridCellCoord2(0,0)); }

	// Returns the always loaded (top level) cell
	inline const FGridLevel::FGridCell& GetAlwaysLoadedCell() const { return Levels.Last().GetCell(FGridCellCoord2(0,0)); }

	// Returns the cell at the given coord
	inline const FGridLevel::FGridCell& GetCell(const FGridCellCoord& InCoords) const { return Levels[InCoords.Z].GetCell(FGridCellCoord2(InCoords.X, InCoords.Y)); }

	/**
	 * Returns the cell bounds
	 *
	 * @return true if the specified coord was valid
	 */
	inline bool GetCellBounds(const FGridCellCoord& InCoords, FBox2D& OutBounds) const
	{
		if (Levels.IsValidIndex(InCoords.Z))
		{
			return Levels[InCoords.Z].GetCellBounds(FGridCellCoord2(InCoords.X, InCoords.Y), OutBounds);
		}
		return false;
	}

	/**
	 * Returns the cell global coordinates
	 *
	 * @return true if the specified coord was valid
	 */
	inline bool GetCellGlobalCoords(const FGridCellCoord& InCoords, FGridCellCoord& OutGlobalCoords) const
	{
		if (Levels.IsValidIndex(InCoords.Z))
		{
			const FGridLevel& GridLevel = Levels[InCoords.Z];
			if (GridLevel.IsValidCoords(FGridCellCoord2(InCoords.X, InCoords.Y)))
			{
				int64 CoordOffset = Levels[InCoords.Z].GridSize >> 1;
				OutGlobalCoords = InCoords;
				OutGlobalCoords.X -= CoordOffset;
				OutGlobalCoords.Y -= CoordOffset;
				return true;
			}
		}
		return false;
	}

	// Runs a function on all cells
	ENGINE_API void ForEachCells(TFunctionRef<void(const FSquare2DGridHelper::FGridLevel::FGridCell&)> InOperation) const;

	/**
	 * Runs a function on all intersecting cells for the provided box
	 *
	 * @return the number of intersecting cells
	 */
	ENGINE_API int32 ForEachIntersectingCells(const FBox& InBox, TFunctionRef<void(const FGridCellCoord&)> InOperation, int32 InStartLevel = 0) const;

	/**
	 * Runs a function on all intersecting cells for the provided sphere
	 *
	 * @return the number of intersecting cells
	 */
	ENGINE_API int32 ForEachIntersectingCells(const FSphere& InSphere, TFunctionRef<void(const FGridCellCoord&)> InOperation, int32 InStartLevel = 0) const;

	/**
	 * Runs a function on all intersecting cells for the provided spherical sector
	 *
	 * @return the number of intersecting cells
	 */
	ENGINE_API int32 ForEachIntersectingCells(const FSphericalSector& InShape, TFunctionRef<void(const FGridCellCoord&)> InOperation, int32 InStartLevel = 0) const;

public:
	FBox WorldBounds;
	FVector Origin;
	int64 CellSize;
	TArray<FGridLevel> Levels;
};

#if WITH_EDITOR

FSquare2DGridHelper GetGridHelper(const FBox& WorldBounds, const FVector& GridOrigin, int64 GridCellSize, bool bUseAlignedGridLevels);
FSquare2DGridHelper GetPartitionedActors(const FBox& WorldBounds, const FSpatialHashRuntimeGrid& Grid, const TArray<const IStreamingGenerationContext::FActorSetInstance*>& ActorSetInstances, const FSpatialHashSettings& Settings);

#endif // #if WITH_EDITOR
