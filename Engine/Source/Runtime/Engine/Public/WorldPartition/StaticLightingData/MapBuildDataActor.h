// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionActorDesc.h"
#include "GameFramework/Actor.h"

#include "MapBuildDataActor.generated.h"

#define UE_API ENGINE_API

class UMapBuildDataRegistry;

UCLASS(MinimalAPI, NotPlaceable, HideCategories=(Rendering, Replication, Collision, Physics, Navigation, Networking, Input, Actor, LevelInstance, Cooking))
class AMapBuildDataActor : public AActor
{
	GENERATED_UCLASS_BODY()

public:
	
	void SetCellPackage(FName InCellPackage) { CellPackage = InCellPackage; }
	UE_API void SetBounds(FBox& Bounds);

#if WITH_EDITOR
	void SetActorInstances(TArray<FGuid>& InActorInstances) { ActorInstances = InActorInstances; }
#endif

	UE_API UMapBuildDataRegistry* GetBuildData(bool bCreateIfNotFound = false);
	UE_API void SetBuildData(UMapBuildDataRegistry* MapBuildData);

	UE_API void LinkToActor(AActor* Actor);

protected:
	//~ Begin UObject Interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

	//~ Begin AActor Interface.	
	UE_API virtual void GetActorBounds(bool bOnlyCollidingComponents, FVector& OutOrigin, FVector& OutBoxExtent, bool bIncludeFromChildActors) const override;	
	UE_API virtual void PreRegisterAllComponents() override;
	UE_API virtual void PostUnregisterAllComponents() override;
#if WITH_EDITOR
	UE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	UE_API virtual void PreDuplicateFromRoot(FObjectDuplicationParameters& DupParams) override;
	UE_API virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const override;
#endif
	//~ End AActor Interface.

	UPROPERTY(NonPIEDuplicateTransient)	
	TObjectPtr<UMapBuildDataRegistry> BuildData;

	UPROPERTY()
	TObjectPtr<AActor> ForceLinkToActor;

	UPROPERTY()
	FBox ActorBounds;

	UPROPERTY()
	FName CellPackage;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FGuid> ActorInstances;
#endif

	UPROPERTY()
	FGuid LevelBuildDataId;
	
	bool bAddedToWorld;

	UE_API void AddToWorldMapBuildData();
	UE_API void RemoveFromWorldMapBuildData();

	UE_API void InitializeRenderingResources();
	UE_API void ReleaseRenderingResources();

	friend class FMapBuildDataActorDesc;
};

#if WITH_EDITOR
class FMapBuildDataActorDesc : public FWorldPartitionActorDesc
{
	friend class AMapBuildDataActor;

	public:
		
		FName	CellPackage;

	protected:
	
		ENGINE_API FMapBuildDataActorDesc();

		//~ Begin FWorldPartitionActorDesc Interface.
		ENGINE_API virtual void Init(const AActor* InActor) override;
		ENGINE_API virtual bool Equals(const FWorldPartitionActorDesc* Other) const override;
		virtual uint32 GetSizeOf() const override { return sizeof(FMapBuildDataActorDesc); }
		ENGINE_API virtual void Serialize(FArchive& Ar) override;
		virtual bool IsRuntimeRelevant(const FWorldPartitionActorDescInstance* InActorDescInstance) const override;
		//~ End FWorldPartitionActorDesc Interface.
};

DEFINE_ACTORDESC_TYPE(AMapBuildDataActor, FMapBuildDataActorDesc );

#endif

#undef UE_API
