// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPtr.h"

class AActor;
class UMapBuildDataRegistry;
class AMapBuildDataActor;

#if WITH_EDITOR

struct FLightingActorDesc
{
	// Actor 
	FGuid ActorGuid;
	
	// Actor Package
	FName ActorPackage;
	
	// All precomputed data guids associated with this actor
	TArray<FGuid> PrecomputedLightingGuids;

	// Actor
	TSoftObjectPtr<AActor> Actor;
	
	// CellLevelPackage Name 
	FName CellLevelPackage;
};

struct FLightingCellDesc
{	
	TSoftObjectPtr<UMapBuildDataRegistry> MapBuildData;
	TSoftObjectPtr<AMapBuildDataActor> DataActor;
	TArray<FGuid> ActorInstanceGuids;
	FBox Bounds;
	TArray<FName> DataLayers;
	FName RuntimeGrid;

	// CellLevelPackage Name 
	FName CellLevelPackage;
};

struct FStaticLightingDescriptors
{	
	struct FActorPackage
	{
		FName PackageName;
		FGuid Guid;
		FName AssociatedLevelPackage;
	};

	TMap<FGuid, FLightingActorDesc> ActorGuidsToDesc;
	TMap<FName, FLightingCellDesc> LightingCellsDescs;
	TArray<FActorPackage> StaleMapDataActorsPackage;
	TArray<FActorPackage> MapDataActorsPackage;
	UWorld* World;
	UNREALED_API void InitializeFromWorld(UWorld* World);

	UNREALED_API UMapBuildDataRegistry* GetOrCreateRegistryForActor(AActor* Actor);
	UNREALED_API UMapBuildDataRegistry* GetRegistryForActor(AActor* Actor, bool bCreateIfNotFound = false);

	[[nodiscard]] UNREALED_API TArray<UMapBuildDataRegistry*> GetAllMapBuildData();

	UNREALED_API bool CreateAndUpdateActors();

	UNREALED_API static void Set(FStaticLightingDescriptors*);
	UNREALED_API static FStaticLightingDescriptors* Get();
};

#endif