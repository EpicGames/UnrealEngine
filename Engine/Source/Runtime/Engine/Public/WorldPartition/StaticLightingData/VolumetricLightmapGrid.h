// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Serialization/BulkData.h"
#include "VolumetricLightmapGrid.generated.h"


class FPrecomputedVolumetricLightmapData;
class UMapBuildDataRegistry;

USTRUCT()
struct FVolumetricLightMapGridCell
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY()
	FBox Bounds;

	FByteBulkData BulkData;

#if WITH_EDITOR
	// ptr to the just recently built version of the FPrecomputedVolumetricLightmapData
	// not used at runtime
	FPrecomputedVolumetricLightmapData* EditorData = nullptr;
#endif
	
	UPROPERTY()
	uint32 CellID = -1;
};	

USTRUCT()
struct FVolumetricLightMapGridDesc
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FBox GridBounds;
	
	UPROPERTY()
	FGuid Guid;	

	UPROPERTY()
	float CellSize = 0.0f;

	UPROPERTY()
	float DetailCellSize = 0.0f;

	UPROPERTY()
	float BrickSize = 0.0f;

	UPROPERTY()
	TArray<FVolumetricLightMapGridCell>		Cells;
	
	bool IsValid() const { return Guid.IsValid(); }

	ENGINE_API FGuid	GetCellGuid(uint32 CellID) const;
	ENGINE_API FString	GetCellDesc(FGuid LevelId) const;

	ENGINE_API FVolumetricLightMapGridCell* GetCell(FGuid LevelId) const;	
	ENGINE_API FVolumetricLightMapGridCell* GetCell(const FVector& InPosition) const; 
	
	[[nodiscard]] ENGINE_API TArray<FVolumetricLightMapGridCell*> GetIntersectingCells(const FBox& InBounds, bool bInWithData);	
	
	ENGINE_API void LoadVolumetricLightMapCell(FVolumetricLightMapGridCell& Cell, FPrecomputedVolumetricLightmapData*& OutData);

	ENGINE_API FPrecomputedVolumetricLightmapData* GetPrecomputedVolumetricLightmapBuildData(FGuid LevelId) const;
	ENGINE_API FPrecomputedVolumetricLightmapData* GetOrCreatePrecomputedVolumetricLightmapBuildData(FGuid LevelId);

#if WITH_EDITOR
	ENGINE_API void InitializeBulkData();
	ENGINE_API void Initialize(UWorld* InWorld, const FBox& InBounds);
#endif

	void SerializeBulkData(FArchive& Ar, UObject* Owner);
};
