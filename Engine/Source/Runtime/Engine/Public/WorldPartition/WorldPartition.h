// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Optional.h"
#include "GameFramework/Actor.h"
#include "Templates/SubclassOf.h"
#include "UObject/LinkerInstancingContext.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/ActorDescContainerInstanceCollection.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageGenerator.h"

#if WITH_EDITOR
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "WorldPartition/WorldPartitionEditorLoaderAdapter.h"
#include "WorldPartition/WorldPartitionRuntimeCellTransformer.h"
#include "ActorReferencesUtils.h"
#include "ExternalDirtyActorsTracker.h"
#include "PackageSourceControlHelper.h"
#include "CookPackageSplitter.h"
#include "Delegates/DelegateCombinations.h"
#endif

#include "WorldPartition.generated.h"

class UActorDescContainer;
class UWorldPartitionEditorHash;
class UWorldPartitionRuntimeCell;
class UWorldPartitionRuntimeHash;
class URuntimeHashExternalStreamingObjectBase;
class UWorldPartitionStreamingPolicy;
class IWorldPartitionCell;
class UDataLayerManager;
class IStreamingGenerationErrorHandler;
class FLoaderAdapterAlwaysLoadedActors;
class FLoaderAdapterActorList;
class FHLODActorDesc;
class UHLODLayer;
class UCanvas;
class ULevel;
class FAutoConsoleVariableRef;
class FWorldPartitionDraw2DContext;
class FContentBundleEditor;
class IStreamingGenerationContext;
class UExternalDataLayerManager;
class IWorldPartitionCookPackageObject;

struct IWorldPartitionStreamingSourceProvider;
struct IHLODCreationFilter;

enum class EWorldPartitionRuntimeCellState : uint8;
enum class EWorldPartitionStreamingPerformance : uint8;

enum class EWorldPartitionInitState
{
	Uninitialized,
	Initializing,
	Initialized,
	Uninitializing
};

UENUM()
enum class EWorldPartitionServerStreamingMode : uint8
{
	ProjectDefault = 0 UMETA(ToolTip = "Use project default (wp.Runtime.EnableServerStreaming)"),
	Disabled = 1 UMETA(ToolTip = "Server streaming is disabled"),
	Enabled = 2 UMETA(ToolTip = "Server streaming is enabled"),
	EnabledInPIE = 3 UMETA(ToolTip = "Server streaming is only enabled in PIE"),
};

UENUM()
enum class EWorldPartitionServerStreamingOutMode : uint8
{
	ProjectDefault = 0 UMETA(ToolTip = "Use project default (wp.Runtime.EnableServerStreamingOut)"),
	Disabled = 1 UMETA(ToolTip = "Server streaming out is disabled"),
	Enabled = 2 UMETA(ToolTip = "Server streaming out is enabled"),
};

UENUM()
enum class EWorldPartitionDataLayersLogicOperator : uint8
{
	Or,
	And
};

#if WITH_EDITOR
/**
 * Interface for the world partition editor
 */
struct IWorldPartitionEditor
{
	virtual void Refresh() {}
	virtual void Reconstruct() {}
	virtual void FocusBox(const FBox& Box) const {}
};

class ISourceControlHelper
{
public:
	virtual FString GetFilename(const FString& PackageName) const =0;
	virtual FString GetFilename(UPackage* Package) const =0;
	virtual bool Checkout(UPackage* Package) const =0;
	virtual bool Add(UPackage* Package) const =0;
	virtual bool Delete(const FString& PackageName) const =0;
	virtual bool Delete(const TArray<FString>& PackageNames) const =0;
	virtual bool Delete(UPackage* Package) const =0;
	virtual bool Delete(const TArray<UPackage*>& Packages) const =0;
	virtual bool Save(UPackage* Package) const =0;
	virtual bool Save(const TArray<UPackage*>& Packages) const =0;
	virtual bool Copy(const FString& SrcFilePath, const FString& DstFilePath) const =0;
};
#endif

/** Holds an instance of a runtime cell transformer. */
USTRUCT()
struct FRuntimeCellTransformerInstance
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	inline void PreTransform(ULevel* InLevel) const { if (CanTransform(InLevel)) { Instance->PreTransform(InLevel); } }
	inline void Transform(ULevel* InLevel) const { if (CanTransform(InLevel)) { Instance->Transform(InLevel); } }
	inline void PostTransform(ULevel* InLevel) const { if (CanTransform(InLevel)) { Instance->PostTransform(InLevel); } }

	inline bool ShouldStripRuntimeCell(const UWorldPartitionRuntimeCell* InCell) const { if (CanTransform(InCell)) { return Instance->ShouldStripRuntimeCell(InCell); } return false; }
	inline void TransformRuntimeCell(UWorldPartitionRuntimeCell* InCell) const { if (CanTransform(InCell)) { Instance->TransformRuntimeCell(InCell); } }

	/** Runtime cell transformer class */
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay)
	TSubclassOf<UWorldPartitionRuntimeCellTransformer> Class;

	/** Transformer object instance */
	UPROPERTY(VisibleAnywhere, Category = WorldPartitionSetup, Instanced, Meta = (EditCondition = "Class != nullptr", HideEditConditionToggle, NoResetToDefault))
	TObjectPtr<UWorldPartitionRuntimeCellTransformer> Instance;

private:
	inline bool CanTransform(ULevel* InLevel) const { return Instance && Instance->IsEnabled() && Instance->IsLevelTransformable(InLevel); }
	inline bool CanTransform(const UWorldPartitionRuntimeCell* InCell) const { return Instance && Instance->IsEnabled() && Instance->IsCellTransformable(InCell); }
#endif
};

UCLASS(AutoExpandCategories=(WorldPartition), MinimalAPI)
class UWorldPartition final : public UObject, public FActorDescContainerInstanceCollection, public IWorldPartitionCookPackageGenerator
{
	GENERATED_UCLASS_BODY()

	friend class FWorldPartitionActorDesc;
	friend class FWorldPartitionConverter;
	friend class UWorldPartitionConvertCommandlet;
	friend class FWorldPartitionEditorModule;
	friend class FWorldPartitionDetails;
	friend class FUnrealEdMisc;
	friend class UActorDescContainer;
	friend class UActorDescContainerInstance;

public:
#if WITH_EDITOR
	static ENGINE_API UWorldPartition* CreateOrRepairWorldPartition(AWorldSettings* WorldSettings, TSubclassOf<UWorldPartitionEditorHash> EditorHashClass = {}, TSubclassOf<UWorldPartitionRuntimeHash> RuntimeHashClass = {});
	static ENGINE_API bool RemoveWorldPartition(AWorldSettings* WorldSettings);
#endif

#if WITH_EDITOR
	ENGINE_API TArray<FBox> GetUserLoadedEditorRegions() const;

public:
	ENGINE_API void SetEnableStreaming(bool bInEnableStreaming);

	ENGINE_API void OnEnableStreamingChanged();
	ENGINE_API void OnEnableLoadingInEditorChanged();

	ENGINE_API bool IsStreamingEnabledInEditor() const;

	DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionEditorGameWorldEvent, UWorldPartition*);
	static ENGINE_API FWorldPartitionEditorGameWorldEvent OnPrepareEditorGameWorldForPIE;
	static ENGINE_API FWorldPartitionEditorGameWorldEvent OnShutdownEditorGameWorldForPIE;

private:
	ENGINE_API void SavePerUserSettings();
		
	ENGINE_API void OnPackageDirtyStateChanged(UPackage* Package);

	// PIE/Game
	ENGINE_API void OnPreBeginPIE(bool bStartSimulate);
	ENGINE_API void OnPrePIEEnded(bool bWasSimulatingInEditor);
	ENGINE_API void OnCancelPIE();

	ENGINE_API void PrepareEditorGameWorld();
	ENGINE_API void ShutdownEditorGameWorld();

	// WorldDeletegates Events
	ENGINE_API void OnWorldRenamed(UWorld* RenamedWorld);

	// ActorDescContainerInstance Events
	void OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* NewActorDescInstance);
	void OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* ActorDescInstance);
	void OnActorDescInstanceUpdating(FWorldPartitionActorDescInstance* ActorDescInstance);
	void OnActorDescInstanceUpdated(FWorldPartitionActorDescInstance* ActorDescInstance);

	bool ShouldHashUnhashActorDescInstances() const;
	ENGINE_API void InitializeActorDescContainerEditorStreaming(UActorDescContainerInstance* InActorDescContainer);
#endif

	ENGINE_API void OnBeginPlay();

public:
	ENGINE_API const FTransform& GetInstanceTransform() const;
	//~ End UActorDescContainer Interface

	inline bool HasInstanceTransform() const { return InstanceTransform.IsSet(); }

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	ENGINE_API virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;
	ENGINE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
#endif //WITH_EDITOR
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual UWorld* GetWorld() const override;
	ENGINE_API virtual bool ResolveSubobject(const TCHAR* SubObjectPath, UObject*& OutObject, bool bLoadIfExists) override;
	ENGINE_API virtual void BeginDestroy() override;
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface

	// Editor/Runtime conversions
	ENGINE_API bool ConvertEditorPathToRuntimePath(const FSoftObjectPath& InPath, FSoftObjectPath& OutPath) const;

#if WITH_EDITOR
	ENGINE_API bool ConvertContainerPathToEditorPath(const FActorContainerID& InContainerID, const FSoftObjectPath& InPath, FSoftObjectPath& OutPath) const;

	void SetInstanceTransform(const FTransform& InInstanceTransform) { InstanceTransform = InInstanceTransform; }
	ENGINE_API FName GetWorldPartitionEditorName() const;

	// Streaming generation
	bool CanGenerateStreaming() const { return !StreamingPolicy; }

	struct FGenerateStreamingParams
	{
		FGenerateStreamingParams() 
			: ErrorHandler(nullptr)
		{}
		
		FGenerateStreamingParams& SetContainerInstanceCollection(const FActorDescContainerInstanceCollection& InContainerInstanceCollection, const FStreamingGenerationContainerInstanceCollection::ECollectionType& InCollectionType)
		{
			check(ContainerInstanceCollection.IsEmpty());
			ContainerInstanceCollection.SetCollectionType(InCollectionType);
			ContainerInstanceCollection.Append(InContainerInstanceCollection);
			return *this;
		}
		FGenerateStreamingParams& SetErrorHandler(IStreamingGenerationErrorHandler* InErrorHandler) { ErrorHandler = InErrorHandler; return *this; }
		FGenerateStreamingParams& SetOutputLogType(const FString& InOutputLogType) { OutputLogType = InOutputLogType; return *this; }
		FGenerateStreamingParams& SetFilteredClasses(const TArray<TSubclassOf<AActor>>& InFilteredClasses) { FilteredClasses = InFilteredClasses; return *this; }
		
	private:

		TArray<TSubclassOf<AActor>> FilteredClasses;
		FStreamingGenerationContainerInstanceCollection ContainerInstanceCollection;
		TOptional<const FString> OutputLogType;
		IStreamingGenerationErrorHandler* ErrorHandler;

		friend class UWorldPartition;
	};

	struct FGenerateStreamingContext
	{
		FGenerateStreamingContext()
		{}

		// Level Packages to generate
		 TArray<FString>* PackagesToGenerate = nullptr;
		
		// Generated External Streaming Objects
		TArray<URuntimeHashExternalStreamingObjectBase*>* GeneratedExternalStreamingObjects = nullptr;

		 TOptional<FString> OutputLogFilename;

		FGenerateStreamingContext& SetLevelPackagesToGenerate(TArray<FString>* InLevelPackagesToGenerate) { PackagesToGenerate = InLevelPackagesToGenerate; return *this; }
		FGenerateStreamingContext& SetGeneratedExternalStreamingObjects(TArray<URuntimeHashExternalStreamingObjectBase*>* InGeneratedExternalStreamingObjects) { GeneratedExternalStreamingObjects = InGeneratedExternalStreamingObjects; return *this; }
	};

	ENGINE_API bool GenerateStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingContext& InContext);
	ENGINE_API bool GenerateContainerStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingContext& InContext);
	ENGINE_API TUniquePtr<IStreamingGenerationContext> GenerateStreamingGenerationContext(const FGenerateStreamingParams& InParams, FGenerateStreamingContext& InContext);

	ENGINE_API void FlushStreaming();
	ENGINE_API URuntimeHashExternalStreamingObjectBase* FlushStreamingToExternalStreamingObject();

	// Event when world partition was enabled/disabled in the world
	DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionChangedEvent, UWorld*);
	static ENGINE_API FWorldPartitionChangedEvent WorldPartitionChangedEvent;

	DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionGenerateStreamingDelegate, TArray<FString>*);
	FWorldPartitionGenerateStreamingDelegate OnPreGenerateStreaming;

	/**
	 * Experimental: event used to gather actor descriptor mutators.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FWorldPartitionGenerateStreamingActorDescsMutatePhase, const IStreamingGenerationContext* StreamingGenerationContext, TArray<FActorDescViewMutatorInstance>& ActorDescsMutatorsInstances);
	FWorldPartitionGenerateStreamingActorDescsMutatePhase OnGenerateStreamingActorDescsMutatePhase;

	ENGINE_API void RemapSoftObjectPath(FSoftObjectPath& ObjectPath) const;
	ENGINE_API bool IsValidPackageName(const FString& InPackageName);

	// Cooking events
	DECLARE_MULTICAST_DELEGATE_OneParam(FWorldPartitionCookEventDelegate, IWorldPartitionCookPackageContext&);

	ENGINE_API void BeginCook(IWorldPartitionCookPackageContext& CookContext);
	ENGINE_API void EndCook(IWorldPartitionCookPackageContext& CookContext);

	FWorldPartitionCookEventDelegate OnBeginCook;	
	FWorldPartitionCookEventDelegate OnEndCook;

	//~ Begin IWorldPartitionCookPackageGenerator Interface 
	ENGINE_API virtual bool GatherPackagesToCook(IWorldPartitionCookPackageContext& CookContext) override;
	ENGINE_API virtual bool PrepareGeneratorPackageForCook(IWorldPartitionCookPackageContext& CookContext, TArray<UPackage*>& OutModifiedPackages) override;
	ENGINE_API virtual bool PopulateGeneratorPackageForCook(IWorldPartitionCookPackageContext& CookContext, const TArray<FWorldPartitionCookPackage*>& InPackagesToCook, TArray<UPackage*>& OutModifiedPackages) override;
	ENGINE_API virtual bool PopulateGeneratedPackageForCook(IWorldPartitionCookPackageContext& CookContext, const FWorldPartitionCookPackage& InPackagesToCool, TArray<UPackage*>& OutModifiedPackages) override;
	ENGINE_API virtual UWorldPartitionRuntimeCell* GetCellForPackage(const FWorldPartitionCookPackage& InPackageToCook) const override;
	//~ End IWorldPartitionCookPackageGenerator Interface 
	// End Cooking

	ENGINE_API FBox GetEditorWorldBounds() const;
	ENGINE_API FBox GetRuntimeWorldBounds() const;

	inline TConstArrayView<FRuntimeCellTransformerInstance> GetRuntimeCellsTransformerStack() const { return RuntimeCellsTransformerStack; }
	ENGINE_API bool ShouldStripRuntimeCellByTransformerStack(const UWorldPartitionRuntimeCell* InCell) const;
	ENGINE_API void ApplyRuntimeCellsTransformerStack(UWorldPartitionRuntimeCell* InCell);
	ENGINE_API void ApplyRuntimeCellsTransformerStack(ULevel* InLevel);
	ENGINE_API bool HasRuntimeCellTransformerOfType(const UClass* InCellTransformerClass);
	ENGINE_API void AddRuntimeCellTransformerOfType(TSubclassOf<UWorldPartitionRuntimeCellTransformer> InCellTransformerClass, int32 PositionToInsert = -1);
	
	UHLODLayer* GetDefaultHLODLayer() const { return DefaultHLODLayer; }
	void SetDefaultHLODLayer(UHLODLayer* InDefaultHLODLayer) { DefaultHLODLayer = InDefaultHLODLayer; }

	/* Struct of optional parameters passed to SetupHLODActors function. */
	struct FSetupHLODActorsParams
	{
		FSetupHLODActorsParams()
			: SourceControlHelper(nullptr)
			, bReportOnly(false)
			, bConsiderUnsavedHLODActors(false)
			, bSaveActors(true)
		{}

		ISourceControlHelper* SourceControlHelper;
		bool bReportOnly;
		bool bConsiderUnsavedHLODActors;
		bool bSaveActors;
		mutable TArray<TObjectPtr<UWorldPartition>> OutAdditionalWorldPartitionsForStandaloneHLOD;
		TArray<TSharedPtr<IHLODCreationFilter> > Filters;

		FSetupHLODActorsParams& SetSourceControlHelper(ISourceControlHelper* InSourceControlHelper) { SourceControlHelper = InSourceControlHelper; return *this; }
		FSetupHLODActorsParams& SetReportOnly(bool bInReportOnly) { bReportOnly = bInReportOnly; return *this; }
		FSetupHLODActorsParams& SetConsiderUnsavedHLODActors(bool bInConsiderUnsavedHLODActors) { bConsiderUnsavedHLODActors = bInConsiderUnsavedHLODActors; return *this; }
		FSetupHLODActorsParams& SetSaveActors(bool bInSaveActors) { bSaveActors = bInSaveActors; return *this; }
		FSetupHLODActorsParams& SetFilters(const TArray<TSharedPtr<IHLODCreationFilter> >& InFilters) { Filters = InFilters; return *this; }
	};

	ENGINE_API void SetupHLODActors(const FSetupHLODActorsParams& Params);

	// Debugging
	ENGINE_API void DrawRuntimeHashPreview();
	ENGINE_API void DumpActorDescs(const FString& Path);

	ENGINE_API void CheckForErrors(IStreamingGenerationErrorHandler* ErrorHandler) const;

	/* Struct of optional parameters passed to check for errors function. */
	struct FCheckForErrorsParams
	{
		ENGINE_API FCheckForErrorsParams();

		IStreamingGenerationErrorHandler* ErrorHandler;
		bool bEnableStreaming;

		const FActorDescContainerInstanceCollection* ActorDescContainerInstanceCollection;

		TMap<FGuid, const UActorDescContainerInstance*> ActorGuidsToContainerInstanceMap;
				
		FCheckForErrorsParams& SetErrorHandler(IStreamingGenerationErrorHandler* InErrorHandler) { ErrorHandler = InErrorHandler; return *this; }
		FCheckForErrorsParams& SetActorDescContainerInstanceCollection(const FActorDescContainerInstanceCollection* InActorDescContainerInstanceCollection) { ActorDescContainerInstanceCollection = InActorDescContainerInstanceCollection; return *this; }
		FCheckForErrorsParams& SetEnableStreaming(bool bInEnableStreaming) { bEnableStreaming = bInEnableStreaming; return *this; }
		FCheckForErrorsParams& SetActorGuidsToContainerInstanceMap(const TMap<FGuid, const UActorDescContainerInstance*>& InActorGuidsToContainerInstanceMap) { ActorGuidsToContainerInstanceMap = InActorGuidsToContainerInstanceMap; return *this; }
	};

	static ENGINE_API void CheckForErrors(const FCheckForErrorsParams& Params);

	using FStreamingGenerationErrorHandlerOverride = TFunction<IStreamingGenerationErrorHandler*(IStreamingGenerationErrorHandler* InErrorHandler)>;
	ENGINE_API static TOptional<FStreamingGenerationErrorHandlerOverride> StreamingGenerationErrorHandlerOverride;

	ENGINE_API void AppendAssetRegistryTags(FAssetRegistryTagsContext Context) const;

	struct FContainerRegistrationParams
	{
		FContainerRegistrationParams(FName InPackageName)
			: PackageName(InPackageName)
		{}

		/* The long package name of the container package on disk. */
		FName PackageName;

		/* Custom filter function used to filter actors descriptors. */
		TUniqueFunction<bool(const FWorldPartitionActorDesc*)> FilterActorDescFunc;
	};

	// Event when world partition was enabled/disabled in the world
	DECLARE_DELEGATE_TwoParams(FActorDescContainerInstancePreInitializeDelegate, UActorDescContainerInstance::FInitializeParams&, UActorDescContainerInstance*);
	FActorDescContainerInstancePreInitializeDelegate OnActorDescContainerInstancePreInitialize;

	ENGINE_API void UninitializeActorDescContainers();

	DECLARE_MULTICAST_DELEGATE_OneParam(FActorDescContainerRegistrationDelegate, UActorDescContainer*);
		
	ENGINE_API UActorDescContainerInstance* RegisterActorDescContainerInstance(const UActorDescContainerInstance::FInitializeParams& InInitializationParams);
	ENGINE_API bool UnregisterActorDescContainerInstance(UActorDescContainerInstance* InContainerInstance);

	DECLARE_MULTICAST_DELEGATE_OneParam(FActorDescContainerInstanceRegistrationDelegate, UActorDescContainerInstance*);
	FActorDescContainerInstanceRegistrationDelegate OnActorDescContainerInstanceRegistered;
	FActorDescContainerInstanceRegistrationDelegate OnActorDescContainerInstanceUnregistered;

	// Actors pinning
	ENGINE_API void PinActors(const TArray<FGuid>& ActorGuids);
	ENGINE_API void UnpinActors(const TArray<FGuid>& ActorGuids);
	ENGINE_API bool IsActorPinned(const FGuid& ActorGuid) const;

	ENGINE_API void LoadLastLoadedRegions(const TArray<FBox>& EditorLastLoadedRegions);
	ENGINE_API void LoadLastLoadedRegions();

	bool HasLoadedUserCreatedRegions() const { return !!NumUserCreatedLoadedRegions; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FLoaderAdapterStateChangedDelegate, const IWorldPartitionActorLoaderInterface::ILoaderAdapter*);
	FLoaderAdapterStateChangedDelegate LoaderAdapterStateChanged;
		
	ENGINE_API void OnLoaderAdapterStateChanged(IWorldPartitionActorLoaderInterface::ILoaderAdapter* InLoaderAdapter);

	bool IsEnablingStreamingJustified() const { return bEnablingStreamingJustified; }

	bool IsHLODsInEditorAllowed() const { return bAllowShowingHLODsInEditor; }

	UFUNCTION()
	ENGINE_API bool IsStandaloneHLODAllowed() const;

	void SetIsStandaloneHLODWorld(bool bInIsStandaloneHLODWorld) { bIsStandaloneHLODWorld = bInIsStandaloneHLODWorld; }
#endif

	bool HasStandaloneHLOD() const { return bHasStandaloneHLOD; }
	bool IsStandaloneHLODWorld() const { return bIsStandaloneHLODWorld; }
	bool ShouldExternalizeHLODAssets() const { return bExternalizeHLODAssets; }

public:
	static ENGINE_API bool IsSimulating(bool bIncludeTestEnableSimulationStreamingSource = true);
	ENGINE_API int32 GetStreamingStateEpoch() const;

	ENGINE_API bool CanInitialize(UWorld* InWorld) const;
	ENGINE_API void Initialize(UWorld* World, const FTransform& InTransform);
	ENGINE_API bool IsInitialized() const;
	ENGINE_API void Uninitialize();

	ENGINE_API bool SupportsStreaming() const;
	ENGINE_API bool IsStreamingEnabled() const;
	ENGINE_API bool CanStream() const;
	ENGINE_API bool IsServer() const;
	ENGINE_API bool IsServerStreamingEnabled() const;
	bool IsContentBundleEnabled() const { return !bDisableContentBundles; }
	ENGINE_API bool IsServerStreamingOutEnabled() const;
	ENGINE_API bool UseMakingVisibleTransactionRequests() const;
	ENGINE_API bool UseMakingInvisibleTransactionRequests() const;
	ENGINE_API void InvalidateVisiblityTransactionRequestCachedVariables();

	ENGINE_API bool IsMainWorldPartition() const;

	ENGINE_API bool CanAddCellToWorld(const IWorldPartitionCell* InCell) const;
	ENGINE_API bool IsStreamingCompleted(const TArray<FWorldPartitionStreamingSource>* InStreamingSources) const;
	ENGINE_API bool IsStreamingCompleted(EWorldPartitionRuntimeCellState QueryState, const TArray<FWorldPartitionStreamingQuerySource>& QuerySources, bool bExactState) const;
	ENGINE_API bool GetIntersectingCells(const TArray<FWorldPartitionStreamingQuerySource>& InSources, TArray<const IWorldPartitionCell*>& OutCells) const;

	ENGINE_API bool IsExternalStreamingObjectInjected(URuntimeHashExternalStreamingObjectBase* InExternalStreamingObject) const;
	ENGINE_API bool InjectExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject);
	ENGINE_API bool RemoveExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject);

	ENGINE_API const TArray<FWorldPartitionStreamingSource>& GetStreamingSources() const;

	// Debugging
	ENGINE_API bool DrawRuntimeHash2D(FWorldPartitionDraw2DContext& DrawContext);
	ENGINE_API void DrawRuntimeHash3D();
	ENGINE_API void DrawRuntimeCellsDetails(UCanvas* Canvas, FVector2D& Offset);

	ENGINE_API void OnCellShown(const UWorldPartitionRuntimeCell* InCell);
	ENGINE_API void OnCellHidden(const UWorldPartitionRuntimeCell* InCell);

	ENGINE_API EWorldPartitionStreamingPerformance GetStreamingPerformance() const;

	ENGINE_API bool IsStreamingInEnabled() const;
	ENGINE_API void DisableStreamingIn();
	ENGINE_API void EnableStreamingIn();

	ENGINE_API UDataLayerManager* GetDataLayerManager() const;
	ENGINE_API UDataLayerManager* GetResolvingDataLayerManager() const;
	ENGINE_API UExternalDataLayerManager* GetExternalDataLayerManager() const;

	inline EWorldPartitionDataLayersLogicOperator GetDataLayersLogicOperator() const { return DataLayersLogicOperator; }

#if WITH_EDITORONLY_DATA
	UPROPERTY(DuplicateTransient)
	TObjectPtr<UWorldPartitionEditorHash> EditorHash;

	FLoaderAdapterAlwaysLoadedActors* AlwaysLoadedActors;
	FLoaderAdapterActorList* ForceLoadedActors;
	FLoaderAdapterActorList* PinnedActors;

	IWorldPartitionEditor* WorldPartitionEditor;

private:
	/** Class of WorldPartitionStreamingPolicy to be used to manage world partition streaming. */
	UPROPERTY()
	TSubclassOf<UWorldPartitionStreamingPolicy> WorldPartitionStreamingPolicyClass;

	/** Used to know if it's the first time streaming is enabled on this world. */
	UPROPERTY()
	bool bStreamingWasEnabled;

	/** Used to know if we need to recheck if the user should enable streaming based on world size. */
	bool bShouldCheckEnableStreamingWarning;

#if WITH_EDITOR
	/** Holds the world actor references, filled in PreSave and used in AppendAssetRegistryTags. */
	mutable TArray<ActorsReferencesUtils::FActorReference> WorldExternalActorReferences;
#endif
#endif

private:
	void OnCleanupLevel();

public:
#if WITH_EDITOR
	UActorDescContainerInstance* GetActorDescContainerInstance() const { return ActorDescContainerInstance; }

	void SetContainerInstanceClass(TSubclassOf<UActorDescContainerInstance> InContainerInstanceClass)
	{
		check(!IsInitialized());
		ContainerInstanceClass = InContainerInstanceClass;
	}
#endif			

	UPROPERTY()
	TObjectPtr<UWorldPartitionRuntimeHash> RuntimeHash;

	/** Enables streaming for this world. */
	UPROPERTY()
	bool bEnableStreaming;

	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay, meta = (EditConditionHides, EditCondition = "bEnableStreaming", HideEditConditionToggle))
	EWorldPartitionServerStreamingMode ServerStreamingMode;

	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay, meta = (EditConditionHides, EditCondition = "bEnableStreaming", HideEditConditionToggle))
	EWorldPartitionServerStreamingOutMode ServerStreamingOutMode;

private:
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay)
	EWorldPartitionDataLayersLogicOperator DataLayersLogicOperator;

#if WITH_EDITORONLY_DATA
	/** Whether HLODs should be allowed to be displayed in the editor for this map */
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay, meta = (EditConditionHides, EditCondition = "bEnableStreaming", HideEditConditionToggle))
	uint8 bAllowShowingHLODsInEditor : 1;
#endif

	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, meta = (DisplayName = "Build Standalone HLOD", EditConditionHides, EditCondition = "IsStandaloneHLODAllowed()", HideEditConditionToggle))
	uint8 bHasStandaloneHLOD : 1;

	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, meta = (EditConditionHides, EditCondition = "bHasStandaloneHLOD", HideEditConditionToggle))
	uint8 bExternalizeHLODAssets : 1;

	UPROPERTY()
	uint8 bIsStandaloneHLODWorld : 1;

	/** if set to true, this removes any content bundles from this world and also removes content bundle editing */
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, AdvancedDisplay)
	uint8 bDisableContentBundles : 1;

	TObjectPtr<UWorld> World;

#if WITH_EDITOR
	bool bForceRefreshAlwaysLoaded;
	bool bForceRefreshEditor;
	bool bEnablingStreamingJustified;
	bool bIsPIE;
	int32 NumUserCreatedLoadedRegions;
#endif

#if WITH_EDITORONLY_DATA
	/** Runtime cells transform stack objects */
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup)
	TArray<FRuntimeCellTransformerInstance> RuntimeCellsTransformerStack;

	/** Runtime cells transform stack objects execution stats */
	float RuntimeCellsTransformerStackDumpTime = 0.0f;
	TMap<UClass*, TPair<double, int32>> RuntimeCellsTransformerStackTimes;

	// Default HLOD layer
	UPROPERTY(EditAnywhere, Category = WorldPartitionSetup, meta = (DisplayName = "Default HLOD Layer", EditCondition="bEnableStreaming", EditConditionHides, HideEditConditionToggle))
	TObjectPtr<class UHLODLayer> DefaultHLODLayer;

	TArray<FWorldPartitionReference> LoadedSubobjects;

	struct FWorldPartitionExternalDirtyActorsTrackerReference
	{
		using Type = FWorldPartitionReference;
		using OwnerType = UWorldPartition;
		static FWorldPartitionReference Store(UWorldPartition* InOwner, AActor* InActor) { return FWorldPartitionReference(InOwner, InActor->GetActorGuid()); }
	};

	class FWorldPartitionExternalDirtyActorsTracker : public TExternalDirtyActorsTracker<FWorldPartitionExternalDirtyActorsTrackerReference>
	{
	public:
		FWorldPartitionExternalDirtyActorsTracker();
		FWorldPartitionExternalDirtyActorsTracker(UWorldPartition* InWorldPartition);

		//~ Begin TExternalDirtyActorsTracker interface
		virtual void OnRemoveNonDirtyActor(const TWeakObjectPtr<AActor> InActor, FWorldPartitionReference& InValue) override;
		virtual void Tick(float InDeltaTime) override;
		//~ End TExternalDirtyActorsTracker interface

		void SetNonDirtyTrackingDisabled(bool bInIsNonDirtyTrackingDisabled) { bIsNonDirtyTrackingDisabled = bInIsNonDirtyTrackingDisabled; }
		bool IsNonDirtyTrackingDisabled() const { return bIsNonDirtyTrackingDisabled; }
	private:
		TSet<TPair<TWeakObjectPtr<AActor>, FWorldPartitionReference>> NonDirtyActors;
		bool bIsNonDirtyTrackingDisabled = false;
	};

	TUniquePtr<FWorldPartitionExternalDirtyActorsTracker> ExternalDirtyActorsTracker;

	TSet<FString> GeneratedLevelStreamingPackageNames;

	UPROPERTY(Transient)
	TObjectPtr<UActorDescContainerInstance> ActorDescContainerInstance;

	UPROPERTY(Transient)
	TSubclassOf<UActorDescContainerInstance> ContainerInstanceClass;
public:
	TOptional<bool> bOverrideEnableStreamingInEditor;

	friend class FDisableNonDirtyActorTrackingScope;

	// Use scope around actor package save calls to prevent newly created spatial actors from being pinned (actors will get unloaded instead)
	class FDisableNonDirtyActorTrackingScope
	{
	public:
		ENGINE_API FDisableNonDirtyActorTrackingScope(UWorldPartition* InWorldPartition, bool bInDisableTracking);
		ENGINE_API ~FDisableNonDirtyActorTrackingScope();
	private:
		UWorldPartition* WorldPartition = nullptr;
		bool bPreviousValue = false;
	};
private:
#endif

	//~ Begin Verse support
	ENGINE_API void AddReferencedObject(UObject* InObject);
	ENGINE_API void RemoveReferencedObject(UObject* InObject);
	const TSet<TObjectPtr<UObject>>& GetReferencedObjects() const { return ReferencedObjects; }
	//~ End Verse support

	/** Referenced objects (used by verse) */
	UPROPERTY(Transient)
	TSet<TObjectPtr<UObject>> ReferencedObjects;

	friend class UVerseWorldPartitionHelperBase;

	EWorldPartitionInitState InitState;
	TOptional<FTransform> InstanceTransform;

	// Defaults to true, can be set to false to temporarly disable Streaming in of new cells.
	bool bStreamingInEnabled;

	mutable TOptional<bool> bCachedUseMakingInvisibleTransactionRequests;
	mutable TOptional<bool> bCachedUseMakingVisibleTransactionRequests;
	mutable TOptional<bool> bCachedIsServerStreamingEnabled;
	mutable TOptional<bool> bCachedIsServerStreamingOutEnabled;

	UPROPERTY(Transient)
	TObjectPtr<UDataLayerManager> DataLayerManager;

	UPROPERTY(Transient)
	TObjectPtr<UExternalDataLayerManager> ExternalDataLayerManager;

	UPROPERTY(Transient)
	mutable TObjectPtr<UWorldPartitionStreamingPolicy> StreamingPolicy;

#if WITH_EDITOR
	static ENGINE_API int32 LoadingRangeBugItGo;
	static ENGINE_API int32 EnableSimulationStreamingSource;
	static ENGINE_API int32 WorldExtentToEnableStreaming;
	static ENGINE_API bool DebugDedicatedServerStreaming;
	static ENGINE_API FAutoConsoleVariableRef CVarLoadingRangeBugItGo;
	static ENGINE_API FAutoConsoleVariableRef CVarEnableSimulationStreamingSource;
	static ENGINE_API FAutoConsoleVariableRef CVarWorldExtentToEnableStreaming;
	static ENGINE_API FAutoConsoleVariableRef CVarDebugDedicatedServerStreaming;
#endif

	mutable int32 StreamingStateEpoch;
	static ENGINE_API int32 GlobalEnableServerStreaming;
	static ENGINE_API bool bGlobalEnableServerStreamingOut;
	static ENGINE_API bool bUseMakingVisibleTransactionRequests;
	static ENGINE_API bool bUseMakingInvisibleTransactionRequests;
	static ENGINE_API FAutoConsoleVariableRef CVarEnableServerStreaming;
	static ENGINE_API FAutoConsoleVariableRef CVarEnableServerStreamingOut;
	static ENGINE_API FAutoConsoleVariableRef CVarUseMakingVisibleTransactionRequests;
	static ENGINE_API FAutoConsoleVariableRef CVarUseMakingInvisibleTransactionRequests;
	
	ENGINE_API void OnWorldMatchStarting();
	ENGINE_API void OnWorldPreBeginPlay();
	ENGINE_API void OnStreamingStateUpdated();
	ENGINE_API void Tick(float DeltaSeconds);
	void OnPreChangeStreamingContent();
	int32 GetUpdateStreamingStateEpoch() const;

	// Delegates registration
	ENGINE_API void RegisterDelegates();
	ENGINE_API void UnregisterDelegates();	

#if WITH_EDITOR
	ENGINE_API void OnLevelActorDeleted(AActor* Actor);
	ENGINE_API void OnPostBugItGoCalled(const FVector& Loc, const FRotator& Rot);

	void HashActorDescInstance(FWorldPartitionActorDescInstance* ActorDescInstance);
	void UnhashActorDescInstance(FWorldPartitionActorDescInstance* ActorDescInstance);
	void OnContentBundleRemovedContent(const FContentBundleEditor* ContentBundle);
	IWorldPartitionCookPackageObject* GetCookPackageObject(const FWorldPartitionCookPackage& PackageToCook) const;
	bool HasStreamingContent() const;

public:
	// Editor loader adapters management
	template <typename T, typename... ArgsType>
	UWorldPartitionEditorLoaderAdapter* CreateEditorLoaderAdapter(ArgsType&&... Args)
	{
		UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter = NewObject<UWorldPartitionEditorLoaderAdapter>(GetTransientPackage());
		EditorLoaderAdapter->SetLoaderAdapter(new T(Forward<ArgsType>(Args)...));
		RegisteredEditorLoaderAdapters.Add(EditorLoaderAdapter);
		return EditorLoaderAdapter;
	}

	void ReleaseEditorLoaderAdapter(UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter)
	{
		verify(RegisteredEditorLoaderAdapters.Remove(EditorLoaderAdapter) != INDEX_NONE);
		EditorLoaderAdapter->Release();
	}

	const TSet<TObjectPtr<UWorldPartitionEditorLoaderAdapter>>& GetRegisteredEditorLoaderAdapters() const
	{
		return RegisteredEditorLoaderAdapters;
	}
#endif

public:
	/**
	 * Experimental: World Asset Streaming can be used to inject streaming levels into the runtime grids dynamically, with one level of HLODs support.
	 */
	struct FRegisterWorldAssetStreamingParams
	{
		struct FWorldAssetDesc
		{
			TSoftObjectPtr<UWorld> WorldAsset;
			FName TargetGrid;
		};

		FRegisterWorldAssetStreamingParams() 
		{}

		FWorldAssetDesc WorldAssetDesc;
		TArray<FWorldAssetDesc> HLODWorldAssetDescs;

		FGuid Guid;
		FTransform Transform;
		FBox Bounds;
		int32 Priority = 0;
		FString CellInstanceSuffix;
		bool bBoundsPlacement = false;

		bool IsValid() const
		{
			return !WorldAssetDesc.WorldAsset.IsNull() && !WorldAssetDesc.TargetGrid.IsNone() && Guid.IsValid() && Bounds.IsValid;
		}

		FRegisterWorldAssetStreamingParams& SetWorldAsset(const TSoftObjectPtr<UWorld>& InWorldAsset, const FName& InTargetGrid) { WorldAssetDesc.WorldAsset = InWorldAsset; WorldAssetDesc.TargetGrid = InTargetGrid; return *this; }
		FRegisterWorldAssetStreamingParams& AddHLODWorldAsset(const TSoftObjectPtr<UWorld>& InHLODWorldAsset, const FName& InHLODTargetGrid) { HLODWorldAssetDescs.Add({ InHLODWorldAsset, InHLODTargetGrid }); return *this; }
		FRegisterWorldAssetStreamingParams& RemoveAllHLODWorldAssets() { HLODWorldAssetDescs.Reset(); return *this; }
		FRegisterWorldAssetStreamingParams& SetGuid(const FGuid InGuid) { Guid = InGuid; return *this; }
		FRegisterWorldAssetStreamingParams& SetTransform(const FTransform InTransform) { Transform = InTransform; return *this; }
		FRegisterWorldAssetStreamingParams& SetBounds(const FBox& InBounds) { Bounds = InBounds; return *this; }
		FRegisterWorldAssetStreamingParams& SetPriority(const int32& InPriority) { Priority = InPriority; return *this; }
		FRegisterWorldAssetStreamingParams& SetCellInstanceSuffix(const FString& InCellInstanceSuffix) { CellInstanceSuffix = InCellInstanceSuffix; return *this; }
		FRegisterWorldAssetStreamingParams& SetBoundsPlacement(bool bInBoundsPlacement) { bBoundsPlacement = bInBoundsPlacement; return *this; }
	};

	ENGINE_API bool SupportsWorldAssetStreaming(const FName& InTargetGrid);
	ENGINE_API FGuid RegisterWorldAssetStreaming(const FRegisterWorldAssetStreamingParams& InParams);
	ENGINE_API bool UnregisterWorldAssetStreaming(const FGuid& InWorldAssetStreamingGuid);
	ENGINE_API TArray<UWorldPartitionRuntimeCell*> GetWorldAssetStreamingCells(const FGuid& InWorldAssetStreamingGuid);

private:

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient, NonTransactional)
	TSet<TObjectPtr<UWorldPartitionEditorLoaderAdapter>> RegisteredEditorLoaderAdapters;
#endif

#if !UE_BUILD_SHIPPING
	ENGINE_API void GetOnScreenMessages(FCoreDelegates::FSeverityMessageMap& OutMessages);
#endif
	class AWorldPartitionReplay* Replay;

	friend struct FWorldPartitionStreamingContext;
	friend class AWorldPartitionReplay;
	friend class UWorldPartitionSubsystem;
	friend class UExternalDataLayerManager;
#if WITH_EDITOR
	friend class FScopedCookingExternalStreamingObject;
#endif
};
