// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Hash/CityHash.h"
#include "WorldPartitionActorContainerID.generated.h"

USTRUCT()
struct FActorContainerPath
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FGuid> ContainerGuids;

	friend uint32 GetTypeHash(const FActorContainerPath& InActorContainerPath)
	{
		if (InActorContainerPath.ContainerGuids.Num() > 0)
		{
			FGuid Guid = InActorContainerPath.ContainerGuids[0];
			for (int32 i = 1; i < InActorContainerPath.ContainerGuids.Num(); ++i)
			{
				Guid = FGuid::Combine(Guid, InActorContainerPath.ContainerGuids[i]);
			}

			return GetTypeHash(Guid);
		}

		return 0;
	}

	friend FArchive& operator<<(FArchive& Ar, FActorContainerPath& InActorContainerPath)
	{
		Ar << InActorContainerPath.ContainerGuids;
		return Ar;
	}

	bool operator==(const FActorContainerPath& InOther) const = default;
	bool operator!=(const FActorContainerPath& InOther) const = default;
};

USTRUCT()
struct FActorContainerID
{
	GENERATED_USTRUCT_BODY()

	FActorContainerID()
	{}

	FActorContainerID(const FActorContainerID& InOther)
		: Guid(InOther.Guid)
	{}

	FActorContainerID(const FActorContainerID& InParent, const FActorContainerID& InOther)
		: Guid(FGuid::Combine(InParent.Guid, InOther.Guid))
	{}

	FActorContainerID(const FActorContainerID& InParent, const FGuid& InActorGuid)
		: Guid(FGuid::Combine(InParent.Guid, InActorGuid))
	{}

	FActorContainerID(const FActorContainerID& InParent, const FActorContainerPath& InPath)
		: Guid(InParent.Guid)
	{
		for (const FGuid& ContainerGuid : InPath.ContainerGuids)
		{
			Guid = FGuid::Combine(Guid, ContainerGuid);
		}
	}

	void operator=(const FActorContainerID& InOther)
	{
		Guid = InOther.Guid;
	}

	bool operator==(const FActorContainerID& InOther) const
	{
		return Guid == InOther.Guid;
	}

	bool operator!=(const FActorContainerID& InOther) const
	{
		return Guid != InOther.Guid;
	}

	bool operator<(const FActorContainerID& InOther) const
	{
		return Guid < InOther.Guid;
	}

	friend FArchive& operator<<(FArchive& Ar, FActorContainerID& InActorContainer)
	{
		return Ar << InActorContainer.Guid;
	}

	bool IsMainContainer() const
	{
		return !Guid.IsValid();
	}

	FString ToString() const
	{
		return Guid.ToString();
	}

	FString ToShortString() const
	{
		const uint64 ID = CityHash64((const char*)&Guid, sizeof(FGuid));
		return FString::Printf(TEXT("%016llx"), ID);
	}

	FGuid GetActorGuid(const FGuid& InActorGuid) const
	{
		// Preserve the original actor guid for top level actors
		return Guid.IsValid() ? FGuid::Combine(Guid, InActorGuid) : InActorGuid;
	}

	friend inline uint32 GetTypeHash(const FActorContainerID& InContainerID)
	{
		return GetTypeHash(InContainerID.Guid);
	}

	static FActorContainerID GetMainContainerID()
	{
		return FActorContainerID();
	}

private:
	UPROPERTY()
	FGuid Guid;
};
