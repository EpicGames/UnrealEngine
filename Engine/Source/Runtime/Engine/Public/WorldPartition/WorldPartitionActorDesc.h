// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "PropertyPairsMap.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/SubclassOf.h"
#include "Misc/Guid.h"
#include "Misc/TVariant.h"
#include "WorldPartition/WorldPartitionActorDescType.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"

class FActorDescArchive;
class UActorComponent;

// Struct used to create actor descriptor
struct FWorldPartitionActorDescInitData
{
	FWorldPartitionActorDescInitData()
		: DataSource(TInPlaceType<TArray<uint8>>(), TArray<uint8>())
	{}

	FWorldPartitionActorDescInitData(FActorDescArchive* InArchive)
		: DataSource(TInPlaceType<FActorDescArchive*>(), InArchive)
	{}

	UClass* NativeClass;
	FName PackageName;
	FSoftObjectPath ActorPath;

	TArray<uint8>& GetSerializedData()
	{
		check(DataSource.IsType<TArray<uint8>>());
		return DataSource.Get<TArray<uint8>>();
	}

	const TArray<uint8>& GetSerializedData() const
	{
		check(DataSource.IsType<TArray<uint8>>());
		return DataSource.Get<TArray<uint8>>();
	}

	FActorDescArchive* GetArchive() const
	{
		check(DataSource.IsType<FActorDescArchive*>());
		return DataSource.Get<FActorDescArchive*>();
	}

	bool IsUsingArchive() const
	{
		return DataSource.IsType<FActorDescArchive*>();
	}
		
	FWorldPartitionActorDescInitData& SetNativeClass(UClass* InNativeClass) { NativeClass = InNativeClass; return *this; }
	FWorldPartitionActorDescInitData& SetPackageName(FName InPackageName) { PackageName = InPackageName; return *this; }
	FWorldPartitionActorDescInitData& SetActorPath(const FSoftObjectPath& InActorPath) { ActorPath = InActorPath; return *this; }

private:
	// Provide SerializedData or an already initialized Archive
	TVariant<TArray<uint8>, FActorDescArchive*> DataSource;
};

struct FWorldPartitionRelativeBounds
{
	FWorldPartitionRelativeBounds()
		: bIsValid(false)
	{ }

	FWorldPartitionRelativeBounds(EForceInit ForceInit)
		: Center(ForceInit)
		, Rotation(ForceInit)
		, Extents(ForceInit)
		, bIsValid(false)
	{ }

	FWorldPartitionRelativeBounds(const FVector& InCenter, const FQuat& InRotation, const FVector& InExtents)
		: Center(InCenter)
		, Rotation(InRotation)
		, Extents(InExtents)
		, bIsValid(true)
	{ }

	FWorldPartitionRelativeBounds(const FBox& InBox)
		: FWorldPartitionRelativeBounds(ForceInit)
	{
		if (InBox.IsValid)
		{
			InBox.GetCenterAndExtents(Center, Extents);
			Rotation = FQuat::Identity;
			bIsValid = true;
		}
	}

	bool Equals(const FWorldPartitionRelativeBounds& Other, double Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return (!bIsValid && !Other.bIsValid) 
			|| ((bIsValid && Other.bIsValid) && Center.Equals(Other.Center, Tolerance) && Rotation.Equals(Other.Rotation, Tolerance) && Extents.Equals(Other.Extents, Tolerance));
	}

	bool IsValid() const
	{
		return bIsValid;
	}

	FBox ToAABB() const
	{
		FBox Result(ForceInit);

		if (bIsValid)
		{
			double X = Extents.X;
			double Y = Extents.Y;
			double Z = Extents.Z;

			Result += Rotation * FVector(-X, -Y, -Z) + Center;
			Result += Rotation * FVector( X, -Y, -Z) + Center;
			Result += Rotation * FVector(-X,  Y, -Z) + Center;
			Result += Rotation * FVector( X,  Y, -Z) + Center;
			Result += Rotation * FVector(-X, -Y,  Z) + Center;
			Result += Rotation * FVector( X, -Y,  Z) + Center;
			Result += Rotation * FVector(-X,  Y,  Z) + Center;
			Result += Rotation * FVector( X,  Y,  Z) + Center;
		}

		return Result;
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("IsValid=%s, Center=(%s), Rotation=(%s), Extents=(%s)"), bIsValid ? TEXT("true") : TEXT("false"), *Center.ToString(), *Rotation.Euler().ToString(), *Extents.ToString());
	}

	friend FArchive& operator<<(FArchive& Ar, FWorldPartitionRelativeBounds& Bounds)
	{
		return Ar << Bounds.Center << Bounds.Rotation << Bounds.Extents << Bounds.bIsValid;
	}

	[[nodiscard]] FWorldPartitionRelativeBounds TransformBy(const FTransform& Transform) const
	{
		if (!bIsValid)
		{
			return FWorldPartitionRelativeBounds(ForceInit);
		}

		FWorldPartitionRelativeBounds NewBounds;
		NewBounds.Center = Transform.TransformPosition(Center);
		NewBounds.Rotation = Transform.GetRotation() * Rotation;
		NewBounds.Extents = Extents * Transform.GetScale3D();
		NewBounds.bIsValid = true;

		return NewBounds;
	}

	[[nodiscard]] FWorldPartitionRelativeBounds InverseTransformBy(const FTransform& Transform) const
	{
		if (!bIsValid)
		{
			return FWorldPartitionRelativeBounds(ForceInit);
		}

		FWorldPartitionRelativeBounds NewBounds;
		NewBounds.Center = Transform.InverseTransformPosition(Center);
		NewBounds.Rotation = Transform.GetRotation().Inverse() * Rotation;
		NewBounds.Extents = Extents * FTransform::GetSafeScaleReciprocal(Transform.GetScale3D());
		NewBounds.bIsValid = true;

		return NewBounds;
	}

private:
	FVector Center;
	FQuat Rotation;
	FVector Extents;
	bool bIsValid;
};

struct FWorldPartitionAssetDataPatcher
{
	virtual ~FWorldPartitionAssetDataPatcher() = default;
	virtual bool DoPatch(FString& InOutString) = 0;
	virtual bool DoPatch(FName& InOutName) = 0;
	virtual bool DoPatch(FSoftObjectPath& InOutSoft) = 0;
	virtual bool DoPatch(FTopLevelAssetPath& InOutPath) = 0;
};

class AActor;
class UWorldPartition;
class UActorDescContainerInstance;
class FWorldPartitionActorDescInstance;
class UActorDescContainer;
class IWorldPartitionActorDescInstanceView;
class IStreamingGenerationErrorHandler;

enum class EContainerClusterMode : uint8
{
	Partitioned, // Per Actor Partitioning
};

template <typename T, class F>
inline bool CompareUnsortedArrays(const TArray<T>& Array1, const TArray<T>& Array2, F Func)
{
	if (Array1.Num() == Array2.Num())
	{
		TArray<T> SortedArray1(Array1);
		TArray<T> SortedArray2(Array2);
		SortedArray1.Sort(Func);
		SortedArray2.Sort(Func);
		return SortedArray1 == SortedArray2;
	}
	return false;
}

template <typename T>
inline bool CompareUnsortedArrays(const TArray<T>& Array1, const TArray<T>& Array2)
{
	return CompareUnsortedArrays(Array1, Array2, [](const T& A, const T& B) { return A < B; });
}

template <>
inline bool CompareUnsortedArrays(const TArray<FName>& Array1, const TArray<FName>& Array2)
{
	return CompareUnsortedArrays(Array1, Array2, [](const FName& A, const FName& B) { return A.LexicalLess(B); });
}
#endif // WITH_EDITOR

/**
 * Represents a Actor Metadata (editor-only)
 */
struct FWorldPartitionComponentDescInitData
{
#if WITH_EDITOR
	FTopLevelAssetPath Type;
	FName Name;

	friend FArchive& operator<<(FArchive& Ar, FWorldPartitionComponentDescInitData& ComponentDescInitData)
	{
		return Ar << ComponentDescInitData.Type << ComponentDescInitData.Name;
	}
#endif // WITH_EDITOR
};

class FWorldPartitionComponentDesc
{
#if WITH_EDITOR
	friend class FActorDescArchive;
	friend class FWorldPartitionActorDesc;

public:
	virtual ~FWorldPartitionComponentDesc() = default;

	virtual uint32 GetSizeOf() const = 0;
	ENGINE_API virtual void Init(const UActorComponent* InComponent);
	ENGINE_API virtual void Serialize(FArchive& Ar);
	
	inline FTopLevelAssetPath GetBaseClass() const { return BaseClass; }
	inline FTopLevelAssetPath GetNativeClass() const { return NativeClass; }
	inline UClass* GetComponentNativeClass() const { return ComponentNativeClass; }

	inline FString ToString() const
	{
		return FString::Printf(TEXT("BaseClass=%s, NativeClass=%s, ComponentName=%s"), *BaseClass.ToString(), *NativeClass.ToString(), *ComponentName.ToString());
	}

protected:
	ENGINE_API FWorldPartitionComponentDesc();

	FTopLevelAssetPath BaseClass;
	FTopLevelAssetPath NativeClass;
	UClass* ComponentNativeClass;
	FName ComponentName;
#endif // WITH_EDITOR
};

class FWorldPartitionActorDesc
{
#if WITH_EDITOR
	friend class AActor;
	friend class UWorldPartition;
	friend struct FWorldPartitionHandleImpl;
	friend struct FWorldPartitionReferenceImpl;
	friend struct FWorldPartitionActorDescUtils;
	friend struct FWorldPartitionActorDescUnitTestAcccessor;
	friend class FAssetHeaderPatcherInner;
	friend class FActorDescArchive;
	friend class FWorldPartitionActorDescInstance;
	friend class FStreamingGenerationTempActorDescInstance;
	friend class FPropertyOverrideUtils;
	template<class U> friend class TActorDescContainerCollection;

public:
	struct FContainerInstance
	{
		UActorDescContainerInstance* ContainerInstance = nullptr;
		FTransform Transform = FTransform::Identity;
		EContainerClusterMode ClusterMode;
		TMap<FActorContainerID, TSet<FGuid>> FilteredActors;
	};

	virtual ~FWorldPartitionActorDesc() = default;

	inline const FGuid& GetGuid() const { return Guid; }
	
	inline FTopLevelAssetPath GetBaseClass() const { return BaseClass; }
	inline FTopLevelAssetPath GetNativeClass() const { return NativeClass; }
	inline UClass* GetActorNativeClass() const { return ActorNativeClass; }

	inline FName GetRuntimeGrid() const { return RuntimeGrid; }
	inline bool GetIsSpatiallyLoaded() const { return RuntimeBounds.IsValid ? bIsSpatiallyLoaded : false; }
	inline bool GetIsSpatiallyLoadedRaw() const { return bIsSpatiallyLoaded; }
	inline bool GetActorIsEditorOnly() const { return bActorIsEditorOnly; }
	ENGINE_API bool GetActorIsEditorOnlyLoadedInPIE() const;
	inline bool GetActorIsRuntimeOnly() const { return bActorIsRuntimeOnly; }

	inline bool GetActorIsHLODRelevant() const { return bActorIsHLODRelevant; }
	inline FSoftObjectPath GetHLODLayer() const { return HLODLayer; }
	ENGINE_API TArray<FName> GetDataLayers(bool bIncludeExternalDataLayer = true) const;
	inline const TArray<FName>& GetNonExternalDataLayers() const { return DataLayers; }
	ENGINE_API FName GetExternalDataLayer() const;
	inline const FSoftObjectPath& GetExternalDataLayerAsset() const { return ExternalDataLayerAsset; }

	inline const TArray<FName>& GetTags() const { return Tags; }

	inline FName GetActorPackage() const { return ActorPackage; }
	inline FSoftObjectPath GetActorSoftPath() const { return ActorPath; }
	inline FName GetActorLabel() const { return ActorLabel; }
	inline FName GetFolderPath() const { return FolderPath; }
	inline const FGuid& GetFolderGuid() const { return FolderGuid; }
	inline const FTransform& GetActorTransform() const { return ActorTransform; }
	inline const FTransform& GetActorTransformRelative() const { return ActorTransformRelative; }

	ENGINE_API FBox GetEditorBounds() const;
	ENGINE_API FBox GetRuntimeBounds() const;

	inline const FGuid& GetParentActor() const { return ParentActor; }

	UE_DEPRECATED(5.8, "Actors no longer contain data layers without assets")
	inline bool IsUsingDataLayerAsset() const { return true; }

	inline void AddProperty(FName PropertyName, FName PropertyValue = NAME_None) { Properties.AddProperty(PropertyName, PropertyValue); }
	inline bool GetProperty(FName PropertyName, FName* PropertyValue) const { return Properties.GetProperty(PropertyName, PropertyValue); }
	inline bool HasProperty(FName PropertyName) const { return Properties.HasProperty(PropertyName); }
	inline const TArray<TUniquePtr<FWorldPartitionComponentDesc>>& GetComponentDescs() const { return ComponentDescs; }

	ENGINE_API FName GetActorName() const;
	ENGINE_API FName GetActorLabelOrName() const;
	ENGINE_API FName GetDisplayClassName() const;

	// Faster accessors for names as strings
	ENGINE_API const FString& GetActorNameString() const;
	ENGINE_API const FString& GetActorLabelString() const;
	ENGINE_API const FString& GetDisplayClassNameString() const;

	inline bool IsDefaultActorDesc() const { return bIsDefaultActorDesc; }
		
	virtual bool IsChildContainerInstance() const { return false; }
	virtual FName GetChildContainerPackage() const { return NAME_None; }
	virtual FString GetChildContainerName() const { return FString(); }
	virtual EWorldPartitionActorFilterType GetChildContainerFilterType() const { return EWorldPartitionActorFilterType::None; }
	virtual const FWorldPartitionActorFilter* GetChildContainerFilter() const { return nullptr; }
	virtual UActorDescContainer* GetChildContainer() const { return nullptr; }
	
	ENGINE_API FGuid GetContentBundleGuid() const;

	virtual const FGuid& GetSceneOutlinerParent() const { return GetParentActor(); }
	virtual bool IsResaveNeeded() const { return bIsSpatiallyLoaded && !RuntimeBounds.IsValid; }

	ENGINE_API virtual void CheckForErrors(const IWorldPartitionActorDescInstanceView* InActorDescView, IStreamingGenerationErrorHandler* ErrorHandler) const;

	bool operator==(const FWorldPartitionActorDesc& Other) const
	{
		return Guid == Other.Guid;
	}

	friend uint32 GetTypeHash(const FWorldPartitionActorDesc& Key)
	{
		return GetTypeHash(Key.Guid);
	}
	
	//~ Begin Deprecation
	UE_DEPRECATED(5.1, "SetIsSpatiallyLoadedRaw is deprecated and should not be used.")
	inline void SetIsSpatiallyLoadedRaw(bool bNewIsSpatiallyLoaded) { bIsSpatiallyLoaded = bNewIsSpatiallyLoaded; }
	//~ End Deprecation
	
public:
	const TArray<FGuid>& GetReferences() const
	{
		return References;
	}

	const TArray<FGuid>& GetEditorOnlyReferences() const
	{
		return EditorOnlyReferences;
	}

	bool IsEditorOnlyReference(const FGuid& ReferenceGuid) const
	{
		return EditorOnlyReferences.Contains(ReferenceGuid);
	}

	UActorDescContainer* GetContainer() const
	{
		return Container;
	}

	virtual void SetContainer(UActorDescContainer* InContainer)
	{
		check(!Container || !InContainer);
		Container = InContainer;
	}

	ENGINE_API virtual void UpdateActorToWorld(FTransform* InParentTransform = nullptr);

	enum class EToStringMode : uint8
	{
		Guid,
		Compact,
		Full,
		Verbose,
		ForDiff
	};

	ENGINE_API FString ToString(EToStringMode Mode = EToStringMode::Compact) const;

	ENGINE_API virtual void Init(const AActor* InActor);
	ENGINE_API virtual void Init(const FWorldPartitionActorDescInitData& DescData);

	static ENGINE_API void Patch(const FWorldPartitionActorDescInitData& DescData, TArray<uint8>& OutData, FWorldPartitionAssetDataPatcher* InAssetDataPatcher);

	ENGINE_API virtual bool Equals(const FWorldPartitionActorDesc* Other) const;	

	virtual bool HasStandaloneHLOD() const { return false; }

	/**
	 * Returns true if resaving this actor will have an impact on streaming generation. Before class descriptors, properties changed on a Blueprint 
	 * that would affect streaming generation weren't taken into account until the actors affected by the change were resaved.
	 */
	ENGINE_API virtual bool ShouldResave(const FWorldPartitionActorDesc* Other) const;

	ENGINE_API void SerializeTo(TArray<uint8>& OutData, FWorldPartitionActorDesc* BaseDesc = nullptr) const;

	using FActorDescDeprecator = TFunction<void(FArchive&, FWorldPartitionActorDesc*)>;
	static ENGINE_API void RegisterActorDescDeprecator(TSubclassOf<AActor> ActorClass, const FActorDescDeprecator& Deprecator);

	ENGINE_API bool IsMainWorldOnly() const;
	ENGINE_API bool IsListedInSceneOutliner() const;

protected:
	void InitTransientProperties(const FWorldPartitionActorDescInitData& DescData);
	virtual bool GetChildContainerInstance(const FWorldPartitionActorDescInstance* InActorDescInstance, FContainerInstance& OutContainerInstance) const { return false; }
	virtual UActorDescContainerInstance* CreateChildContainerInstance(const FWorldPartitionActorDescInstance* InActorDescInstance) const { return nullptr; }
	virtual UWorldPartition* GetLoadedChildWorldPartition(const FWorldPartitionActorDescInstance* InActorDescInstance) const { return nullptr; }
	ENGINE_API virtual bool IsRuntimeRelevant(const FWorldPartitionActorDescInstance* InActorDescInstance) const;
	ENGINE_API virtual bool IsEditorRelevant(const FWorldPartitionActorDescInstance* InActorDescInstance) const;

	ENGINE_API FWorldPartitionActorDesc();

	ENGINE_API virtual void TransferFrom(const FWorldPartitionActorDesc* From);

	virtual void TransferWorldData(const FWorldPartitionActorDesc* From)
	{
		ActorTransform = From->ActorTransform;
		RuntimeBounds = From->RuntimeBounds;
		EditorBounds = From->EditorBounds;


		ActorTransformRelative = From->ActorTransformRelative;
		RuntimeBoundsRelative = From->RuntimeBoundsRelative;
		EditorBoundsRelative = From->EditorBoundsRelative;
		bHasValidRelativeBounds = From->bHasValidRelativeBounds;
	}

	virtual uint32 GetSizeOf() const { return sizeof(FWorldPartitionActorDesc); }

	ENGINE_API virtual void Serialize(FArchive& Ar);

	ENGINE_API void SetEditorBounds(const FBox& InEditorBounds);
	ENGINE_API void SetRuntimeBounds(const FBox& InRuntimeBounds);

	// Persistent
	FGuid							Guid;
	FTopLevelAssetPath				BaseClass;
	FTopLevelAssetPath				NativeClass;	// Not serialized, comes from initialization data
	FName							ActorPackage;	// Not serialized, comes from initialization data
	FSoftObjectPath					ActorPath;		// Not serialized, comes from initialization data
	FName							ActorLabel;
	FTransform						ActorTransformRelative;			// Relative transform for actors with parent actor, actor-to-world transform otherwise
	FWorldPartitionRelativeBounds	RuntimeBoundsRelative;	// Runtime bounds relative to actor-to-world transform
	FWorldPartitionRelativeBounds	EditorBoundsRelative;	// Editor bounds relative to actor-to-world transform
	FName							RuntimeGrid;
	bool							bIsSpatiallyLoaded;
	bool							bActorIsEditorOnly;
	bool							bActorIsRuntimeOnly;
	bool							bActorIsMainWorldOnly;
	bool							bActorIsHLODRelevant;
	bool							bActorIsListedInSceneOutliner;
	bool							bIsUsingDataLayerAsset; // No longer used, but serialized manually and needed if we ever load old data. Treat as a UPROPERTY in terms of deprecation.
	FSoftObjectPath					HLODLayer;
	TArray<FName>					DataLayers;
	FSoftObjectPath					ExternalDataLayerAsset;
	TArray<FGuid>					References;
	TArray<FGuid>					EditorOnlyReferences; // References that aren't necessarily editor only but referenced through an editor only property.
	TArray<FName>					Tags;
	FPropertyPairsMap				Properties;
	FName							FolderPath;
	FGuid							FolderGuid;
	FGuid							ParentActor; // Used to validate settings against parent (to warn on layer/placement compatibility issues)
	FGuid							ContentBundleGuid;

	TArray<TUniquePtr<FWorldPartitionComponentDesc>> ComponentDescs;
	
	// Transient
	UClass*							ActorNativeClass;
	FName							ActorName;
	FString							ActorNameString;
	FString							ActorLabelString;
	FString							ActorDisplayClassNameString;
	UActorDescContainer*			Container;
	FTransform						ActorTransform;	// Current actor-to-world transform, considering parent transform
	FBox							RuntimeBounds;	// Current runtime bounds, considering parent transform
	FBox							EditorBounds;	// Current editor bounds, considering parent transform
	bool							bIsDefaultActorDesc;
	bool							bHasValidRelativeBounds;

	static ENGINE_API TMap<TSubclassOf<AActor>, FActorDescDeprecator> Deprecators;

private:
	void FixupStreamingBounds();
#endif // WITH_EDITOR
};

#if WITH_EDITOR

class FWorldPartitionActorDescStringBuilder : public TStringBuilder<1024>
{
public:
	FWorldPartitionActorDescStringBuilder(FWorldPartitionActorDesc::EToStringMode InMode);
	void AppendFromString(const TCHAR* Name, const FString& Value, bool bUseLineStart = true);
	void AppendFromBool(const TCHAR* Name, bool bValue, bool bUseLineStart = true);

	template <typename Type>
	void AppendToString(const TCHAR* Name, const Type& Value, bool bUseLineStart = true)
	{
		if constexpr (std::is_same_v<Type, FBox>)
		{
			if (!Value.IsValid)
			{
				AppendPair(Name, TEXT("IsValid=false"), bUseLineStart);
				return;
			}
		}

		if constexpr (std::is_same_v<Type, FWorldPartitionRelativeBounds>)
		{
			if (!Value.IsValid())
			{
				AppendPair(Name, TEXT("IsValid=false"), bUseLineStart);
				return;
			}
		}

		AppendFromString(Name, Value.ToString(), bUseLineStart);
	}

private:
	void AppendPair(const TCHAR* Name, const TCHAR* Value, bool bUseLineStart);

	FWorldPartitionActorDesc::EToStringMode Mode;
	const TCHAR LineStart;
	const TCHAR* LineEnd;
};

#endif // WITH_EDITOR