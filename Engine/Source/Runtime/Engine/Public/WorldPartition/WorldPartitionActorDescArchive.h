// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "Serialization/Archive.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

class FWorldPartitionActorDesc;
class FWorldPartitionComponentDesc;
struct FWorldPartitionAssetDataPatcher;

class FActorDescArchive : public FArchiveProxy
{
public:
	FActorDescArchive(FArchive& InArchive, FWorldPartitionActorDesc* InActorDesc, const FWorldPartitionActorDesc* InBaseActorDesc = nullptr);

	void Init(const FTopLevelAssetPath InClassPath = FTopLevelAssetPath());
	void SetComponentDesc(FWorldPartitionComponentDesc* InComponentDesc);

	//~ Begin FArchive Interface
	virtual FArchive& operator<<(FName& Value) override { FArchiveProxy::operator<<(Value); return *this; }
	virtual FArchive& operator<<(FText& Value) override { unimplemented(); return *this; }
	virtual FArchive& operator<<(UObject*& Value) override { unimplemented(); return *this; }
	virtual FArchive& operator<<(FLazyObjectPtr& Value) override { unimplemented(); return *this; }
	virtual FArchive& operator<<(FObjectPtr& Value) override { unimplemented(); return *this; }
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override { unimplemented(); return *this; }
	virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	virtual FArchive& operator<<(FWeakObjectPtr& Value) override { unimplemented(); return *this; }
	//~ End FArchive Interface

	using FArchive::operator<<;
	virtual FArchive& operator<<(FTopLevelAssetPath& Value); 

	template <typename DestPropertyType, typename SourcePropertyType>
	struct TDeltaSerializer
	{
		using FDeprecateFunction = TFunction<void(DestPropertyType&, const SourcePropertyType&)>;

		explicit TDeltaSerializer(DestPropertyType& InValue)
			: Value(InValue)
		{}

		explicit TDeltaSerializer(DestPropertyType& InValue, FDeprecateFunction InFunc)
			: Value(InValue)
			, Func(InFunc)
		{}

		friend FArchive& operator<<(FArchive& Ar, const TDeltaSerializer<DestPropertyType, SourcePropertyType>& V)
		{
			FActorDescArchive& ActorDescAr = (FActorDescArchive&)Ar;
			
			check(ActorDescAr.BaseDesc || Ar.IsSaving());
			check(ActorDescAr.ActorDesc);

			Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

			auto GetBaseDefaultValue = [&V, &ActorDescAr](UPTRINT BasePtr, UPTRINT DefaultPtr, uint32 SizeOf) -> const DestPropertyType*
			{
				const UPTRINT PropertyOffset = (UPTRINT)&V.Value - BasePtr;

				if ((PropertyOffset + sizeof(V.Value)) <= SizeOf)
				{
					const DestPropertyType* RefValue = (const DestPropertyType*)(DefaultPtr + PropertyOffset);
					return RefValue;
				}

				return nullptr;
			};

			uint8 bSerialize = 1;

			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionActorClassDescSerialize)
			{
				if (Ar.IsSaving())
				{
					if (ActorDescAr.ComponentDesc)
					{
						if (ActorDescAr.BaseComponentDesc)
						{
							if constexpr (std::is_same_v<DestPropertyType, SourcePropertyType>)
							{
								// When saving, we expect the class descriptor to be the exact type as what we are serializing.
								const DestPropertyType* BaseDefaultValue = GetBaseDefaultValue(*(UPTRINT*)&ActorDescAr.ComponentDesc, *(UPTRINT*)&ActorDescAr.BaseComponentDesc, ActorDescAr.BaseComponentDescSizeof);
								check(BaseDefaultValue);

								bSerialize = (V.Value != *BaseDefaultValue) ? 1 : 0;
							}
						}
					}
					else if (ActorDescAr.BaseDesc)
					{
						if constexpr (std::is_same_v<DestPropertyType, SourcePropertyType>)
						{
							// When saving, we expect the class descriptor to be the exact type as what we are serializing.
							const DestPropertyType* BaseDefaultValue = GetBaseDefaultValue(*(UPTRINT*)&ActorDescAr.ActorDesc, *(UPTRINT*)&ActorDescAr.BaseDesc, ActorDescAr.BaseDescSizeof);
							check(BaseDefaultValue);

							bSerialize = (V.Value != *BaseDefaultValue) ? 1 : 0;
						}
					}
				}

				Ar << bSerialize;
			}

			if (bSerialize)
			{
				if constexpr (std::is_same_v<DestPropertyType, SourcePropertyType>)
				{
					Ar << V.Value;
				}
				else
				{
					check(Ar.IsLoading());

					SourcePropertyType SourceValue;
					Ar << SourceValue;
					V.Func(V.Value, SourceValue);
				}
			}
			else if (Ar.IsLoading())
			{
				// When loading, we need to handle a different class descriptor in case of missing classes, etc.
				if (ActorDescAr.ComponentDesc)
				{
					if (const DestPropertyType* BaseDefaultValue = GetBaseDefaultValue(*(UPTRINT*)&ActorDescAr.ComponentDesc, *(UPTRINT*)&ActorDescAr.BaseComponentDesc, ActorDescAr.BaseComponentDescSizeof))
					{
						V.Value = *BaseDefaultValue;
					}
				}
				else if (const DestPropertyType* BaseDefaultValue = GetBaseDefaultValue(*(UPTRINT*)&ActorDescAr.ActorDesc, *(UPTRINT*)&ActorDescAr.BaseDesc, ActorDescAr.BaseDescSizeof))
				{				
					V.Value = *BaseDefaultValue;
				}
				else
				{
					check(ActorDescAr.bIsMissingBaseDesc);
				}
			}

			return Ar;
		}

		DestPropertyType& Value;
		FDeprecateFunction Func;
	};

	FWorldPartitionActorDesc* ActorDesc;
	FWorldPartitionComponentDesc* ComponentDesc;
	const FWorldPartitionActorDesc* BaseDesc;
	const FWorldPartitionComponentDesc* BaseComponentDesc;
	uint32 BaseDescSizeof;
	uint32 BaseComponentDescSizeof;
	bool bIsMissingBaseDesc;
};

class FActorDescArchivePatcher : public FActorDescArchive
{
public:
	FActorDescArchivePatcher(FArchive& InArchive, FWorldPartitionActorDesc* InActorDesc, FArchive& OutArchive, FWorldPartitionAssetDataPatcher* InAssetDataPatcher)
		: FActorDescArchive(InArchive, InActorDesc)
		, OutAr(OutArchive)
		, AssetDataPatcher(InAssetDataPatcher)
		, bIsPatching(false)
	{
		check(AssetDataPatcher);
	}

	//~ Begin FArchive Interface
	virtual FArchive& operator<<(FName& Value) override;
	virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	virtual void Serialize(void* V, int64 Length) override;
	//~ End FArchive Interface

	virtual FArchive& operator<<(FTopLevelAssetPath& Value) override;

private:
	FArchive& OutAr;
	FWorldPartitionAssetDataPatcher* AssetDataPatcher;
	bool bIsPatching;
};

template <typename DestPropertyType, typename SourcePropertyType = DestPropertyType>
using TDeltaSerialize = FActorDescArchive::TDeltaSerializer<DestPropertyType, SourcePropertyType>;
#endif
