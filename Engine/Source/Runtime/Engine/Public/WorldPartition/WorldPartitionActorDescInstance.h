// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/DataLayer/DataLayerInstanceNames.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class UWorldPartition;
class UActorDescContainerInstance;
#endif // WITH_EDITOR

class FWorldPartitionActorDescInstance
{
#if WITH_EDITOR
	friend struct FWorldPartitionHandleImpl;
	friend struct FWorldPartitionReferenceImpl;
	friend struct FWorldPartitionActorDescUtils;
	friend class FWorldPartitionActorDesc;
	friend class UActorDescContainerInstance;
	friend class FWorldPartitionLoadingContext;
	friend struct FWorldPartitionActorDescUnitTestAcccessor;
	friend class UDataLayerManager;
	friend class UWorldPartition;
	friend class FWorldPartitionLoadingContext;
	friend class IWorldPartitionActorLoaderInterface;
	friend class UWorldPartitionEditorSpatialHash;
	friend class ILoaderAdapterSpatial;

protected:
	ENGINE_API FWorldPartitionActorDescInstance();
public:
	ENGINE_API FWorldPartitionActorDescInstance(UActorDescContainerInstance* InContainerInstance, FWorldPartitionActorDesc* InActorDesc);
	virtual ~FWorldPartitionActorDescInstance() = default;
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FActorDescUpdatedEvent, FWorldPartitionActorDescInstance*);
	UE_EXPERIMENTAL(5.8, "This new static delegate is experimental and the API is subject to change.")
	static ENGINE_API FActorDescUpdatedEvent OnActorDescUpdated;

	virtual const FGuid& GetGuid() const { return ActorDesc->GetGuid(); }
	virtual FTopLevelAssetPath GetBaseClass() const { return ActorDesc->GetBaseClass(); }
	virtual FTopLevelAssetPath GetNativeClass() const { return ActorDesc->GetNativeClass(); }
	virtual UClass* GetActorNativeClass() const { return ActorDesc->GetActorNativeClass(); }
	
	virtual FName GetRuntimeGrid() const { return ActorDesc->GetRuntimeGrid(); }
	virtual bool GetIsSpatiallyLoaded() const { return !GetForceNonSpatiallyLoaded() && ActorDesc->GetIsSpatiallyLoaded(); }
	virtual bool GetActorIsEditorOnly() const { return ActorDesc->GetActorIsEditorOnly(); }
	virtual bool GetActorIsEditorOnlyLoadedInPIE() const { return ActorDesc->GetActorIsEditorOnlyLoadedInPIE(); }
	virtual bool GetActorIsRuntimeOnly() const { return ActorDesc->GetActorIsRuntimeOnly(); }
	ENGINE_API virtual bool IsRuntimeRelevant() const;
	ENGINE_API virtual bool IsEditorRelevant() const;

	UE_DEPRECATED(5.8, "Actors no longer contain data layers without assets")
	virtual bool IsUsingDataLayerAsset() const { return true; }
	virtual TArray<FName> GetDataLayers() const { return ActorDesc->GetDataLayers(); }

	virtual bool GetActorIsHLODRelevant() const { return ActorDesc->GetActorIsHLODRelevant(); }
	virtual FSoftObjectPath GetHLODLayer() const { return ActorDesc->GetHLODLayer(); }

	virtual const TArray<FName>& GetTags() const { return ActorDesc->GetTags(); }
	virtual FName GetActorPackage() const { return ActorDesc->GetActorPackage(); }
	ENGINE_API virtual FSoftObjectPath GetActorSoftPath() const;
	virtual FName GetActorLabel() const { return ActorDesc->GetActorLabel(); }
	ENGINE_API virtual FName GetActorName() const;
	virtual FName GetFolderPath() const { return ActorDesc->GetFolderPath(); }
	virtual const FGuid& GetFolderGuid() const { return ActorDesc->GetFolderGuid(); }
	virtual const FTransform& GetActorTransform() const { return ActorDesc->GetActorTransform(); }

	ENGINE_API virtual FBox GetEditorBounds() const;
	ENGINE_API virtual FBox GetRuntimeBounds() const;

	int32 GetAsyncLoadID() const { return AsyncLoadID; }

	virtual bool GetProperty(FName PropertyName, FName* PropertyValue) const { return ActorDesc->GetProperty(PropertyName, PropertyValue); }
	virtual bool HasProperty(FName PropertyName) const { return ActorDesc->HasProperty(PropertyName); }

	virtual const TArray<FGuid>& GetReferences() const { return ActorDesc->GetReferences(); }
	virtual const TArray<FGuid>& GetEditorOnlyReferences() const { return ActorDesc->GetEditorOnlyReferences(); }
	virtual bool IsEditorOnlyReference(const FGuid& ReferenceGuid) const { return ActorDesc->IsEditorOnlyReference(ReferenceGuid); }
	
	virtual const FGuid& GetParentActor() const { return ActorDesc->GetParentActor(); }
		
	virtual FGuid GetContentBundleGuid() const { return ActorDesc->GetContentBundleGuid(); }
	virtual const FSoftObjectPath& GetExternalDataLayerAsset() const { return ActorDesc->GetExternalDataLayerAsset(); }

	virtual bool IsChildContainerInstance() const { return ChildContainerInstance || ActorDesc->IsChildContainerInstance(); }
	virtual FName GetChildContainerPackage() const { return ActorDesc->GetChildContainerPackage(); }
	virtual FString GetChildContainerName() const { return ActorDesc->GetChildContainerName(); }
	virtual EWorldPartitionActorFilterType GetChildContainerFilterType() const { return ActorDesc->GetChildContainerFilterType(); }
	virtual const FWorldPartitionActorFilter* GetChildContainerFilter() const { return ActorDesc->GetChildContainerFilter(); }
	virtual bool GetChildContainerInstance(FWorldPartitionActorDesc::FContainerInstance& OutContainerInstance) const { return ActorDesc->GetChildContainerInstance(this, OutContainerInstance); }
			
	virtual bool IsMainWorldOnly() const { return ActorDesc->IsMainWorldOnly(); }
	virtual bool IsListedInSceneOutliner() const { return ActorDesc->IsListedInSceneOutliner(); }
	virtual const FGuid& GetSceneOutlinerParent() const { return ActorDesc->GetSceneOutlinerParent(); }
			
	inline const FWorldPartitionActorDesc* GetActorDesc() const { return ActorDesc; }
	
	virtual bool HasResolvedDataLayerInstanceNames() const { return ResolvedDataLayerInstanceNames.IsSet(); }
	ENGINE_API virtual const FDataLayerInstanceNames& GetDataLayerInstanceNames() const;

	ENGINE_API virtual bool IsLoaded(bool bEvenIfPendingKill = false) const;
	ENGINE_API virtual AActor* GetActor(bool bEvenIfPendingKill = true, bool bEvenIfUnreachable = false) const;
	
	ENGINE_API virtual FString ToString(FWorldPartitionActorDesc::EToStringMode Mode = FWorldPartitionActorDesc::EToStringMode::Compact) const;
	
	virtual UActorDescContainerInstance* GetContainerInstance() const { return ContainerInstance; }

	inline FName GetActorLabelOrName() const { return GetActorLabel().IsNone() ? GetActorName() : GetActorLabel(); }

	inline UActorDescContainerInstance* GetChildContainerInstance() const { return ChildContainerInstance; }

	ENGINE_API virtual TWeakObjectPtr<AActor>* GetActorPtr(bool bEvenIfPendingKill = true, bool bEvenIfUnreachable = false) const;
	ENGINE_API virtual bool IsValid() const;
				
	inline void SetForceNonSpatiallyLoaded(bool bForce) { bIsForcedNonSpatiallyLoaded = bForce; }
	inline bool GetForceNonSpatiallyLoaded() const { return bIsForcedNonSpatiallyLoaded; }

	inline void SetUnloadedReason(FText* InUnloadedReason) { UnloadedReason = InUnloadedReason; }

	FName GetDisplayClassName() const { return ActorDesc->GetDisplayClassName(); }
	
	ENGINE_API const FText& GetUnloadedReason() const;

	ENGINE_API const FString& GetActorNameString() const;
	ENGINE_API const FString& GetActorLabelString() const;
	ENGINE_API const FString& GetDisplayClassNameString() const;

protected:
	virtual FBox GetLocalEditorBounds(bool bInForceComputeOnChildContainerInstance = false) const;

	ENGINE_API bool StartAsyncLoad();
	ENGINE_API void FlushAsyncLoad() const;
	void AsyncLoadFinished(const FName& ActorPackage, const FName& PackageName, UPackage* Package, bool bSuccessful) const;
	ENGINE_API void MarkUnload();

	UWorldPartition* GetLoadedChildWorldPartition() const { return ActorDesc->GetLoadedChildWorldPartition(this); }
	ENGINE_API void UpdateActorDesc(FWorldPartitionActorDesc* InActorDesc);
	void Invalidate();
		
	inline uint32 IncSoftRefCount() const
	{
		return ++SoftRefCount;
	}

	inline uint32 DecSoftRefCount() const
	{
		check(SoftRefCount > 0);
		return --SoftRefCount;
	}

	inline uint32 IncHardRefCount() const
	{
		return ++HardRefCount;
	}

	inline uint32 DecHardRefCount() const
	{
		check(HardRefCount > 0);
		return --HardRefCount;
	}

	inline uint32 GetSoftRefCount() const
	{
		return SoftRefCount;
	}

	inline uint32 GetHardRefCount() const
	{
		return HardRefCount;
	}

	inline void SetDataLayerInstanceNames(const FDataLayerInstanceNames& InDataLayerInstanceNames) { ResolvedDataLayerInstanceNames = InDataLayerInstanceNames; }

	virtual ENGINE_API void RegisterChildContainerInstance();
	virtual ENGINE_API void UnregisterChildContainerInstance();
	virtual ENGINE_API void UpdateChildContainerInstance();

private:
	/** Called by UActorDescContainerInstance::OnActorDescPreUpdate */
	void OnPreUpdate();
	void InvalidateCachedBounds();

protected:
	TObjectPtr<UActorDescContainerInstance>		ContainerInstance;

	mutable uint32								SoftRefCount;
	mutable uint32								HardRefCount;
	TOptional<FDataLayerInstanceNames>			ResolvedDataLayerInstanceNames;
	bool										bIsForcedNonSpatiallyLoaded;
	bool										bIsRegisteringOrUnregistering;
	mutable FText*								UnloadedReason;
	mutable int32								AsyncLoadID;

	// Instancing Path if set
	TOptional<FSoftObjectPath>					ActorPath;

	mutable TWeakObjectPtr<AActor>				ActorPtr;
	FWorldPartitionActorDesc*					ActorDesc;

	TObjectPtr<UActorDescContainerInstance>		ChildContainerInstance;

	mutable TOptional<FBox>						ChildContainerLocalEditorBoundsCache;
#endif // WITH_EDITOR
};
