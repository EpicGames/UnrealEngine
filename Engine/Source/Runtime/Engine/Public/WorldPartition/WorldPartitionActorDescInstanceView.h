// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstanceViewInterface.h"

/**
 * Base class for a view on top of an actor descriptor instance pointer.
 */
class FWorldPartitionActorDescInstanceView : public IWorldPartitionActorDescInstanceView
{
public:
	FWorldPartitionActorDescInstanceView(const FWorldPartitionActorDescInstance* InActorDescInstance)
		: ActorDescInstance(InActorDescInstance)
	{}
		
	//~ Begin IWorldPartitionActorDescInstanceView interface
	virtual const FGuid& GetGuid() const override { return ActorDescInstance->GetGuid(); }

	virtual FTopLevelAssetPath GetBaseClass() const override { return ActorDescInstance->GetBaseClass(); }
	virtual FTopLevelAssetPath GetNativeClass() const override { return ActorDescInstance->GetNativeClass(); }
	virtual UClass* GetActorNativeClass() const override { return ActorDescInstance->GetActorNativeClass(); }

	virtual FName GetRuntimeGrid() const override { return ActorDescInstance->GetRuntimeGrid(); }
	virtual bool GetIsSpatiallyLoaded() const override { return ActorDescInstance->GetIsSpatiallyLoaded(); }
	virtual bool GetActorIsEditorOnly() const override { return ActorDescInstance->GetActorIsEditorOnly(); }
	virtual bool GetActorIsEditorOnlyLoadedInPIE() const override { return ActorDescInstance->GetActorIsEditorOnlyLoadedInPIE(); }
	virtual bool GetActorIsRuntimeOnly() const override { return ActorDescInstance->GetActorIsRuntimeOnly(); }
	virtual bool IsRuntimeRelevant() const override { return ActorDescInstance->IsRuntimeRelevant(); }
	virtual bool IsEditorRelevant() const override { return ActorDescInstance->IsEditorRelevant(); }

	UE_DEPRECATED(5.8, "Actors no longer contain data layers without assets")
	virtual bool IsUsingDataLayerAsset() const override { return true; }
	virtual TArray<FName> GetDataLayers() const override { return ActorDescInstance->GetDataLayers(); }

	virtual bool GetActorIsHLODRelevant() const override { return ActorDescInstance->GetActorIsHLODRelevant(); }
	virtual FSoftObjectPath GetHLODLayer() const override { return ActorDescInstance->GetHLODLayer(); }

	virtual const TArray<FName>& GetTags() const override { return ActorDescInstance->GetTags(); }
	virtual FName GetActorPackage() const override { return ActorDescInstance->GetActorPackage(); }
	virtual FSoftObjectPath GetActorSoftPath() const override { return ActorDescInstance->GetActorSoftPath(); }
	virtual FName GetActorLabel() const override { return ActorDescInstance->GetActorLabel(); }
	virtual FName GetActorName() const override { return ActorDescInstance->GetActorName(); }
	virtual FName GetFolderPath() const override { return ActorDescInstance->GetFolderPath(); }
	virtual const FGuid& GetFolderGuid() const override { return ActorDescInstance->GetFolderGuid(); }

	virtual FBox GetEditorBounds() const override { return ActorDescInstance->GetEditorBounds(); }
	virtual FBox GetRuntimeBounds() const override { return ActorDescInstance->GetRuntimeBounds(); }

	virtual bool GetProperty(FName PropertyName, FName* PropertyValue) const override { return ActorDescInstance->GetProperty(PropertyName, PropertyValue); }
	virtual bool HasProperty(FName PropertyName) const override { return ActorDescInstance->HasProperty(PropertyName); }

	virtual const TArray<FGuid>& GetReferences() const override { return ActorDescInstance->GetReferences(); }
	virtual const TArray<FGuid>& GetEditorOnlyReferences() const override { return ActorDescInstance->GetEditorOnlyReferences(); }
	virtual bool IsEditorOnlyReference(const FGuid& ReferenceGuid) const override { return ActorDescInstance->IsEditorOnlyReference(ReferenceGuid); }

	virtual const FGuid& GetParentActor() const override { return ActorDescInstance->GetParentActor(); }

	virtual FGuid GetContentBundleGuid() const override { return ActorDescInstance->GetContentBundleGuid(); }
	virtual const FSoftObjectPath& GetExternalDataLayerAsset() const override { return ActorDescInstance->GetExternalDataLayerAsset(); }

	virtual bool IsChildContainerInstance() const override { return ActorDescInstance->IsChildContainerInstance(); }
	virtual FName GetChildContainerPackage() const override { return ActorDescInstance->GetChildContainerPackage(); }
	virtual EWorldPartitionActorFilterType GetChildContainerFilterType() const override { return ActorDescInstance->GetChildContainerFilterType(); }
	virtual const FWorldPartitionActorFilter* GetChildContainerFilter() const override { return ActorDescInstance->GetChildContainerFilter(); }
	virtual bool GetChildContainerInstance(FWorldPartitionActorDesc::FContainerInstance& OutContainerInstance) const override { return ActorDescInstance->GetChildContainerInstance(OutContainerInstance); }

	virtual bool IsMainWorldOnly() const override { return ActorDescInstance->IsMainWorldOnly(); }
	virtual bool IsListedInSceneOutliner() const override { return ActorDescInstance->IsListedInSceneOutliner(); }
	virtual const FGuid& GetSceneOutlinerParent() const override { return ActorDescInstance->GetSceneOutlinerParent(); }

	virtual void CheckForErrors(IStreamingGenerationErrorHandler* ErrorHandler) const override { GetActorDesc()->CheckForErrors(this, ErrorHandler); }

	virtual FString ToString(FWorldPartitionActorDesc::EToStringMode Mode) const override { return ActorDescInstance->ToString(Mode); }
	virtual const FWorldPartitionActorDesc* GetActorDesc() const override { return ActorDescInstance->GetActorDesc(); }

	virtual bool HasResolvedDataLayerInstanceNames() const override { return ActorDescInstance->HasResolvedDataLayerInstanceNames(); }
	const FDataLayerInstanceNames& GetDataLayerInstanceNames() const override { return ActorDescInstance->GetDataLayerInstanceNames(); }

	virtual AActor* GetActor(bool bEvenIfPendingKill = true, bool bEvenIfUnreachable = false) const override { return ActorDescInstance->GetActor(bEvenIfPendingKill, bEvenIfUnreachable); }
	virtual bool IsLoaded(bool bEvenIfPendingKill = false) const override { return ActorDescInstance->IsLoaded(bEvenIfPendingKill); }

	virtual UActorDescContainerInstance* GetContainerInstance() const override { return ActorDescInstance->GetContainerInstance(); }
	//~ End IWorldPartitionActorDescInstanceView interface

private:
	const FWorldPartitionActorDescInstance* ActorDescInstance;
};
#endif // WITH_EDITOR