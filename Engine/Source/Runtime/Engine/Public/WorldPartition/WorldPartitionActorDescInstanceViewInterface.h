// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/DataLayer/DataLayerInstanceNames.h"

class AActor;
class IStreamingGenerationErrorHandler;
struct FWorldPartitionActorFilter;
enum class EWorldPartitionActorFilterType : uint8;

/**
 * Interface for a view on top of an actor descriptor instance, used to cache information that can be different 
 * than the actor descriptor instance itself.
 */
class IWorldPartitionActorDescInstanceView
{
public:
	virtual ~IWorldPartitionActorDescInstanceView() = default;

	virtual const FGuid& GetGuid() const = 0;

	virtual FTopLevelAssetPath GetBaseClass() const = 0;
	virtual FTopLevelAssetPath GetNativeClass() const = 0;
	virtual UClass* GetActorNativeClass() const = 0;
	
	virtual FName GetRuntimeGrid() const = 0;
	virtual bool GetIsSpatiallyLoaded() const = 0;
	virtual bool GetActorIsEditorOnly() const = 0;
	virtual bool GetActorIsEditorOnlyLoadedInPIE() const = 0;
	virtual bool GetActorIsRuntimeOnly() const = 0;
	virtual bool IsRuntimeRelevant() const = 0;
	virtual bool IsEditorRelevant() const = 0;

	virtual bool IsUsingDataLayerAsset() const = 0;
	virtual TArray<FName> GetDataLayers() const = 0;

	virtual bool GetActorIsHLODRelevant() const = 0;
	virtual FSoftObjectPath GetHLODLayer() const = 0;
	
	virtual const TArray<FName>& GetTags() const = 0;
	virtual FName GetActorPackage() const = 0;
	virtual FSoftObjectPath GetActorSoftPath() const = 0;
	virtual FName GetActorLabel() const = 0;
	virtual FName GetActorName() const = 0;
	virtual FName GetFolderPath() const = 0;
	virtual const FGuid& GetFolderGuid() const = 0;
		
	virtual FBox GetEditorBounds() const = 0;
	virtual FBox GetRuntimeBounds() const = 0;
		
	virtual bool GetProperty(FName PropertyName, FName* PropertyValue) const = 0;
	virtual bool HasProperty(FName PropertyName) const = 0;

	virtual const TArray<FGuid>& GetReferences() const = 0;
	virtual const TArray<FGuid>& GetEditorOnlyReferences() const = 0;
	virtual bool IsEditorOnlyReference(const FGuid& ReferenceGuid) const = 0;
		
	virtual const FGuid& GetParentActor() const = 0;
	
	virtual FGuid GetContentBundleGuid() const = 0;
	virtual const FSoftObjectPath& GetExternalDataLayerAsset() const = 0;
	
	virtual bool IsChildContainerInstance() const = 0;
	virtual FName GetChildContainerPackage() const = 0;
	virtual EWorldPartitionActorFilterType GetChildContainerFilterType() const = 0;
	virtual const FWorldPartitionActorFilter* GetChildContainerFilter() const = 0;
	virtual bool GetChildContainerInstance(FWorldPartitionActorDesc::FContainerInstance& OutContainerInstance) const = 0;
			
	virtual bool IsMainWorldOnly() const = 0;
	virtual bool IsListedInSceneOutliner() const = 0;
	virtual const FGuid& GetSceneOutlinerParent() const = 0;
		
	virtual FString ToString(FWorldPartitionActorDesc::EToStringMode Mode) const = 0;
	virtual const FWorldPartitionActorDesc* GetActorDesc() const = 0;

	virtual FName GetActorLabelOrName() const { return GetActorLabel().IsNone() ? GetActorName() : GetActorLabel(); }

	virtual bool HasResolvedDataLayerInstanceNames() const = 0;
	virtual const FDataLayerInstanceNames& GetDataLayerInstanceNames() const = 0;

	virtual AActor* GetActor(bool bEvenIfPendingKill = true, bool bEvenIfUnreachable = false) const = 0;
	virtual bool IsLoaded(bool bEvenIfPendingKill = false) const = 0;

	virtual UActorDescContainerInstance* GetContainerInstance() const = 0;

	virtual void CheckForErrors(IStreamingGenerationErrorHandler* ErrorHandler) const = 0;
};
#endif // WITH_EDITOR
