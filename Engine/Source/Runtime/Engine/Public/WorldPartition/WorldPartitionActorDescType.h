// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/Requires.h"
#include "Templates/UnrealTypeTraits.h"

#include <type_traits>

class AActor;
class UActorComponent;

#if WITH_EDITOR
template <typename ActorType UE_REQUIRES(std::is_base_of_v<AActor, ActorType>)>
struct FWorldPartitionActorDescType
{};

#define DEFINE_ACTORDESC_TYPE(ActorType, ActorDescType) \
template <>												\
struct FWorldPartitionActorDescType<ActorType>			\
{														\
	typedef class ActorDescType Type;					\
};

template <typename ComponentType UE_REQUIRES(std::is_base_of_v<UActorComponent, ComponentType>)>
struct FWorldPartitionComponentDescType
{};

#define DEFINE_COMPONENTDESC_TYPE(ComponentType, ComponentDescType) \
template <>															\
struct FWorldPartitionComponentDescType<ComponentType>				\
{																	\
	typedef class ComponentDescType Type;							\
};

#else
#define DEFINE_ACTORDESC_TYPE(ActorType, ActorDescType)
#define DEFINE_COMPONENTDESC_TYPE(ComponentType, ComponentDescType)
#endif
