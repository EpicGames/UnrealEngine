// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "WorldPartition/WorldPartitionActorDesc.h"

class AActor;
struct FAssetData;
class FWorldPartitionActorDescInstance;

struct FWorldPartitionActorDescUtils
{
	/** @return The asset registry tag name for ActorMetaDataClass */
	static ENGINE_API FName ActorMetaDataClassTagName();

	/** @return The asset registry tag name for ActorMetaData */
	static ENGINE_API FName ActorMetaDataTagName();

	struct FActorDescInitParams
	{
		FActorDescInitParams() {}
		FActorDescInitParams(const AActor* InActor);
		FActorDescInitParams(const FString& InString);

		FString ToString() const;

		FName PathName;
		FName BaseClassName;
		FName NativeClassName;
		FString AssetData;
	};

	/** 
	 * Checks if the provided asset data contains a valid actor descriptor.
	 * @param InAssetData	The asset data to look into.
	 * @return				Whether the provideed asset data contains a valid actor descriptor or not.
	 */
	static ENGINE_API bool IsValidActorDescriptorFromAssetData(const FAssetData& InAssetData);

	/** 
	 * Retrieve the actor's native class from the asset data.
	 * @param InAssetData	The asset data to look into.
	 * @return				The actor's native class.
	 */
	static ENGINE_API UClass* GetActorNativeClassFromAssetData(const FAssetData& InAssetData);

	/**
	 * Retrieve the actor's native class from the string.
	 * @param InClassPath	The class path to resolve.
	 * @return				The actor's native class.
	 */
	static ENGINE_API UClass* GetActorNativeClassFromString(const FString& InClassPath);

	/** 
	 * Creates a valid actor descriptor from the provided actor descriptor init params.
	 * @param InActorDescInitParams	The actor dfescriptor input params.
	 * @param InPackageName			The package name the actor descriptor comes from.
	 * @return						Actor descriptor retrieved from the provided asset data.
	 */
	static ENGINE_API TUniquePtr<FWorldPartitionActorDesc> GetActorDescriptorFromInitParams(const FActorDescInitParams& InActorDescInitParams, FName InPackageName);


	/** 
	 * Creates a valid actor descriptor from the provided asset data.
	 * @param InAssetData	The asset data to look into.
	 * @return				Actor descriptor retrieved from the provided asset data.
	 */
	static ENGINE_API TUniquePtr<FWorldPartitionActorDesc> GetActorDescriptorFromAssetData(const FAssetData& InAssetData);

	/**
	 * Appends the actor's actor descriptor data into the provided asset registry tags.
	 * @param InActor		The actor that will append its actor descriptor.
	 * @param Context		Context to output into.
	 */
	static ENGINE_API void AppendAssetDataTagsFromActor(const AActor* InActor, FAssetRegistryTagsContext Context);

	/** 
	 * Return the actor descriptor data.
	 * @param InActorDesc	The actor descriptor that will generate the data.
	 * @return				The actor descriptor data.
	 */
	static ENGINE_API FString GetAssetDataFromActorDescriptor(TUniquePtr<FWorldPartitionActorDesc>& InActorDesc);

	/** 
	 * Patches an actor descriptor's asset data
	 * @param InAssetData			The asset data to be patched.
	 * @param OutAssetData			The patched asset data.
	 * @param InAssetDataPatcher	The patcher object.
	 * @return						The actor descriptor patched data.
	 */
	static ENGINE_API bool GetPatchedAssetDataFromAssetData(const FAssetData& InAssetData, FString& OutAssetData, FWorldPartitionAssetDataPatcher* InAssetDataPatcher);

	/** 
	 * Update an actor descriptor with new values coming from the provided actor.
	 * @param InActor		The actor to update from.
	 * @param OutActorDesc	Actor descriptor unique pointer that will get updated with the new actor descriptor.
	 */
	static ENGINE_API void UpdateActorDescriptorFromActor(const AActor* InActor, TUniquePtr<FWorldPartitionActorDesc>& OutActorDesc);

	/** 
	 * Update an actor descriptor with new values coming from the provided actor descriptor.
	 * @param InActorDesc	The actor descriptor to update from.
	 * @param OutActorDesc	Actor descriptor unique pointer that will get updated with the new actor descriptor.
	 */
	static ENGINE_API void UpdateActorDescriptorFromActorDescriptor(TUniquePtr<FWorldPartitionActorDesc>& InActorDesc, TUniquePtr<FWorldPartitionActorDesc>& OutActorDesc);

	/** 
	 * Replaces the actor descriptor's actor pointer with the provided new actor pointer.
	 * @param InOldActor	The old actor that the provided actor descriptor was representing.
	 * @param InNewActor	The new actor that the provided actor descriptor should be representing.
	 * @param InActorDescInstance	Actor descriptor instance that will get its actor pointer updated.
	 */
	static ENGINE_API void ReplaceActorDescriptorPointerFromActor(const AActor* InOldActor, AActor* InNewActor, FWorldPartitionActorDescInstance* InActorDescInstance);

	static ENGINE_API UClass* GetNativeClassFromClassName(FName InClassName);
};
#endif
