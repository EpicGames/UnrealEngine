// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"
#include "Misc/Optional.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

UE_DEPRECATED_HEADER(5.8, "The contents of this header have been fully deprecated..") 