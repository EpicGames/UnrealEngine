// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/OverrideVoidReturnInvoker.h"
#include "WorldPartition/WorldPartitionActorDesc.h"

template <class Type>
class UE_DEPRECATED(5.8, "This class is not used anymore") TActorDescViewMap
{
	friend class FWorldPartitionStreamingGenerator;

private:
	template <class Func>
	void ForEachActorDescView(Func InFunc)
	{
		TOverrideVoidReturnInvoker Invoker(true, InFunc);

		for (TUniquePtr<Type>& ActorDescView : ActorDescViewList)
		{
			if (!Invoker(*((Type*)ActorDescView.Get())))
			{
				return;
			}
		}
	}

	Type* FindByGuid(const FGuid& InGuid)
	{
		if (Type** ActorDescViewPtr = ActorDescViewsByGuid.Find(InGuid))
		{
			return *ActorDescViewPtr;
		}
		return nullptr;
	}

	Type& FindByGuidChecked(const FGuid& InGuid)
	{
		return *ActorDescViewsByGuid.FindChecked(InGuid);
	}

public:
	TActorDescViewMap()
	{}

	// Non-copyable but movable
	TActorDescViewMap(const TActorDescViewMap&) = delete;
	TActorDescViewMap(TActorDescViewMap&&) = default;
	TActorDescViewMap& operator=(const TActorDescViewMap&) = delete;
	TActorDescViewMap& operator=(TActorDescViewMap&&) = default;

	Type* Emplace(const FGuid& InActorGuid, const Type& InActorDescView)
	{
		Type* NewActorDescView = (Type*)ActorDescViewList.Emplace_GetRef(MakeUnique<Type>(InActorDescView)).Get();
	
		const UClass* NativeClass = NewActorDescView->GetActorNativeClass();
		const FName NativeClassName = NativeClass->GetFName();

		ActorDescViewsByGuid.Emplace(InActorGuid, NewActorDescView);
		ActorDescViewsByClass.Add(NativeClassName, NewActorDescView);

		return NewActorDescView;
	}

	Type* Emplace(const FWorldPartitionActorDesc* InActorDesc)
	{
		return Emplace(InActorDesc->GetGuid(), Type(InActorDesc));
	}

	inline int32 Num() const
	{
		return ActorDescViewList.Num();
	}

	template <class Func>
	void ForEachActorDescView(Func InFunc) const
	{
		TOverrideVoidReturnInvoker Invoker(true, InFunc);

		for (const TUniquePtr<Type>& ActorDescView : ActorDescViewList)
		{
			if (!Invoker(*ActorDescView))
			{
				return;
			}
		}
	}

	const Type* FindByGuid(const FGuid& InGuid) const
	{
		if (const Type* const* ActorDescViewPtr = ActorDescViewsByGuid.Find(InGuid))
		{
			return *ActorDescViewPtr;
		}
		return nullptr;
	}

	const Type& FindByGuidChecked(const FGuid& InGuid) const
	{
		return *ActorDescViewsByGuid.FindChecked(InGuid);
	}

	template <class ClassType>
	TArray<const Type*> FindByExactNativeClass() const
	{
		return FindByExactNativeClass(ClassType::StaticClass());
	}

	TArray<const Type*> FindByExactNativeClass(UClass* InExactNativeClass) const
	{
		check(InExactNativeClass->IsNative());
		const FName NativeClassName = InExactNativeClass->GetFName();
		TArray<const Type*> Result;
		ActorDescViewsByClass.MultiFind(NativeClassName, Result);
		return Result;
	}

	const TMap<FGuid, Type*>& GetActorDescViewsByGuid() const { return ActorDescViewsByGuid; }

protected:
	TArray<TUniquePtr<Type>> ActorDescViewList;

	TMap<FGuid, Type*> ActorDescViewsByGuid;
	TMultiMap<FName, const Type*> ActorDescViewsByClass;
};

#endif
