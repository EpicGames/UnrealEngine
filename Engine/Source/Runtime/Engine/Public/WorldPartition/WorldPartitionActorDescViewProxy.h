// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "WorldPartitionActorDescView.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#endif // WITH_EDITOR

UE_DEPRECATED_HEADER(5.8, "The contents of this header have been fully deprecated..") 