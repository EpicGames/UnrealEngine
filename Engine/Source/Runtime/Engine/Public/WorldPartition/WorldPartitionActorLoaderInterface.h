// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "Misc/Optional.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartitionActorLoaderInterface.generated.h"

UINTERFACE(MinimalAPI)
class UWorldPartitionActorLoaderInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IWorldPartitionActorLoaderInterface
{
	GENERATED_IINTERFACE_BODY()

#if WITH_EDITOR
	using FReferenceMap = TMap<FGuid, FWorldPartitionReference>;
	using FActorReferenceMap = TMap<FGuid, FReferenceMap>;
	using FContainerReferenceMap = TMap<TWeakObjectPtr<UActorDescContainerInstance>, FActorReferenceMap>;

public:
	/** Base class for actor loaders */
	class ILoaderAdapter
	{
	public:
		ENGINE_API ILoaderAdapter(UWorld* InWorld);
		ENGINE_API virtual ~ILoaderAdapter();

		ENGINE_API void Load();
		ENGINE_API void Unload();
		ENGINE_API bool IsLoaded() const;

		UWorld* GetWorld() const { return World; }

		bool GetUserCreated() const { return bUserCreated; }
		void SetUserCreated(bool bValue) { bUserCreated = bValue; }

		// Public interface
		virtual TOptional<FBox> GetBoundingBox() const { return TOptional<FBox>(); }
		virtual TOptional<FString> GetLabel() const { return TOptional<FString>(); }
		virtual TOptional<FColor> GetColor() const { return TOptional<FColor>(); }

	protected:
		void OnActorDescContainerInstanceInitialize(UActorDescContainerInstance* ContainerInstance);
		void OnActorDescContainerInstanceUninitialize(UActorDescContainerInstance* ContainerInstance);

		// Private interface
		virtual void ForEachActor(TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const =0;

		ENGINE_API bool ShouldActorBeLoaded(const FWorldPartitionHandle& Actor) const;

		ENGINE_API void RegisterDelegates();
		ENGINE_API void UnregisterDelegates();

		// Actors filtering
		ENGINE_API virtual bool PassActorDescFilter(const FWorldPartitionHandle& Actor) const;
		ENGINE_API void RefreshLoadedState();

		ENGINE_API void PostLoadedStateChanged(int32 NumLoads, int32 NumUnloads, bool bClearTransactions);
		ENGINE_API void AddReferenceToActor(FWorldPartitionHandle& Actor);
		ENGINE_API void RemoveReferenceToActor(FWorldPartitionHandle& Actor);
		ENGINE_API void OnRefreshLoadedState(bool bFromUserOperation);

		// Helpers
		ENGINE_API FActorReferenceMap& GetContainerReferences(UActorDescContainerInstance* InContainerInstance);
		ENGINE_API const FActorReferenceMap* GetContainerReferencesConst(UActorDescContainerInstance* InContainerInstance) const;
		ENGINE_API UWorldPartition* GetLoadedChildWorldPartition(const FWorldPartitionHandle& Handle) const;

	private:
		UWorld* World;

		uint8 bLoaded : 1;
		uint8 bUserCreated : 1;

		FContainerReferenceMap ContainerActorReferences;
	};

	virtual ILoaderAdapter* GetLoaderAdapter() =0;

	class FActorDescFilter
	{
	public:
		virtual ~FActorDescFilter() = default;
		virtual bool PassFilter(class UWorld*, const FWorldPartitionHandle&) = 0;

		// Higher priority filters are called first
		virtual uint32 GetFilterPriority() const = 0;
		virtual FText* GetFilterReason() const = 0;
	};
		
	static ENGINE_API void RegisterActorDescFilter(const TSharedRef<FActorDescFilter>& InActorDescFilter);

	static ENGINE_API void RefreshLoadedState(bool bIsFromUserChange);

private:
	static ENGINE_API UWorldPartition* GetLoadedChildWorldPartition(const FWorldPartitionHandle& Handle);
	
	DECLARE_EVENT_OneParam(IWorldPartitionActorLoaderInterface, FOnActorLoaderInterfaceRefreshState, bool /*bIsFromUserChange*/);
	static ENGINE_API FOnActorLoaderInterfaceRefreshState ActorLoaderInterfaceRefreshState;

	static ENGINE_API TArray<TSharedRef<FActorDescFilter>> ActorDescFilters;
	friend class ILoaderAdapter;
#endif
};
