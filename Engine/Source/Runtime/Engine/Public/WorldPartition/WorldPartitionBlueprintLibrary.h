// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterActorList.h"
#include "WorldPartitionBlueprintLibrary.generated.h"

class FWorldPartitionActorDesc;
class FWorldPartitionActorDescInstance;
class UWorldPartition;
class UDataLayerManager;

/**
 * Snapshot of an actor descriptor, which represents the state of an actor on disk.
 * The actor may or may not be loaded.
 */
USTRUCT(BlueprintType)
struct FActorDesc
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FActorDesc();

#if WITH_EDITOR
	ENGINE_API FActorDesc(const FWorldPartitionActorDescInstance& InActorDesc);
#endif

	/** The actor GUID of this descriptor. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	FGuid Guid;

	/** Actor first native class. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	TObjectPtr<UClass> NativeClass;

	/** Actor class, can point to a native or Blueprint class and may be redirected. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	FSoftObjectPath Class;

	/** Internal name of the actor. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	FName Name;

	/** Actor's label. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	FName Label;

	/** Streaming bounds of this actor. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	FBox Bounds;

	/** Actor's target runtime grid. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	FName RuntimeGrid;

	/** Actor's streaming state. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	bool bIsSpatiallyLoaded;

	/** Actor's editor-only property. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	bool bActorIsEditorOnly;

	/** Actor's package name. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	FName ActorPackage;

	/** Actor's path name. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	FName ActorPath;

	/** Actor's data layer assets. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Actor)
	TArray<FSoftObjectPath> DataLayerAssets;
};

UCLASS(MinimalAPI)
class UWorldPartitionBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

#if WITH_EDITOR
	static UWorld* GetEditorWorld();
	static UWorldPartition* GetWorldPartition();

	static void OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition);
	static TMap<UWorldPartition*, TUniquePtr<FLoaderAdapterActorList>> LoaderAdapterActorListMap;
	static FDelegateHandle OnWorldPartitionUninitializedHandle;

	static bool GetActorDescs(const UWorldPartition* WorldPartition, TArray<FActorDesc>& OutActorDescs);
	static bool GetActorDescs(const UActorDescContainerInstance* InContainer, TArray<FActorDesc>& OutActorDescs);
	static bool GetIntersectingActorDescs(UWorldPartition* WorldPartition, const FBox& InBox, TArray<FActorDesc>& OutActorDescs);
	static bool GetIntersectingActorDescs(const UActorDescContainerInstance* InContainerInstance, const FBox& InBox, TArray<FActorDesc>& OutActorDescs);
	static bool HandleIntersectingActorDesc(const FWorldPartitionActorDescInstance* ActorDescInstance, const FBox& InBox, TArray<FActorDesc>& OutActorDescs);
#endif

public:
	/**
	 * Gets the editor world bounds, which includes all actor descriptors.
	 * @return The editor world bounds.
	 */
	UFUNCTION(BlueprintCallable, Category="World Partition", meta=(DevelopmentOnly))
	static FBox GetEditorWorldBounds();

	/**
	 * Gets the runtime world bounds, which only includes actor descriptors that aren't editor only.
	 * @return The runtime world bounds.
	 */
	UFUNCTION(BlueprintCallable, Category="World Partition", meta=(DevelopmentOnly))
	static FBox GetRuntimeWorldBounds();

	/**
	 * Load actors
	 */
	UFUNCTION(BlueprintCallable, Category="World Partition", meta=(DevelopmentOnly))
	static void LoadActors(const TArray<FGuid>& InActorsToLoad);

	/**
	 * Unload actors
	 */
	UFUNCTION(BlueprintCallable, Category="World Partition", meta=(DevelopmentOnly))
	static void UnloadActors(const TArray<FGuid>& InActorsToUnload);

	/**
	 * Pin actors
	 */
	UFUNCTION(BlueprintCallable, Category="World Partition", meta=(DevelopmentOnly))
	static void PinActors(const TArray<FGuid>& InActorsToPin);

	/**
	 * Unpin actors
	 */
	UFUNCTION(BlueprintCallable, Category="World Partition", meta=(DevelopmentOnly))
	static void UnpinActors(const TArray<FGuid>& InActorsToUnpin);

	/**
	 * Gets all the actor descriptors into the provided array, recursing into actor containers.
	 * @return True if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category="World Partition", meta=(DevelopmentOnly))
	static bool GetActorDescs(TArray<FActorDesc>& OutActorDescs);

	/**
	 * Gets all the actor descriptors intersecting the provided box into the provided array, recursing into actor containers.
	 * @return True if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category="World Partition", meta=(DevelopmentOnly))
	static bool GetIntersectingActorDescs(const FBox& InBox, TArray<FActorDesc>& OutActorDescs);

	/**
	 * Gets all the actor descriptors from the provided actor pointers, which represents descriptors on disk, e.g. will not 
	 * reflect properties of unsaved actors.
	 * @return True if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category="World Partition", meta=(DevelopmentOnly))
	static bool GetActorDescsForActors(const TArray<AActor*>& InActors, TArray<FActorDesc>& OutActorDescs);

	/**
	 * Returns the Data Layer Manager for the current world context.
	 */
	UFUNCTION(BlueprintCallable, Category = "World Partition", meta = (WorldContext = "WorldContextObject"))
	static UDataLayerManager* GetDataLayerManager(class UObject* WorldContextObject);

	/**
	 * Returns the Data Layer Manager for the provided object.
	 */
	UFUNCTION(BlueprintCallable, Category = "World Partition")
	static UDataLayerManager* GetDataLayerManagerForObject(class UObject* Object);
};
