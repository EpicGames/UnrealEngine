// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "Misc/LazySingleton.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "UObject/ObjectMacros.h"
#include "WorldPartition/ActorDescList.h"
#include "ModuleDescriptor.h"

class UActorDescContainer;

class FWorldPartitionClassDescRegistry : public FActorDescList
{
	using FNameClassDescMap = TMap<FTopLevelAssetPath, TUniquePtr<FWorldPartitionActorDesc>*>;
	using FParentClassMap = TMap<FTopLevelAssetPath, FTopLevelAssetPath>;
	using FRedirectClassMap = TMap<FTopLevelAssetPath, FTopLevelAssetPath>;

public:
	static ENGINE_API FWorldPartitionClassDescRegistry& Get();
	static ENGINE_API void TearDown();

	ENGINE_API void Initialize();
	ENGINE_API void Uninitialize();
	
	ENGINE_API bool IsInitialized() const;

	ENGINE_API void PrefetchClassDescs(const TArray<FTopLevelAssetPath>& InClassPaths);

	ENGINE_API bool IsRegisteredClass(const FTopLevelAssetPath& InClassPath) const;

	ENGINE_API bool IsDerivedFrom(const FWorldPartitionActorDesc* InClassDesc, const FWorldPartitionActorDesc* InParentClassDesc) const;

	DECLARE_EVENT_OneParam(FWorldPartitionClassDescRegistry, FClassDescriptorUpdated, const FWorldPartitionActorDesc*);
	FClassDescriptorUpdated& OnClassDescriptorUpdated() { return ClassDescriptorUpdatedEvent; }

private:
	ENGINE_API void PrefetchClassDesc(UClass* InClass);

	ENGINE_API void RegisterClassDescriptor(FWorldPartitionActorDesc* InClassDesc);
	ENGINE_API void UnregisterClassDescriptor(FWorldPartitionActorDesc* InClassDesc);

	ENGINE_API void RegisterClassDescriptorFromAssetData(const FAssetData& InAssetData);
	ENGINE_API void RegisterClassDescriptorFromActorClass(const UClass* InActorClass);

	friend class FActorDescArchive;
	friend class UActorDescContainer;

public:
	ENGINE_API const FWorldPartitionActorDesc* GetClassDescDefault(const FTopLevelAssetPath& InClassPath) const;
	ENGINE_API const FWorldPartitionActorDesc* GetClassDescDefaultForActor(const FTopLevelAssetPath& InClassPath) const;
	ENGINE_API const FWorldPartitionActorDesc* GetClassDescDefaultForClass(const FTopLevelAssetPath& InClassPath) const;

private:
	friend class UWorldPartitionResaveActorsBuilder;
	const FParentClassMap& GetParentClassMap() const { check(IsInitialized()); return ParentClassMap; }

	ENGINE_API void OnObjectPreSave(UObject* InObject, FObjectPreSaveContext InSaveContext);
	ENGINE_API void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

	ENGINE_API void OnPluginLoadingPhaseComplete(ELoadingPhase::Type InLoadingPhase, bool bInPhaseSuccessful);

	ENGINE_API void OnAssetRemoved(const FAssetData& InAssetData);
	ENGINE_API void OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath);

	ENGINE_API void RegisterClass(UClass* Class);
	ENGINE_API void RegisterClasses();

	// Returns the updated descriptor if one was modified (for deferred broadcast outside the lock), or nullptr.
	ENGINE_API const FWorldPartitionActorDesc* UpdateClassDescriptor(UObject* InObject, bool bOnlyIfExists);

	ENGINE_API void ValidateInternalState();

	ENGINE_API FTopLevelAssetPath RedirectClassPath(const FTopLevelAssetPath& InClassPath) const;

	// Internal variants that assume ClassDescLock is already held by the caller
	const FWorldPartitionActorDesc* GetClassDescDefaultInternal(const FTopLevelAssetPath& InClassPath) const;
	FTopLevelAssetPath RedirectClassPathInternal(const FTopLevelAssetPath& InClassPath) const;
	void PrefetchClassDescsInternal(const TArray<FTopLevelAssetPath>& InClassPaths);

	// Protects ClassByPath, ParentClassMap, and RedirectClassMap from concurrent access
	mutable FTransactionallySafeCriticalSection ClassDescLock;

	FNameClassDescMap ClassByPath;
	FParentClassMap ParentClassMap;
	FRedirectClassMap RedirectClassMap;
	FClassDescriptorUpdated ClassDescriptorUpdatedEvent;
};
#endif
