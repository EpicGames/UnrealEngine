// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionEditorHash.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"

#if WITH_EDITOR

class AActor;
class ULevel;
class ULevelStreaming;
class UPackage;
class UWorld;

class FWorldPartitionConverter
{
public:
	struct FParameters
	{
		FParameters()
		: bConvertSubLevels(true)
		, bEnableStreaming(true)
		, bEnableLoadingInEditor(true)
		, bUseActorFolders(true)
		, EditorHashClass(nullptr)
		, RuntimeHashClass(nullptr)
		{}

		bool bConvertSubLevels;
		bool bEnableStreaming;
		bool bEnableLoadingInEditor;
		bool bUseActorFolders;
		TSubclassOf<UWorldPartitionEditorHash> EditorHashClass;
		TSubclassOf<UWorldPartitionRuntimeHash> RuntimeHashClass;
	};

	static ENGINE_API bool Convert(UWorld* InWorld, const FWorldPartitionConverter::FParameters& InParameters);

private:

	FWorldPartitionConverter(UWorld* InWorld, const FWorldPartitionConverter::FParameters& InParameters);
	bool Convert();
	bool ShouldDeleteActor(AActor* InActor, bool bIsMainLevel) const;
	void ChangeObjectOuter(UObject* InObject, UObject* InNewOuter);
	void GatherAndPrepareSubLevelsToConvert(ULevel* InLevel, TArray<ULevel*>& OutSubLevels);
	bool PrepareStreamingLevelForConversion(ULevelStreaming* InStreamingLevel);
	bool LevelHasLevelScriptBlueprint(ULevel* InLevel);
	void FixupSoftObjectPaths(UPackage* OuterPackage);
	
	UWorld* World;
	FWorldPartitionConverter::FParameters Parameters;
	TMap<FString, FString> RemapSoftObjectPaths;
};

#endif
