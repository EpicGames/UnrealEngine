// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/NameTypes.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "HAL/IConsoleManager.h"

class FAutoConsoleCommand;
class UCanvas;
class UFont;
struct FLinearColor;

class FWorldPartitionDebugHelper
{
public:
	static bool IsDebugDataLayerShown(FName DataLayerName);
	static bool AreDebugDataLayersShown(const TArray<FName>& DataLayerNames);
	static bool IsDebugStreamingStatusShown(EStreamingStatus StreamingStatus);
	static bool IsDebugRuntimeHashGridShown(FName Name);
	static bool IsDebugCellNameShown(const FString& Name);
	static void DrawText(UCanvas* Canvas, const FString& Text, const UFont* Font, const FColor& Color, FVector2D& Pos, float* MaxTextWidth = nullptr);
	static void DrawLegendItem(UCanvas* Canvas, const FString& Text, const UFont* Font, const FColor& Color, const FColor& TextColor, FVector2D& Pos, float* MaxItemWidth = nullptr);
	inline static bool IsRuntimeSpatialHashCellStreamingPriorityShown() { return ShowRuntimeSpatialHashCellStreamingPriorityMode != 0; }
	inline static int32 GetRuntimeSpatialHashCellStreamingPriorityMode() { return ShowRuntimeSpatialHashCellStreamingPriorityMode; }
	inline static bool CanDrawContentBundles() { return bCanDrawContentBundles; }

private:
	static FAutoConsoleCommand DebugFilterByStreamingStatusCommand;
	static TSet<EStreamingStatus> DebugStreamingStatusFilter;

	static FAutoConsoleCommand DebugFilterByDataLayerCommand;
	static TSet<FName> DebugDataLayerFilter;

	static FAutoConsoleCommand DebugFilterByRuntimeHashGridNameCommand;
	static TSet<FName> DebugRuntimeHashFilter;

	static FAutoConsoleCommand DebugFilterByCellNameCommand;
	static TArray<FString> DebugCellNameFilter;

	static FAutoConsoleVariableRef ShowRuntimeSpatialHashCellStreamingPriorityModeCommand;
	static int32 ShowRuntimeSpatialHashCellStreamingPriorityMode;

	static FAutoConsoleVariableRef DrawContentBundlesCommand;
	static bool bCanDrawContentBundles;
};