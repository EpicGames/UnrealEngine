// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartitionEditorHash.generated.h"

UCLASS(Abstract, Config=Engine, Within = WorldPartition, MinimalAPI)
class UWorldPartitionEditorHash : public UObject
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	virtual void Initialize() PURE_VIRTUAL(UWorldPartitionEditorHash::Initialize, return;);
	virtual void SetDefaultValues() PURE_VIRTUAL(UWorldPartitionEditorHash::SetDefaultValues, return;);
	virtual FName GetWorldPartitionEditorName() const PURE_VIRTUAL(UWorldPartitionEditorHash::GetWorldPartitionEditorName, return FName(NAME_None););
	virtual FBox GetEditorWorldBounds() const  PURE_VIRTUAL(UWorldPartitionEditorHash::GetEditorWorldBounds, return FBox(ForceInit););
	virtual FBox GetRuntimeWorldBounds() const  PURE_VIRTUAL(UWorldPartitionEditorHash::GetRuntimeWorldBounds, return FBox(ForceInit););
	virtual FBox GetNonSpatialBounds() const PURE_VIRTUAL(UWorldPartitionEditorHash::GetNonSpatialBounds, return FBox(ForceInit););
	virtual void Tick(float DeltaSeconds) PURE_VIRTUAL(UWorldPartitionEditorHash::Tick, return;);

	virtual void HashActor(FWorldPartitionHandle& InActorHandle) PURE_VIRTUAL(UWorldPartitionEditorHash::HashActor, ;);
	virtual void UnhashActor(FWorldPartitionHandle& InActorHandle) PURE_VIRTUAL(UWorldPartitionEditorHash::UnhashActor, ;);

	/* Struct of optional parameters passed to ForEachIntersectingActor. */
	struct FForEachIntersectingActorParams
	{
		ENGINE_API FForEachIntersectingActorParams();

		/** Should we include spatially loaded actors in the query? */
		bool bIncludeSpatiallyLoadedActors;

		/** Should we include non-spatially loaded actors in the query? */
		bool bIncludeNonSpatiallyLoadedActors;

		/** Optional minimum box to stop searching for actors */
		TOptional<FBox> MinimumBox;

		FForEachIntersectingActorParams& SetIncludeSpatiallyLoadedActors(bool bInIncludeSpatiallyLoadedActors) { bIncludeSpatiallyLoadedActors = bInIncludeSpatiallyLoadedActors; return *this; };
		FForEachIntersectingActorParams& SetIncludeNonSpatiallyLoadedActors(bool bInIncludeNonSpatiallyLoadedActors) { bIncludeNonSpatiallyLoadedActors = bInIncludeNonSpatiallyLoadedActors; return *this; };
		FForEachIntersectingActorParams& SetMinimumBox(const FBox& InMinimumBox) { MinimumBox = InMinimumBox; return *this; };
	};

	virtual int32 ForEachIntersectingActor(const FBox& Box, TFunctionRef<void(FWorldPartitionActorDescInstance*)> InOperation, const FForEachIntersectingActorParams& Params = FForEachIntersectingActorParams()) PURE_VIRTUAL(UWorldPartitionEditorHash::ForEachIntersectingActor, return 0;);
#endif
};
