// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "WorldPartitionEditorLoaderAdapter.generated.h"

UCLASS(MinimalAPI)
class UWorldPartitionEditorLoaderAdapter : public UObject, public IWorldPartitionActorLoaderInterface
{
public:
	GENERATED_BODY()

#if WITH_EDITOR
	virtual IWorldPartitionActorLoaderInterface::ILoaderAdapter* GetLoaderAdapter() override { return LoaderAdapter; }

	virtual void BeginDestroy() override
	{
		Super::BeginDestroy();
		check(!LoaderAdapter);
	}
private:
	friend class UWorldPartition;
	void SetLoaderAdapter(IWorldPartitionActorLoaderInterface::ILoaderAdapter* InLoaderAdapter)
	{
		check(!LoaderAdapter);
		LoaderAdapter = InLoaderAdapter;
	}

	void Release()
	{
		if (LoaderAdapter)
		{
			delete LoaderAdapter;
			LoaderAdapter = nullptr;
		}
	}

	IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = nullptr;
#endif
};
