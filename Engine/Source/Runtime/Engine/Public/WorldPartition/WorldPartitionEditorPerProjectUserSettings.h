// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/PackageName.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "Engine/World.h"
#include "Engine/DeveloperSettings.h"
#include "WorldPartitionEditorPerProjectUserSettings.generated.h"

USTRUCT()
struct FWorldPartitionPerWorldSettings
{
	GENERATED_BODY();

#if WITH_EDITOR
	FWorldPartitionPerWorldSettings()
	{}

	void Reset()
	{
		LoadedEditorRegions.Empty();
		LoadedEditorLocationVolumes.Empty();
		NotLoadedDataLayers.Empty();
		LoadedDataLayers.Empty();
		bForceDisableDynamicLoadingRangeScaling = false;
	}
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FBox> LoadedEditorRegions;

	UPROPERTY()
	TArray<FName> LoadedEditorLocationVolumes;

	UPROPERTY()
	TArray<TSoftObjectPtr<const UDataLayerAsset>> NotLoadedDataLayers;

	UPROPERTY()
	TArray<TSoftObjectPtr<const UDataLayerAsset>> LoadedDataLayers;

	/** Lets users disable the map's DynamicLoadingRangeScaling enable setting locally without having to checkout the map or risk submitting the setting */
	UPROPERTY()
	bool bForceDisableDynamicLoadingRangeScaling = false;
#endif
};

UCLASS(config = EditorPerProjectUserSettings, MinimalAPI, meta = (DisplayName = "World Partition (Local)"))
class UWorldPartitionEditorPerProjectUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UWorldPartitionEditorPerProjectUserSettings(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
		, bHideEditorDataLayers(false)
		, bHideRuntimeDataLayers(false)
		, bHideDataLayerActors(true)
		, bHideUnloadedActors(false)
		, bShowOnlySelectedActors(false)
		, bHighlightSelectedDataLayers(true)
		, bHideLevelInstanceContent(true)
		, bDisableLoadingOfLastLoadedRegions(false)
		, MinimapUnloadedOpacity(0.66f)
#endif
	{}

#if WITH_EDITOR
	ENGINE_API TArray<FBox> GetEditorLoadedRegions(UWorld* InWorld) const;
	ENGINE_API void SetEditorLoadedRegions(UWorld* InWorld, const TArray<FBox>& InEditorLoadedRegions);

	ENGINE_API TArray<FName> GetEditorLoadedLocationVolumes(UWorld* InWorld) const;
	ENGINE_API void SetEditorLoadedLocationVolumes(UWorld* InWorld, const TArray<FName>& InEditorLoadedLocationVolumes);

	bool GetEnableLoadingOfLastLoadedRegions() const
	{
		return !bDisableLoadingOfLastLoadedRegions;
	}

	bool GetBugItGoLoadRegion() const
	{
		return bBugItGoLoadRegion;
	}

	void SetBugItGoLoadRegion(bool bInBugItGoLoadRegion)
	{
		if (bBugItGoLoadRegion != bInBugItGoLoadRegion)
		{
			bBugItGoLoadRegion = bInBugItGoLoadRegion;
			SaveConfig();
		}
	}

	bool GetShowCellCoords() const
	{
		return bShowCellCoords;
	}

	void SetShowCellCoords(bool bInShowCellCoords)
	{
		if (bShowCellCoords != bInShowCellCoords)
		{
			bShowCellCoords = bInShowCellCoords;
			SaveConfig();
		}
	}

	float GetMinimapUnloadedOpacity() const
	{
		return MinimapUnloadedOpacity;
	}

	void SetMinimapUnloadedOpacity(float InMinimapUnloadedOpacity)
	{
		if (MinimapUnloadedOpacity != InMinimapUnloadedOpacity)
		{
			MinimapUnloadedOpacity = InMinimapUnloadedOpacity;
			SaveConfig();
		}
	}

	void ClearPerWorldSettings()
	{
		PerWorldEditorSettings.Empty();
		SaveConfig();
	}

	UE_DEPRECATED(5.8, "Use GetWorldDataLayerAssetsNotLoadedInEditor instead.")
	TArray<const UDataLayerInstance*> GetWorldDataLayersNotLoadedInEditor(UWorld* InWorld) const { return TArray<const UDataLayerInstance*>(); }
	UE_DEPRECATED(5.8, "Use GetWorldDataLayerAssetsLoadedInEditor instead.")
	TArray<const UDataLayerInstance*> GetWorldDataLayersLoadedInEditor(UWorld* InWorld) const { return TArray<const UDataLayerInstance*>(); }
	UE_DEPRECATED(5.8, "Use GetWorldDataLayerAssetsNotLoadedInEditor instead.")
	TArray<UDataLayerInstance*> GetWorldDataLayersNotLoadedInEditor(UWorld* InWorld) { return TArray<UDataLayerInstance*>(); }
	UE_DEPRECATED(5.8, "Use GetWorldDataLayerAssetsLoadedInEditor instead.")
	TArray<UDataLayerInstance*> GetWorldDataLayersLoadedInEditor(UWorld* InWorld) { return TArray<UDataLayerInstance*>(); }

	ENGINE_API TArray<const UDataLayerAsset*> GetWorldDataLayerAssetsLoadedInEditor(UWorld* InWorld) const;
	ENGINE_API TArray<UDataLayerAsset*> GetWorldDataLayerAssetsLoadedInEditor(UWorld* InWorld);
	ENGINE_API TArray<const UDataLayerAsset*> GetWorldDataLayerAssetsNotLoadedInEditor(UWorld* InWorld) const;
	ENGINE_API TArray<UDataLayerAsset*> GetWorldDataLayerAssetsNotLoadedInEditor(UWorld* InWorld);

	ENGINE_API void SetWorldDataLayersNonDefaultEditorLoadStates(UWorld* InWorld, const TArray<const UDataLayerInstance*>& InDataLayersLoadedInEditor, const TArray<const UDataLayerInstance*>& InDataLayersNotLoadedInEditor);

private:
	const FWorldPartitionPerWorldSettings* GetWorldPartitionPerWorldSettings(UWorld* InWorld) const;
#endif

#if WITH_EDITORONLY_DATA
public:
	ENGINE_API bool GetForceDisableDynamicLoadingRangeScaling(UWorld* InWorld) const;
	ENGINE_API void SetForceDisableDynamicLoadingRangeScaling(UWorld* InWorld, bool bOverride);

	/** True when the Data Layer Outliner is displaying Editor Data Layers */
	UPROPERTY(config, EditAnywhere, Category = "Data Layer")
	uint32 bHideEditorDataLayers : 1;

	/** True when the Data Layer Outliner is displaying Runtime Data Layers */
	UPROPERTY(config, EditAnywhere, Category = "Data Layer")
	uint32 bHideRuntimeDataLayers : 1;

	/** True when the Data Layer Outliner is not displaying actors */
	UPROPERTY(config, EditAnywhere, Category = "Data Layer")
	uint32 bHideDataLayerActors : 1;

	/** True when the Data Layer Outliner is not displaying unloaded actors */
	UPROPERTY(config, EditAnywhere, Category = "Data Layer")
	uint32 bHideUnloadedActors : 1;

	/** True when the Data Layer Outliner is only displaying actors and datalayers for selected actors */
	UPROPERTY(config, EditAnywhere, Category = "Data Layer")
	uint32 bShowOnlySelectedActors : 1;

	/** True when the Data Layer Outliner highlights Data Layers containing actors that are currently selected */
	UPROPERTY(config, EditAnywhere, Category = "Data Layer")
	uint32 bHighlightSelectedDataLayers : 1;

	/** True when the Data Layer Outliner is not displaying Level Instance content */
	UPROPERTY(config, EditAnywhere, Category = "Data Layer")
	uint32 bHideLevelInstanceContent : 1;

private:
	bool ShouldSaveSettings(const UWorld* InWorld) const;

	bool ShouldLoadSettings(const UWorld* InWorld) const;

	UPROPERTY(config, EditAnywhere, Category = Default)
	uint32 bDisableLoadingOfLastLoadedRegions : 1;

	UPROPERTY(config, EditAnywhere, Category = Default)
	uint32 bBugItGoLoadRegion : 1;

	UPROPERTY(config, EditAnywhere, Category = Default)
	uint32 bShowCellCoords : 1;

	UPROPERTY(config, EditAnywhere, Category = Default, meta = (ClampMin=0.0, ClampMax=1.0))
	float MinimapUnloadedOpacity;

	UPROPERTY(config)
	TMap<TSoftObjectPtr<UWorld>, FWorldPartitionPerWorldSettings> PerWorldEditorSettings;
#endif
};
