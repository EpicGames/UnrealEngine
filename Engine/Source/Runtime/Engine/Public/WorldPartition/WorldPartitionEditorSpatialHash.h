// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/HashBuilder.h"
#include "Templates/SubclassOf.h"
#include "WorldPartition/WorldPartitionEditorHash.h"
#include "WorldPartitionEditorSpatialHash.generated.h"

UCLASS(MinimalAPI)
class UWorldPartitionEditorSpatialHash : public UWorldPartitionEditorHash
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	friend class SWorldPartitionEditorGridSpatialHash;

	struct FCell
	{
	public:
		FCell()
			: Bounds(ForceInitToZero)
		{}

		FBox Bounds;
		TSet<FWorldPartitionHandle> Actors;
	};

	struct FCellCoord
	{
		FCellCoord(int64 InX, int64 InY, int64 InZ, int32 InLevel)
			: X(InX)
			, Y(InY)
			, Z(InZ)
			, Level(InLevel)
		{}

		int64 X;
		int64 Y;
		int64 Z;
		int32 Level;

		inline uint32 GetChildIndex() const
		{
			return ((X & 1) << 2) | ((Y & 1) << 1) | (Z & 1);
		}

		inline FCellCoord GetChildCellCoord(uint32 ChildIndex) const
		{
			check(Level);
			check(ChildIndex < 8);

			return FCellCoord(
				(X << 1) | (ChildIndex >> 2),
				(Y << 1) | ((ChildIndex >> 1) & 1),
				(Z << 1) | (ChildIndex & 1),
				Level - 1
			);
		}

		inline FCellCoord GetParentCellCoord() const
		{
			return FCellCoord(X >> 1, Y >> 1, Z >> 1, Level + 1);
		}

		bool operator==(const FCellCoord& Other) const
		{
			return (X == Other.X) && (Y == Other.Y) && (Z == Other.Z) && (Level == Other.Level);
		}

		friend uint32 GetTypeHash(const FCellCoord& CellCoord)
		{
			FHashBuilder HashBuilder;
			HashBuilder << CellCoord.X << CellCoord.Y << CellCoord.Z << CellCoord.Level;
			return HashBuilder.GetHash();
		}
	};

	inline FCellCoord GetCellCoords(const FVector& InPos, int32 Level) const
	{
		check(Level >= 0);
		const int64 CellSizeForLevel = (int64)CellSize * (1LL << Level);
		return FCellCoord(
			FMath::FloorToInt(InPos.X / CellSizeForLevel),
			FMath::FloorToInt(InPos.Y / CellSizeForLevel),
			FMath::FloorToInt(InPos.Z / CellSizeForLevel),
			Level
		);
	}

	inline FBox GetCellBounds(const FCellCoord& InCellCoord) const
	{
		check(InCellCoord.Level >= 0);
		const int64 CellSizeForLevel = (int64)CellSize * (1LL << InCellCoord.Level);
		const FVector Min = FVector(
			static_cast<FVector::FReal>(InCellCoord.X * CellSizeForLevel), 
			static_cast<FVector::FReal>(InCellCoord.Y * CellSizeForLevel), 
			static_cast<FVector::FReal>(InCellCoord.Z * CellSizeForLevel)
		);
		const FVector Max = Min + FVector(static_cast<double>(CellSizeForLevel));
		return FBox(Min, Max);
	}

	inline int32 GetLevelForBox(const FBox& Box) const
	{
		const FVector Extent = Box.GetExtent();
		const FVector::FReal MaxLength = Extent.GetMax() * 2.0;
		return FMath::CeilToInt32(FMath::Max<FVector::FReal>(FMath::Log2(MaxLength / CellSize), 0));
	}

	inline int32 ForEachIntersectingCells(const FBox& InBounds, int32 Level, TFunctionRef<void(const FCellCoord&)> InOperation) const
	{
		int32 NumCells = 0;

		FCellCoord MinCellCoords(GetCellCoords(InBounds.Min, Level));
		FCellCoord MaxCellCoords(GetCellCoords(InBounds.Max, Level));

		for (int64 z=MinCellCoords.Z; z<=MaxCellCoords.Z; z++)
		{
			for (int64 y=MinCellCoords.Y; y<=MaxCellCoords.Y; y++)
			{
				for (int64 x=MinCellCoords.X; x<=MaxCellCoords.X; x++)
				{
					InOperation(FCellCoord(x, y, z, Level));
					NumCells++;
				}
			}
		}

		return NumCells;
	}

	struct FCellNode
	{
		FCellNode()
			: ChildNodesMask(0)
		{}

		inline bool HasChildNodes() const
		{
			return !!ChildNodesMask;
		}

		inline bool HasChildNode(uint32 ChildIndex) const
		{
			check(ChildIndex < 8);
			return !!(ChildNodesMask & (1 << ChildIndex));
		}

		inline void AddChildNode(uint32 ChildIndex)
		{
			check(ChildIndex < 8);
			uint32 ChildMask = 1 << ChildIndex;
			check(!(ChildNodesMask & ChildMask));
			ChildNodesMask |= ChildMask;
		}

		inline void RemoveChildNode(uint32 ChildIndex)
		{
			check(ChildIndex < 8);
			uint32 ChildMask = 1 << ChildIndex;
			check(ChildNodesMask & ChildMask);
			ChildNodesMask &= ~ChildMask;
		}

		inline void ForEachChild(TFunctionRef<void(uint32 ChildIndex)> InOperation) const
		{
			int32 CurChildNodesMask = ChildNodesMask;

			while(CurChildNodesMask)
			{
				const int32 ChildIndex = FMath::CountTrailingZeros(CurChildNodesMask);
				
				check(CurChildNodesMask & (1 << ChildIndex));
				CurChildNodesMask &= ~(1 << ChildIndex);

				InOperation(ChildIndex);
			}
		}

		uint8 ChildNodesMask;
	};

public:
	virtual ~UWorldPartitionEditorSpatialHash() = default;

	// UWorldPartitionEditorHash interface begin
	ENGINE_API virtual void Initialize() override;
	ENGINE_API virtual void SetDefaultValues() override;
	ENGINE_API virtual FName GetWorldPartitionEditorName() const override;
	ENGINE_API virtual FBox GetEditorWorldBounds() const override;
	ENGINE_API virtual FBox GetRuntimeWorldBounds() const override;
	ENGINE_API virtual FBox GetNonSpatialBounds() const override;
	ENGINE_API virtual void Tick(float DeltaSeconds) override;

	ENGINE_API virtual void HashActor(FWorldPartitionHandle& InActorHandle) override;
	ENGINE_API virtual void UnhashActor(FWorldPartitionHandle& InActorHandle) override;

	ENGINE_API virtual int32 ForEachIntersectingActor(const FBox& Box, TFunctionRef<void(FWorldPartitionActorDescInstance*)> InOperation, const FForEachIntersectingActorParams& Params = FForEachIntersectingActorParams()) override;
	// UWorldPartitionEditorHash interface end
#endif

#if WITH_EDITORONLY_DATA
private:
	ENGINE_API int32 ForEachIntersectingCell(const FBox& Box, TFunctionRef<void(FCell*)> InOperation, int32 MinimumLevel = 0);
	ENGINE_API int32 ForEachIntersectingCellInner(const FBox& Box, const FCellCoord& CellCoord, TFunctionRef<void(FCell*)> InOperation, int32 MinimumLevel = 0);

	UPROPERTY(Config)
	int32 CellSize;

	// Dynamic sparse octree structure
	typedef TTuple<FCellNode, TUniquePtr<FCell>> FCellNodeElement;
	typedef TMap<FCellCoord, FCellNodeElement> FCellNodeHashLevel;
	TArray<FCellNodeHashLevel, TInlineAllocator<20>> HashLevels;

	TSet<FCell*> Cells;
	TUniquePtr<FCell> AlwaysLoadedCell;
	
	FBox EditorBounds;
	FBox RuntimeBounds;
	FBox NonSpatialBounds;
	bool bBoundsDirty;

#if DO_CHECK
	TMap<FGuid, FBox> HashedActors;
#endif

public:
	UPROPERTY(Config, meta = (AllowedClasses = "/Script/Engine.Texture2D, /Script/Engine.MaterialInterface"))
	FSoftObjectPath WorldImage;

	UPROPERTY(Config)
	FVector2D WorldImageTopLeftW;

	UPROPERTY(Config)
	FVector2D WorldImageBottomRightW;
#endif
};
