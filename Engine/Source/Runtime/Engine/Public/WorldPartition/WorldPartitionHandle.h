// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/UniquePtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_EDITOR
class UActorDescContainer;
class FWorldPartitionActorDesc;
struct FWorldPartitionHandleImpl;
struct FWorldPartitionReferenceImpl;

class AActor;
class UActorDescContainerInstance;
class FWorldPartitionActorDescInstance;

template<class U>
class TActorDescContainerCollection;

template<class U>
class TActorDescContainerInstanceCollection;

class FWorldPartitionLoadingContext
{
	friend struct FWorldPartitionReferenceImpl;

public:
	/**
	 * Base class for loading contexts
	 */
	class IContext
	{
		friend class FWorldPartitionLoadingContext;

	public:
		ENGINE_API IContext();
		ENGINE_API virtual ~IContext();

	private:
		// Should return true if the context caches objects that can be released outside its control
		virtual bool IsDeferringRawObjects() { return false; };
		virtual void RegisterActor(FWorldPartitionActorDescInstance* InActorDescInstance) = 0;
		virtual void UnregisterActor(FWorldPartitionActorDescInstance* InActorDescInstance) = 0;
		virtual void CancelOperation(FWorldPartitionActorDescInstance* InActorDescInstance) = 0;
	};

	/**
	 * Immediate loading context, which will register and unregister actors on demand.
	 */
	class FImmediate : public IContext
	{
	private:
		ENGINE_API virtual void RegisterActor(FWorldPartitionActorDescInstance* InActorDescInstance) override;
		ENGINE_API virtual void UnregisterActor(FWorldPartitionActorDescInstance* InActorDescInstance) override;
		virtual void CancelOperation(FWorldPartitionActorDescInstance* InActorDescInstance) override {} // Immediate context, nothing to cancel
	};

	/**
	 * Deferred loading context, which will gather actor registrations and unregistrations and
	 * execute them at the end of scope. This respects ULevel components registrations and
	 * construction scripts execution logic.
	 */
	class FDeferred : public IContext
	{
	public:
		ENGINE_API ~FDeferred();

		int32 GetNumRegistrations() const { return NumRegistrations; }
		int32 GetNumUnregistrations() const { return NumUnregistrations; }
		bool GetNeedsClearTransactions() const { return bNeedsClearTransactions; }

	private:
		virtual bool IsDeferringRawObjects() override { return true; };
		ENGINE_API virtual void RegisterActor(FWorldPartitionActorDescInstance* InActorDescInstance) override;
		ENGINE_API virtual void UnregisterActor(FWorldPartitionActorDescInstance* InActorDescInstance) override;
		ENGINE_API virtual void CancelOperation(FWorldPartitionActorDescInstance* InActorDescInstance) override;

		struct FContainerInstanceOps
		{
			TSet<FWorldPartitionActorDescInstance*> Registrations;
			TSet<FWorldPartitionActorDescInstance*> Unregistrations;
		};
		TMap<UActorDescContainerInstance*, FContainerInstanceOps> ContainerInstanceOps;

		TSet<FWorldPartitionActorDescInstance*> ContainerInstanceCancellations;

		int32 NumRegistrations = 0;
		int32 NumUnregistrations = 0;
		bool bNeedsClearTransactions = false;
	};

	/**
	 * Null loading context, which will ignore all loading/unloading commands.
	 */
	class FNull : public IContext
	{
	private:
		virtual void RegisterActor(FWorldPartitionActorDescInstance* InActorDescInstance) override {}
		virtual void UnregisterActor(FWorldPartitionActorDescInstance* InActorDescInstance) override {}
		virtual void CancelOperation(FWorldPartitionActorDescInstance* InActorDescInstance) override {}
	};

	// Returns true if the current context is vulnerable to deleted objects
	static bool IsDeferringRawObjects();
	// If the current context is operating on the actor desc, cancel it
	static void CancelOperationIfAny(FWorldPartitionActorDescInstance* InActorDescInstance);

private:
	static void LoadAndRegisterActor(FWorldPartitionActorDescInstance* ActorDescInstance);
	static void UnloadAndUnregisterActor(FWorldPartitionActorDescInstance* ActorDescInstance);

	static FImmediate DefaultContext;
	static IContext* ActiveContext;	
};

template <typename Impl>
class TWorldPartitionHandle
{
public:
	inline TWorldPartitionHandle()
		: ContainerInstance(nullptr)
		, ActorDescInstance(nullptr)
	{}

	inline TWorldPartitionHandle(TUniquePtr<FWorldPartitionActorDescInstance>* InActorDescInstance)
		: ContainerInstance(Impl::GetActorDescContainerInstance(InActorDescInstance))
		, ActorDescInstance(InActorDescInstance)
	{
		if (IsValid())
		{
			IncRefCount();
		}
	}

	inline TWorldPartitionHandle(FWorldPartitionActorDescInstance* InActorDescInstance)
		: ContainerInstance(Impl::GetActorDescContainerInstance(InActorDescInstance))
		, ActorDescInstance(nullptr)
	{
		if (ContainerInstance != nullptr)
		{
			ActorDescInstance = Impl::GetActorDescInstance(ContainerInstance.Get(), Impl::GetActorDescInstanceGuid(InActorDescInstance));
		}

		if (IsValid())
		{
			IncRefCount();
		}
	}

	inline TWorldPartitionHandle(UActorDescContainerInstance* InContainerInstance, const FGuid& InActorGuid)
		: ContainerInstance(InContainerInstance)
		, ActorDescInstance(Impl::GetActorDescInstance(InContainerInstance, InActorGuid))
	{
		if (IsValid())
		{
			IncRefCount();
		}
	}

	template<class U>
	inline TWorldPartitionHandle(TActorDescContainerInstanceCollection<U>* ContainerCollection, const FGuid& ActorGuid)
		: ContainerInstance(Impl::GetActorDescContainerInstance(ContainerCollection, ActorGuid))
		, ActorDescInstance(nullptr)
	{
		if (ContainerInstance != nullptr)
		{
			ActorDescInstance = Impl::GetActorDescInstance(ContainerInstance.Get(), ActorGuid);
		}

		if (IsValid())
		{
			IncRefCount();
		}
	}
	
	inline TWorldPartitionHandle(const TWorldPartitionHandle& Other)
		: ContainerInstance(nullptr)
		, ActorDescInstance(nullptr)
	{
		*this = Other;
	}

	inline TWorldPartitionHandle(TWorldPartitionHandle&& Other)
		: ContainerInstance(nullptr)
		, ActorDescInstance(nullptr)
	{
		*this = MoveTemp(Other);
	}

	inline TWorldPartitionHandle<FWorldPartitionHandleImpl> ToHandle() const
	{
		return Impl::ToHandle(*this);
	}

	inline TWorldPartitionHandle<FWorldPartitionReferenceImpl> ToReference() const
	{
		return Impl::ToReference(*this);
	}

	template <typename T>
	inline TWorldPartitionHandle<Impl>(TWorldPartitionHandle<T>&& Other)
		: ActorDescInstance(nullptr)
	{
		*this = MoveTemp(Other);
	}

	inline ~TWorldPartitionHandle()
	{
		if (IsValid())
		{
			DecRefCount();
		}
	}

	inline TWorldPartitionHandle& operator=(const TWorldPartitionHandle& Other)
	{
		if (this != &Other)
		{
			if (IsValid())
			{
				DecRefCount();
			}

			ContainerInstance = Other.ContainerInstance;
			ActorDescInstance = Other.ActorDescInstance;

			if (IsValid())
			{
				IncRefCount();
			}
		}

		return *this;
	}

	inline TWorldPartitionHandle<Impl>& operator=(TWorldPartitionHandle&& Other)
	{
		if (this != &Other)
		{
			if (IsValid())
			{
				DecRefCount();
			}

			ContainerInstance = Other.ContainerInstance;
			ActorDescInstance = Other.ActorDescInstance;
			
			Other.ContainerInstance = nullptr;
			Other.ActorDescInstance = nullptr;
		}

		return *this;
	}

	// Conversions
	template <typename T>
	inline TWorldPartitionHandle<Impl>& operator=(const TWorldPartitionHandle<T>& Other)
	{
		if (IsValid())
		{
			DecRefCount();
		}

		ContainerInstance = Other.ContainerInstance;
		ActorDescInstance = Other.ActorDescInstance;

		if (IsValid())
		{
			IncRefCount();
		}

		return *this;
	}

	template <typename T>
	inline TWorldPartitionHandle<Impl>& operator=(TWorldPartitionHandle<T>&& Other)
	{
		if (IsValid())
		{
			DecRefCount();
		}

		ContainerInstance = Other.ContainerInstance;
		ActorDescInstance = Other.ActorDescInstance;

		if (IsValid())
		{
			IncRefCount();
			Other.DecRefCount();

			Other.ContainerInstance = nullptr;
			Other.ActorDescInstance = nullptr;
		}

		return *this;
	}

	inline FWorldPartitionActorDescInstance* operator->() const
	{
		return GetInstance();
	}

	inline FWorldPartitionActorDescInstance* operator*() const
	{
		return GetInstance();
	}

	inline FWorldPartitionActorDescInstance* GetInstance() const
	{
		return IsValid() ? ActorDescInstance->Get() : nullptr;
	}

	inline bool IsValid() const
	{
		return ContainerInstance.IsValid() && ActorDescInstance && ActorDescInstance->IsValid();
	}

	inline bool IsLoaded() const
	{
		return IsValid() && Impl::IsLoaded(ActorDescInstance->Get());
	}

	inline AActor* GetActor() const
	{
		return IsValid() ? Impl::GetActor(ActorDescInstance->Get()) : nullptr;
	}

	inline void Reset()
	{
		if (IsValid())
		{
			DecRefCount();
		}

		ContainerInstance = nullptr;
		ActorDescInstance = nullptr;
	}
	
	friend inline uint32 GetTypeHash(const TWorldPartitionHandle<Impl>& HandleBase)
	{
		return ::PointerHash(HandleBase.ActorDescInstance);
	}

	inline bool operator==(const TWorldPartitionHandle& Other) const
	{
		return ActorDescInstance == Other.ActorDescInstance;
	}

	inline bool operator!=(const TWorldPartitionHandle& Other) const
	{
		return !(*this == Other);
	}

	// Conversions
	template <typename T>
	inline bool operator==(const TWorldPartitionHandle<T>& Other) const
	{
		return GetInstance() == Other.GetInstance();
	}

	template <typename T>
	inline bool operator!=(const TWorldPartitionHandle<T>& Other) const
	{
		return !(*this == Other);
	}

public:
	inline void IncRefCount()
	{
		Impl::IncRefCount(ActorDescInstance->Get());
	}

	inline void DecRefCount()
	{
		Impl::DecRefCount(ActorDescInstance->Get());
	}

	TWeakObjectPtr<UActorDescContainerInstance> ContainerInstance;
	TUniquePtr<FWorldPartitionActorDescInstance>* ActorDescInstance;
};

struct FWorldPartitionImplBase
{
	template<class U>
	static UActorDescContainer* GetActorDescContainer(TActorDescContainerCollection<U>* ContainerCollection, const FGuid& ActorGuid)
	{
		return nullptr;
	}

	static ENGINE_API bool IsLoaded(FWorldPartitionActorDescInstance* InActorDescInstance);
	static ENGINE_API UActorDescContainerInstance* GetActorDescContainerInstance(TUniquePtr<FWorldPartitionActorDescInstance>* InActorDescInstance);
	static ENGINE_API UActorDescContainerInstance* GetActorDescContainerInstance(FWorldPartitionActorDescInstance* InActorDescInstance);
	static ENGINE_API FGuid GetActorDescInstanceGuid(const FWorldPartitionActorDescInstance* InActorDescInstance);
	static ENGINE_API TUniquePtr<FWorldPartitionActorDescInstance>* GetActorDescInstance(UActorDescContainerInstance* InContainerInstance, const FGuid& InActorGuid);
	static ENGINE_API FWorldPartitionActorDesc* GetActorDesc(FWorldPartitionActorDescInstance* InActorDescInstance);

	static ENGINE_API AActor* GetActor(FWorldPartitionActorDescInstance* InActorDescInstance);

	template<class U>
	static UActorDescContainerInstance* GetActorDescContainerInstance(TActorDescContainerInstanceCollection<U>* ContainerCollection, const FGuid& ActorGuid)
	{
		return ContainerCollection ? ContainerCollection->GetActorDescContainerInstance(ActorGuid) : nullptr;
	}
};

struct FWorldPartitionHandleImpl : FWorldPartitionImplBase
{
	static ENGINE_API void IncRefCount(FWorldPartitionActorDescInstance* InActorDescInstance);
	static ENGINE_API void DecRefCount(FWorldPartitionActorDescInstance* InActorDescInstance);

	static ENGINE_API TWorldPartitionHandle<FWorldPartitionReferenceImpl> ToReference(const TWorldPartitionHandle<FWorldPartitionHandleImpl>& Source);
};

struct FWorldPartitionReferenceImpl : FWorldPartitionImplBase
{
	static ENGINE_API void IncRefCount(FWorldPartitionActorDescInstance* InActorDescInstance);
	static ENGINE_API void DecRefCount(FWorldPartitionActorDescInstance* InActorDescInstance);

	static ENGINE_API TWorldPartitionHandle<FWorldPartitionHandleImpl> ToHandle(const TWorldPartitionHandle<FWorldPartitionReferenceImpl>& Source);
};

/**
 * FWorldPartitionHandle will increment/decrement the soft reference count on the actor descriptor.
 * This won't trigger any loading, but will prevent cleanup of the actor descriptor when destroying an
 * actor in the editor.
 */
typedef TWorldPartitionHandle<FWorldPartitionHandleImpl> FWorldPartitionHandle;

/**
 * FWorldPartitionReference will increment/decrement the hard reference count on the actor descriptor.
 * This will trigger actor loading/unloading when the hard reference counts gets to one/zero.
 */
typedef TWorldPartitionHandle<FWorldPartitionReferenceImpl> FWorldPartitionReference;

/**
 * FWorldPartitionHandlePinRefScope will keep a reference if the actor is already loaded. This is useful 
 * when you want to keep an actor loaded during special operations, but don't trigger loading when the 
 * actor is not loaded. Intended to be on stack, as a scoped operation.
 */
struct FWorldPartitionHandlePinRefScope
{
	FWorldPartitionHandlePinRefScope(const FWorldPartitionHandle& InHandle)
	{
		if (InHandle.IsLoaded())
		{
			Reference = InHandle;
		}
	}

	FWorldPartitionHandlePinRefScope(const FWorldPartitionReference& InReference)
	{
		Reference = InReference;
	}

private:
	FWorldPartitionReference Reference;
};
#endif
