// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "WorldPartition/WorldPartitionRuntimeCellInterface.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionHandle.h"
#endif

class UWorld;
class UWorldPartition;

#if WITH_EDITOR
class FWorldPartitionActorDesc;
class FWorldPartitionActorDescInstance;
#endif

namespace FWorldPartitionHelpersPrivate
{
	ENGINE_API UWorldPartition* GetWorldPartitionFromObject(const UObject* InObject);

	template <class T>
	inline UWorldPartition* GetWorldPartition(const T* InObject)
	{
		return IsValid(InObject) ? GetWorldPartitionFromObject(InObject) : nullptr;
	}

	template <>
	inline UWorldPartition* GetWorldPartition(const ULevel* InLevel)
	{
		if (IsValid(InLevel))
		{
			if (const IWorldPartitionCell* Cell = InLevel->GetWorldPartitionRuntimeCell())
			{
				return Cell->GetOuterWorld()->GetWorldPartition();
			}		
			else if (const UWorld* OuterWorld = Cast<UWorld>(InLevel->GetOuter()))
			{
				return OuterWorld->GetWorldPartition();
			}
		}
			
		return nullptr;
	}

	template <>
	inline UWorldPartition* GetWorldPartition(const UWorld* InWorld)
	{
		return IsValid(InWorld) ? GetWorldPartition<ULevel>(InWorld->PersistentLevel) : nullptr;
	}

	template <>
	inline UWorldPartition* GetWorldPartition(const AActor* InActor)
	{
		return IsValid(InActor) ? GetWorldPartition<ULevel>(InActor->GetLevel()) : nullptr;
	}
}

class FWorldPartitionHelpers
{
public:
	/** Returns the owning World Partition for this object. */
	template <class T>
	static inline UWorldPartition* GetWorldPartition(const T* InObject)
	{
		return FWorldPartitionHelpersPrivate::GetWorldPartition(InObject);
	}

	/** Sends an RPC console command to the server (non shipping builds only). */
	static ENGINE_API void ServerExecConsoleCommand(UWorld* InWorld, const FString& InConsoleCommandName, const TArray<FString>& InArgs);

#if WITH_EDITOR
private:
	static ENGINE_API bool IsActorDescClassCompatibleWith(const FWorldPartitionActorDesc* ActorDesc, const UClass* Class);

public:

	template <class ActorClass = AActor>
	static void ForEachIntersectingActorDescInstance(UWorldPartition* WorldPartition, const FBox& Box, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Func)
	{
		ForEachIntersectingActorDescInstance(WorldPartition, Box, ActorClass::StaticClass(), Func);
	}

	template<class ActorClass = AActor>
	static void ForEachActorDescInstance(UWorldPartition* WorldPartition, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Func)
	{
		ForEachActorDescInstance(WorldPartition, ActorClass::StaticClass(), Func);
	}

	static ENGINE_API void ForEachIntersectingActorDescInstance(UWorldPartition* WorldPartition, const FBox& Box, TSubclassOf<AActor> ActorClass, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Func);
	static ENGINE_API void ForEachActorDescInstance(UWorldPartition* WorldPartition, TSubclassOf<AActor> ActorClass, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Func);

	/* Struct of optional parameters passed to foreach actordesc functions. */
	struct FForEachActorWithLoadingParams
	{
		ENGINE_API FForEachActorWithLoadingParams();

		/* Perform a garbage collection per-actor, useful to test if the caller properly handle GCs. */
		bool bGCPerActor;

		/* Prevent clearing of actor references. */
		bool bKeepReferences;

		/* The classes used to filter actors loading. */
		TArray<TSubclassOf<AActor>> ActorClasses;

		/* If not empty, iteration will be done only on those actors. */
		TArray<FGuid> ActorGuids;

		/* Custom filter function used to filter actors loading based on actor desc. */
		TUniqueFunction<bool(const FWorldPartitionActorDesc*)> FilterActorDesc;

		/* Custom filter function used to filter actors loading based on actor desc instance. */
		TUniqueFunction<bool(const FWorldPartitionActorDescInstance*)> FilterActorDescInstance;

		/* Called right before releasing actor references and performing garbage collection. */
		TUniqueFunction<void()> OnPreGarbageCollect;
	};

	/* Struct of optional output from foreach actordesc functions. */
	struct FForEachActorWithLoadingResult
	{
		/* Reference to all actors and actor references loaded by the foreach actordesc function */
		TMap<FGuid, FWorldPartitionReference> ActorReferences;
	};
	
	static ENGINE_API void ForEachActorWithLoading(UWorldPartition* WorldPartition, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Func, const FForEachActorWithLoadingParams& Params = FForEachActorWithLoadingParams());
	static ENGINE_API void ForEachActorWithLoading(UWorldPartition* WorldPartition, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Func, const FForEachActorWithLoadingParams& Params, FForEachActorWithLoadingResult& Result);

	static ENGINE_API bool HasExceededMaxMemory();
	static ENGINE_API bool ShouldCollectGarbage();
	static ENGINE_API void DoCollectGarbage();

	// Simulate an engine frame tick
	static ENGINE_API void FakeEngineTick(UWorld* World);

	// Runtime/Editor conversions
	static ENGINE_API bool ConvertRuntimePathToEditorPath(const FSoftObjectPath& InPath, FSoftObjectPath& OutPath);

	// Returns all loaded actors for a specific level, including unregistered and garbage ones
	static ENGINE_API TMap<FGuid, AActor*> GetLoadedActorsForLevel(const ULevel* InLevel);

	// Returns all loaded and registered actors for a specific level
	static ENGINE_API TMap<FGuid, AActor*> GetRegisteredActorsForLevel(const ULevel* InLevel);
#endif // WITH_EDITOR

	// Editor/Runtime conversions
	static ENGINE_API bool ConvertEditorPathToRuntimePath(const FSoftObjectPath& InPath, FSoftObjectPath& OutPath);
};


