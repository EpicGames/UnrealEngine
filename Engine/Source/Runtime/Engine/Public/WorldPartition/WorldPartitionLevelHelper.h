// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * WorldPartitionLevelHelper
 *
 * Helper class to build Levels for World Partition
 *
 */

#pragma once

#include "Engine/World.h"
#include "UObject/LinkerInstancingContext.h"
#include "UObject/UObjectAnnotation.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/WorldPartitionPropertyOverride.h"

class FWorldPartitionPackageHelper;
class UWorldPartition;
struct FActorContainerID;

class FWorldPartitionLevelHelper
{
public:
	ENGINE_API static FString AddActorContainerIDToSubPathString(const FActorContainerID& InContainerID, const FString& InSubPathString);
	ENGINE_API static FString AddActorContainerID(const FActorContainerID& InContainerID, const FString& InActorName);

#if WITH_EDITOR
public:
	static FWorldPartitionLevelHelper& Get();

	struct FPackageReferencer
	{
		~FPackageReferencer() { RemoveReferences(); }

		ENGINE_API void AddReference(UPackage* InPackage);
		ENGINE_API void RemoveReferences();
	};

	static ULevel* CreateEmptyLevelForRuntimeCell(const UWorldPartitionRuntimeCell* Cell, const UWorld* InWorld, const FString& InWorldAssetName, UPackage* DestPackage = nullptr);
	ENGINE_API static void MoveExternalActorsToLevel(const TArray<FWorldPartitionRuntimeCellObjectMapping>& InChildPackages, ULevel* InLevel, TArray<UPackage*>& OutModifiedPackages);
	static void RemapLevelSoftObjectPaths(ULevel* InLevel, UWorldPartition* InWorldPartition);
	
	/* Struct of optional parameters passed to LoadActors function. */
	struct FLoadActorsParams
	{
		FLoadActorsParams()
			: OuterWorld(nullptr)
			, DestLevel(nullptr)
			, PackageReferencer(nullptr)
			, bLoadAsync(false)
			, bSilenceLoadFailures(false)
			, AsyncRequestIDs(nullptr)
		{}

		UWorld* OuterWorld;
		ULevel* DestLevel;
		TArrayView<FWorldPartitionRuntimeCellObjectMapping> ActorPackages;
		FPackageReferencer* PackageReferencer;
		TFunction<void(bool)> CompletionCallback;
		bool bLoadAsync;
		bool bSilenceLoadFailures;
		TArray<int32>* AsyncRequestIDs;
		mutable FLinkerInstancingContext InstancingContext;

		FLoadActorsParams& SetOuterWorld(UWorld* InOuterWorld) { OuterWorld = InOuterWorld; return *this; }
		FLoadActorsParams& SetDestLevel(ULevel* InDestLevel) { DestLevel = InDestLevel; return *this; }
		FLoadActorsParams& SetActorPackages(TArrayView<FWorldPartitionRuntimeCellObjectMapping> InActorPackages) { ActorPackages = InActorPackages; return *this; }
		FLoadActorsParams& SetPackageReferencer(FPackageReferencer* InPackageReferencer) { PackageReferencer = InPackageReferencer; return *this; }
		FLoadActorsParams& SetCompletionCallback(TFunction<void(bool)> InCompletionCallback) { CompletionCallback = InCompletionCallback; return *this; }
		FLoadActorsParams& SetLoadAsync(bool bInLoadAsync, TArray<int32>* OutAsyncRequestIDs = nullptr)
		{ 
			bLoadAsync = bInLoadAsync;
			AsyncRequestIDs = OutAsyncRequestIDs;
			return *this;
		}
		FLoadActorsParams& SetSilenceLoadFailures(bool bInSilenceLoadFailures) { bSilenceLoadFailures = bInSilenceLoadFailures; return *this; }
		FLoadActorsParams& SetInstancingContext(FLinkerInstancingContext InInstancingContext) { InstancingContext = InInstancingContext; return *this; }
	};

	ENGINE_API static bool LoadActors(const FLoadActorsParams& InParams);
	ENGINE_API static bool LoadActors(FLoadActorsParams&& InParams);

	static FSoftObjectPath RemapActorPath(const FActorContainerID& InContainerID, const FString& SourceWorldName, const FSoftObjectPath& InActorPath);

private:
	FWorldPartitionLevelHelper();

	void AddReference(UPackage* InPackage, FPackageReferencer* InReferencer);
	void RemoveReferences(FPackageReferencer* InReferencer);
	void PreGarbageCollect();

	static UWorld::InitializationValues GetWorldInitializationValues();

	friend class FContentBundleEditor;
	static bool RemapLevelCellPathInContentBundle(ULevel* Level, const class FContentBundleEditor* ContentBundleEditor, const UWorldPartitionRuntimeCell* Cell);

	struct FLoadedPropertyOverrides
	{
		TMap<FActorContainerID, const UWorldPartitionPropertyOverride*> PropertyOverrides;
	};

	static bool LoadActorsInternal(FLoadActorsParams&& InParams, FLoadedPropertyOverrides&& InLoadedPropertyOverrides);
	static bool LoadActorsWithPropertyOverridesInternal(FLoadActorsParams&& InParams);

	struct FPackageReference
	{
		TSet<FPackageReferencer*> Referencers;
		TWeakObjectPtr<UPackage> Package;
	};

	friend struct FPackageReferencer;

	TMap<FName, FPackageReference> PackageReferences;

	TSet<TWeakObjectPtr<UPackage>> PreGCPackagesToUnload;

	static void SetForcePackageTrashingAtCleanup(ULevel* Level, bool bForcePackageTrashingAtCleanup);

	friend class UWorldPartitionLevelStreamingDynamic;
	friend class UWorldPartitionSubsystem;
	friend class UWorldPartitionRuntimeLevelStreamingCell;
	friend class UWorldPartitionHLODSourceActorsFromCell;
	// Cache of Property Overrides to apply after ReRunConstructionScript
	// In PIE this will  be done when Streaming state changes to Visible on the UWorldPartitionLevelStreamingDynamic
	// In Cook this will be done on save of the Level cell
	class FActorPropertyOverridesAnnotation
	{
	public:
		FActorPropertyOverridesAnnotation() {}
		FActorPropertyOverridesAnnotation(TArray<FActorPropertyOverride>&& InActorPropertyOverrides, const FTransform& InContainerTransform) : ActorPropertyOverrides(MoveTemp(InActorPropertyOverrides)), ContainerTransform(InContainerTransform) {}

		TArray<FActorPropertyOverride> ActorPropertyOverrides;
		FTransform ContainerTransform;
		inline bool IsDefault()
		{
			return ActorPropertyOverrides.IsEmpty();
		}
	};
	static FUObjectAnnotationSparse<FActorPropertyOverridesAnnotation, true> ActorPropertyOverridesAnnotation;

	// Apply Existing Property Override annotation to Actor
	static void ApplyConstructionScriptPropertyOverridesFromAnnotation(AActor* InActor);
#endif
};

