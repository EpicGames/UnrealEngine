// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * WorldPartitionLevelStreamingDynamic
 *
 * Dynamically controlled world partition level streaming implementation.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LevelStreamingDynamic.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionRuntimeLevelStreamingCell.h"
#if WITH_EDITOR
#include "WorldPartition/WorldPartitionLevelHelper.h"
#endif
#include "WorldPartitionLevelStreamingDynamic.generated.h"

UCLASS(NotPlaceable, NotBlueprintable, HideDropdown, MinimalAPI)
class UWorldPartitionLevelStreamingDynamic : public ULevelStreamingDynamic
{
	GENERATED_UCLASS_BODY()

	ENGINE_API void Load();
	ENGINE_API void Unload();
	ENGINE_API void Activate();
	ENGINE_API void Deactivate();
	ENGINE_API virtual UWorld* GetStreamingWorld() const override;
	void SetShouldBeAlwaysLoaded(bool bInShouldBeAlwaysLoaded) { bShouldBeAlwaysLoaded = bInShouldBeAlwaysLoaded; }
	const UWorldPartitionRuntimeCell* GetWorldPartitionRuntimeCell() const { return StreamingCell.Get(); }

	virtual bool ShouldBeAlwaysLoaded() const override { return bShouldBeAlwaysLoaded; }
	ENGINE_API virtual bool ShouldBlockOnUnload() const override;
	virtual bool ShouldRequireFullVisibilityToRender() const override { return true; }
	ENGINE_API virtual bool RequestVisibilityChange(bool bVisible) override;
	virtual const IWorldPartitionCell* GetWorldPartitionCell() const override { return GetWorldPartitionRuntimeCell(); }

#if !WITH_EDITOR
	ENGINE_API virtual void PostLoad();
#endif

	ENGINE_API void Initialize(const UWorldPartitionRuntimeLevelStreamingCell& InCell);
	ENGINE_API void SetLevelTransform(const FTransform& InLevelTransform);
	const FSoftObjectPath& GetOuterWorldPartition() const { return OuterWorldPartition.ToSoftObjectPath(); }

	virtual bool CanReplicateStreamingStatus() const override { return false; }

#if WITH_EDITOR
	// Override ULevelStreaming
	ENGINE_API virtual bool RequestLevel(UWorld* PersistentWorld, bool bAllowLevelLoadRequests, EReqLevelBlock BlockPolicy) override;
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual TOptional<FFolder::FRootObject> GetFolderRootObject() const override;

	bool GetLoadSucceeded() const { return bLoadSucceeded; }

	void SetShouldPerformStandardLevelLoading(bool bInShouldPerformStandardLevelLoading) { bShouldPerformStandardLevelLoading = bInShouldPerformStandardLevelLoading; }

	UE_DEPRECATED(5.7, "LoadInEditor() is no longer supported and will be removed in a future version.")
	static ENGINE_API UWorldPartitionLevelStreamingDynamic* LoadInEditor(UWorld* World, FName LevelStreamingName, const TArray<FWorldPartitionRuntimeCellObjectMapping>& InPackages);

	UE_DEPRECATED(5.7, "UnloadFromEditor() is no longer supported and will be removed in a future version.")
	static ENGINE_API void UnloadFromEditor(UWorldPartitionLevelStreamingDynamic* InLevelStreaming);

	const TArray<FWorldPartitionRuntimeCellObjectMapping>& GetChildPackages() const { return ChildPackages; };

private:
	ENGINE_API void CreateRuntimeLevel();
	ENGINE_API bool IssueLoadRequests();
	ENGINE_API void FinalizeRuntimeLevel();
	ENGINE_API void OnCleanupLevel();
	virtual void OnCurrentStateChanged(ELevelStreamingState InPrevState, ELevelStreamingState InNewState) override;

	ENGINE_API void Initialize(UWorld* OuterWorld, const TArray<FWorldPartitionRuntimeCellObjectMapping>& InPackages);

	FName OriginalLevelPackageName;
	TArray<FWorldPartitionRuntimeCellObjectMapping> ChildPackages;
	TArray<FWorldPartitionRuntimeCellObjectMapping> ChildPackagesToLoad;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<ULevel> RuntimeLevel;
#endif

	FDelegateHandle OnCleanupLevelDelegateHandle;
	bool bLoadRequestInProgress;
	bool bLoadSucceeded;
	FWorldPartitionLevelHelper::FPackageReferencer PackageReferencer;
#endif

private:
	ENGINE_API void UpdateShouldSkipMakingVisibilityTransactionRequest();
	ENGINE_API bool CanChangeVisibility(bool bMakeVisible) const;

	UPROPERTY()
	bool bShouldBeAlwaysLoaded;

	bool bHasSetLevelTransform;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bShouldPerformStandardLevelLoading;

	UPROPERTY()
	TObjectPtr<UActorContainer> UnsavedActorsContainer;
#endif

	UPROPERTY()
	TWeakObjectPtr<const UWorldPartitionRuntimeLevelStreamingCell> StreamingCell;

	UPROPERTY()
	TSoftObjectPtr<UWorldPartition> OuterWorldPartition;
};
