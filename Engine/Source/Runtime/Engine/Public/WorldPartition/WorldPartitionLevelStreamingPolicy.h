// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * WorldPartitionLevelStreamingPolicy
 *
 * World Partition Streaming Policy that handles load/unload of streaming cell using a corresponding Streaming Level and 
 * a Level that will contain all streaming cell's content.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionStreamingPolicy.h"
#include "WorldPartition/WorldPartitionRuntimeContainerResolving.h"
#include "WorldPartitionLevelStreamingPolicy.generated.h"

struct FResourceSizeEx;
class UWorldPartitionRuntimeLevelStreamingCell;
enum class EWorldPartitionRuntimeCellState : uint8;

UCLASS()
class UWorldPartitionLevelStreamingPolicy : public UWorldPartitionStreamingPolicy
{
	GENERATED_BODY()

public:
	virtual void DrawRuntimeCellsDetails(class UCanvas* Canvas, FVector2D& Offset) override;
	virtual bool IsStreamingCompleted(const TArray<FWorldPartitionStreamingSource>* InStreamingSources) const override;

#if WITH_EDITOR
	virtual TSubclassOf<class UWorldPartitionRuntimeCell> GetRuntimeCellClass() const override;
	virtual void PrepareActorToCellRemapping() override;
	virtual void SetContainerResolver(const FWorldPartitionRuntimeContainerResolver& InContainerResolver) override { ContainerResolver = InContainerResolver; }
	virtual void RemapSoftObjectPath(FSoftObjectPath& ObjectPath) const override;
	static FString GetCellPackagePath(const FName& InCellName, const UWorld* InWorld);

	virtual bool StoreStreamingContentToExternalStreamingObject(URuntimeHashExternalStreamingObjectBase& OutExternalStreamingObject) override;
	virtual bool ConvertContainerPathToEditorPath(const FActorContainerID& InContainerID, const FSoftObjectPath& InPath, FSoftObjectPath& OutPath) const;
#endif

	virtual bool ConvertEditorPathToRuntimePath(const FSoftObjectPath& InPath, FSoftObjectPath& OutPath) const override;
	virtual UObject* GetSubObject(const TCHAR* SubObjectPath) override;

	virtual bool InjectExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject) override;
	virtual bool RemoveExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject) override;

	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
protected:
	void ForEachActiveRuntimeCell(TFunctionRef<void(const UWorldPartitionRuntimeCell*)> Func) const;

private:
	const FName* FindCellNameForSubObject(const FString& InSubObjectString, bool bInResolveContainers, FString& OutSubObjectString, const UObject*& OutLevelMountPointContext, UObject*& OutCellOuter) const;

	UPROPERTY()
	FTopLevelAssetPath SourceWorldAssetPath;

	UPROPERTY()
	TMap<FName, FName> SubObjectsToCellRemapping;

	UPROPERTY()
	FWorldPartitionRuntimeContainerResolver ContainerResolver;

	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase>> ExternalStreamingObjects;

	UPROPERTY(Transient)
	TMap<FName, TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase>> SubObjectsToExternalStreamingObjectsRemapping;
};