// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogWorldPartition, Log, All);
DECLARE_STATS_GROUP(TEXT("World Partition"), STATGROUP_WorldPartition, STATCAT_Advanced);