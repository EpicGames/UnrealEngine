// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Info.h"
#include "WorldPartitionMiniMap.generated.h"

class UTexture2D;

/**
 * A mini map to preview the world in world partition window. (editor-only)
 */
UCLASS(hidecategories = (Actor, Advanced, Display, Events, Object, Attachment, Info, Input, Blueprint, Layers, Tags, Replication, Physics, Cooking, Networking, LevelInstance), notplaceable, MinimalAPI)
class AWorldPartitionMiniMap : public AInfo
{
	GENERATED_BODY()

private:
#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const final { return false; }
	virtual bool ActorTypeSupportsExternalDataLayer() const final { return false; }
	virtual bool ActorTypeIsMainWorldOnly() const override  { return true; }
	virtual bool ShouldImport(FStringView ActorPropString, bool IsMovingLevel) override { return false; }
#endif

public:
	ENGINE_API AWorldPartitionMiniMap(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual bool IsEditorOnly() const final { return true; }
	ENGINE_API virtual void PostLoad() override;

#if WITH_EDITOR
	virtual bool IsLockLocation() const { return true; }
	virtual bool IsUserManaged() const final { return false; }
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API FBox GetMiniMapWorldBounds() const;
	ENGINE_API void GetMiniMapResolution(int32& OutMinimapImageSizeX, int32& OutMinimapImageSizeY, int32& OutWorldUnitsPerPixel) const;
#endif

	/* WorldBounds for MinMapTexture */
	UPROPERTY(VisibleAnywhere, Category = WorldPartitionMiniMap, AdvancedDisplay)
	FBox MiniMapWorldBounds;

	/* UVOffset used to setup Virtual Texture */
	UPROPERTY(VisibleAnywhere, Category = WorldPartitionMiniMap, AdvancedDisplay)
	FBox2D UVOffset;

	/* MiniMap Texture for displaying on world partition window */
	UPROPERTY(VisibleAnywhere, Category = WorldPartitionMiniMap)
	TObjectPtr<UTexture2D> MiniMapTexture;

	/* Datalayers included in MiniMap rendering */
	UPROPERTY(EditAnywhere, Category = WorldPartitionMiniMap)
	TSet<TObjectPtr<UDataLayerAsset>> IncludedDataLayers;

	/* Datalayers excluded from MiniMap rendering */
	UPROPERTY(EditAnywhere, Category = WorldPartitionMiniMap)
	TSet<TObjectPtr<UDataLayerAsset>> ExcludedDataLayers;

	/**
	 * Target world units per pixel for the minimap texture. 
	 * May not end up being the final minimap accuracy if the resulting texture resolution is unsupported.
	 */
	UPROPERTY(EditAnywhere, Category = WorldPartitionMiniMap, meta = (UIMin = "10", UIMax = "100000"), AdvancedDisplay)
	int32 WorldUnitsPerPixel;

	/**
	 * Size of the loading region that will be used when iterating over the whole map during the minimap build process.
	 * A smaller size may help reduce blurriness as it will put less pressure on various graphics pools, at the expanse of an increase in processing time. 
	 */
	UPROPERTY(EditAnywhere, Category = WorldPartitionMiniMap, meta = (UIMin = "3200", UIMax = "204800"), AdvancedDisplay)
	int32 BuilderCellSize;

	/* Specifies which component of the scene rendering should be output to the minimap texture. */
	UPROPERTY(EditAnywhere, Category = WorldPartitionMiniMap, AdvancedDisplay)
	TEnumAsByte<ESceneCaptureSource> CaptureSource;

	/* Number of frames to render before each capture in order to warmup various rendering systems (VT/Nanite/etc). */
	UPROPERTY(EditAnywhere, Category = WorldPartitionMiniMap, meta = (UIMin = "0", UIMax = "10"), AdvancedDisplay)
	uint32 CaptureWarmupFrames;

private:
	UPROPERTY()
	int32 MiniMapTileSize_DEPRECATED = 0;
};
