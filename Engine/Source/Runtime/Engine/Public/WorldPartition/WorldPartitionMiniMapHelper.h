// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * WorldPartitionMiniMapHelper
 *
 * Helper class to build MiniMap texture in World Partition editor.
 *
 */
#pragma once

#if WITH_EDITOR
#include "Math/Box.h"
#include "Math/Matrix.h"
#include "Engine/EngineTypes.h"

class AWorldPartitionMiniMap;
class UWorld;
class UTexture2D;
class AActor;

class FWorldPartitionMiniMapHelper
{
public:
	static ENGINE_API AWorldPartitionMiniMap* GetWorldPartitionMiniMap(UWorld* World, bool bCreateNewMiniMap=false);
	static ENGINE_API void CaptureBoundsMiniMapToTexture(UWorld* InWorld, UObject* InOuterForTexture, uint32 InMiniMapSize, UTexture2D*& InOutMiniMapTexture, const FString& InTextureName, const FBox& InBounds);
	static ENGINE_API void CaptureBoundsMiniMapToTexture(UWorld* InWorld, UObject* InOuterForTexture, uint32 InMiniMapSizeX, uint32 InMiniMapSizeY, UTexture2D*& InOutMiniMapTexture, const FString& InTextureName, const FBox& InBounds, ESceneCaptureSource InCaptureSource, uint32 InNumWarmupFrames, TFunctionRef<void()> WarmupTickCallback);

private:
	static void CalTopViewOfWorld(FMatrix& OutProjectionMatrix, const FBox& WorldBox, uint32 ViewportWidth, uint32 ViewportHeight);
};
#endif

