// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "UObject/NameTypes.h"

class UPackage;

class FWorldPartitionPackageHelper
{
public:
	static void UnloadPackage(UPackage* InPackage);
};

#endif