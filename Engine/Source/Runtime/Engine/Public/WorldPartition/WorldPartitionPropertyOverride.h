// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"

#include "WorldPartitionPropertyOverride.generated.h"

class FProperty;

// Per Sub-Object serialized tagged properties
USTRUCT(NotBlueprintable)
struct FSubObjectPropertyOverride
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY()
	TArray<uint8> SerializedTaggedProperties;
#endif
};

USTRUCT(NotBlueprintable)
struct FPropertyOverrideReferenceTable
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	// Contains SoftObjectPaths from the FSubObjectPropertyOverride serialization so that they can be properly fixed up (fixup redirectors)
	// This table should not be changed outside of serialization of the SubObjectOverrides
	UPROPERTY()
	TArray<FSoftObjectPath> SoftObjectPathTable;

	// Support previous data this will be false until this override is resaved
	UPROPERTY()
	bool bIsValid = false;
#endif
};

// Per Actor overrides, includes a map of Sub-Object name to FSubObjectPropertyOverride data
USTRUCT(NotBlueprintable)
struct FActorPropertyOverride
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	// Used to Serialize newly overriden ActorDescs
	UPROPERTY(Transient)
	TWeakObjectPtr<AActor> Actor;

	UPROPERTY()
	TMap<FString, FSubObjectPropertyOverride> SubObjectOverrides;

	UPROPERTY()
	mutable FPropertyOverrideReferenceTable ReferenceTable;
#endif
};

// Per Container overrides, insludes a map of ActorGuid to FActorPropertyOverride data
USTRUCT(NotBlueprintable)
struct FContainerPropertyOverride
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY()
	TMap<FGuid, FActorPropertyOverride> ActorOverrides;
#endif
};

// Per Container/Sub-Container overrides, includes a map of ContainerPath to FContainerPropertyOverride data
UCLASS(MinimalAPI, NotBlueprintable)
class UWorldPartitionPropertyOverride : public UObject
{
	GENERATED_BODY()
public:
	UWorldPartitionPropertyOverride() {}
	virtual ~UWorldPartitionPropertyOverride() = default;

#if WITH_EDITOR
	const TMap<FActorContainerPath, FContainerPropertyOverride>& GetPropertyOverridesPerContainer() const { return PropertyOverridesPerContainer; }

protected:
	friend class FWorldPartitionLevelHelper;
	friend class UWorldPartitionLevelStreamingDynamic;
	friend class UWorldPartitionSubsystem;

	ENGINE_API static bool ApplyPropertyOverrides(const FActorPropertyOverride* InPropertyOverride, AActor* InActor, bool bConstructionScriptProperties);
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TMap<FActorContainerPath, FContainerPropertyOverride> PropertyOverridesPerContainer;
#endif
};

// Policy used to choose which properties can be overriden (serialized)
UCLASS(Abstract, NotBlueprintable)
class UWorldPartitionPropertyOverridePolicy : public UObject
{
	GENERATED_BODY()
public:
	UWorldPartitionPropertyOverridePolicy() {}
	virtual ~UWorldPartitionPropertyOverridePolicy() = default;
#if WITH_EDITOR
	virtual bool CanOverrideProperty(const FProperty* InProperty) const PURE_VIRTUAL(UWorldPartitionPropertyOverridePolicy::CanOverrideProperty, return false;)
#endif
};




