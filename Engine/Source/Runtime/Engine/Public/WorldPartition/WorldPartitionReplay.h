// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "WorldPartitionReplay.generated.h"

struct FWorldPartitionReplayStreamingSource : public FWorldPartitionStreamingSource
{
	FWorldPartitionReplayStreamingSource()
	{

	}

	FWorldPartitionReplayStreamingSource(const FWorldPartitionStreamingSource& InStreamingSource)
		: FWorldPartitionReplayStreamingSource(InStreamingSource.Name, InStreamingSource.Location, InStreamingSource.Rotation, InStreamingSource.TargetState, InStreamingSource.bBlockOnSlowLoading, InStreamingSource.Priority, InStreamingSource.Velocity)
	{
	}

	FWorldPartitionReplayStreamingSource(FName InName, const FVector& InLocation, const FRotator& InRotation, EStreamingSourceTargetState InTargetState, bool bInBlockOnSlowLoading, EStreamingSourcePriority InPriority, FVector InVelocity)
		: FWorldPartitionStreamingSource(InName, InLocation, InRotation, InTargetState, bInBlockOnSlowLoading, InPriority, false, InVelocity)
	{
		bReplay = true;
	}

	friend FArchive& operator<<(FArchive& Ar, FWorldPartitionReplayStreamingSource& StreamingSource);
};

struct FWorldPartitionReplaySample
{
	FWorldPartitionReplaySample(AWorldPartitionReplay* InReplay);
			
	friend FArchive& operator<<(FArchive& Ar, FWorldPartitionReplaySample& StreamingSource);
	
	TArray<int32> StreamingSourceNameIndices;
	TArray<FWorldPartitionReplayStreamingSource> StreamingSources;
	class AWorldPartitionReplay* Replay = nullptr;
	float TimeSeconds = 0.f;
};

/**
 * Actor used to record world partition replay data (streaming sources for now)
 */
UCLASS(notplaceable, transient, MinimalAPI)
class AWorldPartitionReplay : public AActor
{
	GENERATED_UCLASS_BODY()

public:
	static ENGINE_API void Initialize(UWorld* World);
	static ENGINE_API void Uninitialize(UWorld* World);
	static ENGINE_API bool IsPlaybackEnabled(const UWorld* World);
	static ENGINE_API bool IsRecordingEnabled(const UWorld* World);

	ENGINE_API virtual void RewindForReplay() override;
	ENGINE_API virtual void PreReplication(IRepChangedPropertyTracker& ChangedPropertyTracker) override;
		
	ENGINE_API bool GetReplayStreamingSources(TArray<FWorldPartitionStreamingSource>& OutStreamingSources);
		
private:
	friend FArchive& operator<<(FArchive& Ar, FWorldPartitionReplaySample& StreamingSource);
	friend struct FWorldPartitionReplaySample;
	ENGINE_API TArray<FWorldPartitionStreamingSource> GetRecordingStreamingSources() const;

	UPROPERTY(Transient, Replicated)
	TArray<FName> StreamingSourceNames;

	TArray<FWorldPartitionReplaySample> ReplaySamples;
};

