// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "WorldPartition/WorldPartitionRuntimeCellData.h"
#include "WorldPartition/WorldPartitionRuntimeCellInterface.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageInterface.h"
#include "WorldPartition/DataLayer/DataLayerInstanceNames.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Misc/HierarchicalLogArchive.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Algo/AnyOf.h"
#include "UObject/ObjectMacros.h"
#include "WorldPartitionRuntimeCell.generated.h"

class UActorContainer;
class UDataLayerAsset;
class UDataLayerInstance;
class UExternalDataLayerInstance;
class UWorldPartition;
class UDataLayerManager;
class UExternalDataLayerAsset;
class FStreamingGenerationActorDescView;
struct FHierarchicalLogArchive;
struct FWorldPartitionStreamingContext;

enum class EWorldPartitionDataLayersLogicOperator : uint8;

enum class EWorldPartitionRuntimeCellVisualizeMode
{
	StreamingPriority,
	StreamingStatus
};

USTRUCT()
struct FWorldPartitionRuntimeCellPropertyOverride
{
	GENERATED_USTRUCT_BODY()

	FWorldPartitionRuntimeCellPropertyOverride()
	{}

	FWorldPartitionRuntimeCellPropertyOverride(const FActorContainerID& InOwnerContainerID, const FString& InAssetPath, FName InPackageName, const FActorContainerPath& InContainerPath)
#if WITH_EDITORONLY_DATA
		: OwnerContainerID(InOwnerContainerID)
		, AssetPath(InAssetPath)
		, PackageName(InPackageName)
		, ContainerPath(InContainerPath)
#endif
	{}

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FActorContainerID OwnerContainerID;

	UPROPERTY()
	FString AssetPath;

	UPROPERTY()
	FName PackageName;

	UPROPERTY()
	FActorContainerPath ContainerPath;

	void UpdateHash(FWorldPartitionPackageHashBuilder& Builder) const;
#endif
};

USTRUCT()
struct FWorldPartitionRuntimeCellObjectMapping
{
	GENERATED_USTRUCT_BODY()

	FWorldPartitionRuntimeCellObjectMapping()
#if WITH_EDITORONLY_DATA
		: ContainerTransform(FTransform::Identity)
		, bIsEditorOnly(false)
#endif
	{}

	FWorldPartitionRuntimeCellObjectMapping(
		FName InPackage, 
		FName InPath, 
		const FTopLevelAssetPath& InBaseClass, 
		const FTopLevelAssetPath& InNativeClass, 
		const FActorContainerID& InContainerID, 
		const FTransform& InContainerTransform, 
		const FTransform& InEditorOnlyParentTransform, 
		FName InContainerPackage, 
		FName InWorldPackage, 
		const FGuid& InActorInstanceGuid, 
		bool bInIsEditorOnly
	)
#if WITH_EDITORONLY_DATA
		: Package(InPackage)
		, Path(InPath)
		, BaseClass(InBaseClass)
		, NativeClass(InNativeClass)
		, ContainerID(InContainerID)
		, ContainerTransform(InContainerTransform)
		, EditorOnlyParentTransform(InEditorOnlyParentTransform)
		, ContainerPackage(InContainerPackage)
		, WorldPackage(InWorldPackage)
		, ActorInstanceGuid(InActorInstanceGuid)
		, LoadedPath(InPath)
		, bIsEditorOnly(bInIsEditorOnly)
#endif
	{}

#if WITH_EDITORONLY_DATA
	/** 
	 * The name of the package to load to resolve on disk (can contain a single actor or a data chunk)
	 */
	UPROPERTY()
	FName Package;

	/** 
	 * The complete name path of the contained object
	 */
	UPROPERTY()
	FName Path;

	/** 
	 * The actor's base class
	 */
	UPROPERTY()
	FTopLevelAssetPath BaseClass;

	/** 
	 * The actor's native base class
	 */
	UPROPERTY()
	FTopLevelAssetPath NativeClass;

	/**
	 * ID of the owning container instance
	 */
	UPROPERTY()
	FActorContainerID ContainerID;

	/** 
	 * Transform of the owning container instance
	 */
	UPROPERTY()
	FTransform ContainerTransform;

	/** 
	 * Transform of the owning actor if editor only
	 */
	UPROPERTY()
	FTransform EditorOnlyParentTransform;

	/**
	 * Package of the owning container instance
	 */
	UPROPERTY()
	FName ContainerPackage;

	/**
	 * Package of the World 
	 */
	UPROPERTY()
	FName WorldPackage;

	/**
	 * GUID of the actor instance
	 */
	UPROPERTY()
	FGuid ActorInstanceGuid;

	/**
	* Loaded actor path (when cooking or pie)
	* 
	* Depending on if the actor was part of a container instance or the main partition this will be the path
	* of the loaded or duplicated actor before it is moved into its runtime cell.
	* 
	* If the actor was part of the world partition this path should match the Path property.
	*/
	UPROPERTY()
	FName LoadedPath;

	UPROPERTY()
	bool bIsEditorOnly;
		
	UPROPERTY()
	TArray<FWorldPartitionRuntimeCellPropertyOverride> PropertyOverrides;

	void UpdateHash(FWorldPartitionPackageHashBuilder& Builder) const;
#endif
};

class UActorDescContainer;

/**
 * Cell State
 */
UENUM(BlueprintType)
enum class EWorldPartitionRuntimeCellState : uint8
{
	Unloaded,
	Loaded,
	Activated
};

USTRUCT()
struct FWorldPartitionRuntimeCellDebugInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FName GridName;

	UPROPERTY()
	int64 CoordX = 0;

	UPROPERTY()
	int64 CoordY = 0;

	UPROPERTY()
	int64 CoordZ = 0;
};

static_assert(EWorldPartitionRuntimeCellState::Unloaded < EWorldPartitionRuntimeCellState::Loaded && EWorldPartitionRuntimeCellState::Loaded < EWorldPartitionRuntimeCellState::Activated, "Streaming Query code is dependent on this being true");

/**
 * Represents a PIE/Game streaming cell which points to external actor/data chunk packages
 */
UCLASS(Abstract, MinimalAPI)
class UWorldPartitionRuntimeCell : public UObject, public IWorldPartitionCell, public IWorldPartitionCookPackageObject
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
#endif

	template<class T>
	T* GetTypedOuter() const
	{
		static_assert(!std::is_same<T, UWorld>::value, "Use GetOuterWorld instead");
		static_assert(!std::is_same<T, UWorldPartition>::value, "Use GetOuterWorld()->GetWorldPartition() instead");
		return Super::GetTypedOuter<T>();
	}

	virtual void Load() const PURE_VIRTUAL(UWorldPartitionRuntimeCell::Load,);
	virtual void Unload() const PURE_VIRTUAL(UWorldPartitionRuntimeCell::Unload,);
	virtual bool CanUnload() const PURE_VIRTUAL(UWorldPartitionRuntimeCell::CanUnload, return true;);
	virtual void Activate() const PURE_VIRTUAL(UWorldPartitionRuntimeCell::Activate,);
	virtual void Deactivate() const PURE_VIRTUAL(UWorldPartitionRuntimeCell::Deactivate,);
	virtual ULevel* GetLevel() const PURE_VIRTUAL(UWorldPartitionRuntimeCell::GetLevel, return nullptr;);
	virtual EWorldPartitionRuntimeCellState GetCurrentState() const PURE_VIRTUAL(UWorldPartitionRuntimeCell::GetCurrentState, return EWorldPartitionRuntimeCellState::Unloaded;);
	virtual FLinearColor GetDebugColor(EWorldPartitionRuntimeCellVisualizeMode VisualizeMode) const { static const FLinearColor DefaultColor = FLinearColor::Black.CopyWithNewOpacity(0.25f); return DefaultColor; }
	virtual bool IsAlwaysLoaded() const { return bIsAlwaysLoaded; }
	virtual bool IsSpatiallyLoaded() const { return bIsSpatiallyLoaded; }
	virtual void SetIsAlwaysLoaded(bool bInIsAlwaysLoaded) { bIsAlwaysLoaded = bInIsAlwaysLoaded; }
	virtual void SetIsSpatiallyLoaded(bool bInIsSpatiallyLoaded) { bIsSpatiallyLoaded = bInIsSpatiallyLoaded; }
	virtual void SetStreamingPriority(int32 InStreamingPriority) const PURE_VIRTUAL(UWorldPartitionRuntimeCell::SetStreamingPriority,);
	virtual EStreamingStatus GetStreamingStatus() const { return LEVEL_Unloaded; }
	/** Thread-safe. Returns whether this cell's level is visible (added to world and rendering). */
	virtual bool IsVisible() const { return false; }
	void SetClientOnlyVisible(bool bInClientOnlyVisible) { bClientOnlyVisible = bInClientOnlyVisible; }
	bool GetClientOnlyVisible() const { return bClientOnlyVisible; }
	virtual FGuid const& GetContentBundleID() const { return ContentBundleID; }
	virtual TArray<FName> GetActors() const PURE_VIRTUAL(UWorldPartitionRuntimeCell::GetActors, return TArray<FName>(););

	//~Begin IWorldPartitionCell Interface
	ENGINE_API virtual TArray<const UDataLayerInstance*> GetDataLayerInstances() const override;
	ENGINE_API virtual const UExternalDataLayerInstance* GetExternalDataLayerInstance() const override;
	ENGINE_API virtual bool ContainsDataLayer(const UDataLayerAsset* DataLayerAsset) const override;
	ENGINE_API virtual bool ContainsDataLayer(const UDataLayerInstance* DataLayerInstance) const override;
	ENGINE_API virtual bool HasContentBundle() const override;
	virtual const TArray<FName>& GetDataLayers() const override { return GetDataLayersInline(); }
	virtual FName GetExternalDataLayer() const override { return DataLayers.GetExternalDataLayer(); }
	virtual bool HasAnyDataLayer(const TSet<FName>& InDataLayers) const override
	{
		return Algo::AnyOf(GetDataLayers(), [&InDataLayers](const FName& DataLayer) { return InDataLayers.Contains(DataLayer); });
	}
	ENGINE_API virtual FName GetLevelPackageName() const override;
	ENGINE_API virtual FString GetDebugName() const override;
	ENGINE_API virtual UWorld* GetOwningWorld() const override;
	ENGINE_API virtual UWorld* GetOuterWorld() const override;
	//~End IWorldPartitionCell Interface

	ENGINE_API UDataLayerManager* GetDataLayerManager() const;
	/** Thread-safe resolve and cache of the effective wanted state for this cell's data layers. */
	ENGINE_API EDataLayerRuntimeState GetCellEffectiveWantedState(const FWorldPartitionStreamingContext& Context) const;
	inline bool HasDataLayers() const { return !DataLayers.IsEmpty(); }

	void SetBlockOnSlowLoading(bool bInBlockOnSlowLoading) { bBlockOnSlowLoading = bInBlockOnSlowLoading; }
	bool GetBlockOnSlowLoading() const { return bBlockOnSlowLoading; }
#if WITH_EDITOR
	ENGINE_API bool NeedsActorToCellRemapping() const;
	ENGINE_API void SetDataLayers(const TArray<const UDataLayerInstance*>& InDataLayerInstances);
	void SetContentBundleUID(const FGuid& InContentBundleID) { ContentBundleID = InContentBundleID; }
	void SetLevelPackageName(const FName& InLevelPackageName) { LevelPackageName = InLevelPackageName; }
	virtual FName GetCookPackageName() const PURE_VIRTUAL(UWorldPartitionRuntimeCell::GetCookPackageName, return NAME_None;);
	
	//~Begin IWorldPartitionCell Interface
	virtual TSet<FName> GetActorPackageNames() const override { return TSet<FName>(); }
	//~End IWorldPartitionCell Interface

	virtual void AddActorToCell(const FStreamingGenerationActorDescView& ActorDescView) PURE_VIRTUAL(UWorldPartitionRuntimeCell::AddActorToCell, );

	virtual void Fixup() PURE_VIRTUAL(UWorldPartitionRuntimeCell::Fixup, );
	virtual int32 GetActorCount() const PURE_VIRTUAL(UWorldPartitionRuntimeCell::GetActorCount, return 0;);

	// Cook methods
	virtual bool PrepareCellForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InGeneratedPackage = nullptr) { return true; }

	//~Begin IWorldPartitionCookPackageObject
	virtual bool IsLevelPackage() const override { return true; }
	virtual const UExternalDataLayerAsset* GetExternalDataLayerAsset() const override { return ExternalDataLayerAsset; }
	virtual FString GetPackageNameToCreate() const { return FString(); }
	virtual bool OnPrepareGeneratorPackageForCook(TArray<UPackage*>& OutModifiedPackages) override { return true; }
	virtual bool OnPopulateGeneratorPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InPackage) override { return true; }
	virtual bool OnPopulateGeneratedPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InPackage, TArray<UPackage*>& OutModifiedPackages) override { return true; }
	virtual FWorldPartitionPackageHash GetGenerationHash() const override PURE_VIRTUAL(UWorldPartitionRuntimeCell::GetGenerationHash, return FWorldPartitionPackageHash(););
	//~End IWorldPartitionCookPackageObject

	ENGINE_API virtual void DumpStateLog(FHierarchicalLogArchive& Ar) const;
#endif

	void SetIsHLOD(bool bInIsHLOD) { bIsHLOD = bInIsHLOD; }
	bool GetIsHLOD() const { return bIsHLOD; }

	void SetIsCustomHLODPlaceholderCell(bool bInIsCustomHLODPlaceholderCell) { bIsCustomHLODPlaceholderCell = bInIsCustomHLODPlaceholderCell; }
	bool GetIsCustomHLODPlaceholderCell() const { return bIsCustomHLODPlaceholderCell; }
	
	const FGuid& GetGuid() const { return CellGuid; }
	void SetGuid(const FGuid& InCellGuid) { CellGuid = InCellGuid; }

	const FLinearColor& GetCellDebugColor() const { return CellDebugColor; }
	void SetCellDebugColor(const FLinearColor& InCellDebugColor) { CellDebugColor = InCellDebugColor; }

	const FGuid& GetSourceCellGuid() const { return SourceCellGuid; }
	void SetSourceCellGuid(const FGuid& InSourceCellGuid) { SourceCellGuid = InSourceCellGuid; }

#if !UE_BUILD_SHIPPING
	void SetDebugStreamingPriority(float InDebugStreamingPriority) { DebugStreamingPriority = InDebugStreamingPriority; }
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UActorContainer> UnsavedActorsContainer;
#endif

protected:
	inline const TArray<FName>& GetDataLayersInline() const { return DataLayers.ToArray(); }

	//@todo_ow: Implement ServerOnlyVisible and refactor ClientOnlyVisible.
	//          Instead of this function, server would not not wait for client level visibility 
	//          for server-only visible cells.
	//          Also, server-only & client-only visible cells should be pre-filtered at 
	//          world partition initialization to avoid visiting them at runtime.
	//          The same should be done for injected external streaming objects.
	virtual bool ShouldServerWaitForClientLevelVisibility() const { return true; }

	/** Thread-safe read of the cached effective wanted state (without resolving). */
	EDataLayerRuntimeState GetCellEffectiveWantedStateRaw() const
	{
		if (!HasDataLayers()) { return EDataLayerRuntimeState::Activated; }
		UE::TReadScopeLock ReadLock(EffectiveWantedStateLock);
		return EffectiveWantedState;
	}

	UPROPERTY()
	bool bIsAlwaysLoaded;

	UPROPERTY()
	bool bIsSpatiallyLoaded;

private:
	UPROPERTY()
	FDataLayerInstanceNames DataLayers;

	UPROPERTY()
	bool bClientOnlyVisible;

	UPROPERTY()
	bool bIsHLOD;

	UPROPERTY()
	bool bIsCustomHLODPlaceholderCell;

	UPROPERTY()
	bool bBlockOnSlowLoading;

	UPROPERTY()
	FGuid ContentBundleID;

	UPROPERTY()
	FLinearColor CellDebugColor;

protected:
	UPROPERTY()
	FGuid CellGuid;

	// Used by injected HLOD cells
	UPROPERTY()
	FGuid SourceCellGuid;

#if !UE_BUILD_SHIPPING
	// Represents the streaming priority relative to other cells
	float DebugStreamingPriority;
#endif

#if WITH_EDITOR
	FName LevelPackageName;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<const UExternalDataLayerAsset> ExternalDataLayerAsset;
#endif

	mutable FTransactionallySafeRWLock EffectiveWantedStateLock;
	mutable EDataLayerRuntimeState EffectiveWantedState;
	mutable int32 EffectiveWantedStateEpoch;

public:
	//~Begin UWorldPartitionRuntimeCellData Proxy
	inline void AppendStreamingSourceInfo(const FWorldPartitionStreamingSource& Source, const FSphericalSector& SourceShape, const FWorldPartitionStreamingContext& Context) const { RuntimeCellData->AppendStreamingSourceInfo(Source, SourceShape, Context, bBlockOnSlowLoading); }
	inline void MergeStreamingSourceInfo() const { RuntimeCellData->MergeStreamingSourceInfo(); }
	ENGINE_API int32 SortCompare(const UWorldPartitionRuntimeCell* Other) const;
	
	/** Returns the cell'content s bounds, which is the sum of all actor bounds inside the cell. */
	inline const FBox& GetContentBounds() const { return RuntimeCellData->GetContentBounds(); }

	/** Returns the cell's bounds, which is the uniform size of the cell. */
	inline FBox GetCellBounds() const { return RuntimeCellData->GetCellBounds(); }

	/** Returns the cell's streaming bounds, which is what the underlying runtime hash uses to intersect cells. If the cell is Custom HLOD Placeholder cell, returns cell bounds. */
	inline FBox GetStreamingBounds() const { return bIsCustomHLODPlaceholderCell ? RuntimeCellData->GetCellBounds() : RuntimeCellData->GetStreamingBounds(); }

	ENGINE_API virtual bool IsDebugShown() const;
	//~End UWorldPartitionRuntimeCellData Proxy

	UPROPERTY()
	TObjectPtr<UWorldPartitionRuntimeCellData> RuntimeCellData;

	friend class UWorldPartitionStreamingPolicy;
	friend struct FSpatialHashStreamingGrid;
};

[[nodiscard]] constexpr bool operator == (EWorldPartitionRuntimeCellState a, EDataLayerRuntimeState b)
{
	return static_cast<uint8>(a) == static_cast<uint8>(b);
}

[[nodiscard]] constexpr bool operator == (EDataLayerRuntimeState a, EWorldPartitionRuntimeCellState b)
{
	return static_cast<uint8>(a) == static_cast<uint8>(b);
}

[[nodiscard]] constexpr bool operator != (EWorldPartitionRuntimeCellState a, EDataLayerRuntimeState b)
{
	return !(a == b);
}

[[nodiscard]] constexpr bool operator != (EDataLayerRuntimeState a, EWorldPartitionRuntimeCellState b)
{
	return !(a == b);
}

[[nodiscard]] constexpr bool operator < (EWorldPartitionRuntimeCellState a, EDataLayerRuntimeState b)
{
	return static_cast<uint8>(a) < static_cast<uint8>(b);
}

[[nodiscard]] constexpr bool operator < (EDataLayerRuntimeState a, EWorldPartitionRuntimeCellState b)
{
	return static_cast<uint8>(a) < static_cast<uint8>(b);
}

[[nodiscard]] constexpr bool operator > (EWorldPartitionRuntimeCellState a, EDataLayerRuntimeState b)
{
	return static_cast<uint8>(a) > static_cast<uint8>(b);
}

[[nodiscard]] constexpr bool operator > (EDataLayerRuntimeState a, EWorldPartitionRuntimeCellState b)
{
	return static_cast<uint8>(a) > static_cast<uint8>(b);
}