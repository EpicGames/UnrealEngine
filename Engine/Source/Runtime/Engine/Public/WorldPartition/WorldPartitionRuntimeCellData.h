// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "StringDev.h"
#include "Misc/HierarchicalLogArchive.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "WorldPartitionRuntimeCellData.generated.h"

class UActorContainer;
struct FWorldPartitionStreamingContext;

/** Caches information on streaming source that will be used later on to sort cell. */
UCLASS(MinimalAPI)
class UWorldPartitionRuntimeCellData : public UObject
{
	GENERATED_UCLASS_BODY()

	virtual ~UWorldPartitionRuntimeCellData() = default;

	//~Begin UObject Interface
	ENGINE_API void Serialize(FArchive& Ar);
	//~End UObject Interface

#if WITH_EDITOR
	ENGINE_API virtual void DumpStateLog(FHierarchicalLogArchive& Ar) const;
#endif

	ENGINE_API virtual void ResetStreamingSourceInfo(const FWorldPartitionStreamingContext& Context) const;
	ENGINE_API virtual void AppendStreamingSourceInfo(const FWorldPartitionStreamingSource& Source, const FSphericalSector& SourceShape, const FWorldPartitionStreamingContext& Context, bool bBlockOnSlowLoading) const;
	virtual void MergeStreamingSourceInfo() const {}
	ENGINE_API virtual int32 SortCompare(const UWorldPartitionRuntimeCellData* InOther) const;

	/** Returns the cell's content bounds, which is the sum of all actor bounds inside the cell. */
	ENGINE_API virtual const FBox& GetContentBounds() const;

	/** Returns the cell's bounds, which is the uniform size of the cell. */
	ENGINE_API virtual FBox GetCellBounds() const;

	/** Returns the cell's streaming bounds, which is what the underlying runtime hash uses to intersect cells.  */
	ENGINE_API virtual FBox GetStreamingBounds() const;	

	virtual bool IsDebugShown() const { return true; }
	ENGINE_API virtual FString GetDebugName() const;

	// Minimum affecting source priority
	mutable uint8 CachedMinSourcePriority;

	// Determine if the cell was requested by a blocking source
	mutable bool bCachedWasRequestedByBlockingSource;

	// Square distance from the cell to the closest blocking streaming source
	mutable double CachedMinSquareDistanceToBlockingSource;

	// Ratio used to determine the cell streaming performance status
	mutable float CachedMinBlockOnSlowStreamingRatio;

	// Square distance from the cell to the closest streaming source
	mutable double CachedMinSquareDistanceToSource;

	// Ratio used to determine the cell streaming performance status
	mutable float CachedMinSlowStreamingRatio;

	// Spatial priority based on distance and angle from source
	mutable double CachedMinSpatialSortingPriority;

	// Epoch used to dirty cache
	mutable int32 CachedSourceInfoEpoch;

	UPROPERTY()
	FBox ContentBounds;

	// Optional cell bounds
	UPROPERTY()
	TOptional<FBox> CellBounds;

	UPROPERTY()
	FName GridName;

	UPROPERTY()
	int32 Priority;

	UPROPERTY()
	int32 HierarchicalLevel;

	FStringTest DebugName;
};
