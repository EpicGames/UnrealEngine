// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionRuntimeCellData.h"
#include "WorldPartitionRuntimeCellDataSpatialHash.generated.h"

class UActorContainer;
class UWorldPartitionRuntimeCell;

UCLASS(Within = WorldPartitionRuntimeCell, MinimalAPI)
class UWorldPartitionRuntimeCellDataSpatialHash : public UWorldPartitionRuntimeCellData
{
	GENERATED_UCLASS_BODY()

	//~Begin UWorldPartitionRuntimeCellData
	ENGINE_API virtual void ResetStreamingSourceInfo(const FWorldPartitionStreamingContext& Context) const override;
	ENGINE_API virtual void AppendStreamingSourceInfo(const FWorldPartitionStreamingSource& Source, const FSphericalSector& SourceShape, const FWorldPartitionStreamingContext& Context, bool bBlockOnSlowLoading) const override;
	ENGINE_API virtual void MergeStreamingSourceInfo() const override;
	ENGINE_API virtual int32 SortCompare(const UWorldPartitionRuntimeCellData* InOther) const override;
	ENGINE_API virtual FBox GetCellBounds() const override;
	ENGINE_API virtual FBox GetStreamingBounds() const override;
	ENGINE_API virtual bool IsDebugShown() const override;
	//~End UWorldPartitionRuntimeCellData

	UPROPERTY()
	FVector Position;

	UPROPERTY()
	float Extent;

private:
	float ComputeSourceToCellAngleFactor(const FSphericalSector& SourceShape) const;

	// Modulated distance to the different streaming sources used to sort relative priority amongst streaming cells
	// The value is affected by :
	// - All sources intersecting the cell
	// - The priority of each source
	// - The distance between the cell and each source
	// - The angle between the cell and each source orientation
	mutable double CachedSourceSortingDistance;

	// Source Priorities
	mutable TArray<float> CachedSourcePriorityWeights;

	// Square distance from the cell to  the intersecting streaming sources
	mutable TArray<double> CachedSourceSquaredDistances;

	// Intersecting streaming source shapes
	mutable TArray<FSphericalSector> CachedInstersectingShapes;

	// 2D version of CachedMinBlockOnSlowStreamingRatio
	mutable double CachedMinBlockOnSlowStreamingRatio2D;

	// 2D version of CachedMinSquareDistanceToBlockingSource
	mutable double CachedMinSquareDistanceToBlockingSource2D;

	// 2D version of CachedMinSlowStreamingRatio
	mutable double CachedMinSlowStreamingRatio2D;

	// 2D version of CachedMinSquareDistanceToSource
	mutable double CachedMinSquareDistanceToSource2D;
};
