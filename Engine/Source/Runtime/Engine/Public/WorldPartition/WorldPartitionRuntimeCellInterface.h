// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "WorldPartitionRuntimeCellInterface.generated.h"

class UDataLayerAsset;
class UDataLayerInstance;
class UExternalDataLayerInstance;

UINTERFACE(MinimalAPI)
class UWorldPartitionCell : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IWorldPartitionCell
{
	GENERATED_IINTERFACE_BODY()

public:
	/** Returns the cell content associated data layer instances. */
	virtual TArray<const UDataLayerInstance*> GetDataLayerInstances() const = 0;
	/** Returns the cell associated external data layer instance. */
	virtual const UExternalDataLayerInstance* GetExternalDataLayerInstance() const = 0;
	/** Returns whether the cell data layers referenced the provided data layer asset or not. */
	virtual bool ContainsDataLayer(const UDataLayerAsset* DataLayerAsset) const = 0;
	/** Returns whether the cell data layers referenced the provided data layer instance or not. */
	virtual bool ContainsDataLayer(const UDataLayerInstance* DataLayerInstance) const = 0;
	/** Returns whether the cell content is associated with content bundle or not. */
	virtual bool HasContentBundle() const = 0;
	/** Returns the cell content associated data layers. */
	virtual const TArray<FName>& GetDataLayers() const = 0;
	/** Returns the cell content associated external data layer. */
	virtual FName GetExternalDataLayer() const = 0;
	/** Returns whether the cell data layers referenced any of the provided data layer or not. */
	virtual bool HasAnyDataLayer(const TSet<FName>& InDataLayers) const = 0;
	/** Returns the cell bounds computed using cell content. */
	virtual const FBox& GetContentBounds() const = 0;
	/** Returns the cell bounds. */
	virtual FBox GetCellBounds() const = 0;
	/** Returns the associated level package name. */
	virtual FName GetLevelPackageName() const = 0;
	/** Returns the debug name associated with this cell. */
	virtual FString GetDebugName() const = 0;
	/**  Returns the owning world of this cell. */
	virtual UWorld* GetOwningWorld() const = 0;
	/**  REturns the outer world of this cell. */
	virtual UWorld* GetOuterWorld() const = 0;

#if WITH_EDITOR
	/** Returns the referenced actor packages. */
	virtual TSet<FName> GetActorPackageNames() const = 0;
#endif
};

