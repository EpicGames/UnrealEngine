// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "WorldPartitionRuntimeCellTransformer.generated.h"

class UWorldPartitionRuntimeCell;

#define UE_API ENGINE_API

UCLASS(MinimalAPI, Config=Engine)
class UWorldPartitionRuntimeCellTransformerSettings : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(Config)
	TArray<TSubclassOf<UActorComponent>> IgnoredComponentClasses;

	UPROPERTY(Config)
	TArray<TSubclassOf<UActorComponent>> IgnoredExactComponentClasses;
#endif
};

UCLASS(MinimalAPI, Abstract)
class UWorldPartitionRuntimeCellTransformer : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UObject Interface.
	UE_API virtual void PostLoad() override;
	//~ End UObject Interface.

#if WITH_EDITOR
	/** Whether the level can be processed by the transformer. */
	UE_API virtual bool IsLevelTransformable(ULevel* InLevel) const;
	UE_API virtual void PreTransform(ULevel* InLevel);
	UE_API virtual void Transform(ULevel* InLevel);
	UE_API virtual void PostTransform(ULevel* InLevel);

	/** Whether the cell can be process by the transformer. */
	UE_API virtual bool IsCellTransformable(const UWorldPartitionRuntimeCell* InCell) const;

	/**
	 * Vote to strip this runtime cell from the grid entirely. If any transformer returns true, the cell
	 * is dropped before its packages are generated. Default implementation returns false (retain).
	 *
	 * Called during streaming generation, before the cell has been populated with actors — at this point
	 * the cell's grid/data-layer/guid metadata is set, but its ULevel has not been built and no actors
	 * have been loaded into it. Implementations must make the strip decision purely from cell metadata
	 * (data layers, bounds, grid, etc.), not from actor content.
	 */
	UE_API virtual bool ShouldStripRuntimeCell(const UWorldPartitionRuntimeCell* InCell) const;

	/**
	 * Hook to transform the runtime cell structure itself.
	 * Note: Modification of the cell contents must be done in the ULevel* hook.
	 */
	UE_API virtual void TransformRuntimeCell(UWorldPartitionRuntimeCell* InCell);

	UE_API virtual void ForEachIgnoredComponentClass(TFunctionRef<bool(const TSubclassOf<UActorComponent>&)> Func) const;
	UE_API virtual void ForEachIgnoredExactComponentClass(TFunctionRef<bool(const TSubclassOf<UActorComponent>&)> Func) const;
#endif

	bool IsEnabled() const { return bEnabled; }

protected:
#if WITH_EDITOR
	UE_API virtual bool CanIgnoreComponent(const UActorComponent* InComponent) const;
	UE_API virtual bool IsBlueprintActorWithLogic(AActor* InActor) const;
#endif
	
protected:
	// Tag used to force exclude actors from any cell transformation
	UE_API static const FName NAME_CellTransformerIgnoreActor;

private:
	UPROPERTY(EditAnywhere, Category = Transformer)
	bool bEnabled = true;
};

#undef UE_API