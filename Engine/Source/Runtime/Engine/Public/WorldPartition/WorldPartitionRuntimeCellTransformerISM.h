// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WorldPartition/WorldPartitionRuntimeCellTransformer.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"

#include "WorldPartitionRuntimeCellTransformerISM.generated.h"

UCLASS()
class UWorldPartitionRuntimeCellTransformerISM : public UWorldPartitionRuntimeCellTransformer
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
public:
	virtual void Transform(ULevel* InLevel) override;

protected:
	/** Whether the actor can be processed by the transformer. */
	virtual bool IsActorTransformable(AActor* InActor) const { return true; }
	/** Whether the component can be processed by the transformer. */
	virtual bool IsComponentTransformable(UActorComponent* InComponent) const { return true; }

	virtual bool CanIgnoreComponent(const UActorComponent* InComponent) const override;
	virtual bool CanAutoInstanceActor(AActor* InActor) const;
	virtual bool CanRemoveActor(AActor* InActor) const;
#endif

#if WITH_EDITORONLY_DATA
public:
	/** Allowed classes (recursive) to convert to instances */
	UPROPERTY(EditAnywhere, Category = ISM)
	TArray<TSubclassOf<AActor>> AllowedClasses;

	/** Disallowed classes (non-recursive) to convert to instances */
	UPROPERTY(EditAnywhere, Category = ISM)
	TArray<TSubclassOf<AActor>> DisallowedClasses;

	/** Minimum number of instances required to allow converting actors to ISM */
	UPROPERTY(EditAnywhere, Category = ISM)
	uint32 MinNumInstances;

	/** When true, bucket additionally on UPrimitiveComponent fields the merge would otherwise drop (e.g. CanCharacterStepUpOn). */
	UPROPERTY(EditAnywhere, Category = ISM)
	bool bStrictBucketing = false;
#endif
};

/** Actor class used by UWorldPartitionRuntimeCellTransformerISM to save transformed result */
UCLASS(NotPlaceable, NotBlueprintable, NotBlueprintType, MinimalAPI)
class AWorldPartitionAutoInstancedActor : public AActor
{
	GENERATED_UCLASS_BODY()
};