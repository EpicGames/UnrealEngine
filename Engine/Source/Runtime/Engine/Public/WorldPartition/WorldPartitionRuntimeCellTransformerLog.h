// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartitionRuntimeCellTransformer.h"
#include "WorldPartitionRuntimeCellTransformerLog.generated.h"

UCLASS()
class UWorldPartitionRuntimeCellTransformerLog : public UWorldPartitionRuntimeCellTransformer
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
public:
	virtual void PreTransform(ULevel* InLevel) override;
	virtual void PostTransform(ULevel* InLevel) override;

private:
	void GatherLevelContentStats(ULevel* InLevel, TMap<UClass*, int32>& OutClassNumInstances);
	void DumpLevelContentStats(ULevel* InLevel, const TMap<UClass*, int32>& InClassNumInstances);
#endif

#if WITH_EDITORONLY_DATA
public:
	/** Only log when the level was actually transformed */
	UPROPERTY(EditAnywhere, Category = ISM)
	bool bOnlyLogDifferences;

	TMap<UClass*, int32> ClassNumInstancesBefore;
#endif
};