// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "Misc/Guid.h"
#include "Containers/Map.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "WorldPartitionRuntimeContainerResolving.generated.h"

USTRUCT()
struct FWorldPartitionRuntimeContainerInstance
{
	GENERATED_USTRUCT_BODY()
public:
	FWorldPartitionRuntimeContainerInstance()
	{}

	FWorldPartitionRuntimeContainerInstance(const FGuid& InActorGuid, FName InContainerPackage)
		: ActorGuid(InActorGuid), ContainerPackage(InContainerPackage)
	{}

private:
	UPROPERTY()
	FGuid ActorGuid;

	UPROPERTY()
	FName ContainerPackage;

	friend struct FWorldPartitionRuntimeContainerResolver;
};

USTRUCT()
struct FWorldPartitionRuntimeContainer
{
	GENERATED_USTRUCT_BODY()
public:
#if WITH_EDITOR
	void AddContainerInstance(FName InActorName, const FWorldPartitionRuntimeContainerInstance& InContainerInstance)
	{
		check(!ContainerInstances.Contains(InActorName));
		ContainerInstances.Add(InActorName, InContainerInstance);
	}
#endif
	SIZE_T GetAllocatedSize() const;
private:
	UPROPERTY()
	TMap<FName, FWorldPartitionRuntimeContainerInstance> ContainerInstances;

	friend struct FWorldPartitionRuntimeContainerResolver;
};

/**
 * FWorldPartitionRuntimeContainerResolver
 *
 * Helper class that allows resolving a hierarchy of Container actor names (Editor Path) to a resolvable persistent level path (Runtime Path)
 * 
 * Editor Path : /Game/Map.Map:PersistentLevel.LevelInstance1.LevelInstance2.StaticMeshActor 
 * Runtime Path : /Game/Map.Map:PersistentLevel.StaticMeshActor_{ContainerID}
 */
USTRUCT()
struct FWorldPartitionRuntimeContainerResolver
{
	GENERATED_USTRUCT_BODY()
public:
	bool IsValid() const { return !MainContainerPackage.IsNone(); }
	bool ResolveContainerPath(const FString& InSubObjectString, FString& OutSubObjectString) const;
	SIZE_T GetAllocatedSize() const;
	
#if WITH_EDITOR
	void SetMainContainerPackage(FName InContainerPackage)
	{
		check(MainContainerPackage.IsNone());
		MainContainerPackage = InContainerPackage;
	}

	FWorldPartitionRuntimeContainer& AddContainer(FName InContainerPackage)
	{
		check(!Containers.Contains(InContainerPackage));
		return Containers.Add(InContainerPackage);	
	}

	bool ContainsContainer(FName InContainerPackage)
	{
		return Containers.Contains(InContainerPackage);
	}

	void BuildContainerIDToEditorPathMap();
	const FString* FindContainerEditorPath(const FActorContainerID& InContainerID) const { return ContainerIDToEditorPath.Find(InContainerID); }
#endif
private:
	UPROPERTY()
	TMap<FName, FWorldPartitionRuntimeContainer> Containers;
	UPROPERTY()
	FName MainContainerPackage;

#if WITH_EDITORONLY_DATA
	// Reverse lookup :
	// ex: {ContainerID}, "LevelInstance1.LevelInstance2"
	UPROPERTY()
	TMap<FActorContainerID, FString> ContainerIDToEditorPath;
#endif
};