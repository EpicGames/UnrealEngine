// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Misc/Optional.h"
#include "Engine/World.h"
#include "WorldPartition.h"
#include "WorldPartition/ActorDescList.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"
#include "WorldPartition/WorldPartitionStreamingGenerationContext.h"
#include "WorldPartition/WorldPartitionRuntimeContainerResolving.h"
#include "WorldPartition/DataLayer/DataLayerInstanceProviderInterface.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#if WITH_EDITOR
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "CookPackageSplitter.h"
#include "Misc/HierarchicalLogArchive.h"
#endif
#include "WorldPartitionRuntimeHash.generated.h"

struct FHierarchicalLogArchive;
struct FDataLayerInstanceNames;
class FWorldPartitionDraw2DContext;
class UExternalDataLayerAsset;
class UExternalDataLayerInstance;

extern ENGINE_API float GSlowStreamingRatio;
extern ENGINE_API float GSlowStreamingWarningFactor;

extern ENGINE_API float GBlockOnSlowStreamingRatio;
extern ENGINE_API float GBlockOnSlowStreamingWarningFactor;

UENUM()
enum class EWorldPartitionStreamingPerformance : uint8
{
	Good,
	Slow,
	Critical,
	Immediate,
};
ENGINE_API const TCHAR* EnumToString(EWorldPartitionStreamingPerformance InState);

USTRUCT()
struct FWorldPartitionRuntimeCellStreamingData
{
	GENERATED_BODY()

	UPROPERTY()
	FString PackageName;

	UPROPERTY()
	FSoftObjectPath WorldAsset;
};

UCLASS(Abstract, MinimalAPI)
class URuntimeHashExternalStreamingObjectBase : public UObject, public IWorldPartitionCookPackageObject, public IDataLayerInstanceProvider
{
	GENERATED_BODY()

	friend class UWorldPartitionRuntimeHash;

public:
	//~ Begin UObject Interface
	virtual class UWorld* GetWorld() const override final { return GetOwningWorld(); }
#if DO_CHECK
	virtual void BeginDestroy() override;
#endif
	//~ End UObject Interface

	UWorld* GetOwningWorld() const;
	UWorld* GetOuterWorld() const { return OuterWorld.Get(); }

	ENGINE_API void ForEachStreamingCells(TFunctionRef<void(UWorldPartitionRuntimeCell&)> Func);
	
	ENGINE_API void OnStreamingObjectLoaded(UWorld* InjectedWorld);

	// ~Being IDataLayerInstanceProvider
	ENGINE_API virtual TSet<TObjectPtr<UDataLayerInstance>>& GetDataLayerInstances() override;
	virtual const TSet<TObjectPtr<UDataLayerInstance>>& GetDataLayerInstances() const override { return const_cast<URuntimeHashExternalStreamingObjectBase*>(this)->GetDataLayerInstances(); }
	virtual const UExternalDataLayerInstance* GetRootExternalDataLayerInstance() const override { return RootExternalDataLayerInstance; }
	// ~End IDataLayerInstanceProvider
	UExternalDataLayerInstance* GetRootExternalDataLayerInstance() { return const_cast<UExternalDataLayerInstance*>(RootExternalDataLayerInstance.Get()); }
	ENGINE_API const UObject* GetLevelMountPointContextObject() const;

#if WITH_EDITOR
	//~Begin IWorldPartitionCookPackageObject interface
	virtual bool IsLevelPackage() const override { return false; }
	virtual const UExternalDataLayerAsset* GetExternalDataLayerAsset() const override { return ExternalDataLayerAsset; }
	ENGINE_API virtual FString GetPackageNameToCreate() const override;
	virtual bool OnPrepareGeneratorPackageForCook(TArray<UPackage*>& OutModifiedPackages) override { return true; }
	ENGINE_API virtual bool OnPopulateGeneratorPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InPackage) override;
	ENGINE_API virtual bool OnPopulateGeneratedPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InPackage, TArray<UPackage*>& OutModifiedPackages) override;
	ENGINE_API virtual FWorldPartitionPackageHash GetGenerationHash() const override;
	//~End IWorldPartitionCookPackageObject interface

	const static TCHAR* GetCookedExternalStreamingObjectName() { return TEXT("RuntimeHashExternalStreamingObjectBase"); }
	ENGINE_API const FString& GetPackagePathToCreate() const;
	ENGINE_API void SetPackagePathToCreate(const FString& InPath);

protected:
	virtual void DumpStateLog(FHierarchicalLogArchive& Ar);
	UWorldPartitionRuntimeCell* GetCellForCookPackage(const FString& InCookPackageName) const;
	bool PrepareForCook(const IWorldPartitionCookPackageContext& InCookContext);
#endif

public:
	UPROPERTY();
	TMap<FName, FName> SubObjectsToCellRemapping;

	UPROPERTY()
	FWorldPartitionRuntimeContainerResolver ContainerResolver;

protected:
	TOptional<TWeakObjectPtr<UWorld>> OwningWorld;

	UPROPERTY();
	TSoftObjectPtr<UWorld> OuterWorld;

	UPROPERTY();
	TMap<FName, FWorldPartitionRuntimeCellStreamingData> CellToStreamingData;

	UPROPERTY()
	TSet<TObjectPtr<UDataLayerInstance>> DataLayerInstances;

	UPROPERTY()
	TObjectPtr<const UExternalDataLayerInstance> RootExternalDataLayerInstance;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UWorldPartitionRuntimeCell>> PackagesToGenerateForCook;

	UPROPERTY(Transient)
	TObjectPtr<const UExternalDataLayerAsset> ExternalDataLayerAsset;
#endif

#if DO_CHECK
	TWeakObjectPtr<UWorldPartition> TargetInjectedWorldPartition;
#endif

#if WITH_EDITOR
	FString PackagePathToCreate;
#endif

	friend class UWorldPartition;
	friend class UExternalDataLayerManager;
};

struct FWorldPartitionQueryCache
{
public:
	void AddCellInfo(const UWorldPartitionRuntimeCell* Cell, const FSphericalSector& SourceShape);
	double GetCellMinSquareDist(const UWorldPartitionRuntimeCell* Cell) const;

private:
	TMap<const UWorldPartitionRuntimeCell*, double> CellToSourceMinSqrDistances;
};

struct FWorldPartitionStreamingContext
{
public:
	static ENGINE_API FWorldPartitionStreamingContext Create(const UWorld* InWorld);
	ENGINE_API FWorldPartitionStreamingContext();
	bool IsValid() const { return bIsValid; }
	ENGINE_API EDataLayerRuntimeState GetDataLayerEffectiveRuntimeState(FName DataLayerName) const;
#if WITH_EDITOR
	const FDynamicLoadingRangeScales& GetDynamicLoadingRangeScales() const { return DynamicLoadingRangeScales; }
#endif // WITH_EDITOR

private:
	ENGINE_API FWorldPartitionStreamingContext(const UWorld* InWorld);
	ENGINE_API FWorldPartitionStreamingContext(EWorldPartitionDataLayersLogicOperator InDataLayersLogicOperator, const FWorldDataLayersEffectiveStates& InDataLayerEffectiveStates, int32 InUpdateStreamingStateEpoch
#if WITH_EDITOR
		, const FDynamicLoadingRangeScales& InDynamicLoadingRangeScales = {}
#endif // WITH_EDITOR
	);

	int32 GetResolvingDataLayersRuntimeStateEpoch() const { check(IsValid()); return DataLayerEffectiveStates->GetUpdateEpoch(); }
	int32 GetUpdateStreamingStateEpoch() const { check(IsValid()); return UpdateStreamingStateEpoch; }
	ENGINE_API EDataLayerRuntimeState ResolveDataLayerRuntimeState(const FDataLayerInstanceNames& InDataLayerNames) const;

	bool bIsValid;
	EWorldPartitionDataLayersLogicOperator DataLayersLogicOperator;
	const FWorldDataLayersEffectiveStates* DataLayerEffectiveStates;
	int32 UpdateStreamingStateEpoch;
#if WITH_EDITOR
	FDynamicLoadingRangeScales DynamicLoadingRangeScales;
#endif // WITH_EDITOR

	friend class UWorldPartitionStreamingPolicy;
	friend class UWorldPartitionRuntimeCell;
	friend class UWorldPartitionRuntimeCellData;
};

#if WITH_EDITOR
struct FFixedGridInfo
{
	uint32 CellSize = 0;
	FVector Origin = FVector::ZeroVector;
	bool bIs2D = false;
};
#endif

UCLASS(Abstract, Config=Engine, AutoExpandCategories=(WorldPartition), Within=WorldPartition, MinimalAPI)
class UWorldPartitionRuntimeHash : public UObject
{
	GENERATED_UCLASS_BODY()

	friend class UWorldPartition;
	friend class URuntimePartition;
	friend class UWorldPartitionStreamingPolicy;

#if WITH_EDITOR
	virtual void SetDefaultValues() {}
	virtual bool SupportsHLODs() const { return false; }
	ENGINE_API virtual TArray<UWorldPartitionRuntimeCell*> GetAlwaysLoadedCells() const;
	ENGINE_API virtual bool GenerateStreaming(class UWorldPartitionStreamingPolicy* StreamingPolicy, const IStreamingGenerationContext* StreamingGenerationContext, TArray<FString>* OutPackagesToGenerate);
	virtual bool SetupHLODActors(const IStreamingGenerationContext* StreamingGenerationContext, const UWorldPartition::FSetupHLODActorsParams& Params) const { return false; }
	virtual FName GetDefaultGrid() const { return NAME_None; }
	virtual bool IsValidGrid(FName GridName, const UClass* ActorClass) const { return false; }
	virtual bool IsValidHLODLayer(FName GridName, const FSoftObjectPath& HLODLayerPath) const { return false; }
	virtual TOptional<FFixedGridInfo> GetFixedGridInfo(FName InGridName) const { return {}; }
	virtual void ForEachHLODLayer(TFunctionRef<bool(FName, FName, int32)> Func) const {}; // RuntimePartitionName, HLODSetupName, HLODSetupIndex
	virtual int32 ComputeHLODHierarchyDepth() const { return 0; }
	virtual FName ResolveRuntimeGridNameForHLODLayer(FName RuntimeGrid, const FSoftObjectPath& HLODLayerPath) const { return NAME_None; }
	virtual bool CopyGridSetupFrom(const UWorldPartitionRuntimeHash* Other) { return false; }
	virtual void DrawPreview() const {}

	virtual bool HasStreamingContent() const { return false; }
	ENGINE_API URuntimeHashExternalStreamingObjectBase* StoreStreamingContentToExternalStreamingObject();

	ENGINE_API virtual void FlushStreamingContent();
	virtual TSubclassOf<URuntimeHashExternalStreamingObjectBase> GetExternalStreamingObjectClass() const PURE_VIRTUAL(UWorldPartitionRuntimeHash::GetExternalStreamingObjectClass, return nullptr;);
	ENGINE_API virtual void DumpStateLog(FHierarchicalLogArchive& Ar) const;
#endif

	virtual void OnBeginPlay() {}

#if WITH_EDITOR
	// PIE/Game methods
	ENGINE_API virtual void PrepareEditorGameWorld();
	ENGINE_API virtual void ShutdownEditorGameWorld();

protected:
	ENGINE_API virtual void StoreStreamingContentToExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* OutExternalStreamingObject);
	ENGINE_API bool PopulateCellActorInstances(const TArray<const IStreamingGenerationContext::FActorSetInstance*>& ActorSetInstances, bool bIsMainWorldPartition, bool bIsCellAlwaysLoaded, TArray<IStreamingGenerationContext::FActorInstance>& OutCellActorInstances);
	ENGINE_API void PopulateRuntimeCell(UWorldPartitionRuntimeCell* RuntimeCell, const TArray<IStreamingGenerationContext::FActorInstance>& ActorInstances, TArray<FString>* OutPackagesToGenerate);
#endif

public:
	class FStreamingSourceCells
	{
	public:
		void AddCell(const UWorldPartitionRuntimeCell* Cell, const FWorldPartitionStreamingSource& Source, const FSphericalSector& SourceShape, const FWorldPartitionStreamingContext& Context);
		void Reset() { Cells.Reset(); }
		int32 Num() const { return Cells.Num(); }
		TSet<const UWorldPartitionRuntimeCell*>& GetCells() { return Cells; }

	private:
		TSet<const UWorldPartitionRuntimeCell*> Cells;
	};

	// Streaming interface
	virtual void ForEachStreamingCells(TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func) const {}
	virtual void ForEachStreamingCellsQuery(const FWorldPartitionStreamingQuerySource& QuerySource, TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func, FWorldPartitionQueryCache* QueryCache = nullptr) const {}
	virtual void ForEachStreamingCellsSources(const TArray<FWorldPartitionStreamingSource>& Sources, TFunctionRef<bool(const UWorldPartitionRuntimeCell*, EStreamingSourceTargetState)> Func, const FWorldPartitionStreamingContext& Context = FWorldPartitionStreamingContext()) const {}
	// Computes a hash value of all runtime hash specific dependencies that affects the update of the streaming
	virtual uint32 ComputeUpdateStreamingHash() const { return 0; }

	ENGINE_API bool IsCellRelevantFor(bool bClientOnlyVisible) const;

	ENGINE_API EWorldPartitionStreamingPerformance GetStreamingPerformance(const TSet<const UWorldPartitionRuntimeCell*>& CellsToActivate, bool& bOutShouldBlock) const;	

	ENGINE_API virtual bool IsExternalStreamingObjectInjected(URuntimeHashExternalStreamingObjectBase* InExternalStreamingObject) const;
	ENGINE_API virtual bool InjectExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject);
	ENGINE_API virtual bool RemoveExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject);

	virtual bool Draw2D(FWorldPartitionDraw2DContext& DrawContext) const { return false; }
	virtual void Draw3D(const TArray<FWorldPartitionStreamingSource>& Sources) const {}
	virtual bool ContainsRuntimeHash(const FString& Name) const { return false; }
	virtual bool IsStreaming3D() const { return true; }
	virtual bool GetShouldMergeStreamingSourceInfo() const { return false; }
	virtual const FGuid* GetStandaloneHLODActorSourceCellOverride(const FGuid& InActorGuid) const { return nullptr; }
	virtual const FGuid* GetCustomHLODActorSourceCellOverride(const FGuid& InActorGuid) const { return nullptr; }

	static ENGINE_API URuntimeHashExternalStreamingObjectBase* CreateExternalStreamingObject(TSubclassOf<URuntimeHashExternalStreamingObjectBase> InClass, UObject* InOuter, UWorld* InOuterWorld);
	ENGINE_API UWorldPartitionRuntimeCell* CreateRuntimeCell(UClass* CellClass, UClass* CellDataClass, const FString& CellName, const FString& CellInstanceSuffix, UObject* InOuter = nullptr);

protected:
	virtual EWorldPartitionStreamingPerformance GetStreamingPerformanceForCell(const UWorldPartitionRuntimeCell* Cell, bool& bOutShouldBlock) const;

#if WITH_EDITORONLY_DATA
	struct FEditorAlwaysLoadedActor
	{
		FEditorAlwaysLoadedActor(const FWorldPartitionReference& InReference, AActor* InActor)
			: Reference(InReference)
			, Actor(InActor)
		{}

		FWorldPartitionReference Reference;
		TWeakObjectPtr<AActor> Actor;
	};

	TArray<FEditorAlwaysLoadedActor> EditorAlwaysLoadedActor;

	TMap<FString, TObjectPtr<UWorldPartitionRuntimeCell>> PackagesToGenerateForCook;
#endif

protected:
#if WITH_EDITOR
	ENGINE_API void ForceExternalActorLevelReference(bool bForceExternalActorLevelReference);
	ENGINE_API bool ResolveBlockOnSlowStreamingForCell(bool bInOwnerBlockOnSlowStreaming, bool bInIsHLODCell, const TArray<const UDataLayerInstance*>& InCellDataLayerInstances) const;
	ENGINE_API int32 GetDataLayersStreamingPriority(const TArray<const UDataLayerInstance*>& InCellDataLayerInstances) const;
#endif

	TSet<TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase>> InjectedExternalStreamingObjects;

	// Variable set in scoped guard to skip non spatials cells in repeated queries
	mutable bool bQueryNonSpatialCells = true;

	virtual bool SupportsWorldAssetStreaming(const FName& InTargetGrid) { return false; }
	virtual FGuid RegisterWorldAssetStreaming(const UWorldPartition::FRegisterWorldAssetStreamingParams& InParams) { return FGuid(); }
	virtual bool UnregisterWorldAssetStreaming(const FGuid& InWorldAssetStreamingGuid) { return false; }
	virtual TArray<UWorldPartitionRuntimeCell*> GetWorldAssetStreamingCells(const FGuid& InWorldAssetStreamingGuid) { return {}; }

#if WITH_EDITOR
private:
	UWorldPartitionRuntimeCell* GetCellForCookPackage(const FString& InCookPackageName) const;
	friend class UWorldPartition;

	using FRuntimeHashConvertFunc = TFunction<UWorldPartitionRuntimeHash*(const UWorldPartitionRuntimeHash*)>;
	static TMap<TPair<const UClass*, const UClass*>, FRuntimeHashConvertFunc> WorldPartitionRuntimeHashConverters;

public:
	static ENGINE_API void RegisterWorldPartitionRuntimeHashConverter(const UClass* InSrcClass, const UClass* InDstClass, FRuntimeHashConvertFunc&& InConverter);
	static ENGINE_API UWorldPartitionRuntimeHash* ConvertWorldPartitionHash(const UWorldPartitionRuntimeHash* InSrcHash, const UClass* InDstClass);

	static ENGINE_API void ExecutePreSetupHLODActors(const UWorldPartition* InWorldPartition, const UWorldPartition::FSetupHLODActorsParams& InParams);
	static ENGINE_API void ExecutePostSetupHLODActors(const UWorldPartition* InWorldPartition, const UWorldPartition::FSetupHLODActorsParams& InParams);
	virtual void PreSetupHLODActors(const UWorldPartition* InWorldPartition, const UWorldPartition::FSetupHLODActorsParams& InParams) const {}
	virtual void PostSetupHLODActors(const UWorldPartition* InWorldPartition, const UWorldPartition::FSetupHLODActorsParams& InParams) const {}
#endif
};
