// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/WorldPartitionRuntimeCellData.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#if WITH_EDITOR
#include "WorldPartition/WorldPartitionLevelHelper.h"
#endif
#include "WorldPartitionRuntimeLevelStreamingCell.generated.h"

class UWorld;
class FStreamingGenerationActorDescView;

UCLASS(MinimalAPI)
class UWorldPartitionRuntimeLevelStreamingCell : public UWorldPartitionRuntimeCell
{
	GENERATED_UCLASS_BODY()

	//~Begin UWorldPartitionRuntimeCell interface
	ENGINE_API virtual void Load() const override;
	ENGINE_API virtual void Unload() const override;
	ENGINE_API virtual bool CanUnload() const override;
	ENGINE_API virtual void Activate() const override;
	ENGINE_API virtual void Deactivate() const override;
	ENGINE_API virtual ULevel* GetLevel() const override;
	ENGINE_API virtual EWorldPartitionRuntimeCellState GetCurrentState() const override;
	ENGINE_API virtual FLinearColor GetDebugColor(EWorldPartitionRuntimeCellVisualizeMode VisualizeMode) const override;
	ENGINE_API virtual void SetIsAlwaysLoaded(bool bInIsAlwaysLoaded) override;
	ENGINE_API virtual EStreamingStatus GetStreamingStatus() const override;
	ENGINE_API virtual bool IsVisible() const override;
	//~End UWorldPartitionRuntimeCell interface

	//~Begin IWorldPartitionCell interface
	ENGINE_API FName GetLevelPackageName() const override;
	//~End IWorldPartitionCell interface

	ENGINE_API virtual void SetStreamingPriority(int32 InStreamingPriority) const override;
	ENGINE_API class UWorldPartitionLevelStreamingDynamic* GetLevelStreaming() const;

	ENGINE_API bool HasActors() const;
	ENGINE_API virtual TArray<FName> GetActors() const override;

	ENGINE_API void CreateAndSetLevelStreaming(const FString& InPackageName, const FSoftObjectPath& InWorldAsset = FSoftObjectPath());
	ENGINE_API bool CreateAndSetLevelStreaming(const TSoftObjectPtr<UWorld>& InWorldAsset, const FTransform& InInstanceTransform) const;
	ENGINE_API class UWorldPartitionLevelStreamingDynamic* CreateLevelStreaming(const FString& InPackageName = FString(), const FSoftObjectPath& InWorldAsset = FSoftObjectPath()) const;
	

#if WITH_EDITOR
	//~Begin UWorldPartitionRuntimeCell interface
	ENGINE_API virtual void AddActorToCell(const FStreamingGenerationActorDescView& ActorDescView) override;
	ENGINE_API virtual void Fixup() override;
	ENGINE_API virtual int32 GetActorCount() const override;
	ENGINE_API virtual void DumpStateLog(FHierarchicalLogArchive& Ar) const override;
	ENGINE_API virtual bool PrepareCellForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InGeneratedPackage) override;
	ENGINE_API virtual FName GetCookPackageName() const override;
	//~End UWorldPartitionRuntimeCell interface

	virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;

	//~Begin IWorldPartitionCookPackageObject interface
	ENGINE_API virtual FString GetPackageNameToCreate() const override;
	ENGINE_API virtual bool OnPrepareGeneratorPackageForCook(TArray<UPackage*>& OutModifiedPackages) override;
	ENGINE_API virtual bool OnPopulateGeneratorPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InPackage) override;
	ENGINE_API virtual bool OnPopulateGeneratedPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InPackage, TArray<UPackage*>& OutModifiedPackages) override;
	ENGINE_API virtual FWorldPartitionPackageHash GetGenerationHash() const override;
	//~End IWorldPartitionCookPackageObject interface

	//~Begin IWorldPartitionCell Interface
	ENGINE_API virtual TSet<FName> GetActorPackageNames() const override;
	//~End IWorldPartitionCell Interface

	const TArray<FWorldPartitionRuntimeCellObjectMapping>& GetPackages() const { return Packages; }
#endif

protected:
	// Called when cell is shown
	ENGINE_API virtual void OnCellShown() const;
	// Called when cell is hidden
	ENGINE_API virtual void OnCellHidden() const;

private:
	UFUNCTION()
	ENGINE_API void OnLevelShown();
	
	UFUNCTION()
	ENGINE_API void OnLevelHidden();

	ENGINE_API class UWorldPartitionLevelStreamingDynamic* GetOrCreateLevelStreaming() const;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FWorldPartitionRuntimeCellObjectMapping> Packages;

	TSet<FName> ActorContainerPackageDependencies;
#endif

	UPROPERTY()
	mutable TObjectPtr<class UWorldPartitionLevelStreamingDynamic> LevelStreaming;
};
