// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/HierarchicalLogArchive.h"
#include "Math/MathFwd.h"
#include "GameFramework/Info.h"

#include "WorldPartition.h"
#include "WorldPartition/DataLayer/DataLayersID.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartitionRuntimeHash.h"
#include "WorldPartitionRuntimeSpatialHashGridPreviewer.h"
#include "WorldPartitionRuntimeSpatialHash.generated.h"

typedef FInt64Vector3 FGridCellCoord;
typedef FInt64Vector2 FGridCellCoord2;
class FWorldPartitionDraw2DContext;

namespace UE::Private::WorldPartition
{
	struct FStreamingDescriptor;
};

UENUM()
enum class EWorldPartitionCVarProjectDefaultOverride : uint8
{
	ProjectDefault,
	Disabled,
	Enabled,
};

USTRUCT()
struct FSpatialHashSettings
{
	GENERATED_USTRUCT_BODY()

	FSpatialHashSettings()
		: bUseAlignedGridLevels(true)
		, bSnapNonAlignedGridLevelsToLowerLevels(true)
#if WITH_EDITORONLY_DATA
		, bPlaceSmallActorsUsingLocation(false)
		, bPlacePartitionActorsUsingLocation(true)
#endif
	{}

	friend bool operator==(const FSpatialHashSettings& Lhs, const FSpatialHashSettings& Rhs) = default;
	friend bool operator!=(const FSpatialHashSettings& Lhs, const FSpatialHashSettings& Rhs) = default;

	void UpdateSettings(const UWorldPartitionRuntimeSpatialHash& RuntimeSpatialHash);
	
	UPROPERTY();
	bool bUseAlignedGridLevels;

	UPROPERTY();
	bool bSnapNonAlignedGridLevelsToLowerLevels;

#if WITH_EDITORONLY_DATA
	UPROPERTY();
	bool bPlaceSmallActorsUsingLocation;

	UPROPERTY();
	bool bPlacePartitionActorsUsingLocation;
#endif
};

USTRUCT()
struct FSpatialHashStreamingGridLayerCell
{
	GENERATED_USTRUCT_BODY()

	FSpatialHashStreamingGridLayerCell() = default;

	// Move constructor.
	FSpatialHashStreamingGridLayerCell(const FSpatialHashStreamingGridLayerCell& Other) = delete;
	FSpatialHashStreamingGridLayerCell(FSpatialHashStreamingGridLayerCell&& Other) = default;

	// Move assignment.
	FSpatialHashStreamingGridLayerCell& operator=(const FSpatialHashStreamingGridLayerCell& Other) = delete;
	FSpatialHashStreamingGridLayerCell& operator=(FSpatialHashStreamingGridLayerCell&& Other) = default;

	UPROPERTY()
	TArray<TObjectPtr<UWorldPartitionRuntimeCell>> GridCells;
};

// Disable copy of this USTRUCT by the autogenerated code.
template<> struct TStructOpsTypeTraits<FSpatialHashStreamingGridLayerCell> : public TStructOpsTypeTraitsBase2<FSpatialHashStreamingGridLayerCell> { enum { WithCopy = false }; };

/**
 * Represents a PIE/Game streaming grid level
 */
USTRUCT()
struct FSpatialHashStreamingGridLevel
{
	GENERATED_USTRUCT_BODY()

	FSpatialHashStreamingGridLevel() = default;

	// Move constructor.
	FSpatialHashStreamingGridLevel(const FSpatialHashStreamingGridLevel& Other) = delete;
	FSpatialHashStreamingGridLevel(FSpatialHashStreamingGridLevel&& Other) = default;

	// Move assignment.
	FSpatialHashStreamingGridLevel& operator=(const FSpatialHashStreamingGridLevel& Other) = delete;
	FSpatialHashStreamingGridLevel& operator=(FSpatialHashStreamingGridLevel&& Other) = default;

	/** 
	 * Represents the grid cells
	 */
	UPROPERTY()
	TArray<FSpatialHashStreamingGridLayerCell> LayerCells;

	/** 
	 * Represents the grid cells index mapping
	 */
	UPROPERTY()
	TMap<int64, int32> LayerCellsMapping;
};

// Disable copy of this USTRUCT by the autogenerated code.
template<> struct TStructOpsTypeTraits<FSpatialHashStreamingGridLevel> : public TStructOpsTypeTraitsBase2<FSpatialHashStreamingGridLevel> { enum { WithCopy = false }; };

struct FSquare2DGridHelper;

/**
 * Represents a PIE/Game streaming grid
 */
USTRUCT()
struct FSpatialHashStreamingGrid
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FSpatialHashStreamingGrid();
	ENGINE_API ~FSpatialHashStreamingGrid();

	// Move constructor.
	FSpatialHashStreamingGrid(const FSpatialHashStreamingGrid& Other) = delete;
	FSpatialHashStreamingGrid(FSpatialHashStreamingGrid&& Other) = default;

	// Move assignment.
	FSpatialHashStreamingGrid& operator=(const FSpatialHashStreamingGrid& Other) = delete;
	FSpatialHashStreamingGrid& operator=(FSpatialHashStreamingGrid&& Other) = default;

	UPROPERTY()
	FName GridName;

	UPROPERTY()
	FVector Origin;

	UPROPERTY()
	int32 CellSize;

	UPROPERTY()
	float LoadingRange;

	UPROPERTY()
	bool bBlockOnSlowStreaming;

	UPROPERTY()
	FLinearColor DebugColor;

	UPROPERTY()
	TArray<FSpatialHashStreamingGridLevel> GridLevels;

	UPROPERTY()
	FBox WorldBounds;

	UPROPERTY()
	bool bClientOnlyVisible;

	UPROPERTY()
	int32 GridIndex;
		
	UPROPERTY()
	FSpatialHashSettings Settings;
			
	ENGINE_API void InjectExternalStreamingObjectGrid(const FSpatialHashStreamingGrid& InExternalObjectStreamingGrid) const;
	ENGINE_API void RemoveExternalStreamingObjectGrid(const FSpatialHashStreamingGrid& InExternalObjectStreamingGrid) const;

	ENGINE_API bool InsertGridCell(UWorldPartitionRuntimeCell* InGridCell, const FGridCellCoord& InGridCellCoords);

	// Used by PIE/Game
	ENGINE_API int64 GetCellSize(int32 Level) const;
	ENGINE_API void GetCells(const FWorldPartitionStreamingQuerySource& QuerySource, TSet<const UWorldPartitionRuntimeCell*>& OutCells, bool bEnableZCulling, FWorldPartitionQueryCache* QueryCache = nullptr, bool bQueryNonSpatialCells = true) const;
	ENGINE_API void GetCells(const TArray<FWorldPartitionStreamingSource>& Sources, UWorldPartitionRuntimeHash::FStreamingSourceCells& OutActivateCells, UWorldPartitionRuntimeHash::FStreamingSourceCells& OutLoadCells, bool bEnableZCulling, const FWorldPartitionStreamingContext& Context, float LoadingRangeScale = 1.f) const;
	ENGINE_API void GetNonSpatiallyLoadedCells(TSet<const UWorldPartitionRuntimeCell*>& OutActivateCells, TSet<const UWorldPartitionRuntimeCell*>& OutLoadCells, const FWorldPartitionStreamingContext& Context) const;
	ENGINE_API void Draw2D(const class UWorldPartitionRuntimeSpatialHash* Owner, const FBox2D& Region2D, const FBox2D& GridScreenBounds, TFunctionRef<FVector2D(const FVector2D&, bool)> WorldToScreen, FWorldPartitionDraw2DContext& DrawContext) const;
	ENGINE_API void Draw3D(const class UWorldPartitionRuntimeSpatialHash* Owner, const TArray<FWorldPartitionStreamingSource>& Sources, const FTransform& Transform) const;
	ENGINE_API void ForEachRuntimeCell(TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func) const;
	ENGINE_API const FSquare2DGridHelper& GetGridHelper() const;
	ENGINE_API float GetLoadingRange() const;

#if WITH_EDITOR
	void DumpStateLog(FHierarchicalLogArchive& Ar) const;
#endif

private:
	ENGINE_API void ForEachRuntimeCell(const FGridCellCoord& Coords, TFunctionRef<void(const UWorldPartitionRuntimeCell*)> Func) const;
	ENGINE_API void ForEachLayerCell(const FGridCellCoord& Coords, TFunctionRef<void(const FSpatialHashStreamingGridLayerCell*)> Func) const;
	ENGINE_API void DrawStreamingSource2D(const FBox2D& GridScreenBounds, const FSphericalSector& Shape, TFunctionRef<FVector2D(const FVector2D&)> WorldToScreen, const FColor& Color, FWorldPartitionDraw2DContext& DrawContext, float LoadingRangeScale = 1.f) const;
	ENGINE_API void DrawStreamingSource3D(UWorld* World, const FSphericalSector& Shape, const FTransform& Transform, const FColor& Color) const;
	ENGINE_API void GetFilteredCellsForDebugDraw(const FSpatialHashStreamingGridLayerCell* LayerCell, TArray<const UWorldPartitionRuntimeCell*>& FilteredCells) const;
	ENGINE_API EWorldPartitionRuntimeCellVisualizeMode GetStreamingCellVisualizeMode() const;
	mutable FSquare2DGridHelper* GridHelper;

	// Contains cells injected at runtime from content bundles
	UPROPERTY(Transient)
	mutable TArray<FSpatialHashStreamingGridLevel> InjectedGridLevels;

	/** Console command used to change loading range for a given streaming grid */
	static class FAutoConsoleCommand OverrideLoadingRangeCommand;
	static TMap<int32, float> OverriddenLoadingRanges;
	static bool bAddedWorldPartitionSubsystemDeinitializedCallback;
};

// Disable copy of this USTRUCT by the autogenerated code.
template<> struct TStructOpsTypeTraits<FSpatialHashStreamingGrid> : public TStructOpsTypeTraitsBase2<FSpatialHashStreamingGrid> { enum { WithCopy = false }; };

/**
 * Represents a runtime grid (editing)
 */
USTRUCT()
struct FSpatialHashRuntimeGrid
{
	GENERATED_USTRUCT_BODY()

	FSpatialHashRuntimeGrid()
#if WITH_EDITORONLY_DATA
		: CellSize(12800)
		, LoadingRange(25600)
		, bBlockOnSlowStreaming(false)
		, Origin(FVector2D::ZeroVector)
		, Priority(0)
		, DebugColor(FLinearColor::MakeRandomColor())
		, bClientOnlyVisible(false)
		, HLODLayer(nullptr)
#endif
	{}

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Settings)
	FName GridName;

	UPROPERTY(EditAnywhere, Category=Settings)
	int32 CellSize;

	UPROPERTY(EditAnywhere, Category=Settings)
	float LoadingRange;

	/** Should streaming block in situations where cells aren't getting loaded fast enough. */
	UPROPERTY(EditAnywhere, Category=Settings)
	bool bBlockOnSlowStreaming;

	UPROPERTY(EditAnywhere, Category=Settings)
	FVector2D Origin;

	UPROPERTY(EditAnywhere, Category=Settings)
	int32 Priority;

	UPROPERTY(EditAnywhere, Category=Settings, meta = (IgnoreForMemberInitializationTest))
	FLinearColor DebugColor;

	UPROPERTY()
	bool bClientOnlyVisible;

	UPROPERTY()
	TObjectPtr<const UHLODLayer> HLODLayer;

	bool operator == (const FSpatialHashRuntimeGrid& Other) const;
	bool operator != (const FSpatialHashRuntimeGrid& Other) const;
#endif
};

/**
 * Actor keeping information regarding a runtime grid
 */
UCLASS(NotPlaceable)
class ASpatialHashRuntimeGridInfo : public AInfo
{
	GENERATED_UCLASS_BODY()

	virtual void Serialize(FArchive& Ar) override;

public:
	UPROPERTY()
	FSpatialHashRuntimeGrid	GridSettings;
};

UCLASS(MinimalAPI)
class URuntimeSpatialHashExternalStreamingObject : public URuntimeHashExternalStreamingObjectBase
{
	GENERATED_BODY()

#if WITH_EDITOR
	virtual void DumpStateLog(FHierarchicalLogArchive& Ar) override;
#endif

public:
	UPROPERTY();
	TArray<FSpatialHashStreamingGrid> StreamingGrids;
};

UCLASS(MinimalAPI)
class UWorldPartitionRuntimeSpatialHash : public UWorldPartitionRuntimeHash
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	ENGINE_API virtual void SetDefaultValues() override;
	virtual bool SupportsHLODs() const override { return true; }
	ENGINE_API virtual void PreSetupHLODActors(const UWorldPartition* InWorldPartition, const UWorldPartition::FSetupHLODActorsParams& InParams) const override;
	ENGINE_API virtual void PostSetupHLODActors(const UWorldPartition* InWorldPartition, const UWorldPartition::FSetupHLODActorsParams& InParams) const override;
	ENGINE_API virtual bool SetupHLODActors(const IStreamingGenerationContext* StreamingGenerationContext, const UWorldPartition::FSetupHLODActorsParams& Params) const override;
	ENGINE_API virtual FName GetDefaultGrid() const override;
	ENGINE_API virtual bool IsValidGrid(FName GridName, const UClass* ActorClass) const override;
	virtual bool IsValidHLODLayer(FName GridName, const FSoftObjectPath& HLODLayerPath) const override { return true; }
	ENGINE_API virtual TOptional<FFixedGridInfo> GetFixedGridInfo(FName InGridName) const override;
	ENGINE_API virtual void DrawPreview() const override;

	virtual TSubclassOf<URuntimeHashExternalStreamingObjectBase> GetExternalStreamingObjectClass() const override { return URuntimeSpatialHashExternalStreamingObject::StaticClass(); }

	static ENGINE_API FString GetHLODGridName(uint32 InLODLevel, int32 InCellSize, double InLoadingRange);
	static ENGINE_API FString GetCellNameString(UWorld* InOuterWorld, FName InGridName, const FGridCellCoord& InCellGlobalCoord, const FDataLayersID& InDataLayerID, const FGuid& InContentBundleID, FString* OutInstanceSuffix = nullptr);
	static ENGINE_API FGuid GetCellGuid(FName InGridName, int32 InCellSize, const FGridCellCoord& InCellGlobalCoord, const FDataLayersID& InDataLayerID, const FGuid& InContentBundleID);

	ENGINE_API bool GetPreviewGrids() const;
	ENGINE_API void SetPreviewGrids(bool bInPreviewGrids);

	ENGINE_API int32 GetPreviewGridLevel() const;
	ENGINE_API void SetPreviewGridLevel(int32 InPreviewGridLevel);

	ENGINE_API bool GetUseAlignedGridLevels() const;
#endif

	// streaming interface
	ENGINE_API virtual void ForEachStreamingCells(TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func) const override;
	ENGINE_API virtual void ForEachStreamingCellsQuery(const FWorldPartitionStreamingQuerySource& QuerySource, TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func, FWorldPartitionQueryCache* QueryCache = nullptr) const override;
	ENGINE_API virtual void ForEachStreamingCellsSources(const TArray<FWorldPartitionStreamingSource>& Sources, TFunctionRef<bool(const UWorldPartitionRuntimeCell*, EStreamingSourceTargetState)> Func, const FWorldPartitionStreamingContext& Context = FWorldPartitionStreamingContext()) const override;
	ENGINE_API virtual uint32 ComputeUpdateStreamingHash() const override;

	ENGINE_API virtual bool InjectExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject) override;
	ENGINE_API virtual bool RemoveExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject) override;

	ENGINE_API uint32 GetNumGrids() const;

	static ENGINE_API FString GetCellCoordString(const FGridCellCoord& InCellGlobalCoord);

	ENGINE_API void ForEachStreamingGrid(TFunctionRef<void(const FSpatialHashStreamingGrid&)> Func) const;

	ENGINE_API const FSpatialHashStreamingGrid* GetStreamingGridByName(FName InGridName) const;

protected:
	ENGINE_API void ForEachStreamingGrid(TFunctionRef<void(FSpatialHashStreamingGrid&)> Func);
	ENGINE_API void ForEachStreamingGridBreakable(TFunctionRef<bool(const FSpatialHashStreamingGrid&)> Func) const;
	
	ENGINE_API virtual EWorldPartitionStreamingPerformance GetStreamingPerformanceForCell(const UWorldPartitionRuntimeCell* Cell, bool& bOutShouldBlock) const override;

#if WITH_EDITOR
	ENGINE_API virtual bool HasStreamingContent() const override;
	ENGINE_API virtual void StoreStreamingContentToExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* OutExternalStreamingObject) override;
	ENGINE_API virtual void FlushStreamingContent() override;
	ENGINE_API virtual bool GenerateStreaming(class UWorldPartitionStreamingPolicy* StreamingPolicy, const IStreamingGenerationContext* StreamingGenerationContext, TArray<FString>* OutPackagesToGenerate = nullptr) override;
	ENGINE_API virtual void DumpStateLog(FHierarchicalLogArchive& Ar) const override;
#endif

private:
#if WITH_EDITOR
	static ENGINE_API UWorldPartitionRuntimeSpatialHash* CreateFrom(const UWorldPartitionRuntimeHash* SrcHash);
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Config, Category = RuntimeSettings)
	TArray<FSpatialHashRuntimeGrid> Grids;

	/** Whether to preview runtime grids. */
	UPROPERTY(Transient)
	bool bPreviewGrids;

	UPROPERTY(Transient)
	int32 PreviewGridLevel;

	UPROPERTY(Transient)
	mutable FWorldPartitionRuntimeSpatialHashGridPreviewer GridPreviewer;
#endif

#if WITH_EDITORONLY_DATA
	/** Disable to help break the pattern caused by world partition promotion of actors to upper grid levels that are always aligned on child levels. */
	UPROPERTY(EditAnywhere, Config, AdvancedDisplay, Category = RuntimeSettings)
	EWorldPartitionCVarProjectDefaultOverride UseAlignedGridLevels;

	/** Disable to avoid snapping higher levels cells to child cells. Only used when bUseAlignedGridLevels is false. */
	UPROPERTY(EditAnywhere, Config, AdvancedDisplay, Category = RuntimeSettings)
	EWorldPartitionCVarProjectDefaultOverride SnapNonAlignedGridLevelsToLowerLevels;

	/** Enable to place actors smaller than a cell size into their corresponding cell using their location instead of their bounding box. */
	UPROPERTY(EditAnywhere, Config, AdvancedDisplay, Category = RuntimeSettings)
	EWorldPartitionCVarProjectDefaultOverride PlaceSmallActorsUsingLocation;

	/** Enable to place partitioned actors into their corresponding cell using their location instead of their bounding box. */
	UPROPERTY(EditAnywhere, Config, AdvancedDisplay, Category = RuntimeSettings)
	EWorldPartitionCVarProjectDefaultOverride PlacePartitionActorsUsingLocation;
#endif

	/** Whether this hash enables Z culling. */
	UPROPERTY(EditAnywhere, Config, Category = RuntimeSettings)
	bool bEnableZCulling;

protected:
	UPROPERTY()
	FSpatialHashSettings Settings;
	
	/** 
	 * Represents the streaming grids (PIE or Game)
	 */
	UPROPERTY(NonPIEDuplicateTransient)
	TArray<FSpatialHashStreamingGrid> StreamingGrids;
	mutable TMap<FName, const FSpatialHashStreamingGrid*> NameToGridMapping;
	mutable bool bIsNameToGridMappingDirty;

	ENGINE_API virtual bool SupportsWorldAssetStreaming(const FName& InTargetGrid) override;
	ENGINE_API virtual FGuid RegisterWorldAssetStreaming(const UWorldPartition::FRegisterWorldAssetStreamingParams& InParams) override;
	ENGINE_API virtual bool UnregisterWorldAssetStreaming(const FGuid& InWorldAssetStreamingGuid) override;
	ENGINE_API virtual TArray<UWorldPartitionRuntimeCell*> GetWorldAssetStreamingCells(const FGuid& InWorldAssetStreamingGuid) override;

	UPROPERTY(Transient)
	TMap<FGuid, TObjectPtr<URuntimeSpatialHashExternalStreamingObject>> WorldAssetStreamingObjects;

private:
	ENGINE_API virtual bool Draw2D(FWorldPartitionDraw2DContext& DrawContext) const override;
	ENGINE_API virtual void Draw3D(const TArray<FWorldPartitionStreamingSource>& Sources) const override;
	ENGINE_API virtual bool ContainsRuntimeHash(const FString& Name) const override;
	ENGINE_API virtual bool IsStreaming3D() const override;
	virtual bool GetShouldMergeStreamingSourceInfo() const override { return true; }

	ENGINE_API void GetAlwaysLoadedStreamingCells(const FSpatialHashStreamingGrid& StreamingGrid, TSet<const UWorldPartitionRuntimeCell*>& Cells) const;
	ENGINE_API const TMap<FName, const FSpatialHashStreamingGrid*>& GetNameToGridMapping() const;
#if WITH_EDITOR
	ENGINE_API bool CreateStreamingGrid(const FSpatialHashRuntimeGrid& RuntimeGrid, const FSquare2DGridHelper& PartitionedActors, UWorldPartitionStreamingPolicy* StreamingPolicy, TArray<FString>* OutPackagesToGenerate = nullptr);
#endif
	ENGINE_API TArray<const FSpatialHashStreamingGrid*> GetFilteredStreamingGrids() const;

	friend class UWorldPartitionSubsystem;
	friend class UWorldPartitionRuntimeHashSet;
	friend struct FSpatialHashSettings;
	friend UE::Private::WorldPartition::FStreamingDescriptor;
};
