// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartitionRuntimeSpatialHashGridPreviewer.generated.h"

class APostProcessVolume;
class UMaterial;
class UMaterialInstanceDynamic;
struct FSpatialHashRuntimeGrid;

USTRUCT()
struct FWorldPartitionRuntimeSpatialHashGridPreviewer
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API FWorldPartitionRuntimeSpatialHashGridPreviewer();

#if WITH_EDITOR
	ENGINE_API void Draw(UWorld* World, const TArray<FSpatialHashRuntimeGrid>& Grids, bool bEnabled, int32 PreviewGridLevel, bool bUseAlignedGridLevels = true);
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UMaterial> Material;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> MID;

	UPROPERTY()
	TObjectPtr<APostProcessVolume> Volume;

	bool bIsInitialized;

	struct FGridParametersCache
	{
		FGridParametersCache()
			: bEnabled(false)
			, CellSize(0)
			, LoadingRange(0.f)
			, GridColor(0.f, 0.f, 0.f, 0.f)
			, GridOffset(FVector::ZeroVector)
		{}

		bool bEnabled;
		int32 CellSize;
		float LoadingRange;
		FLinearColor GridColor;
		FVector GridOffset;
	};

	enum { MAX_PREVIEW_GRIDS = 4 };
	FGridParametersCache GridParameters[MAX_PREVIEW_GRIDS];
#endif
};
