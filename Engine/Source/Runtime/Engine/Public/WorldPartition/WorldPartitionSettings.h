// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DeveloperSettings.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartitionSettings.generated.h"

#define UE_API ENGINE_API

class UWorldPartitionPropertyOverridePolicy;

USTRUCT()
struct FHLODLayerTypeUnsupportedActorClasses
{
	GENERATED_BODY()

	UPROPERTY(Config)
	TArray<TSubclassOf<AActor>> ActorClasses;
};

UCLASS(MinimalAPI, config = Engine, defaultconfig, DisplayName = "World Partition")
class UWorldPartitionSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UWorldPartitionSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	static UWorldPartitionSettings* Get() { return CastChecked<UWorldPartitionSettings>(UWorldPartitionSettings::StaticClass()->GetDefaultObject()); }

	EWorldPartitionDataLayersLogicOperator GetNewMapsDataLayersLogicOperator() const { return NewMapsDataLayersLogicOperator; }
	bool GetNewMapsEnableWorldPartition() const { return bNewMapsEnableWorldPartition; }
	bool GetNewMapsEnableWorldPartitionStreaming() const { return bNewMapsEnableWorldPartitionStreaming; }
	
	TSubclassOf<UWorldPartitionEditorHash> GetEditorHashDefaultClass() const { return EditorHashDefaultClass; }
	TSubclassOf<UWorldPartitionRuntimeHash> GetRuntimeHashDefaultClass() const { return RuntimeHashDefaultClass; }

	UWorldPartitionPropertyOverridePolicy* GetPropertyOverridePolicy() const
	{
		return PropertyOverridePolicy;
	}

	const TMap<EHLODLayerType, FHLODLayerTypeUnsupportedActorClasses>& GetUnsupportedActorClassesPerHLODLayerType() const
	{
		return UnsupportedActorClassesPerHLODLayerType;
	}

#if WITH_EDITOR
	bool ShouldEDLPackagesInheritWorldChunkAssignmentsDuringCook() const { return bShouldEDLPackagesInheritWorldChunkAssignmentsDuringCook;	}

	/**
	 * EDL Filters
	 * Multi product build optimization, where it is advantageous to cook EDLs on other machines for example, and where producing the map more than once is a waste and can lead to duplication in game packages.
	 * If this list is empty, EDLs are not filtered and we have a full cook.
	 * If there are paths in the list however, only these EDLs are cooked, excluding other EDLs not in the list, and most importantly excluding the base map
	 */
	static ENGINE_API const TSet<FTopLevelAssetPath>& GetEDLCookFilters();
	static ENGINE_API const bool ShouldCookEDLGenerator(const UPackage* Generator);
#endif

protected:
	friend class ULevelInstanceSettings;
	void SetPropertyOverridePolicy(UWorldPartitionPropertyOverridePolicy* InPropertyOverridePolicy)
	{
		PropertyOverridePolicy = InPropertyOverridePolicy;
	}
	
	/** Set the default logical operator for actor data layers activation for new maps */
	UPROPERTY(EditAnywhere, Config, Category = WorldPartition)
	EWorldPartitionDataLayersLogicOperator NewMapsDataLayersLogicOperator = EWorldPartitionDataLayersLogicOperator::Or;

	/** Set the default to whether enable world partition for new maps created in the content broswer */
	UPROPERTY(EditAnywhere, Config, Category = WorldPartition)
	bool bNewMapsEnableWorldPartition = false;

	/** Set the default to whether enable world partition streaming for new maps created in the content broswer */
	UPROPERTY(EditAnywhere, Config, Category = WorldPartition)
	bool bNewMapsEnableWorldPartitionStreaming = true;

	/** Set the default editor hash class to use for the editor */
	UPROPERTY(Config)
	TSubclassOf<UWorldPartitionEditorHash> EditorHashDefaultClass;

	/** Set the default runtime hash class to use for new maps  */
	UPROPERTY(EditAnywhere, Config, NoClear, Category = WorldPartition)
	TSubclassOf<UWorldPartitionRuntimeHash> RuntimeHashDefaultClass;

	UPROPERTY(Transient)
	TObjectPtr<UWorldPartitionPropertyOverridePolicy> PropertyOverridePolicy;

	UPROPERTY(Config)
	TMap<EHLODLayerType, FHLODLayerTypeUnsupportedActorClasses> UnsupportedActorClassesPerHLODLayerType;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Config, Category = WorldPartition, meta=(DisplayName="Cook - External Data Layers packages inherit World chunk assignments"))
	bool bShouldEDLPackagesInheritWorldChunkAssignmentsDuringCook = true;

	UPROPERTY(EditAnywhere, Config, Category = WorldPartition, meta=(ToolTip="EDLs to cook in exclusion from base maps and other EDLs"))
	TSet<TSoftObjectPtr<UExternalDataLayerAsset>> EDLCookFilters;
#endif

private:
	struct CookFilterCommandLineLoader
	{
		CookFilterCommandLineLoader();

		TSet<FTopLevelAssetPath> EDLFilters;
		TSet<FName> GeneratorPackageFilters;
	};

	static const CookFilterCommandLineLoader& GetCookFilterCommandLineLoader();
};

#undef UE_API
