// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box.h"
#include "Misc/HierarchicalLogArchive.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"
#include "Templates/SubclassOf.h"

#if WITH_EDITOR

class AActor;
class UWorld;

namespace UE::Private::WorldPartition
{
	struct FStreamingDescriptor
	{
		/** Represents a streaming actor */
		struct FStreamingActor
		{
			/** Actor class */
			FTopLevelAssetPath BaseClass;

			/** Actor native class */
			FTopLevelAssetPath NativeClass;

			/** Actor full path */
			FSoftObjectPath Path;

			/** Actor package on disk */
			FName Package;

			/** Actor name/label*/
			FString Label;

			/** ActorGuid or ActorInstanceGuid depending */
			FGuid ActorGuid;

			void DumpStateLog(FHierarchicalLogArchive& Ar);
		};

		/** Represents a streaming cell */
		struct FStreamingCell
		{
			/** The cell bounds, not necessarily the same size for all cells */
			FBox Bounds;

			/** If this cell is always loaded or not */
			bool bIsAlwaysLoaded;

			/** If this cell is spatially loaded or not */
			bool bIsSpatiallyLoaded;

			/** Data layers for this cell */
			TArray<FName> DataLayers;

			/** Actors in this cell */
			TArray<FStreamingActor> Actors;

			/* PackageName for this cell */
			FName CellPackage;

			void DumpStateLog(FHierarchicalLogArchive& Ar);
		};

		/* Represents a streaming grid */
		struct FStreamingGrid
		{
			/* Name of this grid */
			FName Name;

			/** Bounds of this grid */
			FBox Bounds;

			/** If set (over zero), represents the cell size of the grid */
			int32 CellSize;

			/** If set (over zero) , represents the default loading range of the grid */
			int32 LoadingRange;

			/** Streaming cells of this grid */
			TArray<FStreamingCell> StreamingCells;

			/** ExternalStreaming Objects for this grid */
			TArray<FString> ExternalStreamingObjects;

			void DumpStateLog(FHierarchicalLogArchive& Ar);
		};

		struct FStreamingDescriptorParams
		{
			TArray<TSubclassOf<AActor>> FilteredClasses;
		};

		/** Streaming grids */
		TArray<FStreamingGrid> StreamingGrids;

		ENGINE_API void DumpStateLog(FHierarchicalLogArchive& Ar);
		ENGINE_API static bool GenerateStreamingDescriptor(UWorld* InWorld, FStreamingDescriptor& OutStreamingDescriptor, const FStreamingDescriptorParams& Params = FStreamingDescriptorParams());

		struct FGeneratedPackage
		{
			FName PackagePathName;
			bool bIsLevelPackage = false;

			bool operator==(const FGeneratedPackage&) const = default;
			FString ToString() { return PackagePathName.ToString(); }
		};

		ENGINE_API static TMap<FName, TArray<FGeneratedPackage>> GenerateWorldPluginPackagesList(UWorld* InWorld);
	};
};

#endif