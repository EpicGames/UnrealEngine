// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "Templates/OverrideVoidReturnInvoker.h"
#include "WorldPartition/ActorDescContainerCollection.h"
#include "WorldPartition/ActorDescContainerInstanceCollection.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstanceView.h"

// deprecated
class FWorldPartitionActorDescView;

class FStreamingGenerationActorDescViewMap;

class FStreamingGenerationActorDescView : public FWorldPartitionActorDescInstanceView
{
	typedef FWorldPartitionActorDescInstanceView Super;

	friend class FStreamingGenerationActorDescViewMap;
	friend class FWorldPartitionStreamingGenerator;

protected:
	// Used for invalid reference error reporting
	FStreamingGenerationActorDescView(const FWorldPartitionActorDescInstance* InActorDescInstance)
		: FWorldPartitionActorDescInstanceView(InActorDescInstance)
		, ActorDescViewMap(nullptr)
		, ParentView(nullptr)
		, bIsForcedNonSpatiallyLoaded(false)
		, bIsForcedNoRuntimeGrid(false)
		, bIsForcedNoDataLayers(false)
		, bIsForceNoHLODLayer(false)
		, bIsForcedHLODIrrelevant(false)
		, bIsUnsaved(false)
	{}

public:
	struct FInvalidReference
	{
		FName ActorPackage;
		FSoftObjectPath ActorSoftPath;
		FTopLevelAssetPath BaseClass;
		FTopLevelAssetPath NativeClass;
	};

	enum class EToStringMode : uint8
	{
		ViewOnly,
		Full
	};

	FStreamingGenerationActorDescView(const FStreamingGenerationActorDescViewMap& InActorDescViewMap, const FWorldPartitionActorDescInstance* InActorDescInstance, bool bInUnsaved = false)
		: FStreamingGenerationActorDescView(InActorDescInstance)
	{
		ActorDescViewMap = &InActorDescViewMap;
		bIsUnsaved = bInUnsaved;
	}
		
	virtual ~FStreamingGenerationActorDescView() = default;

	//~ Begin FWorldPartitionActorDescInstanceView interface
	ENGINE_API virtual FName GetRuntimeGrid() const override;
	ENGINE_API virtual bool GetIsSpatiallyLoaded() const override;
	ENGINE_API virtual FSoftObjectPath GetHLODLayer() const override;
	ENGINE_API virtual bool GetActorIsHLODRelevant() const override;
	ENGINE_API virtual const TArray<FGuid>& GetReferences() const override;
	ENGINE_API virtual const FDataLayerInstanceNames& GetDataLayerInstanceNames() const override;
	ENGINE_API virtual bool IsEditorOnlyReference(const FGuid& ReferenceGuid) const override;
	ENGINE_API virtual FString ToString(FWorldPartitionActorDesc::EToStringMode Mode) const override;
	//~ End FWorldPartitionActorDescInstanceView interface

	bool IsUnsaved() const { return bIsUnsaved; }
	const FStreamingGenerationActorDescViewMap& GetActorDescViewMap() const { return *ActorDescViewMap; }

	ENGINE_API const TArray<FGuid>& GetEditorReferences() const;
	ENGINE_API const FDataLayerInstanceNames& GetRuntimeDataLayerInstanceNames() const;

	ENGINE_API void AddForcedInvalidReference(const FStreamingGenerationActorDescView* ReferenceView);
	ENGINE_API const FInvalidReference* GetInvalidReference(const FGuid& InGuid) const;
	ENGINE_API void SetIsSpatiallyLoaded(bool bValue);
	ENGINE_API void SetForcedNonSpatiallyLoaded();
	ENGINE_API void SetForcedNoRuntimeGrid();
	ENGINE_API void SetForcedNoDataLayers();
	ENGINE_API void SetRuntimeGrid(FName InRuntimeGrid);
	ENGINE_API void SetRuntimeDataLayerInstanceNames(const FDataLayerInstanceNames& InRuntimeDataLayerInstanceNames);
	ENGINE_API void SetRuntimeReferences(const TArray<FGuid>& InRuntimeReferences);
	ENGINE_API void SetEditorReferences(const TArray<FGuid>& InEditorReferences);
	ENGINE_API void SetDataLayerInstanceNames(const FDataLayerInstanceNames& InDataLayerInstanceNames);
	ENGINE_API void SetParentView(const FStreamingGenerationActorDescView* InParentView);
	ENGINE_API void SetEditorOnlyParentTransform(const FTransform& InEditorOnlyParentTransform);
	FTransform GetEditorOnlyParentTransform() const { return EditorOnlyParentTransform.IsSet() ? EditorOnlyParentTransform.GetValue() : FTransform::Identity; }

	ENGINE_API void SetForcedNoHLODLayer();
	ENGINE_API void SetRuntimeHLODLayer(const FSoftObjectPath& InHLODLayer);
	ENGINE_API void SetIsHLODRelevant(bool bValue);
	ENGINE_API void SetForcedHLODIrrelevant();

	bool operator==(const FStreamingGenerationActorDescView& Other) const
	{
		return GetGuid() == Other.GetGuid();
	}

	friend uint32 GetTypeHash(const FStreamingGenerationActorDescView& Key)
	{
		return GetTypeHash(Key.GetGuid());
	}

private:
	ENGINE_API FString ToString(EToStringMode ActorDescViewMode, FWorldPartitionActorDesc::EToStringMode ActorDescMode, TFunctionRef<bool(const TArray<FName>&, TArray<const UDataLayerInstance*>&)> GetRuntimeDataLayerInstancesFromNamesFunc) const;

	const FStreamingGenerationActorDescViewMap* ActorDescViewMap;
	const FStreamingGenerationActorDescView* ParentView;

	bool bIsForcedNonSpatiallyLoaded;
	bool bIsForcedNoRuntimeGrid;
	bool bIsForcedNoDataLayers;
	bool bIsForceNoHLODLayer;
	bool bIsForcedHLODIrrelevant;
	bool bIsUnsaved;

	// Stores invalid actor references
	TMap<FGuid, FInvalidReference> ForcedInvalidReference;
	
	// Used by GetDataLayerInstanceNames() to avoid returning a copy
	mutable FDataLayerInstanceNames LastReturnedDataLayerInstanceNames; 

	TOptional<FName> ResolvedRuntimeGrid;
	TOptional<FDataLayerInstanceNames> ResolvedDataLayerInstanceNames;
	TOptional<FDataLayerInstanceNames> RuntimeDataLayerInstanceNames;
	TOptional<bool> bIsSpatiallyLoaded;
	TOptional<bool> bIsHLODRelevant;
	TOptional<TArray<FGuid>> RuntimeReferences;
	TOptional<FSoftObjectPath> RuntimeHLODLayer;
	TArray<FGuid> EditorReferences;
	TOptional<FTransform> EditorOnlyParentTransform;
};

class FStreamingGenerationActorDescViewMap
{
	friend class FWorldPartitionStreamingGenerator;

private:
	template <class Func>
	void ForEachActorDescView(Func InFunc)
	{
		TOverrideVoidReturnInvoker Invoker(true, InFunc);

		for (TUniquePtr<FStreamingGenerationActorDescView>& ActorDescView : ActorDescViewList)
		{
			if (!Invoker(*ActorDescView))
			{
				return;
			}
		}
	}

	FStreamingGenerationActorDescView* FindByGuid(const FGuid& InGuid)
	{
		if (FStreamingGenerationActorDescView** ActorDescViewPtr = ActorDescViewsByGuid.Find(InGuid))
		{
			return *ActorDescViewPtr;
		}
		return nullptr;
	}

	FStreamingGenerationActorDescView& FindByGuidChecked(const FGuid& InGuid)
	{
		return *ActorDescViewsByGuid.FindChecked(InGuid);
	}

public:
	ENGINE_API FStreamingGenerationActorDescViewMap();

	// Non-copyable but movable
	FStreamingGenerationActorDescViewMap(const FStreamingGenerationActorDescViewMap&) = delete;
	FStreamingGenerationActorDescViewMap(FStreamingGenerationActorDescViewMap&&) = default;

	FStreamingGenerationActorDescViewMap& operator=(const FStreamingGenerationActorDescViewMap&) = delete;
	FStreamingGenerationActorDescViewMap& operator=(FStreamingGenerationActorDescViewMap&&) = default;

	ENGINE_API FStreamingGenerationActorDescView* Emplace(const FGuid& InActorGuid, FStreamingGenerationActorDescView&& InActorDescView);
	ENGINE_API FStreamingGenerationActorDescView* Emplace(const FWorldPartitionActorDescInstance* InActorDescInstance);
			
	inline int32 Num() const
	{
		return ActorDescViewList.Num();
	}

	template <class Func>
	void ForEachActorDescView(Func InFunc) const
	{
		TOverrideVoidReturnInvoker Invoker(true, InFunc);

		for (const TUniquePtr<FStreamingGenerationActorDescView>& ActorDescView : ActorDescViewList)
		{
			if (!Invoker(*ActorDescView))
			{
				return;
			}
		}
	}

	const FStreamingGenerationActorDescView* FindByGuid(const FGuid& InGuid) const
	{
		if (const FStreamingGenerationActorDescView* const* ActorDescViewPtr = ActorDescViewsByGuid.Find(InGuid))
		{
			return *ActorDescViewPtr;
		}
		return nullptr;
	}

	const FStreamingGenerationActorDescView& FindByGuidChecked(const FGuid& InGuid) const
	{
		return *ActorDescViewsByGuid.FindChecked(InGuid);
	}

	template <class ClassType>
	TArray<const FStreamingGenerationActorDescView*> FindByExactNativeClass() const
	{
		return FindByExactNativeClass(ClassType::StaticClass());
	}

	ENGINE_API TArray<const FStreamingGenerationActorDescView*> FindByExactNativeClass(UClass* InExactNativeClass) const;

	const TMap<FGuid, FStreamingGenerationActorDescView*>& GetActorDescViewsByGuid() const { return ActorDescViewsByGuid; }

protected:
	TArray<TUniquePtr<FStreamingGenerationActorDescView>> ActorDescViewList;

	TMap<FGuid, FStreamingGenerationActorDescView*> ActorDescViewsByGuid;
	TMultiMap<FName, const FStreamingGenerationActorDescView*> ActorDescViewsByClass;
};

class FStreamingGenerationContainerInstanceCollection : public TActorDescContainerInstanceCollection<TObjectPtr<const UActorDescContainerInstance>>
{
public:
	enum class ECollectionType
	{
		Invalid,
		BaseAsContentBundle, // Collection contains only one content bundle container
		BaseAndEDLs,		 // Collection contains the base container and optionally EDL containers
		BaseAndAny			 // Collection contains the base container and any type of container (base,EDL,CB)
	};

	FStreamingGenerationContainerInstanceCollection() : CollectionType(ECollectionType::Invalid) {}
	ENGINE_API FStreamingGenerationContainerInstanceCollection(std::initializer_list<TObjectPtr<const UActorDescContainerInstance>> ActorDescContainerInstanceArray, const ECollectionType& InCollectionType);
	ENGINE_API FStreamingGenerationContainerInstanceCollection(const TArray<const UActorDescContainerInstance*>& ActorDescContainerInstances, const ECollectionType& InCollectionType);

	ENGINE_API UWorld* GetWorld() const;
	ENGINE_API const UActorDescContainerInstance* GetBaseContainerInstance() const;
	ENGINE_API FName GetBaseContainerInstancePackageName() const;
	ENGINE_API FGuid GetContentBundleGuid() const; // @todo_ow : Remove once conversion to ExternalDataLayer is complete. 

private:
	void SetCollectionType(const ECollectionType& InCollectionType) { check(CollectionType == ECollectionType::Invalid); CollectionType = InCollectionType; }
	ECollectionType GetCollectionType() const { return CollectionType; }
	void InitializeCollection();
	FName GetBaseContainerPackageName() const;
	TArrayView<const UActorDescContainerInstance* const> GetExternalDataLayerContainerInstances() const;
	TArrayView<const UActorDescContainerInstance* const> GetContentBundleContainerInstances() const;
	ENGINE_API virtual void OnCollectionChanged() override;
	virtual bool ShouldRegisterDelegates() const override { return false; }

	ECollectionType CollectionType;
	int32 ExternalDataLayerStartIdx = INDEX_NONE;
	int32 ContentBundleStartIdx = INDEX_NONE;
	static constexpr int BaseContainerIdx = 0;

	friend class UWorldPartition;
	friend class FWorldPartitionStreamingGenerator;
	friend struct FGenerateStreamingParams;
};

/**
 * Experimental: an actor descriptor mutator used by external code to change properties on actor descriptors.
 */
struct FActorDescViewMutator
{
	TOptional<bool> bIsSpatiallyLoaded;
	TOptional<FName> RuntimeGrid;
	TOptional<bool> bIsHLODRelevant;
	TOptional<FSoftObjectPath> HLODLayer;
};

struct FActorDescViewMutatorInstance : FActorDescViewMutator
{
	/** The actor GUID to mutate */
	FGuid ActorGuid;

	/** Target container ID to mutate this actor */
	FActorContainerID ContainerId;
};

#endif // WITH_EDITOR
