// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Optional.h"
#include "Templates/OverrideVoidReturnInvoker.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"

class UExternalDataLayerAsset;
class UActorDescContainer;
class UDataLayerInstance;
class FGenerateStreamingActorDescCollection;
class FStreamingGenerationActorDescViewMap;
class FStreamingGenerationActorDescView;
class FWorldPartitionStreamingGenerator;
class FStreamingGenerationContainerInstanceCollection;
class UActorDescContainer;
class UDataLayerInstance;
class FWorldDataLayersActorDesc;
struct FWorldPartitionRuntimeContainerResolver;

#if WITH_EDITOR
class IStreamingGenerationContext
{
public:
	virtual ~IStreamingGenerationContext() = default;

	/**
	 * An actor set represents a group of actors that needs to be part of the same streaming cell, because they have hard references between them.
	 */
	struct FActorSet
	{
		TArray<FGuid> Actors;
	};

	/**
	 * An actor set container represents the list of actor sets in an actor container, e.g. a level instance.
	 */
	struct FActorSetContainerInstance
	{
		FActorSetContainerInstance()
			: ActorDescViewMap(nullptr)
			, DataLayerResolvers(nullptr)
			, ContainerInstanceCollection(nullptr)
		{}

		// Non-copyable
		FActorSetContainerInstance(const FActorSetContainerInstance&) = delete;
		FActorSetContainerInstance& operator=(const FActorSetContainerInstance&) = delete;

		const FStreamingGenerationActorDescViewMap* ActorDescViewMap;
		const TArray<const FWorldDataLayersActorDesc*>* DataLayerResolvers;
		const FStreamingGenerationContainerInstanceCollection* ContainerInstanceCollection; // Only used by UWorldPartitionRuntimeSpatialHash::SetupHLODActors
		TArray<TUniquePtr<FActorSet>> ActorSets;
	};

	/**
	 * An actor set instance is an actual instance of an actor set in the world.
	 */
	struct FActorSetInstance
	{
		FBox Bounds;
		FName RuntimeGrid;
		bool bIsSpatiallyLoaded;
		TArray<const UDataLayerInstance*> DataLayers;
		FGuid ContentBundleID;

		const FActorSetContainerInstance* ActorSetContainerInstance = nullptr;

		FActorContainerID ContainerID;
		FTransform Transform;
		const FActorSet* ActorSet;
		const TSet<FGuid>* FilteredActors;

		template <typename Func>
		void ForEachActor(Func InFunc) const
		{
			TOverrideVoidReturnInvoker Invoker(true, InFunc);

			for (const FGuid& ActorGuid : ActorSet->Actors)
			{
				if (!FilteredActors || !FilteredActors->Contains(ActorGuid))
				{
					if (!Invoker(ActorGuid))
					{
						break;
					}
				}
			}
		}

		const UExternalDataLayerAsset* GetExternalDataLayerAsset() const;
	};

	/**
	 * An actor instance represents a single instanced actor in the world.
	 */
	struct FActorInstance
	{
		FActorInstance(const FGuid& InActorGuid, const FActorSetInstance* InActorSetInstance)
			: ActorGuid(InActorGuid)
			, ActorSetInstance(InActorSetInstance)
		{}

		FGuid ActorGuid;
		const FActorSetInstance* ActorSetInstance;

		ENGINE_API const FStreamingGenerationActorDescView& GetActorDescView() const;
		ENGINE_API const FActorContainerID& GetContainerID() const;
		ENGINE_API const FTransform& GetTransform() const;
		ENGINE_API const FBox GetBounds() const;
	};

	virtual FBox GetWorldBounds() const = 0;
	// Returns the ActorSetContainerInstance that contains the BaseActorDescContainerInstance of this context
	virtual const FActorSetContainerInstance* GetActorSetContainerForContextBaseContainerInstance() const = 0;
	virtual void ForEachActorSetInstance(TFunctionRef<void(const FActorSetInstance&)> Func) const = 0;
	virtual void ForEachActorSetContainerInstance(TFunctionRef<void(const FActorSetContainerInstance&)> Func) const = 0;
};

class FStreamingGenerationContextProxy : public IStreamingGenerationContext
{
public:
	FStreamingGenerationContextProxy(const IStreamingGenerationContext* InSourceContext)
		: SourceContext(InSourceContext)
	{}

	virtual FBox GetWorldBounds() const override
	{
		return SourceContext->GetWorldBounds();
	}

	virtual const FActorSetContainerInstance* GetActorSetContainerForContextBaseContainerInstance() const override
	{
		return SourceContext->GetActorSetContainerForContextBaseContainerInstance();
	}

	virtual void ForEachActorSetInstance(TFunctionRef<void(const FActorSetInstance&)> Func) const override
	{
		if (ActorSetInstanceFilterFunc.IsSet())
		{
			SourceContext->ForEachActorSetInstance([this, Func](const FActorSetInstance& ActorSetInstance)
			{
				if ((*ActorSetInstanceFilterFunc)(ActorSetInstance))
				{
					Func(ActorSetInstance);
				}
			});
		}
		else
		{
			SourceContext->ForEachActorSetInstance(Func);
		}
	}

	virtual void ForEachActorSetContainerInstance(TFunctionRef<void(const FActorSetContainerInstance&)> Func) const override
	{
		SourceContext->ForEachActorSetContainerInstance(Func);
	}

	void SetActorSetInstanceFilter(const TFunction<bool(const IStreamingGenerationContext::FActorSetInstance&)>& InActorSetInstanceFilterFunc) { ActorSetInstanceFilterFunc = InActorSetInstanceFilterFunc; }

protected:
	TOptional<TFunction<bool(const IStreamingGenerationContext::FActorSetInstance&)>> ActorSetInstanceFilterFunc;

	const IStreamingGenerationContext* SourceContext;
};
#endif
