// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * WorldPartitionStreamingPolicy
 *
 * Base class for World Partition Runtime Streaming Policy
 *
 */

#pragma once

#include "Async/TaskGraphFwd.h"
#include "Containers/Set.h"
#include "Misc/CoreDelegates.h"
#include "ProfilingDebugging/WorldStreamingTrace.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "WorldPartition/WorldPartitionRuntimeContainerResolving.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartitionStreamingPolicy.generated.h"

class UWorldPartition;
class FWorldPartitionDraw2DContext;

USTRUCT()
struct FActivatedCells
{
	GENERATED_USTRUCT_BODY()

	void Add(const UWorldPartitionRuntimeCell* InCell);
	void Remove(const UWorldPartitionRuntimeCell* InCell);
	bool Contains(const UWorldPartitionRuntimeCell* InCell) const { return Cells.Contains(InCell); }
	void OnAddedToWorld(const UWorldPartitionRuntimeCell* InCell);
	void OnRemovedFromWorld(const UWorldPartitionRuntimeCell* InCell);
	void Reset();

	const TSet<TObjectPtr<const UWorldPartitionRuntimeCell>>& GetCells() const { return Cells; }
	const TSet<const UWorldPartitionRuntimeCell*>& GetPendingAddToWorldCells() const { return PendingAddToWorldCells; }

private:

	UPROPERTY(Transient)
	TSet<TObjectPtr<const UWorldPartitionRuntimeCell>> Cells;

	TSet<const UWorldPartitionRuntimeCell*> PendingAddToWorldCells;
};

USTRUCT()
struct FWorldPartitionUpdateStreamingTargetState
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Transient)
	TArray<TObjectPtr<const UWorldPartitionRuntimeCell>> ToLoadCells;

	UPROPERTY(Transient)
	TArray<TObjectPtr<const UWorldPartitionRuntimeCell>> ToActivateCells;

	UPROPERTY(Transient)
	TArray<TObjectPtr<const UWorldPartitionRuntimeCell>> ToDeactivateCells;

	UPROPERTY(Transient)
	TArray<TObjectPtr<const UWorldPartitionRuntimeCell>> ToUnloadCells;

	EWorldPartitionStreamingPerformance StreamingPerformance = EWorldPartitionStreamingPerformance::Good;
	bool bBlockOnSlowStreaming = false;

	bool bUpdateServerEpoch = false;

#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
	TArray<TPair<FGuid, float>> DeferredPriorityTraceData;
#endif

	bool IsEmpty() const;
	void Reset();
};

USTRUCT()
struct FWorldPartitionUpdateStreamingCurrentState
{
	GENERATED_USTRUCT_BODY()

	// Streaming Sources
	TArray<FWorldPartitionStreamingSource> StreamingSources;

	UPROPERTY(Transient)
	TSet<TObjectPtr<const UWorldPartitionRuntimeCell>> LoadedCells;

	UPROPERTY(Transient)
	FActivatedCells ActivatedCells;

	void Reset();
	void CopyFrom(const FWorldPartitionUpdateStreamingCurrentState& InCurrentState);
};

UCLASS(Abstract, Within = WorldPartition)
class UWorldPartitionStreamingPolicy : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	virtual bool GetIntersectingCells(const TArray<FWorldPartitionStreamingQuerySource>& InSources, TArray<const IWorldPartitionCell*>& OutCells) const;
	virtual void UpdateStreamingState();
	virtual bool CanAddCellToWorld(const UWorldPartitionRuntimeCell* InCell) const;
	virtual bool DrawRuntimeHash2D(FWorldPartitionDraw2DContext& DrawContext);
	virtual void DrawRuntimeHash3D();
	virtual void DrawRuntimeCellsDetails(class UCanvas* Canvas, FVector2D& Offset) {}
	
	virtual bool IsStreamingCompleted(const TArray<FWorldPartitionStreamingSource>* InStreamingSources) const;
	virtual bool IsStreamingCompleted(EWorldPartitionRuntimeCellState QueryState, const TArray<FWorldPartitionStreamingQuerySource>& QuerySources, bool bExactState = true) const;

	virtual void OnCellShown(const UWorldPartitionRuntimeCell* InCell);
	virtual void OnCellHidden(const UWorldPartitionRuntimeCell* InCell);

#if WITH_EDITOR
	virtual TSubclassOf<class UWorldPartitionRuntimeCell> GetRuntimeCellClass() const PURE_VIRTUAL(UWorldPartitionStreamingPolicy::GetRuntimeCellClass, return UWorldPartitionRuntimeCell::StaticClass(); );

	// PIE/Game methods
	virtual void PrepareActorToCellRemapping() {}
	virtual void SetContainerResolver(const FWorldPartitionRuntimeContainerResolver& InContainerResolver) {}
	virtual void RemapSoftObjectPath(FSoftObjectPath& ObjectPath) const {}

	virtual bool StoreStreamingContentToExternalStreamingObject(URuntimeHashExternalStreamingObjectBase& OutExternalStreamingObject) { return true; }
	virtual bool ConvertContainerPathToEditorPath(const FActorContainerID& InContainerID, const FSoftObjectPath& InPath, FSoftObjectPath& OutPath) const { return false; }
#endif

#if !UE_BUILD_SHIPPING
	virtual void GetOnScreenMessages(FCoreDelegates::FSeverityMessageMap& OutMessages);
#endif

	// Editor/Runtime conversions
	virtual bool ConvertEditorPathToRuntimePath(const FSoftObjectPath& InPath, FSoftObjectPath& OutPath) const { return false; }
	virtual UObject* GetSubObject(const TCHAR* SubObjectPath) { return nullptr; }

	const TArray<FWorldPartitionStreamingSource>& GetStreamingSources() const { return CurrentState.StreamingSources; }

	EWorldPartitionStreamingPerformance GetStreamingPerformance() const { return CurrentStreamingPerformance; }

	static bool IsUpdateStreamingOptimEnabled();

	virtual bool InjectExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject) { return true; }
	virtual bool RemoveExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject) { return true; }

	virtual void SetShouldMergeStreamingSourceInfo(bool bInShouldMergeStreamingSourceInfo) { bShouldMergeStreamingSourceInfo = bInShouldMergeStreamingSourceInfo; }

protected:
	virtual void SetCellStateToLoaded(const UWorldPartitionRuntimeCell* InCell, int32& InOutMaxCellsToLoad);
	virtual void SetCellStateToActivated(const UWorldPartitionRuntimeCell* InCell, int32& InOutMaxCellsToLoad);
	virtual void SetCellsStateToUnloaded(const TArray<TObjectPtr<const UWorldPartitionRuntimeCell>>& ToUnloadCells);
	virtual void GetCellsToUpdate(TArray<const UWorldPartitionRuntimeCell*>& OutToLoadCells, TArray<const UWorldPartitionRuntimeCell*>& OutToActivateCells);
	virtual void GetCellsToReprioritize(TArray<const UWorldPartitionRuntimeCell*>& OutToLoadCells, TArray<const UWorldPartitionRuntimeCell*>& OutToActivateCells);
	virtual void UpdateStreamingSources(bool bCanOptimizeUpdate);

	void UpdateStreamingPerformance(EWorldPartitionStreamingPerformance NewStreamingPerformance, bool bBlockOnSlowStreaming);
	bool IsInBlockTillLevelStreamingCompleted(bool bIsCausedByBadStreamingPerformance = false) const;

	// Outer World Partition
	const UWorldPartition* WorldPartition;

private:
	enum class EAsyncUpdateTaskState
	{
		None,
		Pending,
		Started
	};

	struct FUpdateStreamingStateParams
	{
		FUpdateStreamingStateParams(UWorldPartitionStreamingPolicy* InPolicy, const FWorldPartitionUpdateStreamingCurrentState& InCurrentState);
		FUpdateStreamingStateParams& SetRequiredWorldDataLayersEffectiveStatesCopy(bool bInRequiredEffectiveStatesCopy);

		const UWorld* World;
		const UWorldPartitionRuntimeHash* RuntimeHash;
		const bool bCanStream;
		const bool bIsServer;
		const bool bIsStreamingInEnabled;
		const bool bIsServerStreamingEnabled;
		const bool bIsServerStreamingOutEnabled;
		const bool bIsBlockingCausedByBadStreamingPerformance;
		const bool bIsPlaybackEnabled;
		const bool bMatchStarted;
		const bool bShouldMergeStreamingSourceInfo;
#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
		const bool bShouldTraceContainerPriorities;
#endif
		const int32 PolicyUpdateStreamingStateEpoch;
		const EWorldPartitionDataLayersLogicOperator DataLayersLogicOperator;
		const FTransform WorldPartitionInstanceTransform;
		const FWorldPartitionUpdateStreamingCurrentState& CurrentState;
		const TSet<FName>& ServerDisallowedStreamingOutDataLayers;
		TSet<const UWorldPartitionRuntimeCell*>& FrameActivateCells;
		TSet<const UWorldPartitionRuntimeCell*>& FrameLoadCells;
		TSet<const UWorldPartitionRuntimeCell*>& FrameAnalysePerformanceCells;
#if WITH_EDITOR
		const FDynamicLoadingRangeScales DynamicLoadingRangeScales;
#endif // WITH_EDITOR

		const FWorldDataLayersEffectiveStates& GetWorldDataLayersEffectiveStates() const { return WorldDataLayersEffectiveStatesCopy.Get(WorldDataLayersEffectiveStatesRef); }
	private:
		const FWorldDataLayersEffectiveStates& WorldDataLayersEffectiveStatesRef;
		TOptional<FWorldDataLayersEffectiveStates> WorldDataLayersEffectiveStatesCopy;
	};

	// Update optimization
	uint32 ComputeUpdateStreamingHash(bool bCanOptimizeUpdate) const;
	int32 ComputeServerStreamingEnabledEpoch() const;

	void OnStreamingStateUpdated();
	void OnPreChangeStreamingContent();
	bool WaitForAsyncUpdateStreamingState();
	void PostUpdateStreamingStateInternal_GameThread(FWorldPartitionUpdateStreamingTargetState& InOutTargetState);
	const TSet<FName>& GetServerDisallowedStreamingOutDataLayers() const;
	static void UpdateStreamingStateInternal(const FUpdateStreamingStateParams& InParams, FWorldPartitionUpdateStreamingTargetState& OutTargetState);

	// Current streaming state
	UPROPERTY(Transient)
	FWorldPartitionUpdateStreamingCurrentState CurrentState;

	// Current streaming performance
	UPROPERTY(Transient)
	EWorldPartitionStreamingPerformance CurrentStreamingPerformance;

	// Current block on slow streaming
	UPROPERTY(Transient)
	bool bCurrentBlockOnSlowStreaming;

	// Target state
	UPROPERTY(Transient)
	FWorldPartitionUpdateStreamingTargetState TargetState;

	// Asynchronous update task input payload
	UPROPERTY(Transient)
	FWorldPartitionUpdateStreamingCurrentState AsyncTaskCurrentState;

	// Asynchronous update task output payload
	UPROPERTY(Transient)
	FWorldPartitionUpdateStreamingTargetState AsyncTaskTargetState;

	UPROPERTY()
	bool bShouldMergeStreamingSourceInfo;

	bool bCriticalPerformanceRequestedBlockTillOnWorld;
	int32 CriticalPerformanceBlockTillLevelStreamingCompletedEpoch;
	int32 ProcessedToLoadCells;		// Used to know if last update fully processed ToLoadCells
	int32 ProcessedToActivateCells; // Used to know if last update fully processed ToActivateCells
	int32 ServerStreamingStateEpoch;
	int32 ServerStreamingEnabledEpoch;
	uint32 UpdateStreamingHash;
	uint32 UpdateStreamingSourcesHash;
	uint32 UpdateStreamingStateCounter;

	// Asynchronous update task
	EAsyncUpdateTaskState AsyncUpdateTaskState;
	UE::Tasks::TTask<void> AsyncUpdateStreamingStateTask;

	int32 AsyncShouldSkipUpdateCounter;

	mutable TOptional<TSet<FName>> CachedServerDisallowStreamingOutDataLayers;
	// Used internally by UpdateStreamingStateInternal (avoids re-allocations)
	mutable TSet<const UWorldPartitionRuntimeCell*> FrameActivateCells;
	mutable TSet<const UWorldPartitionRuntimeCell*> FrameLoadCells;
	mutable TSet<const UWorldPartitionRuntimeCell*> FrameAnalysePerformanceCells;

	// CVars to control update optimization
	static bool IsUpdateOptimEnabled;
	static bool IsAsyncUpdateStreamingStateEnabled;
	static int32 ForceUpdateFrameCount;
	static FAutoConsoleVariableRef CVarUpdateOptimEnabled;
	static FAutoConsoleVariableRef CVarAsyncUpdateStreamingStateEnabled;
	static FAutoConsoleVariableRef CVarForceUpdateFrameCount;

#if !UE_BUILD_SHIPPING || UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
	static void UpdateDebugCellsStreamingPriority(const TSet<const UWorldPartitionRuntimeCell*>& InActivateStreamingCells, const TSet<const UWorldPartitionRuntimeCell*>& InLoadStreamingCells, bool bInShouldMergeStreamingSourceInfo, TArray<TPair<FGuid, float>>* OutTraceData);
#endif

#if !UE_BUILD_SHIPPING
	double OnScreenMessageStartTime;
	EWorldPartitionStreamingPerformance  OnScreenMessageStreamingPerformance;
	bool bOnScreenMessageShouldBlock;
#endif

	friend class UWorldPartition;
	friend class UWorldPartitionSubsystem;
};