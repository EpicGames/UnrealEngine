// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/Color.h"
#include "Math/RandomStream.h"
#include "HAL/IConsoleManager.h"
#include "Math/Box2D.h"
#include "WorldPartitionStreamingSource.generated.h"

/** See https://en.wikipedia.org/wiki/Spherical_sector. */
class FSphericalSector
{
public:
	using FReal = FVector::FReal;

	/** Creates and initializes a new spherical sector. */
	FSphericalSector(EForceInit)
		: Center(ForceInit)
		, Radius(0.0f)
		, Axis(ForceInit)
		, Angle(360.0f)
	{}

	/** Creates and initializes a spherical sector using given parameters. */
	FSphericalSector(const FVector& InCenter, FReal InRadius)
		: Center(InCenter)
		, Radius(InRadius)
		, Axis(FVector::ForwardVector)
		, Angle(360.0f)
	{}

	/** Creates and initializes a spherical sector using given parameters. */
	FSphericalSector(const FVector& InCenter, FReal InRadius, const FVector& InAxis, FReal InAngle)
		: Center(InCenter)
		, Radius(InRadius)
		, Axis(InAxis)
		, Angle(InAngle)
	{
		check(InAxis.IsNormalized());
		check(InAngle >= 0 && InAngle <= 360);
	}

	void SetCenter(const FVector& InCenter) { Center = InCenter; }
	const FVector& GetCenter() const { return Center; }

	void SetRadius(FReal InRadius) { Radius = InRadius; }
	FReal GetRadius() const { return Radius; }

	void SetAngle(FReal InAngle) { Angle = (InAngle <= 0.0f || InAngle > 360.0f) ? 360.0f : InAngle; }
	FReal GetAngle() const { return Angle; }

	void SetAxis(const FVector& InAxis) { Axis = InAxis.GetSafeNormal(); }
	FVector GetAxis() const { return Axis; }
	FVector GetScaledAxis() const { return Axis * Radius; }

	void SetAsSphere() { Angle = 360.0f; }
	bool IsSphere() const { return FMath::IsNearlyEqual(Angle, (FReal)360.0); }

	bool IsNearlyZero() const { return FMath::IsNearlyZero(Radius) || Axis.IsNearlyZero() || FMath::IsNearlyZero(Angle); }
	bool IsValid() const { return !IsNearlyZero(); }

	FBox CalcBounds() const
	{
		const FVector Offset(Radius);
		return FBox(Center - Offset, Center + Offset);
	}

	/** Get result of Transforming spherical sector with transform. */
	FSphericalSector TransformBy(const FTransform& M) const
	{
		return FSphericalSector(M.TransformPosition(Center), M.GetMaximumAxisScale() * Radius, M.TransformVector(Axis), Angle);
	}

	/** Helper method that builds a list of debug display segments */
	TArray<TPair<FVector, FVector>> BuildDebugMesh() const;

	/** Returns whether spherical sector intersects 2D box */
	bool IntersectsBox(const FBox2D& InBox) const;

private:
	/** Sphere center point. */
	FVector Center;

	/** Sphere radius. */
	FReal Radius;

	/** Sector axis (direction). */
	FVector Axis;

	/** Optional sector angle in degree (360 = regular sphere). */
	FReal Angle;
};

USTRUCT(BlueprintType)
struct FStreamingSourceShape
{
	GENERATED_BODY()

	FStreamingSourceShape()
	: bUseGridLoadingRange(true)
	, LoadingRangeScale(1.f)
	, Radius(10000.0f)
	, bIsSector(false)
	, SectorAngle(360.0f)
	, Location(ForceInitToZero)
	, Rotation(ForceInitToZero)
	{}

	/* If True, streaming source shape radius is bound to loading range radius. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming)
	bool bUseGridLoadingRange;

	/* Applies a scale to the grid's loading range (used if bUseGridLoadingRange is True). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming, meta = (EditCondition = "bUseGridLoadingRange"))
	float LoadingRangeScale;

	/* Custom streaming source shape radius (not used if bUseGridLoadingRange is True). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming, meta = (EditCondition = "!bUseGridLoadingRange"))
	float Radius;

	/* Whether the source shape is a spherical sector instead of a regular sphere source. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming)
	bool bIsSector;

	/* Shape's spherical sector angle in degree (not used if bIsSector is False). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming, meta = (EditCondition = "bIsSector", ClampMin = 0, ClampMax = 360))
	float SectorAngle;

	/* Streaming source shape location (local to streaming source). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming)
	FVector Location;

	/* Streaming source shape rotation (local to streaming source). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Streaming)
	FRotator Rotation;

	FString ToString() const;

	friend uint32 GetTypeHash(const FStreamingSourceShape& InShape)
	{
		uint32 Hash = GetTypeHash(InShape.bUseGridLoadingRange);
		Hash = HashCombineFast(Hash, GetTypeHash(InShape.LoadingRangeScale));
		Hash = HashCombineFast(Hash, GetTypeHash(InShape.Radius));
		Hash = HashCombineFast(Hash, GetTypeHash(InShape.bIsSector));
		Hash = HashCombineFast(Hash, GetTypeHash(InShape.SectorAngle));
		Hash = HashCombineFast(Hash, GetTypeHash(InShape.Location.X));
		Hash = HashCombineFast(Hash, GetTypeHash(InShape.Location.Y));
		Hash = HashCombineFast(Hash, GetTypeHash(InShape.Location.Z));
		Hash = HashCombineFast(Hash, GetTypeHash(InShape.Rotation.Pitch));
		Hash = HashCombineFast(Hash, GetTypeHash(InShape.Rotation.Yaw));
		Hash = HashCombineFast(Hash, GetTypeHash(InShape.Rotation.Roll));
		return Hash;
	}
};

UENUM(BlueprintType)
enum class EStreamingSourceTargetBehavior : uint8
{
	Include,
	Exclude
};

/** Helper class used to iterate over streaming source shapes. */
class FStreamingSourceShapeHelper
{
public:
	inline static bool IsSourceAffectingGrid(const TSet<FName>& InSourceTargetGrids, EStreamingSourceTargetBehavior InSourceTargetBehavior, FName InGridName)
	{
		switch (InSourceTargetBehavior)
		{
		case EStreamingSourceTargetBehavior::Include:
			return InSourceTargetGrids.IsEmpty() || InSourceTargetGrids.Contains(InGridName);
		case EStreamingSourceTargetBehavior::Exclude:
			return !InSourceTargetGrids.Contains(InGridName);
		default:
			checkNoEntry();
		}

		return false;
	}

	inline static void ForEachShape(float InGridLoadingRange, float InDefaultRadius, bool bInProjectIn2D, const FVector& InLocation, const FRotator& InRotation, const TArray<FStreamingSourceShape>& InShapes, TFunctionRef<void(const FSphericalSector&)> InOperation, float InExtraRadius = 0.f, float InExtraAngle = 0.f)
	{
		const FTransform Transform(bInProjectIn2D ? FRotator(0, InRotation.Yaw, 0) : InRotation, InLocation);
		if (InShapes.IsEmpty())
		{
			if ((InDefaultRadius + InExtraRadius) > 0)
			{
				const FSphericalSector LocalShape(FVector::ZeroVector, InDefaultRadius + InExtraRadius);
				check(LocalShape.IsValid()); // Radius, axis and angle should be all valid here

				InOperation(LocalShape.TransformBy(Transform));
			}
		}
		else
		{
			for (const FStreamingSourceShape& Shape : InShapes)
			{
				const FVector::FReal ShapeRadius = (Shape.bUseGridLoadingRange ? (InGridLoadingRange * Shape.LoadingRangeScale) : Shape.Radius) + InExtraRadius;
				const FVector::FReal ShapeAngle = Shape.bIsSector ? FMath::Min(Shape.SectorAngle + InExtraAngle, 360.0f) : 360.0f;
				const FRotator ShapeRotation = bInProjectIn2D ? FRotator(0, Shape.Rotation.Yaw, 0) : Shape.Rotation;
				const FVector ShapeAxis = ShapeRotation.IsNearlyZero() ? FVector::ForwardVector : ShapeRotation.Vector();
				const FSphericalSector LocalShape(bInProjectIn2D ? FVector(Shape.Location.X, Shape.Location.Y, 0) : Shape.Location, ShapeRadius, ShapeAxis, ShapeAngle);
				
				if (LocalShape.IsValid())
				{
					InOperation(LocalShape.TransformBy(Transform));
				}
			}
		}
	}
};

/**
 * Streaming Source Target State
 */
UENUM(BlueprintType)
enum class EStreamingSourceTargetState : uint8
{
	Loaded,
	Activated
};

inline const TCHAR* GetStreamingSourceTargetStateName(EStreamingSourceTargetState StreamingSourceTargetState)
{
	switch(StreamingSourceTargetState)

	{
	case EStreamingSourceTargetState::Loaded: return TEXT("Loaded");
	case EStreamingSourceTargetState::Activated: return TEXT("Activated");
	default: check(0);
	}
	return TEXT("Invalid");
}

/**
 * Structure containing all properties required to query a streaming state
 */
USTRUCT(BlueprintType)
struct FWorldPartitionStreamingQuerySource
{
	GENERATED_USTRUCT_BODY()

	FWorldPartitionStreamingQuerySource()
		: Location(FVector::ZeroVector)
		, Radius(0.f)
		, bUseGridLoadingRange(true)
		, bDataLayersOnly(false)
		, bIncludeAnyDataLayer(false)
		, bSpatialQuery(true)
		, Rotation(ForceInitToZero)
		, TargetBehavior(EStreamingSourceTargetBehavior::Include)
	{}

	FWorldPartitionStreamingQuerySource(const FVector& InLocation)
		: Location(InLocation)
		, Radius(0.f)
		, bUseGridLoadingRange(true)
		, bDataLayersOnly(false)
		, bIncludeAnyDataLayer(false)
		, bSpatialQuery(true)
		, Rotation(ForceInitToZero)
		, TargetBehavior(EStreamingSourceTargetBehavior::Include)
	{}

	/* Location to query. (not used if bSpatialQuery is false) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Query")
	FVector Location;

	/* Radius to query. (not used if bSpatialQuery is false) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Query")
	float Radius;

	/* If True, Instead of providing a query radius, query can be bound to loading range radius. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Query")
	bool bUseGridLoadingRange;

	/* Optional list of data layers to specialize the query. If empty only non data layer cells will be returned by the query. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Query", meta = (EditCondition = "!bIncludeAnyDataLayer"))
	TArray<FName> DataLayers;

	/* If True, Only cells that are in a data layer found in DataLayers property will be returned by the query. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Query")
	bool bDataLayersOnly;

	/* If True, Include cells that have DataLayers without caring for a match in the DataLayers array. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Query")
	bool bIncludeAnyDataLayer;
	

	/* If False, Location/Radius will not be used to find spatial cells. Non-spatial cells are always returned by the query, but can be further filtered by bDataLayersOnly. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Query")
	bool bSpatialQuery;

	/* Reserved settings used by UWorldPartitionStreamingSourceComponent::IsStreamingCompleted. */
	FRotator Rotation;
	
	/** Defines how TargetGrids will be applied to this streaming source. */
	EStreamingSourceTargetBehavior TargetBehavior;

	TSet<FName> TargetGrids;
	
	TArray<FStreamingSourceShape> Shapes;

	/** Helper method that iterates over all shapes. If none is provided, it will still pass a sphere shape using Radius or grid's loading range (see bUseGridLoadingRange). */
	inline void ForEachShape(float InGridLoadingRange, bool bInProjectIn2D, TFunctionRef<void(const FSphericalSector&)> InOperation) const
	{
		if (bSpatialQuery)
		{
			FStreamingSourceShapeHelper::ForEachShape(InGridLoadingRange, bUseGridLoadingRange ? InGridLoadingRange : Radius, bInProjectIn2D, Location, Rotation, Shapes, InOperation);
		}
	}

	/** Helper method that iterates over all shapes. If none is provided, it will still pass a sphere shape using Radius or grid's loading range (see bUseGridLoadingRange). */
	inline void ForEachShape(float InGridLoadingRange, FName InGridName, bool bInProjectIn2D, TFunctionRef<void(const FSphericalSector&)> InOperation) const
	{
		if (FStreamingSourceShapeHelper::IsSourceAffectingGrid(TargetGrids, TargetBehavior, InGridName))
		{
			ForEachShape(InGridLoadingRange, bInProjectIn2D, InOperation);
		}
	}
};

/**
 * Streaming Source Priority
 */
UENUM(BlueprintType)
enum class EStreamingSourcePriority : uint8
{
	Highest = 0,
	High = 64,
	Normal = 128,
	Low = 192,
	Lowest = 255,
	Default = Normal
};

/**
 * Structure containing all properties required to stream from a source
 */
struct FWorldPartitionStreamingSource
{
	FWorldPartitionStreamingSource()
		: Name(NAME_None)
		, Location(FVector::ZeroVector)
		, Rotation(FRotator::ZeroRotator)
		, TargetState(EStreamingSourceTargetState::Activated)
		, bBlockOnSlowLoading(false)
		, Priority(EStreamingSourcePriority::Default)
		, Velocity(FVector::Zero())
		, bUseVelocityContributionToCellsSorting(false)
		, DebugColor(ForceInit)
		, TargetBehavior(EStreamingSourceTargetBehavior::Include)
		, bReplay(false)
		, bRemote(false)
		, bForce2D(false)
		, Hash2D(0)
		, Hash3D(0)
		, QuantizedLocation(FVector::ZeroVector)
		, QuantizedRotation(FRotator::ZeroRotator)
		, ExtraRadius(0)
		, ExtraAngle(0)
	{}

	FWorldPartitionStreamingSource(FName InName, const FVector& InLocation, const FRotator& InRotation, EStreamingSourceTargetState InTargetState, bool bInBlockOnSlowLoading, EStreamingSourcePriority InPriority, bool bRemote, FVector InVelocity = FVector::Zero())
		: Name(InName)
		, Location(InLocation)
		, Rotation(InRotation)
		, TargetState(InTargetState)
		, bBlockOnSlowLoading(bInBlockOnSlowLoading)
		, Priority(InPriority)
		, Velocity(InVelocity)
		, bUseVelocityContributionToCellsSorting(false)
		, DebugColor(ForceInit)
		, TargetBehavior(EStreamingSourceTargetBehavior::Include)
		, bReplay(false)
		, bRemote(bRemote)
		, bForce2D(false)
		, Hash2D(0)
		, Hash3D(0)
		, QuantizedLocation(InLocation)
		, QuantizedRotation(InRotation)
		, ExtraRadius(0)
		, ExtraAngle(0)
	{}

	FColor GetDebugColor() const
	{
		if (!DebugColor.ToPackedBGRA())
		{
			return FColor::MakeRedToGreenColorFromScalar(FRandomStream(Name).GetFraction());
		}

		return FColor(DebugColor.R, DebugColor.G, DebugColor.B, 255);
	}

	ENGINE_API void UpdateHash();
	uint32 GetHash(bool bInclude3DInformation = true) const { return bInclude3DInformation ? Hash3D : Hash2D; }

	/** Source unique name. */
	FName Name;

	/** Source location. */
	FVector Location;

	/** Source orientation (can impact streaming cell prioritization). */
	FRotator Rotation;

	/** Target streaming state. */
	EStreamingSourceTargetState TargetState;

	/** Whether this source will be considered when world partition detects slow loading and waits for cell streaming to complete. */
	bool bBlockOnSlowLoading;

	/** Streaming source priority. */
	EStreamingSourcePriority Priority;

	/** Source velocity (computed automatically). */
	FVector Velocity;
	
	/** Use velocity contribution to streaming cells priority streaming. */
	bool bUseVelocityContributionToCellsSorting;

	/** Color used for debugging. */
	FColor DebugColor;

	/** Defines how TargetGrids will be applied to this streaming source. */
	EStreamingSourceTargetBehavior TargetBehavior;

	/** When set, this will change how this streaming source is applied to the provided runtime streaming grids based on the TargetBehavior. */
	TSet<FName> TargetGrids;

	/** Source internal shapes. When none are provided, a sphere is automatically used. It's radius is equal to grid's loading range and center equals source's location. */
	TArray<FStreamingSourceShape> Shapes;

	/** If true, this streaming source is from a replay recording */
	bool bReplay;

	/** If true, this streaming source is from a remote session */
	bool bRemote;

	/** If true, this streaming source will force intersecting cells in 2D */
	bool bForce2D;

	/** Returns a box encapsulating all shapes. */
	inline FBox CalcBounds(float InGridLoadingRange, FName InGridName, bool bCalcIn2D = false) const
	{
		FBox OutBounds(ForceInit);
		ForEachShape(InGridLoadingRange, InGridName, bCalcIn2D, [&OutBounds](const FSphericalSector& Sector)
		{
			OutBounds += Sector.CalcBounds();
		});
		return OutBounds;
	}

	/** Helper method that iterates over all shapes. If none is provided, it will still pass a sphere shape using grid's loading range. */
	inline void ForEachShape(float InGridLoadingRange, bool bInProjectIn2D, TFunctionRef<void(const FSphericalSector&)> InOperation) const
	{
		FStreamingSourceShapeHelper::ForEachShape(InGridLoadingRange, InGridLoadingRange, bInProjectIn2D, Location, Rotation, Shapes, InOperation, ExtraRadius, ExtraAngle);
	}

	/** Helper method that iterates over all shapes affecting a specific grid. If none is provided, it will still pass a sphere shape using grid's loading range. */
	inline void ForEachShape(float InGridLoadingRange, FName InGridName, bool bInProjectIn2D, TFunctionRef<void(const FSphericalSector&)> InOperation) const
	{
		if (FStreamingSourceShapeHelper::IsSourceAffectingGrid(TargetGrids, TargetBehavior, InGridName))
		{
			FStreamingSourceShapeHelper::ForEachShape(InGridLoadingRange, InGridLoadingRange, bInProjectIn2D, Location, Rotation, Shapes, InOperation, ExtraRadius, ExtraAngle);
		}
	}

	FString ToString() const;

	static int32 GetLocationQuantization() { return LocationQuantization; }
	static int32 GetRotationQuantization() { return RotationQuantization; }

private:
	static ENGINE_API int32 LocationQuantization;
	static ENGINE_API int32 RotationQuantization;
	static ENGINE_API FAutoConsoleVariableRef CVarLocationQuantization;
	static ENGINE_API FAutoConsoleVariableRef CVarRotationQuantization;

	static ENGINE_API int32 DebugDisplaySpeedUnit;
	static ENGINE_API FAutoConsoleVariableRef CVarDebugDisplaySpeedUnit;

	/** Hash of streaming source (used to detect changes) */
	uint32 Hash2D;
	uint32 Hash3D;

	/** Source values used for hash computations. */
	FVector QuantizedLocation;
	FRotator QuantizedRotation;

	/** Used internally for server streaming */
	float ExtraRadius;
	float ExtraAngle;

	friend struct FSetStreamingSourceExtraRadius;
	friend struct FSetStreamingSourceExtraAngle;
};

struct FSetStreamingSourceExtraRadius
{
private:
	FSetStreamingSourceExtraRadius(FWorldPartitionStreamingSource& InStreamingSource, float InExtraRadius)
	{
		if (ensure(InExtraRadius >= 0.f))
		{
			InStreamingSource.ExtraRadius = InExtraRadius;
		}
	}

	friend class UWorldPartitionSubsystem;
};

struct FSetStreamingSourceExtraAngle
{
private:
	FSetStreamingSourceExtraAngle(FWorldPartitionStreamingSource& InStreamingSource, float InExtraAngle)
	{
		if (ensure(InExtraAngle >= 0.f))
		{
			InStreamingSource.ExtraAngle = InExtraAngle;
		}
	}

	friend class UWorldPartitionSubsystem;
};

/**
 * Interface for world partition streaming sources
 */
struct IWorldPartitionStreamingSourceProvider
{
	virtual bool GetStreamingSource(FWorldPartitionStreamingSource& StreamingSource) const
	{
		return false;
	}

	virtual bool GetStreamingSources(TArray<FWorldPartitionStreamingSource>& StreamingSources) const
	{
		FWorldPartitionStreamingSource StreamingSource;
		if (GetStreamingSource(StreamingSource))
		{
			StreamingSources.Add(MoveTemp(StreamingSource));
			return true;
		}
		return false;
	}

	virtual const UObject* GetStreamingSourceOwner() const { return nullptr; }
};
