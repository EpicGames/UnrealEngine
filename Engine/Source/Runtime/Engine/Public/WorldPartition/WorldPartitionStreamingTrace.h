// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/WorldStreamingTrace.h"

// EXPERIMENTAL: The macros (UE_TRACE_WP_STREAMING_*) declared in this header are subject to change without deprecation.
// Code, build scripts, or capture commands depending on these names may need updates between engine versions.

#if UE_TRACE_WORLD_STREAMING_ENABLED

class ULevelStreaming;
class UWorld;
class UWorldPartitionRuntimeCell;
struct FWorldPartitionStreamingSource;
enum class ELevelStreamingState : uint8;

struct UE_EXPERIMENTAL(5.8, "World streaming trace API is experimental and subject to change.") FWorldPartitionStreamingTrace
{
public:
	ENGINE_API static void TraceStreamingStateChange(const UWorld* InWorld, const ULevelStreaming* InStreamingLevel, ELevelStreamingState InState);
	ENGINE_API static void TraceStreamingSourceUpdate(const UWorld* InWorld, const FWorldPartitionStreamingSource& InStreamingSource);
	ENGINE_API static void EndStreamingSourceUpdates(const UWorld* InWorld);
#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
	ENGINE_API static void TraceStreamingContainerPriorities(const UWorld* InWorld, TArrayView<const TPair<FGuid, float>> InCellPriorities);
#endif

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	ENGINE_API static UE::WorldStreamingTrace::EStreamingContainerState GetStreamingContainerState(ELevelStreamingState InState);
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

private:
	static void EnsureContainerHierarchyDescribed(UWorldStreamingTraceSubsystem* InSubsystem, uint64 InCellId, const UWorldPartitionRuntimeCell* InCell);
};

#define UE_TRACE_WP_STREAMING_CONTAINER_STATE(World, StreamingLevel, NewState) FWorldPartitionStreamingTrace::TraceStreamingStateChange(World, StreamingLevel, NewState);
#define UE_TRACE_WP_STREAMING_SOURCE_STATE(World, StreamingSource) FWorldPartitionStreamingTrace::TraceStreamingSourceUpdate(World, StreamingSource);
#define UE_TRACE_WP_STREAMING_SOURCE_END_UPDATES(World) FWorldPartitionStreamingTrace::EndStreamingSourceUpdates(World);
#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
#define UE_TRACE_WP_STREAMING_CONTAINER_PRIORITIES(World, CellPriorities) FWorldPartitionStreamingTrace::TraceStreamingContainerPriorities(World, CellPriorities);
#else
#define UE_TRACE_WP_STREAMING_CONTAINER_PRIORITIES(...)
#endif

#else // UE_TRACE_WORLD_STREAMING_ENABLED

#define UE_TRACE_WP_STREAMING_CONTAINER_STATE(...)
#define UE_TRACE_WP_STREAMING_SOURCE_STATE(...)
#define UE_TRACE_WP_STREAMING_SOURCE_END_UPDATES(...)
#define UE_TRACE_WP_STREAMING_CONTAINER_PRIORITIES(...)

#endif // UE_TRACE_WORLD_STREAMING_ENABLED
