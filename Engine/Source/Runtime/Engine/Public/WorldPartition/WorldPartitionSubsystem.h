// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartitionStreamingSource.h"
#include "Subsystems/WorldSubsystem.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "Streaming/StreamingWorldSubsystemInterface.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Misc/Guid.h"
#include "WorldPartitionSubsystem.generated.h"

class UGameInstance;
class ULevel;
class ULevelStreaming;
class UWorldPartition;
class UActorDescContainer;
class UWorldPartitionLevelStreamingDynamic;
class FWorldPartitionActorDesc;
class FWorldPartitionDraw2DContext;

enum class EWorldPartitionRuntimeCellState : uint8;
enum class ELevelStreamingState : uint8;
enum class ELevelStreamingTargetState : uint8;

/**
 * Helper to compute streaming source velocity based on position history.
 */
struct FStreamingSourceVelocity
{
	FStreamingSourceVelocity(const FName& InSourceName);
	void Invalidate() { bIsValid = false; }
	bool IsValid() { return bIsValid; }
	FVector GetAverageVelocity(const FVector& NewPosition, double CurrentTime);

private:
	bool bIsValid;
	FName SourceName;
	double LastUpdateTime;
	FVector LastPosition;
	FVector AvgVelocity;
};

// Container for the different domains of loading range scales
struct FDynamicLoadingRangeScales
{
	// Increases linearly with time
	float LinearScale = 1.0f;
	// Square root of LinearScale
	float Scale2D = 1.0f;
	// Cubic root of LinearScale
	float Scale3D = 1.0f;
};

/**
 * UWorldPartitionSubsystem
 */

UCLASS(MinimalAPI)
class UWorldPartitionSubsystem : public UTickableWorldSubsystem, public IStreamingWorldSubsystemInterface
{
	GENERATED_BODY()

public:
	ENGINE_API UWorldPartitionSubsystem();

	//~ Begin USubsystem Interface.
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;
	//~ End USubsystem Interface.

	//~ Begin UWorldSubsystem Interface.
	ENGINE_API virtual void OnWorldBeginPlay(UWorld& InWorld) override;
	ENGINE_API virtual void OnWorldComponentsUpdated(UWorld& World) override;
	//~ End UWorldSubsystem Interface.

	//~Begin IStreamingWorldSubsystemInterface
	ENGINE_API virtual void OnUpdateStreamingState() override;
	//~End IStreamingWorldSubsystemInterface

	//~ Begin FTickableGameObject
	ENGINE_API virtual void Tick(float DeltaSeconds) override;
	virtual bool IsTickableInEditor() const override { return true; }
	ENGINE_API virtual ETickableTickType GetTickableTickType() const override;
	ENGINE_API virtual TStatId GetStatId() const override;
	//~End FTickableGameObject

	UFUNCTION(BlueprintCallable, Category = Streaming)
	ENGINE_API bool IsStreamingCompleted(EWorldPartitionRuntimeCellState QueryState, const TArray<FWorldPartitionStreamingQuerySource>& QuerySources, bool bExactState) const;

	/** Returns true if world partition is done streaming levels, adding them to the world or removing them from the world. */
	UFUNCTION(BlueprintCallable, Category = Streaming)
	ENGINE_API bool IsAllStreamingCompleted();

	/*
	 * Returns true if world partition is done streaming levels, adding them to the world or removing them from the world. 
	 * When provided, the test is reduced to streaming levels affected by the optional streaming source provider.
	 */
	ENGINE_API bool IsStreamingCompleted(const IWorldPartitionStreamingSourceProvider* InStreamingSourceProvider = nullptr) const;

	ENGINE_API void DumpStreamingSources(FOutputDevice& OutputDevice) const;
	ENGINE_API void DumpWorldPartitions(FOutputDevice& OutputDevice) const;

	ENGINE_API TSet<IWorldPartitionStreamingSourceProvider*> GetStreamingSourceProviders() const;
	ENGINE_API void RegisterStreamingSourceProvider(IWorldPartitionStreamingSourceProvider* StreamingSource);
	ENGINE_API bool IsStreamingSourceProviderRegistered(IWorldPartitionStreamingSourceProvider* StreamingSource) const;
	ENGINE_API bool UnregisterStreamingSourceProvider(IWorldPartitionStreamingSourceProvider* StreamingSource);

	DECLARE_DELEGATE_RetVal_OneParam(bool, FWorldPartitionStreamingSourceProviderFilter, const IWorldPartitionStreamingSourceProvider*);
	FWorldPartitionStreamingSourceProviderFilter& OnIsStreamingSourceProviderFiltered() { return IsStreamingSourceProviderFiltered; }

	DECLARE_MULTICAST_DELEGATE(FWorldPartitionStreamingStateUpdated);
	FWorldPartitionStreamingStateUpdated& OnStreamingStateUpdated() { return StreamingStateUpdated; }

	static ENGINE_API TMulticastDelegate<void(UWorldPartitionSubsystem*, UWorld*)> OnWorldPartitionSubsystemInitialized;
	static ENGINE_API TMulticastDelegate<void(UWorldPartitionSubsystem*, UWorld*)> OnWorldPartitionSubsystemDeinitialized;

	ENGINE_API void ForEachWorldPartition(TFunctionRef<bool(UWorldPartition*)> Func);

#if !UE_BUILD_SHIPPING
	static ENGINE_API void SetOverrideLoadingRange(FName Name, int32 LoadingRange);
	static ENGINE_API bool GetOverrideLoadingRange(FName Name, int32& LoadingRange);
	static ENGINE_API uint32 GetOverriddenLoadingRangesEpoch();
#endif

#if WITH_EDITOR
	ENGINE_API FWorldPartitionActorFilter GetWorldPartitionActorFilter(const FString& InWorldPackage, EWorldPartitionActorFilterType InFilterTypes = EWorldPartitionActorFilterType::Loading) const;
	ENGINE_API TMap<FActorContainerID, TSet<FGuid>> GetFilteredActorsPerContainer(const FActorContainerID& InContainerID, const FString& InWorldPackage, const FWorldPartitionActorFilter& InActorFilter, EWorldPartitionActorFilterType InFilterTypes = EWorldPartitionActorFilterType::Loading);

	static ENGINE_API bool IsRunningConvertWorldPartitionCommandlet();

	// When enabled, returns a linear loading range scaling ramp to boost initial PIE availability, at the cost of popping. When disabled, the values are 1.f
	const FDynamicLoadingRangeScales& GetDynamicLoadingRangeScales() const { return LoadingRangeScales; }

	TSet<FWorldPartitionHandle> SelectedActorHandles;

private:
	ENGINE_API FWorldPartitionActorFilter GetWorldPartitionActorFilterInternal(const FString& InWorldPackage, EWorldPartitionActorFilterType InFilterTypes, TSet<FString>& InOutVisitedPackageStack) const;
#endif

protected:
	//~ Begin USubsystem Interface.
	ENGINE_API virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override;
	//~ End USubsystem Interface.

private:
#if !UE_BUILD_SHIPPING
	static TMap<FName, int32> OverriddenLoadingRanges;
	static uint32 OverriddenLoadingRangesEpoch;
	static class FAutoConsoleCommand OverrideLoadingRangeCommand;
#endif

	// Streaming Sources
	void UpdateStreamingSources();
	ENGINE_API void GetStreamingSources(const UWorldPartition* InWorldPartition, TArray<FWorldPartitionStreamingSource>& OutStreamingSources) const;
	ENGINE_API void UpdateStreamingSourcesVelocity(const UWorldPartition* InWorldPartition, TArray<FWorldPartitionStreamingSource>& InOutStreamingSources) const;
	uint32 GetStreamingSourcesHash() const { return StreamingSourcesHash; }

#if WITH_EDITOR
	void OnStartGameInstance(UGameInstance* GameInstance);
#endif // WITH_EDITOR
	void OnWorldPartitionInitialized(UWorldPartition* InWorldPartition);
	void OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition);
	void OnLevelStreamingStateChanged(UWorld* InWorld, const ULevelStreaming* InStreamingLevel, ULevel* LevelIfLoaded, ELevelStreamingState PreviousState, ELevelStreamingState NewState);
	void OnLevelStreamingTargetStateChanged(UWorld* InWorld, const ULevelStreaming* InStreamingLevel, ULevel* InLevelIfLoaded, ELevelStreamingState InCurrentState, ELevelStreamingTargetState InPrevTarget, ELevelStreamingTargetState InNewTarget);
	void OnLevelBeginMakingVisible(UWorld* InWorld, const ULevelStreaming* InStreamingLevel, ULevel* InLoadedLevel);
	void OnLevelBeginMakingInvisible(UWorld* InWorld, const ULevelStreaming* InStreamingLevel, ULevel* InLoadedLevel);
	void UpdateLoadingAndPendingLoadStreamingLevels(const ULevelStreaming* InStreamingLevel);
	bool IncrementalUpdateStreamingState();
	
	// Server information
	uint32 GetServerClientsVisibleLevelsHash() const { return ServerClientsVisibleLevelsHash; }
	void UpdateServerClientsVisibleLevelNames();

	static ENGINE_API void UpdateStreamingStateInternal(const UWorld* InWorld, UWorldPartition* InWorldPartition = nullptr);
	static bool IsServer(const UWorld* InWorld);

	ENGINE_API bool HasAnyWorldPartitionServerStreamingEnabled() const;
	ENGINE_API bool HasUninitializationPendingStreamingLevels(const UWorldPartition* InWorldPartition) const;

	ENGINE_API UWorldPartition* GetWorldPartition();
	ENGINE_API const UWorldPartition* GetWorldPartition() const;
	ENGINE_API bool CanDebugDraw() const;
	ENGINE_API void Draw(class UCanvas* Canvas, class APlayerController* PC);
	ENGINE_API void DrawStreamingStatusLegend(class UCanvas* Canvas, FVector2D& Offset, const UWorldPartition* InWorldPartition);
	friend class UWorldPartition;
	friend class UWorldPartitionStreamingPolicy;

#if WITH_EDITOR
	// If enabled, compute a linear loading range scaling ramp to boost initial PIE availability, at the cost of popping.
	void InitDynamicLoadingRangeScale();
	void UpdateDynamicLoadingRangeScale();

	float DynamicLoadingRangeScaleStartTime = -1.0f;
	FDynamicLoadingRangeScales LoadingRangeScales{1.f};
#endif // WITH_EDITOR

	// Registered world partitions & incremental update
	TArray<TObjectPtr<UWorldPartition>> RegisteredWorldPartitions;
	TSet<TObjectPtr<UWorldPartition>> IncrementalUpdateWorldPartitions;
	TSet<TObjectPtr<UWorldPartition>> IncrementalUpdateWorldPartitionsPendingAdd;

	// Streaming Sources
	TSet<IWorldPartitionStreamingSourceProvider*> StreamingSourceProviders;
	FWorldPartitionStreamingSourceProviderFilter IsStreamingSourceProviderFiltered;
	FWorldPartitionStreamingStateUpdated StreamingStateUpdated;
	TArray<FWorldPartitionStreamingSource> StreamingSources;
	TMap<FName, FStreamingSourceVelocity> StreamingSourcesVelocity;
	uint32 StreamingSourcesHash;

	// Server information
	int32 NumWorldPartitionServerStreamingEnabled;
	TSet<FName> ServerClientsVisibleLevelNames;
	uint32 ServerClientsVisibleLevelsHash;

	// Debug draw
	TArray<FWorldPartitionDraw2DContext> WorldPartitionsDraw2DContext;
	FDelegateHandle	DrawHandle;

	// GC backup values
	int32 LevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge;
	int32 LevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge;
	int32 LevelStreamingForceGCAfterLevelStreamedOut;

	// Tracks streaming levels of uninitialized world partition used to delay next initialization until they're done being removed
	TMap<FSoftObjectPath, TSet<TWeakObjectPtr<UWorldPartitionLevelStreamingDynamic>>> WorldPartitionUninitializationPendingStreamingLevels;

	// Tracks world partition loading and pending loads
	TSet<TWeakObjectPtr<const ULevelStreaming>> WorldPartitionLoadingAndPendingLoadStreamingLevels;

	bool bHasBegunPlay = false;

#if WITH_EDITOR
	bool bIsRunningConvertWorldPartitionCommandlet;
#endif
};
