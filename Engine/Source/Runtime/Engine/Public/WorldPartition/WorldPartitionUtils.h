// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "UObject/WeakObjectPtr.h"
#include "Templates/SubclassOf.h"
#include "Misc/ArchiveMD5.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "Memory/MemoryView.h"

class UWorld;
class UWorldPartition;
class IWorldPartitionCell;
class FWorldPartitionCookPackageContext;
struct FWorldPartitionStreamingQuerySource;
class AActor;
class URuntimeHashExternalStreamingObjectBase;

struct FWorldPartitionUtils
{
	struct FSimulateCookSessionParams
	{
		TArray<TSubclassOf<AActor>> FilteredClasses;
	};

	class FSimulateCookedSession
	{
	public:

		ENGINE_API FSimulateCookedSession(UWorld* InWorld, const FSimulateCookSessionParams& Params = FSimulateCookSessionParams());
		ENGINE_API ~FSimulateCookedSession();

		bool IsValid() const { return !!CookContext; }
		ENGINE_API bool ForEachStreamingCells(TFunctionRef<void(const IWorldPartitionCell*)> Func);
		ENGINE_API bool GetIntersectingCells(const TArray<FWorldPartitionStreamingQuerySource>& InSources, TArray<const IWorldPartitionCell*>& OutCells);

	private:
		ENGINE_API bool SimulateCook(const FSimulateCookSessionParams& Params);

		TArray<TObjectPtr<URuntimeHashExternalStreamingObjectBase>> InjectedStreamingObjects;
		FWorldPartitionCookPackageContext* CookContext;
		TWeakObjectPtr<UWorldPartition> WorldPartition;
	};
};

namespace UE::Private::WorldPartition
{
	// Hash helpers
	template<class T, class HashBuilder> void UpdateHash(HashBuilder& Builder, const T& Value);

	template<class HashBuilder> void UpdateHash(HashBuilder& Builder, const FGuid& Value)
	{
		static_assert(sizeof(FGuid) == 4 * sizeof(uint32));
		Builder.Update(&Value, sizeof(FGuid));
	}

	template<class HashBuilder> void UpdateHash(HashBuilder& Builder, const FName& Value)
	{
		TCHAR NameBuffer[FName::StringBufferSize];
		uint32 Length = Value.ToStringTruncate(NameBuffer);
		Builder.Update(MakeMemoryView(NameBuffer, Length * sizeof(TCHAR)));
	}

	template<class HashBuilder> void UpdateHash(HashBuilder& Builder, const FString& Value)
	{
		Builder.Update(*Value, Value.Len() * sizeof(FString::ElementType));
	}


	template<class HashBuilder> void UpdateHash(HashBuilder& Builder, const FTopLevelAssetPath& Value)
	{
		UpdateHash(Builder, Value.GetPackageName());
		UpdateHash(Builder, Value.GetAssetName());
	}

	template<class HashBuilder> void UpdateHash(HashBuilder& Builder, const FTransform& Value)
	{
		Builder.Update(&Value, sizeof(FTransform));
	}

	template<class HashBuilder> void UpdateHash(HashBuilder& Builder, bool Value)
	{
		Builder.Update(&Value, sizeof(bool));
	}


	template<class HashBuilder> void UpdateHash(HashBuilder& Builder, const FActorContainerID& Value)
	{
		static_assert(sizeof(FActorContainerID) == sizeof(FGuid));
		Builder.Update(&Value, sizeof(FGuid));
	}

	template<class T, class HashBuilder> void UpdateHash(HashBuilder& Builder, const TArray<T>& Values)
	{
		for (const T& Val : Values)
		{
			UpdateHash(Builder, Val);
		}
	}
}

#endif
