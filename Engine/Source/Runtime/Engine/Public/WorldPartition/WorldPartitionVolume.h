// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Volume.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "WorldPartitionVolume.generated.h"

class FLoaderAdapterActor;

UCLASS(Deprecated, meta = (DeprecationMessage = "WorldPartitionVolume has been replaced by LocationVolume"), MinimalAPI)
class ADEPRECATED_WorldPartitionVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

private:
#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return false; }
#endif

public:
	//~ Begin AActor Interface
	virtual bool IsEditorOnly() const override { return true; }
#if WITH_EDITOR
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
#endif
	//~ End AActor Interface
};
