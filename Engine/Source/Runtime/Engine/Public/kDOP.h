// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ChunkedArray.h"
#include "Templates/UnrealTemplate.h"

// Indicates how many "k / 2" there are in the k-DOP. 3 == AABB == 6 DOP. The code relies on this being 3.
#define NUM_PLANES	3

template <typename COLL_DATA_PROVIDER, typename KDOP_IDX_TYPE> struct TkDOPLineCollisionCheck;

struct FkHitResult
{
	/** Normal vector in coordinate system of the returner. Zero==none.	*/
	FVector4	Normal;
	/** Time until hit.													*/
	float		Time;
	/** Primitive data item which was hit, INDEX_NONE=none				*/
	int32			Item;		

	FkHitResult()
	: Normal	(0,0,0,0)
	, Time		(1.0f)
	, Item		(INDEX_NONE)
	{
	}
};

/**
 * Stores XYZ from 4 vectors in one Struct Of Arrays.
 */
struct FVector3SOA
{
	/** X = (v0.x, v1.x, v2.x, v3.x) */
	VectorRegister	X;
	/** Y = (v0.y, v1.y, v2.y, v3.y) */
	VectorRegister	Y;
	/** Z = (v0.z, v1.z, v2.z, v3.z) */
	VectorRegister	Z;
};

/**
 * Stores XYZW from 4 vectors in one Struct Of Arrays.
 */
struct FVector4SOA
{
	/** X = (v0.x, v1.x, v2.x, v3.x) */
	VectorRegister	X;
	/** Y = (v0.y, v1.y, v2.y, v3.y) */
	VectorRegister	Y;
	/** Z = (v0.z, v1.z, v2.z, v3.z) */
	VectorRegister	Z;
	/** W = (v0.w, v1.w, v2.w, v3.w) */
	VectorRegister	W;
};

/**
 * Stores 4 triangles in one struct (Struct Of Arrays).
 */
struct FTriangleSOA
{
	/** 3 vertex positions for each of the 4 triangles. */
	FVector3SOA	Positions[3];
	/** Triangle normal (including -W for a plane) for each of the 4 triangles. */
	FVector4SOA	Normals;
	/** A 32-bit payload value for each of the 4 triangles. */
	uint32		Payload[4];
};

/** Tracks information about a traversal through the kDOP tree. */
template<typename KDOP_IDX_TYPE>
struct TTraversalHistory
{
	/** 
	 * Number of parent nodes to track information for. 
	 * This has been tweaked for optimal boolean shadow ray performance.
	 */
	enum { NodeHistoryLength = 1 };
	KDOP_IDX_TYPE Nodes[NodeHistoryLength];

	TTraversalHistory()
	{
		for (int32 NodeIndex = 0; NodeIndex < NodeHistoryLength; NodeIndex++)
		{
			// Initialize the history to invalid index
			Nodes[NodeIndex] = 0xFFFFFFFF;
		}
	}

	TTraversalHistory AddNode(KDOP_IDX_TYPE NewNodeIndex) const
	{
		TTraversalHistory NewHistory;
		// Move all the indices toward the end of the array one element
		CopyAssignItems(NewHistory.Nodes + 1, Nodes, NodeHistoryLength - 1);
		// Insert the new node at the beginning of the array
		NewHistory.Nodes[0] = NewNodeIndex;
		return NewHistory;
	}

	KDOP_IDX_TYPE GetOldestNode() const
	{
		// Accesses the oldest node that this traversal history is tracking
		return Nodes[NodeHistoryLength - 1];
	}
};

/**
 * Line vs triangle intersection test.
 *
 * @param Start		Start of the line segment
 * @param End		End of the line segment
 * @param Dir		Direction of the line segment (not normalized, just End-Start)
 * @param V0		First vertex of the triangle
 * @param V1		Second vertex of the triangle
 * @param V2		Third vertex of the triangle
 * @param Normal	Triangle normal (including +W for the plane equation)
 * @param IntersectionTime	[in/out] Best intersection time so far (0..1), as in: IntersectionPoint = Start + IntersectionTime * Dir.
 * @return			true if the line intersected the triangle
 */
inline bool appLineCheckTriangle(const FVector4& Start, const FVector4& End, const FVector4& Dir, const FVector4& V0, const FVector4& V1, const FVector4& V2, const FVector4& Normal, float& IntersectionTime)
{
	const float StartDist = FPlane(Normal).PlaneDot(Start);
	const float EndDist = FPlane(Normal).PlaneDot(End);

	// Check if the line is completely on one side of the triangle, or if it's co-planar.
#if 1
	if ((StartDist == EndDist) || (StartDist < -0.001f && EndDist < -0.001f) || (StartDist > 0.001f && EndDist > 0.001f))
#else
	if ( (StartDist * EndDist) > -0.0001f )
#endif
	{
		return false;
	}

	// Figure out when it will hit the triangle
	float Time = -StartDist / (EndDist - StartDist);

	// If this triangle is not closer than the previous hit, reject it
	if (Time < 0.f || Time >= IntersectionTime)
	{
		return false;
	}

	// Calculate the line's point of intersection with the node's plane
	const FVector4& Intersection = Start + Dir * Time;
	const FVector4* Verts[3] = 
	{ 
		&V0, &V1, &V2
	};

	// Check if the point of intersection is inside the triangle's edges.
	for( int32 SideIndex = 0; SideIndex < 3; SideIndex++ )
	{
		const FVector4& SideDirection = Normal ^ (*Verts[(SideIndex + 1) % 3] - *Verts[SideIndex]);
		const float SideW = Dot3(SideDirection, *Verts[SideIndex]);
		const float DotW = Dot3(SideDirection, Intersection);
		if ((DotW - SideW) >= 0.001f)
		{
			return false;
		}
	}
	IntersectionTime = Time;
	return true;
}

/** ( -0.0001f, -0.0001f, -0.0001f, -0.0001f ) */
static const VectorRegister GSmallNegativeNumber = MakeVectorRegister(-0.0001f, -0.0001f, -0.0001f, -0.0001f); //-V1043
//extern const VectorRegister GSmallNegativeNumber;

/** ( 0.0001f, 0.0001f, 0.0001f, 0.0001f ) */
static const VectorRegister GSmallNumber = MakeVectorRegister( 0.0001f, 0.0001f, 0.0001f, 0.0001f ); //-V1043
//extern const VectorRegister GSmallNumber;

static const VectorRegister GZeroVectorRegister = MakeVectorRegister( 0.f, 0.f, 0.f, 0.f ); //-V1043

static const VectorRegister VectorNegativeOne = MakeVectorRegister( -1.0f, -1.0f, -1.0f, -1.0f ); //-V1043

/**
 * Line vs triangle intersection test. Tests 1 line against 4 triangles at once.
 *
 * @param Start		Start of the line segment
 * @param End		End of the line segment
 * @param Dir		Direction of the line segment (not normalized, just End-Start)
 * @param Triangle4	Four triangles
 * @param IntersectionTime	[in/out] Best intersection time so far (0..1), as in: IntersectionPoint = Start + IntersectionTime * Dir.
 * @return			Index (0-3) to specify which of the 4 triangles the line intersected, or -1 if none was found.
 */
static int32 appLineCheckTriangleSOA(const FVector3SOA& Start, const FVector3SOA& End, const FVector3SOA& Dir, const FTriangleSOA& Triangle4, float& InOutIntersectionTime)
{
	VectorRegister TriangleMask;

 	VectorRegister StartDist;
	StartDist = VectorMultiplyAdd( Triangle4.Normals.X, Start.X, Triangle4.Normals.W );
	StartDist = VectorMultiplyAdd( Triangle4.Normals.Y, Start.Y, StartDist );
	StartDist = VectorMultiplyAdd( Triangle4.Normals.Z, Start.Z, StartDist );

	VectorRegister EndDist;
	EndDist = VectorMultiplyAdd( Triangle4.Normals.X, End.X, Triangle4.Normals.W );
	EndDist = VectorMultiplyAdd( Triangle4.Normals.Y, End.Y, EndDist );
	EndDist = VectorMultiplyAdd( Triangle4.Normals.Z, End.Z, EndDist );

	// Are both end-points of the line on the same side of the triangle (or parallel to the triangle plane)?
	TriangleMask = VectorMask_LE( VectorMultiply( StartDist, EndDist ), GSmallNegativeNumber );
	if ( VectorMaskBits(TriangleMask) == 0 )
	{
		return -1;
	}

	// Figure out when it will hit the triangle
	VectorRegister Time = VectorDivide( StartDist, VectorSubtract(StartDist, EndDist) );

	// If this triangle is not closer than the previous hit, reject it
	VectorRegister IntersectionTime = VectorLoadFloat1( &InOutIntersectionTime );
	TriangleMask = VectorBitwiseAnd( TriangleMask, VectorMask_GE( Time, VectorZero() ) );
	TriangleMask = VectorBitwiseAnd( TriangleMask, VectorMask_LT( Time, IntersectionTime ) );
	if ( VectorMaskBits(TriangleMask) == 0 )
	{
		return -1;
	}

	// Calculate the line's point of intersection with the node's plane
	const VectorRegister IntersectionX = VectorMultiplyAdd( Dir.X, Time, Start.X );
	const VectorRegister IntersectionY = VectorMultiplyAdd( Dir.Y, Time, Start.Y );
	const VectorRegister IntersectionZ = VectorMultiplyAdd( Dir.Z, Time, Start.Z );

	// Check if the point of intersection is inside the triangle's edges.
	for( int32 SideIndex = 0; SideIndex < 3; SideIndex++ )
	{
		const VectorRegister EdgeX = VectorSubtract( Triangle4.Positions[(SideIndex + 1) % 3].X, Triangle4.Positions[SideIndex].X );
		const VectorRegister EdgeY = VectorSubtract( Triangle4.Positions[(SideIndex + 1) % 3].Y, Triangle4.Positions[SideIndex].Y );
		const VectorRegister EdgeZ = VectorSubtract( Triangle4.Positions[(SideIndex + 1) % 3].Z, Triangle4.Positions[SideIndex].Z );
		const VectorRegister SideDirectionX = VectorNegateMultiplyAdd( Triangle4.Normals.Z, EdgeY, VectorMultiply( Triangle4.Normals.Y, EdgeZ ) );
		const VectorRegister SideDirectionY = VectorNegateMultiplyAdd( Triangle4.Normals.X, EdgeZ, VectorMultiply( Triangle4.Normals.Z, EdgeX ) );
		const VectorRegister SideDirectionZ = VectorNegateMultiplyAdd( Triangle4.Normals.Y, EdgeX, VectorMultiply( Triangle4.Normals.X, EdgeY ) );
		VectorRegister SideW;
		SideW = VectorMultiply( SideDirectionX, Triangle4.Positions[SideIndex].X );
		SideW = VectorMultiplyAdd( SideDirectionY, Triangle4.Positions[SideIndex].Y, SideW );
		SideW = VectorMultiplyAdd( SideDirectionZ, Triangle4.Positions[SideIndex].Z, SideW );
		VectorRegister DotW;
		DotW = VectorMultiply( SideDirectionX, IntersectionX );
		DotW = VectorMultiplyAdd( SideDirectionY, IntersectionY, DotW );
		DotW = VectorMultiplyAdd( SideDirectionZ, IntersectionZ, DotW );
		TriangleMask = VectorBitwiseAnd( TriangleMask, VectorMask_LT( VectorSubtract(DotW, SideW), GSmallNumber ) );
		if ( VectorMaskBits(TriangleMask) == 0 )
		{
			return -1;
		}
	}

	// Set all non-hitting times to 1.0
	Time = VectorSelect( TriangleMask, Time, VectorOne() );

	// Get the best intersection time out of the 4 possibilities.
	VectorRegister BestTimes = VectorMin( Time, VectorSwizzle(Time, 2, 3, 0, 0) );
	BestTimes = VectorMin( BestTimes, VectorSwizzle(BestTimes, 1, 0, 0, 0) );
	IntersectionTime = VectorReplicate( BestTimes, 0 );

	// Get the triangle index that corresponds to the best time.
	// NOTE: This will pick the first triangle, in case there are multiple hits at the same spot.
	int32 SubIndex = VectorMaskBits( VectorMask_EQ( Time, IntersectionTime ) );
	SubIndex = appCountTrailingZeros( SubIndex );

	// Return results.
	VectorStoreFloat1( IntersectionTime, &InOutIntersectionTime );
	return SubIndex;
}

// This structure is used during the build process. It contains the triangle's
// centroid for calculating which plane it should be split or not with
template<typename KDOP_IDX_TYPE>
struct FkDOPBuildCollisionTriangle
{
	/**
	 * First vertex in the triangle
	 */
	FVector4 V0;
	/**
	 * Second vertex in the triangle
	 */
	FVector4 V1;
	/**
	 * Third vertex in the triangle
	 */
	FVector4 V2;

	/** The material of this triangle */
	KDOP_IDX_TYPE MaterialIndex;

	inline FVector4 GetCentroid() const
	{
		return (V0 + V1 + V2) / 3.f;
	}
	FVector4 GetLocalNormal() const
	{
		FVector4 LocalNormal = ((V1 - V2) ^ (V0 - V2)).GetSafeNormal();
		LocalNormal.W = Dot3(V0, LocalNormal);
		return LocalNormal;
	}

	/**
	 * Sets the indices, material index, calculates the centroid using the
	 * specified triangle vertex positions
	 */
	FkDOPBuildCollisionTriangle(
		KDOP_IDX_TYPE InMaterialIndex,
		const FVector4& vert0,const FVector4& vert1,const FVector4& vert2) :
		V0(vert0), V1(vert1), V2(vert2),
		MaterialIndex(InMaterialIndex)
	{
	}
};

// Forward declarations
template <typename COLL_DATA_PROVIDER,typename KDOP_IDX_TYPE> struct TkDOPNode;
template <typename COLL_DATA_PROVIDER,typename KDOP_IDX_TYPE> struct TkDOPTree;
template <typename COLL_DATA_PROVIDER,typename KDOP_IDX_TYPE> struct TkDOPLineCollisionCheck;

/**
 * Holds the min/max planes that make up a set of 4 bounding volumes.
 */
struct FFourBox
{
	/**
	 * Min planes for this set of bounding volumes. Array index is X/Y/Z.
	 */
	MS_ALIGN(16) FVector4 Min[3]  GCC_ALIGN(16);

	/** 
	 * Max planes for this set of bounding volumes. Array index is X/Y/Z.
	 */
	FVector4 Max[3];

	/**
	 * Sets the box at the passed in index to the passed in box.
	 *
	 * @param	BoundingVolumeIndex		Index of box to set
	 * @param	Box						Box to set
	 */
	void SetBox( int32 BoundingVolumeIndex, const FBox& Box )
	{
		Min[0].Component(BoundingVolumeIndex) = Box.Min.X;
		Min[1].Component(BoundingVolumeIndex) = Box.Min.Y;
		Min[2].Component(BoundingVolumeIndex) = Box.Min.Z;
		Max[0].Component(BoundingVolumeIndex) = Box.Max.X;
		Max[1].Component(BoundingVolumeIndex) = Box.Max.Y;
		Max[2].Component(BoundingVolumeIndex) = Box.Max.Z;
	}

	/**
	 * Returns the bounding volume at the passed in index.
	 *
	 * @param	BoundingVolumeIndex		Index of bounding volume to return
	 *
	 * @return Bounding volume at the passed in index
	 */
	FBox GetBox( int32 BoundingVolumeIndex )
	{
		FBox Box;
		Box.Min = FVector4(Min[0][BoundingVolumeIndex],Min[1][BoundingVolumeIndex],Min[2][BoundingVolumeIndex],1);
		Box.Max = FVector4(Max[0][BoundingVolumeIndex],Max[1][BoundingVolumeIndex],Max[2][BoundingVolumeIndex],1);
		return Box;
	}
};

#define kDOPArray TArray

/**
 * A node in the kDOP tree. The node contains the kDOP volume that encompasses
 * it's children and/or triangles
 */
template <typename COLL_DATA_PROVIDER, typename KDOP_IDX_TYPE>
struct TkDOPNode
{
	/** Exposes data provider type to clients. */
	typedef COLL_DATA_PROVIDER							DataProviderType;

	/** Exposes node type to clients. */
	typedef TkDOPNode<DataProviderType,KDOP_IDX_TYPE>	NodeType;

	/** Set of bounding volumes for child nodes. */
	FFourBox BoundingVolumes;

	// Note this isn't smaller since 4 byte alignment will take over anyway
	bool bIsLeaf;
	uint8 Occupancy;

	// Union of either child kDOP nodes or a list of enclosed triangles
	union
	{
		// This structure contains the left and right child kDOP indices
		// These index values correspond to the array in the FkDOPTree
		struct
		{
			KDOP_IDX_TYPE LeftNode;
			KDOP_IDX_TYPE RightNode;
		} n;
		// This structure contains the list of enclosed triangles
		// These index values correspond to the triangle information in the
		// FkDOPTree using the start and count as the means of delineating
		// which triangles are involved
		struct
		{
			KDOP_IDX_TYPE NumTriangles;
			KDOP_IDX_TYPE StartIndex;
		} t;
	};

	/**
	 * Inits the data to no child nodes and an inverted volume
	 */
	inline TkDOPNode()
	{
		n.LeftNode = ((KDOP_IDX_TYPE) -1);
        n.RightNode = ((KDOP_IDX_TYPE) -1);
	}

	/**
	 * Determines if the node is a leaf or not. If it is not a leaf, it subdivides
	 * the list of triangles again adding two child nodes and splitting them on
	 * the mean (splatter method). Otherwise it sets up the triangle information.
	 *
	 * @param Start -- The triangle index to start processing with
	 * @param NumTris -- The number of triangles to process
	 * @param BuildTriangles -- The list of triangles to use for the build process
	 * @param Nodes -- The list of nodes in this tree
	 * @return bounding box for this node
	 */
	FBox SplitTriangleList(int32 Start,int32 NumTris,
		TArray<FkDOPBuildCollisionTriangle<KDOP_IDX_TYPE> >& BuildTriangles,
		kDOPArray<FTriangleSOA>& SOATriangles,
		kDOPArray<NodeType>& Nodes)
	{
		// Figure out if we are a leaf node or not
		if (NumTris > 4)
		{
			// Still too many triangles, so continue subdividing the triangle list
			bIsLeaf = 0;
			Occupancy = 0;
			int32 BestPlane = -1;
			float BestMean = 0.f;
			float BestVariance = 0.f;

			// Determine how to split using the splatter algorithm
			{
				double Mean[NUM_PLANES] = { 0 };
				double Variance[NUM_PLANES] = { 0 };

				// Compute the mean for the triangle list
				for (int32 nTriangle = Start; nTriangle < Start + NumTris; nTriangle++)
				{
					// Project the centroid of the triangle against the plane
					// normals and accumulate to find the total projected
					// weighting
					FVector4 Centroid = BuildTriangles[nTriangle].GetCentroid();

					for (int32 nPlane = 0; nPlane < NUM_PLANES; nPlane++)
					{
						Mean[nPlane] += Centroid[nPlane];
					}
				}

				// Divide by the number of triangles to get the average
				for (int32 nPlane = 0; nPlane < NUM_PLANES; nPlane++)
				{
					Mean[nPlane] /= NumTris;
				}

				// Compute variance of the triangle list
				for (int32 nTriangle = Start; nTriangle < Start + NumTris; nTriangle++)
				{
					// Project the centroid again
					FVector4 Centroid = BuildTriangles[nTriangle].GetCentroid();

					// Now calculate the variance and accumulate it
					for (int32 nPlane = 0; nPlane < NUM_PLANES; nPlane++)
					{
						Variance[nPlane] += (Centroid[nPlane] - Mean[nPlane]) * (Centroid[nPlane] - Mean[nPlane]);
					}
				}

				// Determine which plane is the best to split on
				for (int32 nPlane = 0; nPlane < NUM_PLANES; nPlane++)
				{
					// Get the average variance
					Variance[nPlane] /= NumTris;
					if (Variance[nPlane] >= BestVariance)
					{
						BestPlane = nPlane;
						BestVariance = (float)Variance[nPlane];
						BestMean = (float)Mean[nPlane];
					}
				}
			}

			// Now that we have the plane to split on, work through the triangle
			// list placing them on the left or right of the splitting plane
			int32 Left = Start - 1;
			int32 Right = Start + NumTris;
			// Keep working through until the left index passes the right; we test that condition after each interior loop
			for (;;)
			{
				// Loop invariants: (1) Left < Right,
				// (2) All triangles <= Left belong on the left, all triangles >= Right belong on the right
				// (3) Left+1 is an untested triangle, Right-1 is an untested triangle
				float Dot;
				// Increment Left until it points to triangle that belongs on the right, or Right==Left
				for (++Left; Left < Right; ++Left)
				{
					Dot = BuildTriangles[Left].GetCentroid()[BestPlane];
					if (Dot < BestMean)
					{
						break;
					}
				}
				if (Left == Right)
				{
					break;
				}
				// Decrement Right until it points to triangle that belongs on the left, or Right==Left
				for (--Right; Left < Right; --Right)
				{
					Dot = BuildTriangles[Right].GetCentroid()[BestPlane];
					if (Dot >= BestMean)
					{
						break;
					}
				}
				if (Left == Right)
				{
					break;
				}
				// Left points to a triangle that belongs on the Right; Right points to a triangle that belongs on the Left. Swap them.
				Swap(BuildTriangles[Left], BuildTriangles[Right]);
			}
			// After loop array is partitioned and Left is the first index that belongs on the right side of the plane.
			// Check for wacky degenerate case where more than GKDOPMaxTrisPerLeaf
			// fall all in the same kDOP
			if (Left == Start + NumTris || Right == Start)
			{
				Left = Start + (NumTris / 2);
			}
			// Add the two child nodes
			n.LeftNode = Nodes.AddZeroed(2);
			n.RightNode = n.LeftNode + 1;
			// Have the left node recursively subdivide it's list and set bounding volume.
			FBox LeftBoundingVolume = Nodes[n.LeftNode].SplitTriangleList(Start,Left - Start,BuildTriangles,SOATriangles,Nodes);
			BoundingVolumes.SetBox(0,LeftBoundingVolume);
			// Set unused index 2,3 to child nodes of left node.
			BoundingVolumes.SetBox(2,Nodes[n.LeftNode].BoundingVolumes.GetBox(0));
			BoundingVolumes.SetBox(3,Nodes[n.LeftNode].BoundingVolumes.GetBox(1));

			// And now have the right node recursively subdivide it's list and set bounding volume.			
			FBox RightBoundingVolume = Nodes[n.RightNode].SplitTriangleList(Left,Start + NumTris - Left,BuildTriangles,SOATriangles,Nodes);
			BoundingVolumes.SetBox(1,RightBoundingVolume);

			// Non-leaf node bounds are the "sum" of the left and right nodes' volumes.
			return LeftBoundingVolume + RightBoundingVolume;
		}
		else
		{
			// Build SOA triangles

			// "NULL triangle", used when a leaf can't fill all 4 triangles in a FTriangleSOA.
			// No line should ever hit these triangles, set the values so that it can never happen.
			FkDOPBuildCollisionTriangle<KDOP_IDX_TYPE> EmptyTriangle(0,FVector4(0,0,0,0),FVector4(0,0,0,0),FVector4(0,0,0,0));
			
			t.StartIndex = SOATriangles.Num();
			t.NumTriangles = Align<int32>(NumTris, 4) / 4;
			SOATriangles.AddZeroed( t.NumTriangles );

			int32 BuildTriIndex = Start;
			for ( uint32 SOAIndex=0; SOAIndex < t.NumTriangles; ++SOAIndex )
			{
				FkDOPBuildCollisionTriangle<KDOP_IDX_TYPE>* Tris[4] = { &EmptyTriangle, &EmptyTriangle, &EmptyTriangle, &EmptyTriangle };
				FTriangleSOA& SOA = SOATriangles[t.StartIndex + SOAIndex];
				int32 SubIndex = 0;
				for ( ; SubIndex < 4 && BuildTriIndex < (Start+NumTris); ++SubIndex, ++BuildTriIndex )
				{
					Tris[SubIndex] = &BuildTriangles[BuildTriIndex];
					SOA.Payload[SubIndex] = Tris[SubIndex]->MaterialIndex;
				}
				for ( ; SubIndex < 4; ++SubIndex )
				{
					SOA.Payload[SubIndex] = 0xffffffff;
				}

				SOA.Positions[0].X = VectorSet( Tris[0]->V0.X, Tris[1]->V0.X, Tris[2]->V0.X, Tris[3]->V0.X );
				SOA.Positions[0].Y = VectorSet( Tris[0]->V0.Y, Tris[1]->V0.Y, Tris[2]->V0.Y, Tris[3]->V0.Y );
				SOA.Positions[0].Z = VectorSet( Tris[0]->V0.Z, Tris[1]->V0.Z, Tris[2]->V0.Z, Tris[3]->V0.Z );
				SOA.Positions[1].X = VectorSet( Tris[0]->V1.X, Tris[1]->V1.X, Tris[2]->V1.X, Tris[3]->V1.X );
				SOA.Positions[1].Y = VectorSet( Tris[0]->V1.Y, Tris[1]->V1.Y, Tris[2]->V1.Y, Tris[3]->V1.Y );
				SOA.Positions[1].Z = VectorSet( Tris[0]->V1.Z, Tris[1]->V1.Z, Tris[2]->V1.Z, Tris[3]->V1.Z );
				SOA.Positions[2].X = VectorSet( Tris[0]->V2.X, Tris[1]->V2.X, Tris[2]->V2.X, Tris[3]->V2.X );
				SOA.Positions[2].Y = VectorSet( Tris[0]->V2.Y, Tris[1]->V2.Y, Tris[2]->V2.Y, Tris[3]->V2.Y );
				SOA.Positions[2].Z = VectorSet( Tris[0]->V2.Z, Tris[1]->V2.Z, Tris[2]->V2.Z, Tris[3]->V2.Z );

				const FVector4& Tris0LocalNormal = Tris[0]->GetLocalNormal();
				const FVector4& Tris1LocalNormal = Tris[1]->GetLocalNormal();
				const FVector4& Tris2LocalNormal = Tris[2]->GetLocalNormal();
				const FVector4& Tris3LocalNormal = Tris[3]->GetLocalNormal();

				SOA.Normals.X = VectorSet( Tris0LocalNormal.X, Tris1LocalNormal.X, Tris2LocalNormal.X, Tris3LocalNormal.X );
				SOA.Normals.Y = VectorSet( Tris0LocalNormal.Y, Tris1LocalNormal.Y, Tris2LocalNormal.Y, Tris3LocalNormal.Y );
				SOA.Normals.Z = VectorSet( Tris0LocalNormal.Z, Tris1LocalNormal.Z, Tris2LocalNormal.Z, Tris3LocalNormal.Z );
				SOA.Normals.W = VectorSet( -Tris0LocalNormal.W, -Tris1LocalNormal.W, -Tris2LocalNormal.W, -Tris3LocalNormal.W );
			}

			// No need to subdivide further so make this a leaf node
			bIsLeaf = 1;
			Occupancy = NumTris;
			
			// Generate bounding volume for leaf which is passed up the call chain.
			FBox BoundingVolume(ForceInit);
			for (int32 TriangleIndex=Start; TriangleIndex<Start + NumTris; TriangleIndex++)
			{
				BoundingVolume += FVector(BuildTriangles[TriangleIndex].V0);
				BoundingVolume += FVector(BuildTriangles[TriangleIndex].V1);
				BoundingVolume += FVector(BuildTriangles[TriangleIndex].V2);
			}
			BoundingVolumes.SetBox(0,BoundingVolume);
			BoundingVolumes.SetBox(1,BoundingVolume);
			BoundingVolumes.SetBox(2,BoundingVolume);
			BoundingVolumes.SetBox(3,BoundingVolume);

			return BoundingVolume;
		}
	}

	/**
	 * The slab testing algorithm is based on the following papers. We chose to use the
	 * faster final hit determination, which means we'll get some false positives.
     *
	 * "A Cross-Platform Framework for Interactive Ray Tracing"
	 * Markus Geimer, Stefan Mueller
	 * http://www.uni-koblenz.de/~cg/publikationen/cp_raytrace.pdf
	 *
  	 * "Efficiency Issues for Ray Tracing"
	 * Brian Smits
	 * http://www.cs.utah.edu/~bes/papers/fastRT/paper-node10.html
	 *
	 * We don't need to handle +/- INF, but we have permission to use Thierry 
	 * Berger-Perrin trick available below.
	 *
     * http://www.flipcode.com/archives/SSE_RayBox_Intersection_Test.shtml
	 *
	 * @param	Check				Information about the ray to trace
	 * @param	HitTime	[out]	Time of hit
	 */
	inline void LineCheckBounds(TkDOPLineCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>& Check, FVector4& HitTime, int32 NodeHit[4] ) const
	{
#define PLAIN_C 0
#if PLAIN_C
		for( int32 BoxIndex=0; BoxIndex<4; BoxIndex++ )
		{
			// 0: Create constants
			FVector4 BoxMin( BoundingVolumes.Min[0][BoxIndex], BoundingVolumes.Min[1][BoxIndex], BoundingVolumes.Min[2][BoxIndex], 0 );
			FVector4 BoxMax( BoundingVolumes.Max[0][BoxIndex], BoundingVolumes.Max[1][BoxIndex], BoundingVolumes.Max[2][BoxIndex], 0 );

			// 1: Calculate slabs.
			FVector4 Slab1 = (BoxMin - Check.LocalStart) * Check.LocalOneOverDir;
			FVector4 Slab2 = (BoxMax - Check.LocalStart) * Check.LocalOneOverDir;

			// 2: Figure out per component min/ max
			FVector4 SlabMin = FVector4( FMath::Min(Slab1.X, Slab2.X), FMath::Min(Slab1.Y, Slab2.Y), FMath::Min(Slab1.Z, Slab2.Z), FMath::Min(Slab1.W, Slab2.W) );
			FVector4 SlabMax = FVector4( FMath::Max(Slab1.X, Slab2.X), FMath::Max(Slab1.Y, Slab2.Y), FMath::Max(Slab1.Z, Slab2.Z), FMath::Max(Slab1.W, Slab2.W) );

			// 3: Figure out global min/ max
			float MinTime = Max3( SlabMin.X, SlabMin.Y, SlabMin.Z );
			float MaxTime = Min3( SlabMax.X, SlabMax.Y, SlabMax.Z );

			// 4: Calculate hit time and determine whether there was a hit.
			HitTime[BoxIndex] = MinTime;			
			NodeHit[BoxIndex] = (MaxTime >= 0 && MaxTime >= MinTime && MinTime < Check.Result->Time) ? 0xFFFFFFFF : 0;
		}
#else
		// 0: load everything into registers
		const VectorRegister OriginX		= VectorSetFloat1( Check.LocalStart.X );
		const VectorRegister OriginY		= VectorSetFloat1( Check.LocalStart.Y );
		const VectorRegister OriginZ		= VectorSetFloat1( Check.LocalStart.Z );
		const VectorRegister InvDirX		= VectorSetFloat1( Check.LocalOneOverDir.X );
		const VectorRegister InvDirY		= VectorSetFloat1( Check.LocalOneOverDir.Y );
		const VectorRegister InvDirZ		= VectorSetFloat1( Check.LocalOneOverDir.Z );
		const VectorRegister CurrentHitTime	= VectorSetFloat1( Check.Result->Time );
		// Boxes are FVector2D so we need to unshuffle the data.
		const VectorRegister BoxMinX		= VectorLoadAligned( &BoundingVolumes.Min[0] );
		const VectorRegister BoxMinY		= VectorLoadAligned( &BoundingVolumes.Min[1] );
		const VectorRegister BoxMinZ		= VectorLoadAligned( &BoundingVolumes.Min[2] );
		const VectorRegister BoxMaxX		= VectorLoadAligned( &BoundingVolumes.Max[0] );
		const VectorRegister BoxMaxY		= VectorLoadAligned( &BoundingVolumes.Max[1] );
		const VectorRegister BoxMaxZ		= VectorLoadAligned( &BoundingVolumes.Max[2] );

		// 1: Calculate slabs.
		const VectorRegister BoxMinSlabX	= VectorMultiply( VectorSubtract( BoxMinX, OriginX ), InvDirX );
		const VectorRegister BoxMinSlabY	= VectorMultiply( VectorSubtract( BoxMinY, OriginY ), InvDirY );
		const VectorRegister BoxMinSlabZ	= VectorMultiply( VectorSubtract( BoxMinZ, OriginZ ), InvDirZ );		
		const VectorRegister BoxMaxSlabX	= VectorMultiply( VectorSubtract( BoxMaxX, OriginX ), InvDirX );
		const VectorRegister BoxMaxSlabY	= VectorMultiply( VectorSubtract( BoxMaxY, OriginY ), InvDirY );
		const VectorRegister BoxMaxSlabZ	= VectorMultiply( VectorSubtract( BoxMaxZ, OriginZ ), InvDirZ );

		// 2: Figure out per component min/ max
		const VectorRegister SlabMinX		= VectorMin( BoxMinSlabX, BoxMaxSlabX );
		const VectorRegister SlabMinY		= VectorMin( BoxMinSlabY, BoxMaxSlabY );
		const VectorRegister SlabMinZ		= VectorMin( BoxMinSlabZ, BoxMaxSlabZ );
		const VectorRegister SlabMaxX		= VectorMax( BoxMinSlabX, BoxMaxSlabX );
		const VectorRegister SlabMaxY		= VectorMax( BoxMinSlabY, BoxMaxSlabY );
		const VectorRegister SlabMaxZ		= VectorMax( BoxMinSlabZ, BoxMaxSlabZ );
		
		// 3: Figure out global min/ max
		const VectorRegister SlabMinXY		= VectorMax( SlabMinX , SlabMinY );
		const VectorRegister MinTime		= VectorMax( SlabMinXY, SlabMinZ );
		const VectorRegister SlabMaxXY		= VectorMin( SlabMaxX , SlabMaxY );
		const VectorRegister MaxTime		= VectorMin( SlabMaxXY, SlabMaxZ );

		// 4: Calculate hit time and determine whether there was a hit.		
		VectorStoreAligned( MinTime, &HitTime );
		const VectorRegister OutNodeHit		= VectorBitwiseAnd( VectorCompareGE( MaxTime, VectorZero() ), VectorCompareGE( MaxTime, MinTime ) );
		const VectorRegister CloserNodeHit	= VectorBitwiseAnd( OutNodeHit, VectorCompareGT( CurrentHitTime, MinTime ) );
		VectorStoreAligned( CloserNodeHit, (float*) NodeHit );
#endif
	}

	/** 
	 * Determines the line in the FkDOPLineCollisionCheck intersects this node. It
	 * also will check the child nodes if it is not a leaf, otherwise it will check
	 * against the triangle data.
	 *
	 * @param Check -- The aggregated line check data
	 */
	bool LineCheck(TkDOPLineCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>& Check, TTraversalHistory<KDOP_IDX_TYPE> History) const
	{
		bool bHit = 0;
		// If this is a node, check the two child nodes and pick the closest one
		// to recursively check against and only check the second one if there is
		// not a hit or the hit returned is further out than the second node
		if (bIsLeaf == 0)
		{
			// Check both left and right node at the same time.
			FVector4 NodeHitTime;
			MS_ALIGN(16) int32	NodeHit[4] GCC_ALIGN(16);
			LineCheckBounds( Check, NodeHitTime, NodeHit );

			// Left node was hit.
			if( NodeHit[0] )
			{
				// Left and Right node are hit.
				if( NodeHit[1] )
				{
					// Left node is closer than right node.
					if( NodeHitTime.X < NodeHitTime.Y )
					{
						bHit = Check.Nodes[n.LeftNode].LineCheckPreCalculated(Check,NodeHitTime,History.AddNode(n.LeftNode),NodeHit);
						// Only check right node if it could possibly be a closer hit
						if(	NodeHitTime.Y < Check.Result->Time
						// No need to check if we have a hit and don't care about closest
						&&	(!bHit || Check.bFindClosestIntersection))
						{
							bHit |= Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));
						}
					}
					// Right node is closer than left node.
					else
					{
						bHit = Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));
						// Only check left node if it could possibly be a closer hit
						if(	NodeHitTime.X < Check.Result->Time
						// No need to check if we have a hit and don't care about closest
						&&	(!bHit || Check.bFindClosestIntersection))
						{
							bHit |= Check.Nodes[n.LeftNode].LineCheckPreCalculated(Check,NodeHitTime,History.AddNode(n.LeftNode),NodeHit);
						}
					}
				}
				// Only left node was hit.
				else
				{
					bHit = Check.Nodes[n.LeftNode].LineCheckPreCalculated(Check,NodeHitTime,History.AddNode(n.LeftNode),NodeHit);
				}
			}
			// Left node was not hit.
			else
			{
				// Only right node was hit.
				if( NodeHit[1] )
				{			
					bHit = Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));					
				}
				// No node was hit.
				else
				{
					return false;
				}
			}
		}
		else
		{
			// This is a leaf, check the triangles for a hit
			bHit = LineCheckTriangles(Check,History);
		}
		return bHit;
	}

	/** 
	 * Determines the line in the FkDOPLineCollisionCheck intersects this node. It
	 * also will check the child nodes if it is not a leaf, otherwise it will check
	 * against the triangle data.
	 *
	 * @param Check -- The aggregated line check data
	 */
	bool LineCheckPreCalculated(TkDOPLineCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>& Check, const FVector4& NodeHitTime, TTraversalHistory<KDOP_IDX_TYPE> History, int32* NodeHit) const
	{
		bool bHit = 0;
		// If this is a node, check the two child nodes and pick the closest one
		// to recursively check against and only check the second one if there is
		// not a hit or the hit returned is further out than the second node
		if (bIsLeaf == 0)
		{
			// Left node was hit.
			if( NodeHit[2] )
			{
				// Left and Right node are hit.
				if( NodeHit[3] )
				{
					// Left node is closer than right node.
					if( NodeHitTime.Z < NodeHitTime.W )
					{
						bHit = Check.Nodes[n.LeftNode].LineCheck(Check,History.AddNode(n.LeftNode));
						// Only check right node if it could possibly be a closer hit
						if(	NodeHitTime.W < Check.Result->Time
						// No need to check if we have a hit and don't care about closest
						&&	(!bHit || Check.bFindClosestIntersection))
						{
							bHit |= Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));
						}
					}
					// Right node is closer than left node.
					else
					{
						bHit = Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));
						// Only check left node if it could possibly be a closer hit
						if(	NodeHitTime.Z < Check.Result->Time
						// No need to check if we have a hit and don't care about closest
						&&	(!bHit || Check.bFindClosestIntersection))
						{
							bHit |= Check.Nodes[n.LeftNode].LineCheck(Check,History.AddNode(n.LeftNode));
						}
					}
				}
				// Only left node was hit.
				else
				{
					bHit = Check.Nodes[n.LeftNode].LineCheck(Check,History.AddNode(n.LeftNode));
				}
			}
			// Left node was not hit.
			else
			{
				// Only right node was hit.
				if( NodeHit[3] )
				{			
					bHit = Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));					
				}
				// No node was hit.
				else
				{
					return false;
				}
			}
		}
		else
		{
			// This is a leaf, check the triangles for a hit
			bHit = LineCheckTriangles(Check,History);
		}
		return bHit;
	}

	/**
	 * Works through the list of triangles in this node checking each one for a
	 * collision.
	 *
	 * @param Check -- The aggregated line check data
	 */
	bool LineCheckTriangles(TkDOPLineCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>& Check, const TTraversalHistory<KDOP_IDX_TYPE>& History) const
	{
		// Assume a miss
		bool bHit = false;
		for ( KDOP_IDX_TYPE SOAIndex = t.StartIndex; SOAIndex < (t.StartIndex + t.NumTriangles); SOAIndex++ )
		{
			const FTriangleSOA& TriangleSOA = Check.SOATriangles[SOAIndex];
			int32 SubIndex = appLineCheckTriangleSOA( Check.StartSOA, Check.EndSOA, Check.DirSOA, TriangleSOA, Check.Result->Time );
			if ( SubIndex >= 0 )
			{
				bHit = true;
				Check.LocalHitNormal.X = VectorGetComponentDynamic(TriangleSOA.Normals.X, SubIndex);
				Check.LocalHitNormal.Y = VectorGetComponentDynamic(TriangleSOA.Normals.Y, SubIndex);
				Check.LocalHitNormal.Z = VectorGetComponentDynamic(TriangleSOA.Normals.Z, SubIndex);
				Check.Result->Item = TriangleSOA.Payload[SubIndex];
				Check.HitNodeIndex = History.GetOldestNode();

				// Early out if we don't care about the closest intersection.
				if( !Check.bFindClosestIntersection )
				{
					break;
				}
			}
		}
		return bHit;
	}
};

/**
 * This is the tree of kDOPs that spatially divides the static mesh. It is
 * a binary tree of kDOP nodes.
 */
template<typename COLL_DATA_PROVIDER, typename KDOP_IDX_TYPE>
struct TkDOPTree
{
	/** Exposes data provider type to clients. */
	typedef COLL_DATA_PROVIDER							DataProviderType;

	/** Exposes node type to clients. */
	typedef TkDOPNode<DataProviderType,KDOP_IDX_TYPE>	NodeType;

	/** The list of nodes contained within this tree. Node 0 is always the root node. */
	kDOPArray<NodeType> Nodes;

	/** The list of collision triangles in this tree. */
	kDOPArray<FTriangleSOA> SOATriangles;

	/**
	 * Creates the root node and recursively splits the triangles into smaller
	 * volumes
	 *
	 * @param BuildTriangles -- The list of triangles to use for the build process
	 */
	void Build(TArray<FkDOPBuildCollisionTriangle<KDOP_IDX_TYPE> >& BuildTriangles)
	{
		float kDOPBuildTime = 0;
		{
			// Empty the current set of nodes and preallocate the memory so it doesn't
			// reallocate memory while we are recursively walking the tree
			// With near-perfect packing, we could easily size these to be
			// Nodes = (n / 2) + 1 and SOATriangles = (n / 4) + 1
			Nodes.Empty(BuildTriangles.Num());
			SOATriangles.Empty(BuildTriangles.Num());

			// Add the root node
			Nodes.AddZeroed();

			// Now tell that node to recursively subdivide the entire set of triangles
			Nodes[0].SplitTriangleList(0,BuildTriangles.Num(),BuildTriangles,SOATriangles,Nodes);

			// Don't waste memory.
			Nodes.Shrink();
			SOATriangles.Shrink();
		}
	}

	/**
	 * Figures out whether the check even hits the root node's bounding volume. If
	 * it does, it recursively searches for a triangle to hit.
	 *
	 * @param Check -- The aggregated line check data
	 */
	bool LineCheck(TkDOPLineCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>& Check) const
	{
		// Recursively check for a hit
		TTraversalHistory<KDOP_IDX_TYPE> History;
		bool bHit = Nodes[0].LineCheck(Check, History.AddNode(0));
		return bHit;
	}
};

/**
 * Base struct for all collision checks. Holds a reference to the collision
 * data provider, which is a struct that abstracts out the access to a
 * particular mesh/primitives data
 */
template <typename COLL_DATA_PROVIDER, typename KDOP_IDX_TYPE> struct TkDOPCollisionCheck
{
	/** Exposes data provider type to clients. */
	typedef COLL_DATA_PROVIDER DataProviderType;

	/** Exposes node type to clients. */
	typedef TkDOPNode<DataProviderType,KDOP_IDX_TYPE> NodeType;

	/** Exposes tree type to clients. */
	typedef TkDOPTree<DataProviderType,KDOP_IDX_TYPE> TreeType;

	/**
	 * Used to get access to local->world, vertices, etc. without using virtuals
	 */
	const DataProviderType& CollDataProvider;
	/**
	 * The kDOP tree
	 */
	const TreeType& kDOPTree;
	/**
	 * The array of the nodes for the kDOP tree
	 */
	const kDOPArray<NodeType>& Nodes;
	/**
	 * The collision triangle data for the kDOP tree
	 */
	const kDOPArray<FTriangleSOA>& SOATriangles;

	/**
	 * Hide the default ctor
	 */
	TkDOPCollisionCheck(const DataProviderType& InCollDataProvider) :
		CollDataProvider(InCollDataProvider),
		kDOPTree(CollDataProvider.GetkDOPTree()),
		Nodes(kDOPTree.Nodes),
		SOATriangles(kDOPTree.SOATriangles)
	{
	}
};

/**
 * This struct holds the information used to do a line check against the kDOP
 * tree. The collision provider gives access to various matrices, vertex data
 * etc. without having to use virtual functions.
 */
template <typename COLL_DATA_PROVIDER, typename KDOP_IDX_TYPE> struct TkDOPLineCollisionCheck :
	public TkDOPCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>
{
	/**
	 * Where the collision results get stored
	 */
	FkHitResult* Result;
	// Constant input vars
	const FVector4& Start;
	const FVector4& End;

	/**
	 * Flags for optimizing a trace
	 */
	const bool bFindClosestIntersection;
	// Locally calculated vectors
	FVector4 LocalStart;
	FVector4 LocalEnd;
	FVector4 LocalDir;
	FVector4 LocalOneOverDir;
	// Normal in local space which gets transformed to world at the very end
	FVector4 LocalHitNormal;

	/** Index into the kDOP tree's nodes of the node that was hit. */
	KDOP_IDX_TYPE HitNodeIndex;
	
	/** Start of the line, where each component is replicated into their own vector registers. */
	FVector3SOA	StartSOA;
	/** End of the line, where each component is replicated into their own vector registers. */
	FVector3SOA	EndSOA;
	/** Direction of the line (not normalized, just EndSOA-StartSOA), where each component is replicated into their own vector registers. */
	FVector3SOA	DirSOA;

	/**
	 * Sets up the FkDOPLineCollisionCheck structure for performing line checks
	 * against a kDOPTree. Initializes all of the variables that are used
	 * throughout the line check.
	 *
	 * @param InStart -- The starting point of the trace
	 * @param InEnd -- The ending point of the trace
	 * @param InbFindClosestIntersection -- Whether to stop at the first hit or not
	 * @param InCollDataProvider -- The struct that provides access to mesh/primitive
	 *		specific data, such as L2W, W2L, Vertices, and so on
	 * @param InResult -- The out param for hit result information
	 */
	TkDOPLineCollisionCheck(const FVector4& InStart,const FVector4& InEnd,
		bool bInbFindClosestIntersection,
		const COLL_DATA_PROVIDER& InCollDataProvider,
		FkHitResult* InResult) 
		:
		TkDOPCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>(InCollDataProvider),
		Result(InResult), 
		Start(InStart), 
		End(InEnd), 
		bFindClosestIntersection(bInbFindClosestIntersection), 
		HitNodeIndex(0xFFFFFFFF)
	{
		const FMatrix& WorldToLocal = TkDOPCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>::CollDataProvider.GetWorldToLocal();
		// Move start and end to local space
		LocalStart = WorldToLocal.TransformPosition(Start);
		LocalEnd = WorldToLocal.TransformPosition(End);
		// Calculate the vector's direction in local space
		LocalDir = LocalEnd - LocalStart;
		// Build the one over dir
		LocalOneOverDir.X = LocalDir.X ? 1.f / LocalDir.X : UE_MAX_FLT;
		LocalOneOverDir.Y = LocalDir.Y ? 1.f / LocalDir.Y : UE_MAX_FLT;
		LocalOneOverDir.Z = LocalDir.Z ? 1.f / LocalDir.Z : UE_MAX_FLT;

		// Construct the SOA data
		StartSOA.X = VectorLoadFloat1( &LocalStart.X );
		StartSOA.Y = VectorLoadFloat1( &LocalStart.Y );
		StartSOA.Z = VectorLoadFloat1( &LocalStart.Z );
		EndSOA.X = VectorLoadFloat1( &LocalEnd.X );
		EndSOA.Y = VectorLoadFloat1( &LocalEnd.Y );
		EndSOA.Z = VectorLoadFloat1( &LocalEnd.Z );
		DirSOA.X = VectorLoadFloat1( &LocalDir.X );
		DirSOA.Y = VectorLoadFloat1( &LocalDir.Y );
		DirSOA.Z = VectorLoadFloat1( &LocalDir.Z );
	}

	/**
	 * Transforms the local hit normal into a world space normal using the transpose
	 * adjoint and flips the normal if need be
	 */
	inline FVector4 GetHitNormal(void)
	{
		// Transform the hit back into world space using the transpose adjoint
		FVector4 Normal = TkDOPCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>::CollDataProvider.GetLocalToWorldTransposeAdjoint().TransformVector(LocalHitNormal).GetSafeNormal();
		// Flip the normal if the triangle is inverted
		if (TkDOPCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>::CollDataProvider.GetDeterminant() < 0.f)
		{
			Normal = -Normal;
		}
		return Normal;
	}
};
